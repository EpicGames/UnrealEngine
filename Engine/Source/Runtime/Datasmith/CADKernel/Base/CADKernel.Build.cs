// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class CADKernel : ModuleRules
	{
		public CADKernel(ReadOnlyTargetRules Target)
			: base(Target)
		{
			//OptimizeCode = CodeOptimization.Never;
			CppCompileWarningSettings.DeterministicWarningLevel = WarningLevel.Off; // __DATE__ in Private/CADKernel/Core/System.cpp

			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Private"));
			PublicDefinitions.Add("CADKERNEL_THINZONE=0");

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"GeometryCore",
				}
			);

			PublicDefinitions.Add(Target.Type == TargetType.Program ? "CADKERNEL_DO_ENSURE=0" : "CADKERNEL_DO_ENSURE=1");
		}
	}
}