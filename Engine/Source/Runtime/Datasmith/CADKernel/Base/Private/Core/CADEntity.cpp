// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/CADEntity.h"

#include "Core/EntityGeom.h"

#include "Core/Database.h"
#include "Core/Session.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Surfaces/Surface.h"
#include "Mesh/Criteria/Criterion.h"
#include "Topo/Body.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLink.h"
#include "Topo/TopologicalLoop.h"
#include "Topo/TopologicalVertex.h"

namespace UE::CADKernel
{

	const FPairOfIndex FPairOfIndex::Undefined(-1, -1);

	const TCHAR* FEntity::TypesNames[] = 
	{
		TEXT("UndefinedEntity"),
		TEXT("Curve"),
		TEXT("Surface"),

		TEXT("Edge Link"),
		TEXT("Vertex Link"),
		TEXT("Edge"),
		TEXT("Face"),
 		TEXT("Link"),
		TEXT("Loop"),
		TEXT("Vertex"),
		TEXT("Shell"),
		TEXT("Body"),
		TEXT("Model"),

		TEXT("Mesh Model"),
		TEXT("Mesh"),

		TEXT("Group"),
		TEXT("Criterion"),
		TEXT("Property"),
		nullptr
	};

	const TCHAR* FEntity::GetTypeName(EEntity Type)
	{
		if (Type > EEntity::None && Type < EEntity::EntityTypeEnd)
		{
			return TypesNames[(int32)Type];
		}
		return TypesNames[(int32)EEntity::None];
	}

	FEntity::~FEntity()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FEntity::~FEntity);
	}

	TSharedPtr<FEntity> FEntity::Deserialize(FCADKernelArchive& Archive)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FEntity::Deserialize);

		ensureCADKernel(Archive.IsLoading());

		EEntity Type = EEntity::None;
		Archive << Type;

		TSharedPtr<FEntity> Entity;
		switch (Type)
		{
		case EEntity::Body:
			Entity = FEntity::MakeShared<FBody>(Archive);
			break;
		case EEntity::Curve:
			Entity = FCurve::Deserialize(Archive);
			break;
		case EEntity::Criterion:
			Entity = FCriterion::Deserialize(Archive);
			break;
		case EEntity::EdgeLink:
			Entity = FEntity::MakeShared<FEdgeLink>(Archive);
			break;
		case EEntity::Model:
			Entity = FEntity::MakeShared<FModel>(Archive);
			break;
		case EEntity::Shell:
			Entity = FEntity::MakeShared<FShell>(Archive);
			break;
		case EEntity::Surface:
			Entity = FSurface::Deserialize(Archive);
			break;
		case EEntity::TopologicalEdge:
			Entity = FEntity::MakeShared<FTopologicalEdge>(Archive);
			break;
		case EEntity::TopologicalFace:
			Entity = FEntity::MakeShared<FTopologicalFace>(Archive);
			break;
		case EEntity::TopologicalLoop:
			Entity = FEntity::MakeShared<FTopologicalLoop>(Archive);
			break;
		case EEntity::TopologicalVertex:
			Entity = FEntity::MakeShared<FTopologicalVertex>(Archive);
			break;
		case EEntity::VertexLink:
			Entity = FEntity::MakeShared<FVertexLink>(Archive);
			break;
		default:
			break;
		}

		if (Entity.IsValid())
		{
			Entity->Serialize(Archive);
			Archive.AddEntityFromArchive(Entity);
		}
		return Entity;
	}

#ifdef CADKERNEL_DEV

	void FEntity::InfoEntity() const
	{
		FInfoEntity Info;
		GetInfo(Info);
		Info.Display();
	}

	FInfoEntity& FEntity::GetInfo(FInfoEntity& Info) const
	{
		Info.Set(*this);
		Info.Add(TEXT("Id"), GetId())
			.Add(TEXT("Type"), TypesNames[(uint32)GetEntityType()]);
		return Info;
	}

	FInfoEntity& FEntityGeom::GetInfo(FInfoEntity& Info) const
	{
		return FEntity::GetInfo(Info)
			.Add(TEXT("Kio"), CtKioId);
	}

	void FEntity::AddEntityInDatabase(TSharedRef<FEntity> Entity)
	{
		FSession::Session.GetDatabase().AddEntity(Entity);
	}

#endif

	void FEntity::SerializeIdents(FCADKernelArchive& Ar, TArray<TOrientedEntity<FEntity>>& Array)
	{
		if (Ar.IsLoading())
		{
			int32 ArrayNum = 0;
			Ar << ArrayNum;
			Array.Init(TOrientedEntity<FEntity>(), ArrayNum);
			for (TOrientedEntity<FEntity>& OrientedEntity : Array)
			{
				FIdent OldId = 0;
				Ar << OldId;
				Ar.SetReferencedEntityOrAddToWaitingList(OldId, OrientedEntity.Entity);
				Ar.Serialize(&OrientedEntity.Direction, sizeof(EOrientation));
			}
		}
		else
		{
			int32 ArrayNum = Array.Num();
			Ar << ArrayNum;
			for (TOrientedEntity<FEntity>& OrientedEntity : Array)
			{
				FIdent Id = OrientedEntity.Entity->GetId();
				Ar << Id;
				Ar.Serialize(&OrientedEntity.Direction, sizeof(EOrientation));
				Ar.AddEntityToSave(Id);
			}
		}
	}

	void FEntity::SerializeIdents(FCADKernelArchive& Ar, TArray<FEntity*>& EntityArray, bool bSaveSelection)
	{
		if (Ar.IsLoading())
		{
			int32 ArrayNum = 0;
			Ar << ArrayNum;
			EntityArray.Init(nullptr, ArrayNum);
			for (FEntity*& Entity : EntityArray)
			{
				FIdent OldId = 0;
				Ar << OldId;
				Ar.SetReferencedEntityOrAddToWaitingList(OldId, &Entity);
			}
		}
		else
		{
			int32 ArrayNum = EntityArray.Num();
			Ar << ArrayNum;
			for (FEntity* Entity : EntityArray)
			{
				if (Entity)
				{
					FIdent Id = Entity->GetId();
					Ar << Id;
				}
			}

			if (bSaveSelection)
			{
				for (FEntity* Entity : EntityArray)
				{
					if (Entity)
					{
						Ar.AddEntityToSave(Entity->GetId());
					}
				}
			}
		}
	}

	void FEntity::SerializeIdents(FCADKernelArchive& Ar, TArray<TWeakPtr<FEntity>>& EntityArray, bool bSaveSelection)
	{
		if (Ar.IsLoading())
		{
			int32 ArrayNum = 0;
			Ar << ArrayNum;
			EntityArray.Init(TSharedPtr<FEntity>(), ArrayNum);
			for (TWeakPtr<FEntity>& Entity : EntityArray)
			{
				FIdent OldId = 0;
				Ar << OldId;
				Ar.SetReferencedEntityOrAddToWaitingList(OldId, Entity);
			}
		}
		else
		{
			int32 ArrayNum = EntityArray.Num();
			Ar << ArrayNum;
			for (TWeakPtr<FEntity>& Entity : EntityArray)
			{
				if (Entity.IsValid())
				{
					FIdent Id = Entity.Pin()->GetId();
					Ar << Id;
				}
			}

			if (bSaveSelection)
			{
				for (TWeakPtr<FEntity>& Entity : EntityArray)
				{
					if (Entity.IsValid())
					{
						Ar.AddEntityToSave(Entity.Pin()->GetId());
					}
				}
			}
		}
	}

	void FEntity::SerializeIdents(FCADKernelArchive& Ar, TArray<TSharedPtr<FEntity>>& EntityArray, bool bSaveSelection)
	{
		if (Ar.IsLoading())
		{
			int32 ArrayNum = 0;
			Ar << ArrayNum;
			EntityArray.Init(TSharedPtr<FEntity>(), ArrayNum);
			for (TSharedPtr<FEntity>& Entity : EntityArray)
			{
				FIdent OldId = 0;
				Ar << OldId;
				Ar.SetReferencedEntityOrAddToWaitingList(OldId, Entity);
			}
		}
		else
		{
			int32 ArrayNum = EntityArray.Num();
			Ar << ArrayNum;
			FIdent Id = 0;
			for (TSharedPtr<FEntity>& Entity : EntityArray)
			{
				if (Entity.IsValid() && !Entity->IsDeleted())
				{
					Id = Entity->GetId();
				}
				else
				{
					Id = 0;
				}
				Ar << Id;
			}

			if (bSaveSelection)
			{
				for (TSharedPtr<FEntity>& Entity : EntityArray)
				{
					if(Entity.IsValid() && !Entity->IsDeleted())
					{ 
						Ar.AddEntityToSave(Entity->GetId());
					}
				}
			}
		}
	}

	void FEntity::SerializeIdent(FCADKernelArchive& Ar, TSharedPtr<FEntity>& Entity, bool bSaveSelection)
	{
		if (Ar.IsLoading())
		{
			FIdent OldId = 0;
			Ar << OldId;
			Ar.SetReferencedEntityOrAddToWaitingList(OldId, Entity);
		}
		else
		{
			FIdent Id = Entity.IsValid() ? Entity->GetId() : 0;
			Ar << Id;
			if (bSaveSelection && Id)
			{
				Ar.AddEntityToSave(Id);
			}
		}
	}

	void FEntity::SerializeIdent(FCADKernelArchive& Ar, TWeakPtr<FEntity>& Entity, bool bSaveSelection)
	{
		if (Ar.Archive.IsLoading())
		{
			FIdent OldId;
			Ar.Archive << OldId;
			Ar.SetReferencedEntityOrAddToWaitingList(OldId, Entity);
		}
		else
		{
			FIdent Id = Entity.IsValid() ? Entity.Pin()->GetId() : 0;
			Ar.Archive << Id;
			if (bSaveSelection && Id)
			{
				Ar.AddEntityToSave(Id);
			}
		}
	}

	void FEntity::SerializeIdent(FCADKernelArchive& Ar, FEntity** Entity, bool bSaveSelection)
	{
		if (Ar.Archive.IsLoading())
		{
			FIdent OldId;
			Ar.Archive << OldId;
			Ar.SetReferencedEntityOrAddToWaitingList(OldId, Entity);
		}
		else
		{
			FIdent Id = *Entity ? (*Entity)->GetId() : 0;
			Ar.Archive << Id;
			if (bSaveSelection && Id)
			{
				Ar.AddEntityToSave(Id);
			}
		}
	}
	
	bool FEntity::SetId(FDatabase& Database)
	{
		if (Id < 1)
		{
			Database.AddEntity(AsShared());
			++Database.EntityCount;
			return true;
		}
		return Database.bForceSpawning || false;
	}

} // namespace UE::CADKernel
