// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/Database.h"

#include "Core/CADKernelArchive.h"
#include "Core/Group.h"
#include "Core/Session.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Topo/Body.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/Linkable.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalVertex.h"
#include "UI/Message.h"

#include "HAL/FileManager.h"

static const int32 DataBaseInitialSize = 10000;

namespace UE::CADKernel
{

FDatabase::FDatabase()
{
	DatabaseEntities.Reserve(DataBaseInitialSize);
	AvailableIdents.Reserve(DataBaseInitialSize);
	DatabaseEntities.Add(TSharedPtr<FEntity>());

	Model = FEntity::MakeShared<FModel>();

	AddEntity(Model.ToSharedRef());
}

TSharedPtr<FModel> FDatabase::GetModelAsShared()
{
	if (!Model.IsValid())
	{
		Model = FEntity::MakeShared<FModel>();
		AddEntity(Model.ToSharedRef());
	}
	return Model;
}

FIdent FDatabase::CreateId()
{
	if (AvailableIdents.Num() > 0)
	{
		return  AvailableIdents.Pop(EAllowShrinking::No);
	}
	else
	{
		FIdent NewIdent = (FIdent)DatabaseEntities.Num();
		DatabaseEntities.Add(TSharedPtr<FEntity>());
		return NewIdent;
	}
}

void FDatabase::RemoveEntity(FEntity& Entity)
{
	FIdent EntityId = Entity.GetId();
	if (EntityId == 0)
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::RemoveEntity: the entity is not in the database.\n");
	}
	else if (EntityId >= (FIdent)DatabaseEntities.Num())
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::RemoveEntity: the Id=%d is not yet defined, the entity has never exist.\n", EntityId);
	}
	else if (DatabaseEntities[EntityId].IsValid())
	{
		if (DatabaseEntities[EntityId].Get() != &Entity)
		{
			UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::RemoveEntity: the entity with Id=%d is not in the database.\n", EntityId);
			return;
		}
		DatabaseEntities[EntityId] = TSharedPtr<FEntity>();
		AvailableIdents.Add(EntityId);
	}
	else
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::RemoveEntity: the entity with Id=%d is already deleted.\n", EntityId);
	}
}

void FDatabase::RemoveEntity(FIdent EntityId)
{
	if (EntityId >= (FIdent)DatabaseEntities.Num())
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::RemoveEntity: the Id=%d is not yet defined, the entity has never exist.\n", EntityId);
	}
	else if (DatabaseEntities[EntityId].IsValid())
	{
		DatabaseEntities[EntityId] = TSharedPtr<FEntity>();
		AvailableIdents.Add(EntityId);
	}
	else
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::RemoveEntity: the entity with Id=%d is already deleted.\n", EntityId);
	}
}

TSharedPtr<FEntity> FDatabase::GetEntity(FIdent EntityId) const
{
	if (EntityId >= (FIdent)DatabaseEntities.Num())
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::GetEntity: the Id=%d is not yet defined\n", EntityId);
		return TSharedPtr<FEntity>();
	}
	if (!DatabaseEntities[EntityId].IsValid())
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Warning in FDataBasse::GetEntity: the entity with Id=%d no longer exists.\n", EntityId);
	}
	return DatabaseEntities[EntityId];
}

void FDatabase::GetEntities(const TArray<FIdent>& EntityIds, TArray<TSharedPtr<FEntity>>& Entities) const
{
	Entities.Empty(EntityIds.Num());

	for (FIdent EntityId : EntityIds)
	{
		TSharedPtr<FEntity> Entity = GetEntity(EntityId);
		if (!Entity.IsValid() || Entity->IsDeleted() || Entity->IsProcessed())
		{
			continue;
		}
		Entity->SetProcessedMarker();
		Entities.Add(Entity);
	}
	for (TSharedPtr<FEntity>& Entity : Entities)
	{
		Entity->ResetProcessedMarker();
	}
}

void FDatabase::GetEntities(const TArray<FIdent>& EntityIds, TArray<FEntity*>& Entities) const
{
	Entities.Empty(EntityIds.Num());

	for (FIdent EntityId : EntityIds)
	{
		TSharedPtr<FEntity> Entity = GetEntity(EntityId);
		if (!Entity.IsValid() || Entity->IsDeleted() || Entity->IsProcessed())
		{
			continue;
		}

		Entity->SetProcessedMarker();
		Entities.Add(Entity.Get());
	}
	for (FEntity* Entity : Entities)
	{
		Entity->ResetProcessedMarker();
	}
}

void FDatabase::GetTopologicalEntities(const TArray<FIdent>& EntityIds, TArray<FTopologicalEntity*>& Entities) const
{
	Entities.Empty(EntityIds.Num());

	for (FIdent EntityId : EntityIds)
	{
		TSharedPtr<FEntity> Entity = GetEntity(EntityId);
		if (!Entity.IsValid() || Entity->IsDeleted() || Entity->IsProcessed())
		{
			continue;
		}

		if(Entity->IsTopologicalEntity())
		{
			Entity->SetProcessedMarker();
			Entities.Add((FTopologicalEntity*) Entity.Get());
		}
	}
	for (FTopologicalEntity* Entity : Entities)
	{
		Entity->ResetProcessedMarker();
	}
}

void FDatabase::GetTopologicalShapeEntities(const TArray<FIdent>& EntityIds, TArray<FTopologicalShapeEntity*>& Entities) const
{
	Entities.Empty(EntityIds.Num());

	// to avoid duplicated components
	TSet<FIdent> AddedComponents;
	AddedComponents.Reserve(EntityIds.Num());

	for (FIdent EntityId : EntityIds)
	{
		TSharedPtr<FEntity> Entity = GetEntity(EntityId);
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		if(Entity->IsTopologicalShapeEntity())
		{
			Entity->SetProcessedMarker();
			Entities.Add((FTopologicalShapeEntity*) Entity.Get());
		}
	}
	for (FTopologicalShapeEntity* Entity : Entities)
	{
		Entity->ResetProcessedMarker();
	}
}

void FCADKernelArchive::SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, FEntity** Entity)
{
	Session.Database.SetReferencedEntityOrAddToWaitingList(ArchiveId, Entity);
}

void FCADKernelArchive::SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, TWeakPtr<FEntity>& Entity)
{
	Session.Database.SetReferencedEntityOrAddToWaitingList(ArchiveId, Entity);
}

void FCADKernelArchive::SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, TSharedPtr<FEntity>& Entity)
{
	Session.Database.SetReferencedEntityOrAddToWaitingList(ArchiveId, Entity);
}

void FCADKernelArchive::AddEntityToSave(FIdent Id)
{
	Session.Database.AddEntityToSave(Id);
}

void FCADKernelArchive::AddEntityFromArchive(TSharedPtr<FEntity>& Entity)
{
	Session.Database.AddEntityFromArchive(Entity);
}

void FDatabase::SerializeSelection(FCADKernelArchive& Ar, const TArray<FIdent>& SelectionIds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatabase::Serialize);

	ensureCADKernel(Ar.IsSaving());

	bIsRecursiveSerialization = true;

	Ar.Session.Serialize(Ar);

	int32 DatabaseSize = DatabaseEntities.Num();
	Ar << DatabaseSize;

	EntitiesToBeSerialized.Empty();
	NotYetSerialized.Empty();

	// An archive must gather entity under a single Model. 
	TSharedRef<FModel> TmpModel = FEntity::MakeShared<FModel>();
	TmpModel->SpawnIdent(*this);
	AddEntityToSave(TmpModel->GetId());

	for (FIdent EntityId : SelectionIds)
	{
		TSharedPtr<FEntity> Entity = GetEntity(EntityId);
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		switch(Entity->GetEntityType())
		{
		case EEntity::Model:
			TmpModel->Copy(StaticCastSharedRef<FModel>(Entity.ToSharedRef()));
			break;
		case EEntity::Body:
			TmpModel->Add(StaticCastSharedRef<FBody>(Entity.ToSharedRef()));
			break;
		default:
			AddEntityToSave(EntityId);
			break;
		}
	}

	int32 Index = 0;
	FIdent EntityIdToSave;
	while(NotYetSerialized.Dequeue(EntityIdToSave))
	{
		TSharedPtr<FEntity> Entity = GetEntity(EntityIdToSave);
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		EEntity Type = Entity->GetEntityType();
		Ar << Type;
		Entity->Serialize(Ar);
		Index++;
	}

	EntitiesToBeSerialized.Empty();
	NotYetSerialized.Empty();
	bIsRecursiveSerialization = false;

	RemoveEntity(TmpModel->GetId());
}

void FDatabase::Serialize(FCADKernelArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatabase::Serialize);

	ensureCADKernel(Ar.IsSaving());

	SpawnEntityIdent(GetModel(), true);

	Ar.Session.Serialize(Ar);

	int32 DatabaseSize = DatabaseEntities.Num();
	Ar << DatabaseSize;

	int32 Index = 0;
	for (TSharedPtr<FEntity> Entity : DatabaseEntities)
	{
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			EEntity Type = EEntity::None;
			Ar << Type;
			continue;
		}

		EEntity Type = Entity->GetEntityType();
		Ar << Type;
		Entity->Serialize(Ar);
		Index++;
	}
}

void FDatabase::Deserialize(FCADKernelArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatabase::Deserialize);

	ensureCADKernel(Ar.IsLoading());

	Ar.Session.Serialize(Ar);

	int32 ArchiveSize = 0;
	Ar << ArchiveSize;
	DatabaseEntities.Reserve((int32)(DatabaseEntities.Num() + ArchiveSize * 1.5));
	ArchiveSize += 10;
	ArchiveIdToWaitingSharedPointers.SetNum(ArchiveSize);
	ArchiveIdToWaitingWeakPointers.SetNum(ArchiveSize);
	ArchiveIdToWaitingPointers.SetNum(ArchiveSize);
	ArchiveEntities.Init(TSharedPtr<FEntity>(), ArchiveSize);

	int64 TotalSize = Ar.TotalSize();
	int32 Index = 0;
	for( ; Ar.Tell() < TotalSize; ++Index)
	{
		TSharedPtr<FEntity> Entity = FEntity::Deserialize(Ar);
	}

	// this is a partial archive, a clean of entities has to be performed to remove invalid references in entity descriptions
	if(Index < ArchiveSize)
	{
		CleanArchiveEntities();
	}

	ArchiveIdToWaitingSharedPointers.Empty();
	ArchiveEntities.Empty();
}

void FDatabase::CleanArchiveEntities()
{
	// Initialize the model if not
	GetModel();

	int32 FaceCount = 0;
	int32 ShellCount = 0;
	for (TSharedPtr<FEntity> Entity : ArchiveEntities)
	{
		if (Entity.IsValid() && !Entity->IsDeleted())
		{
			switch (Entity->GetEntityType())
			{
			case EEntity::EdgeLink:
			{
				TSharedPtr<FEdgeLink> EdgeLink = StaticCastSharedPtr<FEdgeLink>(Entity);
				EdgeLink->CleanLink();
				ensureCADKernel(EdgeLink->GetTwinEntityNum());
				break;
			}
			case EEntity::VertexLink:
			{
				TSharedPtr<FVertexLink> VertexLink = StaticCastSharedPtr<FVertexLink>(Entity);
				VertexLink->CleanLink();
				break;
			}
			case EEntity::Body:
			{
				TSharedPtr<FBody> Body = StaticCastSharedPtr<FBody>(Entity);
				for (const TSharedPtr<FShell>& Shell : Body->GetShells())
				{
					Shell->SetMarker1();
				}
				Model->Add(StaticCastSharedPtr<FBody>(Entity));
				break;
			}
			case EEntity::Shell:
			{
				TSharedPtr<FShell> Shell = StaticCastSharedPtr<FShell>(Entity);
				for (const FOrientedFace& Face : Shell->GetFaces())
				{
					Face.Entity->SetMarker1();
				}
				ShellCount++;
				break;
			}
			case EEntity::TopologicalFace:
			{
				FaceCount++;
			}
			default:
				break;
			}
		}
	}


	TArray<TSharedPtr<FTopologicalFace>> IndependantFaces;
	IndependantFaces.Reserve(FaceCount);
	TArray<TSharedPtr<FBody>> NewBodies;
	NewBodies.Reserve(ShellCount);

	// find independent faces and shells
	for (TSharedPtr<FEntity> Entity : ArchiveEntities)
	{
		if (Entity.IsValid() && !Entity->IsDeleted())
		{
			switch (Entity->GetEntityType())
			{
			case EEntity::Shell:
			{
				TSharedPtr<FShell> Shell = StaticCastSharedPtr<FShell>(Entity);
				if (Shell->HasMarker1())
				{
					Shell->ResetMarker1();
				}
				else
				{
					// for independent shell, make a new body to contain it
					TSharedRef<FBody> Body = FEntity::MakeShared<FBody>();
					Body->AddShell(Shell.ToSharedRef());
					NewBodies.Emplace(Body);
				}
				break;
			}
			case EEntity::TopologicalFace:
			{
				TSharedPtr<FTopologicalFace> Face= StaticCastSharedPtr<FTopologicalFace>(Entity);
				if (Face->HasMarker1())
				{
					Face->ResetMarker1();
				}
				else
				{
					IndependantFaces.Emplace(Face);
				}
				break;
			}
			default:
				break;
			}
		}
	}

	Model->Append(NewBodies);
}

void FDatabase::Empty()
{
	Model.Reset();
	DatabaseEntities.Empty(DataBaseInitialSize);
	DatabaseEntities.Add(TSharedPtr<FEntity>());
	AvailableIdents.Empty(DataBaseInitialSize);
}

uint32 FDatabase::SpawnEntityIdents(const TArray<TSharedPtr<FEntity>>& SelectedEntities, bool bInForceSpawning)
{
	bForceSpawning = bInForceSpawning;
	EntityCount = 0;
	for (TSharedPtr<FEntity> Entity : SelectedEntities)
	{
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		Entity->SpawnIdent(*this);
	}
	bForceSpawning = false;
	return EntityCount;
}

uint32 FDatabase::SpawnEntityIdents(const TArray<FEntity*>& SelectedEntities, bool bInForceSpawning)
{
	bForceSpawning = bInForceSpawning;
	EntityCount = 0;
	for (FEntity* Entity : SelectedEntities)
	{
		if (Entity == nullptr || Entity->IsDeleted())
		{
			continue;
		}

		Entity->SpawnIdent(*this);
	}
	bForceSpawning = false;
	return EntityCount;
}

uint32 FDatabase::SpawnEntityIdent(FEntity& Entity, bool bInForceSpawning)
{
	if (Entity.IsDeleted())
	{
		return 0;
	}

	bForceSpawning = bInForceSpawning;
	EntityCount = 0;
	Entity.SpawnIdent(*this);
	bForceSpawning = false;
	return EntityCount;
}

void FDatabase::ExpandSelection(const TArray<TSharedPtr<FEntity>>& Entities, const TSet<EEntity>& Filters, TArray<TSharedPtr<FEntity>>& Selection) const
{
	TSet<TSharedPtr<FEntity>> EntitySet;
	ExpandSelection(Entities, Filters, EntitySet);
	for (TSharedPtr<FEntity> Entity : EntitySet)
	{
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		Selection.Add(Entity);
	}
}

void FDatabase::ExpandSelection(const TArray<TSharedPtr<FEntity>>& Entities, const TSet<EEntity>& Filter, TSet<TSharedPtr<FEntity>>& Selection) const
{
	for (TSharedPtr<FEntity> Entity : Entities)
	{
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		ExpandSelection(Entity, Filter, Selection);
	}
}

void FDatabase::ExpandSelection(TSharedPtr<FEntity> Entity, const TSet<EEntity>& Filter, TSet<TSharedPtr<FEntity>>& Selection) const
{
	if (!Entity.IsValid() || Entity->IsDeleted())
	{
		return;
	}

	EEntity Type = Entity->GetEntityType();

	if (Filter.Contains(Type))
	{
		Selection.Add(Entity);
	}

	switch (Entity->GetEntityType())
	{

	case EEntity::TopologicalEdge:
	{
		TSharedPtr<FTopologicalEdge>Edge = StaticCastSharedPtr<FTopologicalEdge>(Entity);
		ExpandSelection(Edge->GetCurve(), Filter, Selection);
		break;
	}

	case EEntity::Model:
	{
		TSharedPtr<FModel> InModel = StaticCastSharedPtr<FModel>(Entity);
		for (const TSharedPtr<FBody>& Body : InModel->GetBodies())
		{
			ExpandSelection(Body, Filter, Selection);
		}

		break;
	}

	case EEntity::Body:
	{
		TSharedPtr<FBody> Body = StaticCastSharedPtr<FBody>(Entity);
		for (const TSharedPtr<FShell>& Shell : Body->GetShells())
		{
			for (const FOrientedFace& Face : Shell->GetFaces())
			{
				ExpandSelection(Face.Entity, Filter, Selection);
			}
		}
		break;
	}

	case EEntity::MeshModel:
	{
		ensureCADKernel(false);
		break;
	}

	default:
	{
		break;
	}
	}

}

void FDatabase::TopologicalEntitiesSelection(const TArray<TSharedPtr<FEntity>>& Entities, TArray<TSharedPtr<FTopologicalEntity>>& OutTopoEntities, bool bFindSurfaces, bool bFindEdges, bool bFindVertices) const
{
	TSet<EEntity> Filter;
	if (bFindSurfaces)	Filter.Add(EEntity::TopologicalFace);
	if (bFindEdges)		Filter.Add(EEntity::TopologicalEdge);
	if (bFindVertices) 	Filter.Add(EEntity::TopologicalVertex);

	OutTopoEntities.Empty(FMath::Max(Entities.Num(), (int32)100));
	ExpandSelection(Entities, Filter, (TArray<TSharedPtr<FEntity>>&) OutTopoEntities);
}

void FDatabase::CadEntitiesSelection(const TArray<TSharedPtr<FEntity>>& Entities, TArray<TSharedPtr<FEntity>>& OutCADEntities) const
{
	TSet<EEntity> Filter;
	Filter.Add(EEntity::TopologicalFace);
	Filter.Add(EEntity::TopologicalEdge);
	Filter.Add(EEntity::TopologicalVertex);
	Filter.Add(EEntity::Curve);
	Filter.Add(EEntity::Surface);

	OutCADEntities.Empty(FMath::Max(Entities.Num(), (int32)100));
	ExpandSelection(Entities, Filter, OutCADEntities);
}

void FDatabase::MeshEntitiesSelection(const TArray<TSharedPtr<FEntity>>& Entities, TArray<TSharedPtr<FEntity>>& OutMeshEntities) const
{
	TSet<EEntity> Filter;
	Filter.Add(EEntity::Mesh);

	OutMeshEntities.Empty(FMath::Max(Entities.Num(), (int32)100));
	ExpandSelection(Entities, Filter, OutMeshEntities);
}

void FDatabase::GetEntitiesOfType(EEntity Type, TArray<TSharedPtr<FEntity>>& OutEntities) const
{
	TSet<EEntity> Filter;
	Filter.Add(Type);

	GetEntitiesOfTypes(Filter, OutEntities);
}

void FDatabase::GetEntitiesOfTypes(const TSet<EEntity>& Filter, TArray<TSharedPtr<FEntity>>& OutEntities) const
{
	OutEntities.Empty(100);
	for (TSharedPtr<FEntity> Entity : DatabaseEntities)
	{
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		if (Filter.Contains(Entity->GetEntityType()))
		{
			OutEntities.Add(Entity);
		}
	}
}

TSharedPtr<FEntity> FDatabase::GetFirstEntityOfType(EEntity Type) const
{
	for (TSharedPtr<FEntity> Entity : DatabaseEntities)
	{
		if (!Entity.IsValid() || Entity->IsDeleted())
		{
			continue;
		}

		if (Entity->GetEntityType() == Type)
		{
			return Entity;
		}
	}
	return nullptr;
}

}