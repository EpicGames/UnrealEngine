// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Core/Group.h"

#include "Containers/List.h"


namespace UE::CADKernel
{

template<class ElementType>
class TFactory
{
	static const int32 MaxSize = 256;

protected:
	TDoubleLinkedList<TArray<ElementType>> StackOfEntitySet;
	TArray<ElementType*> FreeEntityStack;
#ifdef DEBUG_FACTORY
	int32 AllocatedElementNum = 0;
	int32 UsedElementNum = 0;
	int32 AvailableElementNum = 0;
#endif

private:
	void FillFreeEntityStack()
	{
		StackOfEntitySet.AddHead(TArray<ElementType>());
		TArray<ElementType>& EntityPool = StackOfEntitySet.GetHead()->GetValue();
		EntityPool.SetNum(MaxSize);
		for (ElementType& Entity : EntityPool)
		{
			FreeEntityStack.Add(&Entity);
		}
#ifdef DEBUG_FACTORY
		AllocatedElementNum += MaxSize;
#endif
	}

public:
	TFactory()
	{
		FreeEntityStack.Reserve(MaxSize);
	}

	void DeleteEntity(ElementType* Entity)
	{
		Entity->Clean();
		FreeEntityStack.Add(Entity);
#ifdef DEBUG_FACTORY
		UsedElementNum--;
		ensureCADKernel(UsedElementNum + FreeEntityStack.Num() == AllocatedElementNum);
#endif
	}

	ElementType& New()
	{
#ifdef DEBUG_FACTORY
		ensureCADKernel(UsedElementNum + FreeEntityStack.Num() == AllocatedElementNum);
#endif
		if (FreeEntityStack.IsEmpty())
		{
			FillFreeEntityStack();
		}
#ifdef DEBUG_FACTORY
		UsedElementNum++;
#endif
		return *FreeEntityStack.Pop(EAllowShrinking::No);
	}
};
} // namespace UE::CADKernel

