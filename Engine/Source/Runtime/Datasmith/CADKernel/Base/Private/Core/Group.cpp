// Copyright Epic Games, Inc. All Rights Reserved.

#include "Group.h"

namespace UE::CADKernel
{ 

const TCHAR* GroupOriginNames[] = {
	TEXT("Unknown"),
	TEXT("CAD Group"),
	TEXT("CAD Layer"),
	TEXT("CAD Color"),
	nullptr
};


EEntity FGroup::GetGroupType() const
{
	if (Entities.Num() == 0) 
	{
		return EEntity::None;
	}

	return (*Entities.begin())->GetEntityType();
}

void FGroup::ReplaceEntitiesWithMap(const TMap<TSharedPtr<FEntity>, TSharedPtr<FEntity>>& Map)
{
	for (const TPair<TSharedPtr<FEntity>, TSharedPtr<FEntity>>& Pair : Map)
	{
		if (Entities.Contains(Pair.Key)) 
		{
			Entities.Remove(Pair.Key);
			Entities.Add(Pair.Value);
		}
	}
}

void FGroup::RemoveNonTopologicalEntities()
{
	for (auto It = Entities.CreateIterator(); It; ++It)
	{
		const TSharedPtr<FEntity>& Entity = *It;
		EEntity Type = Entity->GetEntityType();
		if (Type != EEntity::TopologicalFace && Type != EEntity::TopologicalEdge && Type != EEntity::TopologicalVertex) 
		{
			It.RemoveCurrent();
		}
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FGroup::GetInfo(FInfoEntity& Info) const
{
	return FEntity::GetInfo(Info)
	.Add(TEXT("Origin"), GroupOriginNames[(int8) GetOrigin()])
	.Add(TEXT("Entities"), Entities);
}
#endif

void FGroup::SetName(const FString& Name)
{
	GroupName = Name;
}

}