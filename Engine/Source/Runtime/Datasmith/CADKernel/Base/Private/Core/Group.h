// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h"
#include "Core/CADKernelArchive.h"

namespace UE::CADKernel
{
enum class EGroupOrigin : uint8
{
	Unknown,
	CADGroup,
	CADLayer,
	CADColor
};

extern const TCHAR* GroupOriginNames[];

class FGroup : public FEntity
{
	friend FEntity;

protected:
	EGroupOrigin Origin;
	FString GroupName;
	TArray<TSharedPtr<FEntity>> Entities;

	FGroup()
		: Origin(EGroupOrigin::Unknown)
	{
	}

	FGroup(TArray<TSharedPtr<FEntity>>& InEntities)
		: Origin(EGroupOrigin::Unknown)
	{
		Entities.Append(InEntities);
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FEntity::Serialize(Ar);
		Ar << Origin;
		Ar << GroupName;
		SerializeIdents(Ar, (TArray<TSharedPtr<FEntity>>&) Entities);
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities(Entities, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities(Entities);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::None;
	}

	void SetName(const FString& Name);

	const FString& GetName() const
	{
		return GroupName;
	}

	void AddEntity(TSharedPtr<FEntity> Entity)
	{
		Entities.AddUnique(Entity);
	}

	void Empty()
	{
		Entities.Empty();
	}

	void RemoveEntity(TSharedPtr<FEntity> Entity)
	{
		Entities.Remove(Entity);
	}

	bool Contains(TSharedPtr<FEntity> Entity)
	{
		return Entities.Contains(Entity);
	}

	EGroupOrigin GetOrigin() const
	{
		return Origin;
	}

	void SetOrigin(EGroupOrigin InOrigin)
	{
		Origin = InOrigin;
	}

	EEntity GetGroupType() const;

	void GetValidEntities(TArray<TSharedPtr<FEntity>>& OutEntities) const
	{
		for (TSharedPtr<FEntity> Entity : Entities)
		{
			if (Entity.IsValid())
			{
				OutEntities.Add(Entity);
			}
		}
	}

	const TArray<TSharedPtr<FEntity>>& GetEntities() const
	{
		return Entities;
	}

	bool IsEmpty()
	{
		return Entities.IsEmpty();
	}

	void ReplaceEntitiesWithMap(const TMap<TSharedPtr<FEntity>, TSharedPtr<FEntity>>& Map);

	void RemoveNonTopologicalEntities();
};

} // namespace UE::CADKernel

