// Copyright Epic Games, Inc. All Rights Reserved.
#include "Core/Session.h"

#include "Topo/Model.h"
#include "UI/Message.h"

namespace UE::CADKernel
{

#if defined(CADKERNEL_DEV) || defined(CADKERNEL_STDA)
FSession FSession::Session(0.01);
#endif

bool FSession::SaveDatabase(const TCHAR* FileName)
{
	TSharedPtr<FCADKernelArchive> Archive = FCADKernelArchive::CreateArchiveWriter(*this, FileName);
	if (!Archive.IsValid())
	{
		UE_LOGF(LogCADKernelBase, Warning, "The archive file %ls is corrupted\n", FileName);
		return false;
	}

	Database.Serialize(*Archive);
	Archive->Close();

	return true;
}

bool FSession::SaveDatabase(TArray<uint8>& Bytes)
{
	TSharedPtr<FCADKernelArchive> Archive = MakeShared<FCADKernelArchive>(*this, Bytes);
	if (!Archive.IsValid())
	{
		UE_LOGF(LogCADKernelBase, Warning, "The archive is corrupted\n");
		return false;
	}

	Database.Serialize(*Archive);
	Archive->Close();

	return true;
}

TSharedPtr<FModel> FSession::GetModelAsShared()
{
	return Database.GetModelAsShared();
}

void FSession::SaveDatabase(const TCHAR* FileName, const TArray<FEntity*>& SelectedEntities)
{
	TArray<FIdent> EntityIds;
	EntityIds.Reserve(SelectedEntities.Num());

	SpawnEntityIdents(SelectedEntities, true);

	for (const FEntity* Entity : SelectedEntities)
	{
		EntityIds.Add(Entity->GetId());
	}

	TSharedPtr<FCADKernelArchive> Archive = FCADKernelArchive::CreateArchiveWriter(*this, FileName);

	Database.SerializeSelection(*Archive.Get(), EntityIds);
	Archive->Close();
}

void FSession::SaveDatabase(const TCHAR* FileName, const TArray<TSharedPtr<FEntity>>& SelectedEntities)
{
	TArray<FIdent> EntityIds;
	EntityIds.Reserve(SelectedEntities.Num());

	SpawnEntityIdents(SelectedEntities, true);

	for (const TSharedPtr<FEntity>& Entity : SelectedEntities)
	{
		EntityIds.Add(Entity->GetId());
	}

	TSharedPtr<FCADKernelArchive> Archive = FCADKernelArchive::CreateArchiveWriter(*this, FileName);

	Database.SerializeSelection(*Archive.Get(), EntityIds);
	Archive->Close();
}

bool FSession::LoadDatabase(const TCHAR* FilePath)
{
	TGuardValue<double> GeometricToleranceGuard(GeometricTolerance, 0.01);

	TSharedPtr<FCADKernelArchive> Archive = FCADKernelArchive::CreateArchiveReader(*this, FilePath);
	if (!Archive.IsValid())
	{
		UE_LOGF(LogCADKernelBase, Warning, "The archive file %ls is corrupted\n", FilePath);
		return false;
	}

	FModel& Model = GetModel();

	Database.Deserialize(*Archive);
	FModel* ArchiveModel = Archive->ArchiveModel;

	if (ArchiveModel != nullptr)
	{
		ArchiveModel->Empty();
		Database.RemoveEntity(*ArchiveModel);
	}
	Archive->Close();

	return true;
}

bool FSession::LoadDatabase(const TArray<uint8>& Bytes)
{
	TGuardValue<double> GeometricToleranceGuard(GeometricTolerance, 0.01);

	TSharedPtr<FCADKernelArchive> Archive = MakeShared<FCADKernelArchive>(*this, Bytes);
	if (!Archive.IsValid())
	{
		UE_LOGF(LogCADKernelBase, Warning, "The archive buffer is corrupted\n");
		return false;
	}

	FModel& Model = GetModel();

	Database.Deserialize(*Archive);
	FModel* ArchiveModel = Archive->ArchiveModel;

	if (ArchiveModel != nullptr)
	{
		ArchiveModel->Empty();
		Database.RemoveEntity(*ArchiveModel);
	}
	Archive->Close();

	return true;
}

void FSession::AddDatabase(const TArray<uint8>& InRawData)
{
	TGuardValue<double> GeometricToleranceGuard(GeometricTolerance, 0.01);
	FCADKernelArchive Archive = FCADKernelArchive(*this, InRawData);
	Database.Deserialize(Archive);
}

void FSession::SetGeometricTolerance(double NewTolerance)
{
	ensureCADKernel(Database.GetModel().EntityCount() == 0);
	GeometricTolerance = NewTolerance;
	IntersectionTool::SetTolerance(NewTolerance);
}
} // namespace UE::CADKernel
