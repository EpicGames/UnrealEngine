// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/BezierCurve.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Math/BSpline.h"

namespace UE::CADKernel
{

bool FBezierCurve::IsBezier(const FNurbsCurveData& NurbsCurveData)
{
	bool bIsBezier = ((NurbsCurveData.Poles.Num() - 1) % NurbsCurveData.Degree) == 0;
	bIsBezier &= ((NurbsCurveData.NodalVector.Num() - 2) % NurbsCurveData.Degree) == 0;

	if (!bIsBezier)
	{
		return false;
	}

	const int32 NumSegments = (NurbsCurveData.Poles.Num() - 1) / NurbsCurveData.Degree;
	for (int32 Index = 0, KnotIndex = 1; Index < NumSegments && bIsBezier; ++Index)
	{
		const double KValue = NurbsCurveData.NodalVector[KnotIndex++];
		for (int32 Jndex = 1; Jndex < NurbsCurveData.Degree; ++Jndex, ++KnotIndex)
		{
			if (!FMath::IsNearlyEqual(KValue, NurbsCurveData.NodalVector[KnotIndex], UE_DOUBLE_SMALL_NUMBER))
			{
				bIsBezier = false;
				break;
			}
		}
	}

	return bIsBezier;
}

FBezierCurve::FBezierCurve(const FNurbsCurveData& NurbsCurveData)
{
	ensureCADKernel(IsBezier(NurbsCurveData));

	Degree = NurbsCurveData.Degree;
	Dimension = (int8)NurbsCurveData.Dimension;

	NumSegments = (NurbsCurveData.Poles.Num() - 1) / NurbsCurveData.Degree;

	NodalVector.SetNum(NumSegments+1);
	for (int32 Index = 0, Offset = Degree; Index <= NumSegments; ++Index, Offset += Degree)
	{
		NodalVector[Index] = NurbsCurveData.NodalVector[Offset];
	}

	Poles = NurbsCurveData.Poles;
	Weights = NurbsCurveData.Weights;
	if (Weights.Num() != Poles.Num())
	{
		Weights.SetNum(Poles.Num());
		for (double& Weight : Weights)
		{
			Weight = 1.;
		}
	}

	// Validate the curve is actually rational
	if (NurbsCurveData.bIsRational)
	{
		const double WeightRef = Weights[0];
		bIsRational = false;
		for (int32 Index = 1; Index < Weights.Num() && !bIsRational; ++Index)
		{
			if (!FMath::IsNearlyEqual(WeightRef, Weights[Index], UE_DOUBLE_SMALL_NUMBER))
			{
				bIsRational = true;
				break;
			}
		}

		if (!bIsRational && !FMath::IsNearlyEqual(WeightRef, 1., UE_DOUBLE_SMALL_NUMBER))
		{
			const double InvWeight = 1. / WeightRef;
			for (FVector& Pole : Poles)
			{
				Pole *= InvWeight;
			}
		}
	}

	Boundary.Set(NodalVector[0], NodalVector[NumSegments]);
}

void FBezierCurve::EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder) const
{
	// Adjust coordinate value to curve's boundaries
	if (!ensureCADKernel(Coordinate >= (NodalVector[0] - UE_DOUBLE_SMALL_NUMBER)))
	{
		Coordinate = NodalVector[0];
	}

	if (!ensureCADKernel(Coordinate <= (NodalVector[NumSegments] + UE_DOUBLE_SMALL_NUMBER)))
	{
		Coordinate = NodalVector[NumSegments];
	}

	int32 SegmentIndex = 0;
	if (FMath::IsNearlyEqual(Coordinate, NodalVector[NumSegments], UE_DOUBLE_SMALL_NUMBER))
	{
		SegmentIndex = NumSegments - 1;
	}
	else
	{
		for (; SegmentIndex < NumSegments + 1; ++SegmentIndex)
		{
			if (Coordinate < NodalVector[SegmentIndex])
			{
				--SegmentIndex;
				break;
			}
		}
	}
	ensureCADKernel(SegmentIndex < NumSegments);

	OutPoint.DerivativeOrder = DerivativeOrder;
	OutPoint.Init();

	// Normalize input coordinates as Bezier computation is not impacted by knots' values
	double NormalizedValue = (Coordinate - NodalVector[SegmentIndex]) / (NodalVector[SegmentIndex + 1] - NodalVector[SegmentIndex]);
	
	if (Degree == 1)
	{
		// Simple linear interpolation...
		OutPoint.Point = Poles[0] * (1. - NormalizedValue) + Poles[1] * NormalizedValue;
	}
	else
	{
		const int32 Order = Degree + 1;
		
		TArray<double> BernsteinCoeffs;
		BernsteinCoeffs.SetNum(DerivativeOrder > 1 ? 3 * Order : (DerivativeOrder > 0 ? 2 * Order : Order));
		double* Bernstein = BernsteinCoeffs.GetData();
		double* BernsteinD1 = DerivativeOrder > 0 ? Bernstein + Order : nullptr;
		double* BernsteinD2 = DerivativeOrder > 1 ? BernsteinD1 + Order : nullptr;

		BSpline::Bernstein(Degree, NormalizedValue, Bernstein, BernsteinD1, BernsteinD2);

		double Weight = 0.;
		for (int32 Index = 0, PolesIndex = SegmentIndex * Degree; Index < Order; ++Index, ++PolesIndex)
		{
			OutPoint.Point += Poles[PolesIndex] * Bernstein[Index];
			Weight += Weights[PolesIndex] * Bernstein[Index];
		}

		if (bIsRational)
		{
			const double InvWeight = 1. / Weight;
			OutPoint.Point *= InvWeight;
		}

		if (BernsteinD1)
		{
		for (int32 Index = 0, PolesIndex = SegmentIndex * Degree; Index < Order; ++Index, ++PolesIndex)
			{
				OutPoint.Gradient += Poles[PolesIndex] * BernsteinD1[Index];
			}

			if (BernsteinD2)
			{
		for (int32 Index = 0, PolesIndex = SegmentIndex * Degree; Index < Order; ++Index, ++PolesIndex)
				{
					OutPoint.Laplacian += Poles[PolesIndex] * BernsteinD2[Index];
				}
			}
		}
	}
}

TSharedPtr<FEntityGeom> FBezierCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TArray<FVector> TransformedPoles;
	TransformedPoles.Reserve(Poles.Num());

	for (const FVector& Pole : Poles)
	{
		TransformedPoles.Emplace(InMatrix.Multiply(Pole));
	}

	return FEntity::MakeShared<FBezierCurve>(TransformedPoles);
}

void FBezierCurve::Offset(const FVector& OffsetDirection)
{
	for (FVector& Pole : Poles)
	{
		Pole += OffsetDirection;
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FBezierCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info)
		.Add(TEXT("degre"), Poles.Num() - 1)
		.Add(TEXT("poles"), Poles);
}
#endif

void FBezierCurve::ExtendTo(const FVector& Point)
{
	PolylineTools::ExtendTo(Poles, Point);
}

} // namespace UE::CADKernel

