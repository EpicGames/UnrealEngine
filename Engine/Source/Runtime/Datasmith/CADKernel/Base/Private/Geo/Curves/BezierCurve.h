// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"
#include "NURBSCurve.h"

namespace UE::CADKernel
{

class CADKERNEL_API FBezierCurve : public FCurve
{
	friend class FEntity;

protected:
	bool bIsRational = false;

	int32 Degree = 0;
	TArray<double> NodalVector;
	int32 NumSegments = 0;

	TArray<FVector> Poles;
	TArray<double> Weights;

	FBezierCurve(const TArray<FVector>& InPoles)
		: Poles{ InPoles }
	{
		Degree = Poles.Num() - 1;
		NumSegments = 1;
		NodalVector.Add(0.);
		NodalVector.Add(1.);
		Weights.SetNum(Poles.Num());
		for (double& Weight : Weights)
		{
			Weight = 1.;
		}
	}

	FBezierCurve(const FNurbsCurveData& NurbsCurveData);

	FBezierCurve() = default;

public:

	static bool IsBezier(const FNurbsCurveData& NurbsCurveData);

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);

		Ar.Serialize(bIsRational);
		Ar.Serialize(Degree);
		Ar.Serialize(NumSegments);
		Ar.Serialize(NodalVector);
		Ar.Serialize(Poles);
		Ar.Serialize(Weights);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Bezier;
	}

	int32 GetDegre() const
	{
		return Degree;
	}

	const TArray<FVector>& GetPoles() const
	{
		return Poles;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override;
	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		FCurvePoint CurvePoint;
		EvaluatePoint(Coordinate, CurvePoint, DerivativeOrder);
		OutPoint.DerivativeOrder = DerivativeOrder;
		OutPoint.Point[0] = CurvePoint.Point[0];
		OutPoint.Point[1] = CurvePoint.Point[1];
		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient[0] = CurvePoint.Gradient[0];
			OutPoint.Gradient[1] = CurvePoint.Gradient[1];
			
			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian[0] = CurvePoint.Laplacian[0];
				OutPoint.Laplacian[1] = CurvePoint.Laplacian[1];
			}
		}
	}

	virtual void ExtendTo(const FVector& Point) override;


};

} // namespace UE::CADKernel

