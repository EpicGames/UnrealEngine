// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/BoundedCurve.h"

namespace UE::CADKernel
{

void FBoundedCurve::EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder) const
{
	ensureCADKernel(Boundary.Contains(Coordinate));
	return Curve->EvaluatePoint(Coordinate, OutPoint, DerivativeOrder);
}

void FBoundedCurve::Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder) const
{
	ensureCADKernel(Boundary.Contains(Coordinate));
	return Curve->Evaluate2DPoint(Coordinate, OutPoint, DerivativeOrder);
}

void FBoundedCurve::FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const
{
	Curve->FindNotDerivableCoordinates(InBoundary, DerivativeOrder, OutNotDerivableCoordinates);
}

TSharedPtr<FCurve> FBoundedCurve::MakeBoundedCurve(const FLinearBoundary& InBoundary)
{
	ensureCADKernel(Curve.IsValid());

	FLinearBoundary NewBoundary = InBoundary;

	double UMin = Curve->GetUMin();
	double UMax = Curve->GetUMax();

	if(NewBoundary.Min <UMin)
	{
		NewBoundary.Min = UMin;
	}

	if(NewBoundary.Max >UMax)
	{
		NewBoundary.Max = UMax;
	}

	if((NewBoundary.Min -DOUBLE_SMALL_NUMBER)<UMin && (NewBoundary.Max +DOUBLE_SMALL_NUMBER)>UMax)
	{
		return FEntity::MakeShared<FBoundedCurve>(*this);
	}

	return FEntity::MakeShared<FBoundedCurve>(Curve.ToSharedRef(), NewBoundary, Dimension);
}

TSharedPtr<FEntityGeom> FBoundedCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FCurve> TransformedCurve = StaticCastSharedPtr<FCurve>(Curve->ApplyMatrix(InMatrix));
	if (!TransformedCurve.IsValid()) 
	{
		return TSharedPtr<FEntityGeom>();
	}

	return FEntity::MakeShared<FBoundedCurve>(TransformedCurve.ToSharedRef(), Boundary, Dimension);
}

void FBoundedCurve::Offset(const FVector& OffsetDirection)
{
	Curve->Offset(OffsetDirection);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FBoundedCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info).Add(TEXT("base curve"), Curve);
}
#endif

}