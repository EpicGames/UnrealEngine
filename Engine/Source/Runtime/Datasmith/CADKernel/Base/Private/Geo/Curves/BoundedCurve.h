// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"

namespace UE::CADKernel
{

class CADKERNEL_API FBoundedCurve : public FCurve
{
	friend class FEntity;

protected:

	TSharedPtr<FCurve> Curve;

	FBoundedCurve(TSharedRef<FCurve> InCurve, const FLinearBoundary& InBoundary, int8 InDimension)
		: FCurve(InBoundary, InDimension)
		, Curve(InCurve)
	{
	}

	FBoundedCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		SerializeIdent(Ar, Curve);
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		Curve->SpawnIdent(Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		Curve->ResetMarkersRecursively();
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::BoundedCurve;
	}

	TSharedRef<const FCurve> GetBaseCurve()
	{
		return Curve.ToSharedRef();
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override;
	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override;

	virtual void FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const override;

	virtual TSharedPtr<FCurve> MakeBoundedCurve(const FLinearBoundary& InBoundary) override;
};

} // namespace UE::CADKernel

