// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/CompositeCurve.h"

#include "Math/Boundary.h"
#include "Geo/GeoPoint.h"
#include "UI/Message.h"

namespace UE::CADKernel
{

double FCompositeCurve::LocalToGlobalCoordinate(int32 CurveIndex, double Coordinate) const
{
	ensureCADKernel(Curves.IsValidIndex(CurveIndex));

	double GlobalCoord = Coordinate;
	const FLinearBoundary& CurveBoundary = Curves[CurveIndex].Entity->GetBoundary();

	if(Curves[CurveIndex].Direction)
	{
		GlobalCoord -= CurveBoundary.Min;
	}
	else
	{
		GlobalCoord -= CurveBoundary.Max;
	}
	GlobalCoord = FMath::Abs(GlobalCoord);
	GlobalCoord /= CurveBoundary.Length();

	GlobalCoord = Coordinates[CurveIndex] + GlobalCoord * (Coordinates[CurveIndex + 1] - Coordinates[CurveIndex]);

	return GlobalCoord;
}

double FCompositeCurve::GlobalToLocalCoordinate(int32 CurveIndex, double Coordinate) const
{
	ensureCADKernel(Curves.IsValidIndex(CurveIndex));

	double LocalCoord = Coordinate - Coordinates[CurveIndex];
	LocalCoord /= (Coordinates[CurveIndex + 1] - Coordinates[CurveIndex]);

	const FLinearBoundary& CurveBoundary = Curves[CurveIndex].Entity->GetBoundary();

	if (Curves[CurveIndex].Direction == EOrientation::Front)
	{
		LocalCoord = CurveBoundary.Min + LocalCoord * CurveBoundary.Length();
	}
	else
	{
		LocalCoord = CurveBoundary.Max - LocalCoord * CurveBoundary.Length();
	}
	return LocalCoord;
}

void FCompositeCurve::EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder) const
{
	ensureCADKernel(Dimension == 3);

	OutPoint.DerivativeOrder = DerivativeOrder;

	Boundary.MoveInsideIfNot(Coordinate, 0.);

	int32 CurveIndex = 0;
	while (CurveIndex < Curves.Num() && Coordinate > Coordinates[CurveIndex + 1])
	{
		CurveIndex++;
	}

	double LocalCoord = GlobalToLocalCoordinate(CurveIndex, Coordinate);

	Curves[CurveIndex].Entity->EvaluatePoint(LocalCoord, OutPoint, DerivativeOrder);

	// Fix the derivative value by a coefficient
	if (DerivativeOrder > 0)
	{
		const FLinearBoundary& CurveBoundary = Curves[CurveIndex].Entity->GetBoundary();
		double Coefficient = CurveBoundary.Length() / (Coordinates[CurveIndex + 1] - Coordinates[CurveIndex]);

		OutPoint.Gradient = OutPoint.Gradient * Coefficient;

		if (DerivativeOrder > 1)
		{
			OutPoint.Laplacian = OutPoint.Laplacian * Coefficient * Coefficient;
		}
	}
}

void FCompositeCurve::Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder) const
{
	ensureCADKernel(Dimension == 2);

	OutPoint.DerivativeOrder = DerivativeOrder;

	Boundary.MoveInsideIfNot(Coordinate, 0.);

	int32 CurveIndex = 0;
	while (CurveIndex < Curves.Num() && Coordinate > Coordinates[CurveIndex + 1])
	{
		CurveIndex++;
	}

	double LocalCoord = GlobalToLocalCoordinate(CurveIndex, Coordinate);

	Curves[CurveIndex].Entity->Evaluate2DPoint(LocalCoord, OutPoint, DerivativeOrder);

	// Fix the derivative value by a coefficient
	if (DerivativeOrder > 0)
	{
		const FLinearBoundary& CurveBoundary = Curves[CurveIndex].Entity->GetBoundary();
		double Coefficient = CurveBoundary.Length() / (Coordinates[CurveIndex + 1] - Coordinates[CurveIndex]);

		OutPoint.Gradient = OutPoint.Gradient * Coefficient;

		if (DerivativeOrder > 1)
		{
			OutPoint.Laplacian = OutPoint.Laplacian * Coefficient * Coefficient;
		}
	}
}

void FCompositeCurve::FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const
{
	TArray<double> NotDerivableCoordinates;
	OutNotDerivableCoordinates.Empty();

	for (int32 CurveIndex = 0; CurveIndex < Curves.Num(); CurveIndex++)
	{
		if (Coordinates[CurveIndex + 1] <= InBoundary.Min)
		{
			continue;
		}
		if (Coordinates[CurveIndex] >= InBoundary.Max)
		{
			break;
		}

		const FLinearBoundary& CurveBoundary = Curves[CurveIndex].Entity->GetBoundary();

		NotDerivableCoordinates.Empty();
		Curves[CurveIndex].Entity->FindNotDerivableCoordinates(CurveBoundary, DerivativeOrder, NotDerivableCoordinates);

		for (int32 BreakIndex = 0; BreakIndex < NotDerivableCoordinates.Num(); BreakIndex++)
		{
			double GlobalCoord = LocalToGlobalCoordinate(CurveIndex, NotDerivableCoordinates[BreakIndex]);
			if (GlobalCoord < InBoundary.Min) 
			{
				continue;
			}
			if (GlobalCoord > InBoundary.Max)
			{
				break;
			}
			OutNotDerivableCoordinates.Add(GlobalCoord);
		}

		if ((CurveIndex < Curves.Num() - 1) && (Coordinates[CurveIndex + 1] < InBoundary.Max))
		{
			OutNotDerivableCoordinates.Add(Coordinates[CurveIndex + 1]);
		}
	}
}

TSharedPtr<FEntityGeom> FCompositeCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TArray<TSharedPtr<FCurve>> TransformedCurves;

	for (const FOrientedCurve& Curve : Curves)
	{
		TSharedPtr<FCurve> TransformedCurve = StaticCastSharedPtr<FCurve>(Curve.Entity->ApplyMatrix(InMatrix));
		if (!TransformedCurve.IsValid())
		{
			return TSharedPtr<FEntityGeom>();
		}
		TransformedCurves.Add(TransformedCurve);
	}

	if (TransformedCurves.Num() == 0)
	{
		return TSharedPtr<FEntityGeom>();
	}

	return FEntity::MakeShared<FCompositeCurve>(TransformedCurves);
}

void FCompositeCurve::Offset(const FVector& OffsetDirection)
{
	for (FOrientedCurve& Curve : Curves)
	{
		Curve.Entity->Offset(OffsetDirection);
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FCompositeCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info).Add(TEXT("Nb curves"), (int32)Curves.Num())
		.Add(TEXT("curves"), (TArray<TOrientedEntity<FEntity>>&) Curves)
		.Add(TEXT("parameters"), Coordinates);
}
#endif


// =========================================================================================================================================================================================================
// =========================================================================================================================================================================================================
// =========================================================================================================================================================================================================
//
//
//                                                                            NOT YET REVIEWED
//
//
// =========================================================================================================================================================================================================
// =========================================================================================================================================================================================================
// =========================================================================================================================================================================================================

FCompositeCurve::FCompositeCurve(const TArray<TSharedPtr<FCurve>>& CurveList, bool bDoInversions)
{
	ensureCADKernel(false);

#ifdef TODOOOOO
	double GeometricalTolerance = FSystem::Get().GetParameters()->GeometricalTolerance;

	double Perimeter = 0.0;
	Coordinates.Add(Perimeter);

	FVector PreviousPoint1;
	FVector PreviousPoint2;

	for (int32 ICurve = 0; ICurve < CurveList.Num(); ICurve++)
	{
		TSharedPtr<FCurve> Curve = CurveList[ICurve];

		double UMin = Curve->GetUMin();
		double UMax = Curve->GetUMax();

		Perimeter += FMath::Abs(UMax - UMin);
		Coordinates.Add(Perimeter);

		UMins.Add(UMin);
		UMaxs.Add(UMax);

		if (bDoInversions)
		{
			if (ICurve > 0)
			{
				FVector Point1;
				FVector Point2;

				Curve->GetExtremities(&Point1, &Point2);
				if (Point1.Distance(PreviousPoint2) > Point2.Distance(PreviousPoint2))
				{
					Curve = Curve->InvertCurve(GeometricalTolerance);
					PreviousPoint2 = Point1;
				}
				else
				{
					PreviousPoint2 = Point2;
				}
			}
			else
			{
				Curve->GetExtremities(&PreviousPoint1, &PreviousPoint2);
			}
		}
		Curves.Add(Curve);
	}

	Boundary.Set(Coordinates[0], Coordinates[Coordinates.Num() - 1]);
#endif
}

}