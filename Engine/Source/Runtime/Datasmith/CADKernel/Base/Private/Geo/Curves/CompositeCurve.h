// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/OrientedEntity.h"
#include "Geo/Curves/Curve.h"

namespace UE::CADKernel
{
struct FCurvePoint;

class CADKERNEL_API FOrientedCurve : public TOrientedEntity<FCurve>
{
public:
	FOrientedCurve(TSharedPtr<FCurve>& InEntity, EOrientation InDirection)
		: TOrientedEntity(InEntity, InDirection)
	{
	}

	FOrientedCurve()
		: TOrientedEntity()
	{
	}
};

class CADKERNEL_API FCompositeCurve : public FCurve
{
	friend class FEntity;

protected:
	TArray<FOrientedCurve> Curves;
	TArray<double> Coordinates;

	FCompositeCurve(const TArray<TSharedPtr<FCurve>>& Curves, bool bDoInversions = false);

	FCompositeCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		SerializeIdents(Ar, (TArray<TOrientedEntity<FEntity>>&) Curves);
		Ar.Serialize(Coordinates);
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities((TArray<TOrientedEntity<FEntity>>&) Curves, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities((TArray<TOrientedEntity<FEntity>>&) Curves);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Composite;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual TSharedPtr<FCurve> GetCurve(int32 Index) const
	{
		return Curves[Index].Entity;
	}

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override;
	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override;

	virtual void FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const override;

	double LocalToGlobalCoordinate(int32 CurveIndex, double LocalCoordinate) const;
	double GlobalToLocalCoordinate(int32 CurveIndex, double GlobalCoordinate) const;
};

} // namespace UE::CADKernel

