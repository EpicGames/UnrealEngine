// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/Curve.h"

#include "Geo/Curves/BoundedCurve.h"
#include "Geo/Sampling/Polyline.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Geo/Sampling/SurfacicPolyline.h"
#include "Geo/Sampler/SamplerOnChord.h"
#include "Geo/Sampler/SamplerOnParam.h"

namespace UE::CADKernel
{
double FCurve::GetLength(double Tolerance) const
{
	if (!GlobalLength.IsValid())
	{
		switch (Dimension)
		{
		case 3:
			GlobalLength = ComputeLength(Boundary, Tolerance);
			break;
		case 2:
			GlobalLength = ComputeLength2D(Boundary, Tolerance);
			break;
		default:
			GlobalLength = 0;
			break;
		}
	}
	return GlobalLength;
}

void FCurve::EvaluatePoints(const TArray<double>& Coordinates, TArray<FCurvePoint>& OutPoints, int32 DerivativeOrder) const
{
	OutPoints.SetNum(Coordinates.Num());
	for (int32 iPoint = 0; iPoint < Coordinates.Num(); iPoint++)
	{
		EvaluatePoint(Coordinates[iPoint], OutPoints[iPoint], DerivativeOrder);
	}
}

void FCurve::EvaluatePoints(const TArray<double>& Coordinates, TArray<FVector>& OutPoints) const
{
	OutPoints.Empty(Coordinates.Num());
	for (double Coordinate : Coordinates)
	{
		FCurvePoint Point;
		EvaluatePoint(Coordinate, Point, 0);
		OutPoints.Emplace(Point.Point);
	}
}

void FCurve::Evaluate2DPoints(const TArray<double>& Coordinates, TArray<FCurvePoint2D>& OutPoints, int32 DerivativeOrder) const
{
	OutPoints.SetNum(Coordinates.Num());
	for (int32 iPoint = 0; iPoint < Coordinates.Num(); iPoint++)
	{
		Evaluate2DPoint(Coordinates[iPoint], OutPoints[iPoint], DerivativeOrder);
	}
}

/**
 * Evaluate exact 2D points of the curve at the input Coordinates
 * The function can only be used with 2D curve (Dimension == 2)
 */
void FCurve::Evaluate2DPoints(const TArray<double>& Coordinates, TArray<FVector2d>& OutPoints) const
{
	OutPoints.SetNum(Coordinates.Num());
	for (int32 iPoint = 0; iPoint < Coordinates.Num(); iPoint++)
	{
		Evaluate2DPoint(Coordinates[iPoint], OutPoints[iPoint]);
	}
}

void FCurve::FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const
{
}

TSharedPtr<FCurve> FCurve::Rebound(const FLinearBoundary& InBoundary)
{
	if (FMath::IsNearlyEqual(InBoundary.Min, GetUMin()) && FMath::IsNearlyEqual(InBoundary.Max, GetUMax()))
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Invalid rebound (UMin and UMax are nearly equal) on curve %d\n", GetId());
		TSharedPtr<FEntity> Entity = AsShared();
		return StaticCastSharedPtr<FCurve>(Entity);
	}

	ensureCADKernel(false);
	return TSharedPtr<FCurve>();
}

TSharedPtr<FCurve> FCurve::MakeBoundedCurve(const FLinearBoundary& InBoundary)
{
	FLinearBoundary NewBoundary = InBoundary;
	if (NewBoundary.Min < GetUMin())
	{
		NewBoundary.Min = GetUMin();
	}

	if (NewBoundary.Max > GetUMax())
	{
		NewBoundary.Max = GetUMax();
	}

	if (NewBoundary.IsDegenerated())
	{
		UE_LOGF(LogCADKernelBase, Warning, "Invalid bounds (u1=%f u2=%f) on curve %d\n", NewBoundary.Min, NewBoundary.Max, GetId());
		return TSharedPtr<FCurve>();
	}

	if (FMath::IsNearlyEqual(NewBoundary.Min, GetUMin()) && FMath::IsNearlyEqual(NewBoundary.Max, GetUMax()))
	{
		UE_LOGF(LogCADKernelBase, Verbose, "Invalid rebound (UMin and UMax are nearly equal) on curve %d\n", GetId());
		return TSharedPtr<FCurve>();
	}

	return FEntity::MakeShared<FBoundedCurve>(StaticCastSharedRef<FCurve>(AsShared()), NewBoundary, Dimension);
}

double FCurve::ComputeLength(const FLinearBoundary& InBoundary, double Tolerance) const
{
	FPolyline3D Polyline;
	FCurveSamplerOnChord Sampler(*this, Boundary, Tolerance, Polyline);
	Sampler.Sample();
	return Polyline.GetLength(Boundary);
}

double FCurve::ComputeLength2D(const FLinearBoundary& InBoundary, double Tolerance) const
{
	FPolyline2D Polyline;
	FCurve2DSamplerOnChord Sampler(*this, Boundary, Tolerance, Polyline);
	Sampler.Sample();
	return Polyline.GetLength(Boundary);
}

void FCurve::Presample(const FLinearBoundary& InBoundary, double Tolerance, TArray<double>& OutSampling) const
{
	FPolyline3D Presampling;
	FCurveSamplerOnParam Sampler(*this, Boundary, Tolerance * 10., Tolerance, Presampling);
	Sampler.Sample();

	Presampling.SwapCoordinates(OutSampling);
}

} // namespace UE::CADKernel
