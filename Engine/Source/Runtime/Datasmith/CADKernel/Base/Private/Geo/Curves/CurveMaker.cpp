// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/BezierCurve.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Curves/NURBSCurve.h"
#include "Geo/Curves/PolylineCurve.h"
#include "Geo/Curves/SplineCurve.h"

namespace UE::CADKernel
{

TSharedPtr<FCurve> FCurve::MakeNurbsCurve(FNurbsCurveData& InNurbsData)
{
	if (InNurbsData.Degree == 1)
	{
		ensureCADKernel(InNurbsData.Dimension > 1);
		TArray<double> Coordinates(InNurbsData.NodalVector.GetData() + 1, InNurbsData.NodalVector.Num() - 2);
		ensureCADKernel(Coordinates.Num() > 1);
		
		// Make sure Coordinates contains rising or equal values
		if (Coordinates.Num() == 2)
		{
			if (!ensureCADKernel(Coordinates[0] < Coordinates[1]))
			{
				Coordinates[1] = Coordinates[0];
			}
		}
		else
		{
			double LastValue = Coordinates[0];
			for (int32 Index = 1; Index < Coordinates.Num(); ++Index)
			{
				if (Coordinates[Index] < LastValue)
				{
					Coordinates[Index] = LastValue;
				}
				else
				{
					LastValue = Coordinates[Index];
				}
			}
		}

		if (InNurbsData.Dimension == 2)
		{
			TArray<FVector2d> NewPoles;
			NewPoles.Reserve(InNurbsData.Poles.Num());

			if (InNurbsData.bIsRational && InNurbsData.Poles.Num() == InNurbsData.Weights.Num())
			{
				for (int32 Index = 0; Index < InNurbsData.Poles.Num(); ++Index)
				{
					NewPoles.Emplace(InNurbsData.Poles[Index].X, InNurbsData.Poles[Index].Y);
					NewPoles.Last() /= InNurbsData.Weights[Index];
				}
			}
			else
			{
				for (const FVector& Pole : InNurbsData.Poles)
				{
					NewPoles.Emplace(Pole.X, Pole.Y);
				}
			}

			return FEntity::MakeShared<FPolyline2DCurve>(NewPoles, Coordinates);
		}

		if (InNurbsData.bIsRational && InNurbsData.Poles.Num() == InNurbsData.Weights.Num())
		{
			TArray<FVector> NewPoles(InNurbsData.Poles);
			for (int32 Index = 0; Index < NewPoles.Num(); ++Index)
			{
				NewPoles[Index] /= InNurbsData.Weights[Index];
			}

			return FEntity::MakeShared<FPolylineCurve>(NewPoles, Coordinates);
		}

		return FEntity::MakeShared<FPolylineCurve>(InNurbsData.Poles, Coordinates);
	}

	return FEntity::MakeShared<FNURBSCurve>(InNurbsData);
}

TSharedPtr<FCurve> FCurve::MakeBezierCurve(const TArray<FVector>& InPoles)
{
	return FEntity::MakeShared<UE::CADKernel::FBezierCurve>(InPoles);
}

TSharedPtr<FCurve> FCurve::MakeSplineCurve(const TArray<FVector>& InPoles)
{
	return FEntity::MakeShared<UE::CADKernel::FSplineCurve>(InPoles);
}

TSharedPtr<FCurve> FCurve::MakeSplineCurve(const TArray<FVector>& InPoles, const TArray<FVector>& InTangents)
{
	return FEntity::MakeShared<UE::CADKernel::FSplineCurve>(InPoles, InTangents);
}

TSharedPtr<FCurve> FCurve::MakeSplineCurve(const TArray<FVector>& InPoles, const TArray<FVector>& InArriveTangents, const TArray<FVector>& InLeaveTangents)
{
	return FEntity::MakeShared<UE::CADKernel::FSplineCurve>(InPoles, InArriveTangents, InLeaveTangents);
}

}