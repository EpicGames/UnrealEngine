// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/BoundedCurve.h"
#include "Geo/Curves/BezierCurve.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Curves/NURBSCurve.h"
#include "Geo/Curves/SegmentCurve.h"
#include "Geo/Curves/EllipseCurve.h"
#include "Geo/Curves/HyperbolaCurve.h"
#include "Geo/Curves/ParabolaCurve.h"
#include "Geo/Curves/CompositeCurve.h"
#include "Geo/Curves/PolylineCurve.h"
#include "Geo/Curves/RestrictionCurve.h"
#include "Geo/Curves/SurfacicCurve.h"

namespace UE::CADKernel
{

TSharedPtr<FEntity> FCurve::Deserialize(FCADKernelArchive& Archive)
{
	ECurve CurveType = ECurve::None;
	Archive << CurveType;

	switch (CurveType)
	{
	case ECurve::Bezier:
		return FEntity::MakeShared<FBezierCurve>(Archive);
	case ECurve::Segment:
		return FEntity::MakeShared<FSegmentCurve>(Archive);
	case ECurve::Nurbs:
		return FEntity::MakeShared<FNURBSCurve>(Archive);
	case ECurve::Composite:
		return FEntity::MakeShared<FCompositeCurve>(Archive);
	case ECurve::BoundedCurve:
		return FEntity::MakeShared<FBoundedCurve>(Archive);
	case ECurve::Ellipse:
		return FEntity::MakeShared<FEllipseCurve>(Archive);
	case ECurve::Hyperbola:
		return FEntity::MakeShared<FHyperbolaCurve>(Archive);
	case ECurve::Parabola:
		return FEntity::MakeShared<FParabolaCurve>(Archive);
	case ECurve::Polyline3D:
		return FEntity::MakeShared<FPolylineCurve>(Archive);
	case ECurve::Polyline2D:
		return FEntity::MakeShared<FPolyline2DCurve>(Archive);
	case ECurve::Restriction:
		return FEntity::MakeShared<FRestrictionCurve>(Archive);
	case ECurve::Surfacic:
		return FEntity::MakeShared<FSurfacicCurve>(Archive);
	default:
		return TSharedPtr<FEntity>();
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FPolyline2DCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info)
		.Add(TEXT("nbPoints"), this->Points.Num())
		.Add(TEXT("points"), this->Points)
		.Add(TEXT("params"), this->Coordinates);
}

FInfoEntity& FPolylineCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info)
		.Add(TEXT("nbPoints"), this->Points.Num())
		.Add(TEXT("points"), this->Points)
		.Add(TEXT("params"), this->Coordinates);
}
#endif

}