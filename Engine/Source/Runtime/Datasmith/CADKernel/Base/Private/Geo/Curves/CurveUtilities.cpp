// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/CurveUtilities.h"

#include "Geo/Curves/BezierCurve.h"
#include "Geo/Curves/CompositeCurve.h"
#include "Geo/Curves/NURBSCurve.h"
#include "Geo/Curves/PolylineCurve.h"
#include "Geo/Curves/SegmentCurve.h"
#include "Geo/GeoEnum.h"
#include "Geo/Sampling/SurfacicPolyline.h"

namespace UE::CADKernel::CurveUtilities
{
	template<class T>
	bool GetCurvePoles(const T& Curve, TArray<FVector>& PolesOut)
	{
		const TArray<FVector>& Poles = Curve.GetPoles();
		const int32 PoleCount = Poles.Num();

		PolesOut.Reserve(PoleCount);

		if (Curve.GetDimension() == 2)
		{
			for (int32 Index = 0; Index < PoleCount; ++Index)
			{
				const FVector& Pole = Poles[Index];
				PolesOut.Add(FVector(Pole.X, Pole.Y, 0.f));
			}
		}
		else
		{
			for (int32 Index = 0; Index < PoleCount; ++Index)
			{
				const FVector& Pole = Poles[Index];
				PolesOut.Add(FVector(Pole.X, Pole.Y, Pole.Z));
			}
		}

		return PolesOut.Num() > 0;
	}

	template<class PointType, class PointCurveType>
	bool GetPolylinePoints(const TPolylineCurve<PointType, PointCurveType>& Curve, TArray<FVector>& Poles)
	{
		const TArray<PointType>& Points = Curve.GetPolylinePoints();
		const int32 PointCount = Points.Num();

		Poles.Reserve(PointCount);

		if (Curve.GetDimension() == 2)
		{
			for (int32 Index = 0; Index < PointCount; ++Index)
			{
				const PointType Point = Points[Index];
				Poles.Add(FVector(Point[0], Point[1], 0.f));
			}
		}
		else
		{
			for (int32 Index = 0; Index < PointCount; ++Index)
			{
				const PointType Point = Points[Index];
				Poles.Add(FVector(Point[0], Point[1], Point[3]));
			}
		}

		return Poles.Num() > 0;
	}

	TArray<FVector> GetPoles(const UE::CADKernel::FCurve& Curve)
	{
		using namespace UE::CADKernel;

		TArray<FVector> Poles;

		switch (Curve.GetCurveType())
		{
		case ECurve::Bezier:
		{
			GetCurvePoles(static_cast<const FBezierCurve&>(Curve), Poles);
		}
		break;

		case ECurve::Nurbs:
		{
			GetCurvePoles(static_cast<const FNURBSCurve&>(Curve), Poles);
		}
		break;

		case ECurve::Restriction:
		{
			return GetPoles(*static_cast<const FRestrictionCurve&>(Curve).Get2DCurve());
		}

		case ECurve::Segment:
		{
			const FSegmentCurve& Segment = static_cast<const FSegmentCurve&>(Curve);
			Poles.Reserve(2);
			{
				const FVector& Point = Segment.GetStartPoint();
				Poles.Emplace(Point.X, Point.Y, Point.Z);
			}
			{
				const FVector& Point = Segment.GetEndPoint();
				Poles.Emplace(Point.X, Point.Y, Point.Z);
			}
		}
		break;

		case ECurve::Polyline3D:
		{
			GetPolylinePoints<FVector, FCurvePoint>(static_cast<const TPolylineCurve<FVector, FCurvePoint>&>(Curve), Poles);
		}

		case ECurve::Polyline2D:
		{
			GetPolylinePoints<FVector2d, FCurvePoint2D>(static_cast<const TPolylineCurve<FVector2d, FCurvePoint2D>&>(Curve), Poles);
		}
		break;

		default:
			ensureCADKernel(false);
			break;
		}

		return Poles;
	}

	TArray<FVector2d> GetPoles(const UE::CADKernel::FRestrictionCurve& Curve)
	{
		TArray<FVector> Poles = GetPoles(*Curve.Get2DCurve());

		TArray<FVector2d> SurfacicPoints;
		SurfacicPoints.Reserve(Poles.Num());

		for (const FVector& Pole : Poles)
		{
			SurfacicPoints.Emplace(Pole.X, Pole.Y);
		}

		return SurfacicPoints;
	}

	TArray<FVector2d> Get2DPolyline(const UE::CADKernel::FRestrictionCurve& Curve)
	{
		using namespace UE::CADKernel;

		const FSurfacicPolyline& Polyline = Curve.GetPolyline();

		const TArray<FVector2d>& Point2Ds = Polyline.Get2DPoints();

		TArray<FVector2d> SurfacicPoints;
		SurfacicPoints.Reserve(Point2Ds.Num());

		for (const FVector2d& Point2D : Point2Ds)
		{
			SurfacicPoints.Emplace(Point2D.X, Point2D.Y);
		}

		return SurfacicPoints;
	}

	TArray<FVector> Get3DPolyline(const UE::CADKernel::FRestrictionCurve& Curve)
	{
		using namespace UE::CADKernel;

		const FSurfacicPolyline& Polyline = Curve.GetPolyline();

		const TArray<FVector>& Point3Ds = Polyline.GetPoints();

		TArray<FVector> Points;
		Points.Reserve(Point3Ds.Num());

		for (const FVector& Point : Point3Ds)
		{
			Points.Emplace(Point.X, Point.Y, Point.Z);
		}

		return Points;
	}

	int32 GetPoleCount(const UE::CADKernel::FCurve& Curve)
	{
		using namespace UE::CADKernel;

		switch (Curve.GetCurveType())
		{
		case ECurve::Bezier:
		{
			return static_cast<const FBezierCurve&>(Curve).GetPoles().Num();
		}

		case ECurve::Nurbs:
		{
			return static_cast<const FNURBSCurve&>(Curve).GetPoles().Num();
		}

		case ECurve::Restriction:
		{
			return GetPoleCount(*static_cast<const FRestrictionCurve&>(Curve).Get2DCurve());
		}

		case ECurve::Segment:
		{
			return 2;
		}

		case ECurve::Polyline3D:
		{
			return static_cast<const TPolylineCurve<FVector, FCurvePoint>&>(Curve).GetPolylinePoints().Num();
		}

		case ECurve::Polyline2D:
		{
			return static_cast<const TPolylineCurve<FVector2d, FCurvePoint2D>&>(Curve).GetPolylinePoints().Num();
		}

		default:
			ensureCADKernel(false);
			break;
		}

		return 0;
	}

	int32 GetPoleCount(const UE::CADKernel::FRestrictionCurve& Curve)
	{
		return GetPoleCount(*Curve.Get2DCurve());
	}

	int32 GetDegree(const UE::CADKernel::FCurve& Curve)
	{
		using namespace UE::CADKernel;

		TArray<FVector> Poles;

		switch (Curve.GetCurveType())
		{
		case ECurve::Bezier:
		{
			return static_cast<const FBezierCurve&>(Curve).GetDegre();
		}

		case ECurve::Nurbs:
		{
			return static_cast<const FNURBSCurve&>(Curve).GetDegree();
		}

		case ECurve::Restriction:
		{
			return GetDegree(*static_cast<const FRestrictionCurve&>(Curve).Get2DCurve());
		}

		case ECurve::Polyline2D:
		case ECurve::Polyline3D:
		case ECurve::Segment:
		{
			return 1;
		}

		default:
			ensureCADKernel(false);
			break;
		}

		return -1;
	}

	int32 GetDegree(const UE::CADKernel::FRestrictionCurve& Curve)
	{
		return GetDegree(*Curve.Get2DCurve());
	}
}
