// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/EllipseCurve.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FEllipseCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FEllipseCurve>(NewMatrix, RadiusU, RadiusV, Boundary);
}

void FEllipseCurve::Offset(const FVector& OffsetDirection)
{
	FMatrixH Offset = FMatrixH::MakeTranslationMatrix(OffsetDirection);
	Matrix *= Offset;
}

#ifdef CADKERNEL_DEV
FInfoEntity& FEllipseCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info).Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("radius U"), RadiusU)
		.Add(TEXT("radius V"), RadiusV);
}
#endif

}