// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"
#include "Geo/GeoPoint.h"
#include "Math/MathConst.h"

namespace UE::CADKernel
{

class CADKERNEL_API FEllipseCurve : public FCurve
{
	friend class FEntity;

protected:
	FMatrixH Matrix;
	double   RadiusU;
	double   RadiusV;

	FEllipseCurve(const FMatrixH& InMatrix, double InRadiusU, double InRadiusV, int8 InDimension = 3)
		: FCurve(FLinearBoundary(0, PI * 2.), InDimension)
		, Matrix(InMatrix)
		, RadiusU(InRadiusU)
		, RadiusV(InRadiusV)
	{
	}

	FEllipseCurve(const FMatrixH& InMatrix, double InRadiusU, double InRadiusV, const FLinearBoundary& InBounds, int8 InDimension = 3)
		: FCurve(InBounds, InDimension)
		, Matrix(InMatrix)
		, RadiusU(InRadiusU)
		, RadiusV(InRadiusV)
	{
	}

	FEllipseCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		Ar << Matrix;
		Ar << RadiusU;
		Ar << RadiusV;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Ellipse;
	}

	const FMatrixH& GetMatrix() const
	{
		return Matrix;
	}

	bool IsCircular() const
	{
		return FMath::IsNearlyZero(RadiusU - RadiusV);
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 3);

		OutPoint.DerivativeOrder = DerivativeOrder;

		const double CosU = cos(Coordinate);
		const double SinU = sin(Coordinate);

		OutPoint.Point = Matrix.Multiply(FVector(RadiusU * CosU, RadiusV * SinU, 0.));

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = Matrix.MultiplyVector(FVector(RadiusU * (-SinU), RadiusV * CosU, 0.));

			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian = Matrix.MultiplyVector(FVector(RadiusU * (-CosU), RadiusV * (-SinU), 0.));
			}
		}
	}

	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 2);

		OutPoint.DerivativeOrder = DerivativeOrder;

		const double CosU = cos(Coordinate);
		const double SinU = sin(Coordinate);

		OutPoint.Point = Matrix.Multiply2D(FVector2d(RadiusU * CosU, RadiusV * SinU));

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = Matrix.MultiplyVector2D(FVector2d(RadiusU * (-SinU), RadiusV * CosU));

			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian = Matrix.MultiplyVector2D(FVector2d(RadiusU * (-CosU), RadiusV * (-SinU)));
			}
		}
	}
};

} // namespace UE::CADKernel

