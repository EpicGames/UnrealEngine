// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/HyperbolaCurve.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FHyperbolaCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FHyperbolaCurve>(NewMatrix, SemiMajorAxis, SemiImaginaryAxis, Boundary);
}

void FHyperbolaCurve::Offset(const FVector& OffsetDirection)
{
	FMatrixH Offset = FMatrixH::MakeTranslationMatrix(OffsetDirection);
	Matrix *= Offset;
}

#ifdef CADKERNEL_DEV
FInfoEntity& FHyperbolaCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info).Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("semi axis"), SemiMajorAxis)
		.Add(TEXT("semi imag axis"), SemiImaginaryAxis);
}
#endif

}
