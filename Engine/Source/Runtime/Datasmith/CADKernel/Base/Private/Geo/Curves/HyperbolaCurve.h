// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"

namespace UE::CADKernel
{

class CADKERNEL_API FHyperbolaCurve : public FCurve
{
	friend class FIGESEntity104;
	friend class FEntity;
	friend class FEntity;

protected:
	FMatrixH Matrix;
	double SemiMajorAxis;
	double SemiImaginaryAxis;

	FHyperbolaCurve(const FMatrixH& InMatrix, double InSemiAxis, double InSemiImagAxis, const FLinearBoundary& InBounds, int8 InDimension = 3)
		: FCurve(InBounds, InDimension)
		, Matrix(InMatrix)
		, SemiMajorAxis(InSemiAxis)
		, SemiImaginaryAxis(InSemiImagAxis)
	{
	}

	FHyperbolaCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		Ar << Matrix;
		Ar << SemiMajorAxis;
		Ar << SemiImaginaryAxis;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Hyperbola;
	}

	FMatrixH& GetMatrix()
	{
		return Matrix;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;


	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 3);

		OutPoint.DerivativeOrder = DerivativeOrder;

		const double CosHUCoord = cosh(Coordinate);
		const double SinHUCoord = sinh(Coordinate);

		OutPoint.Point = Matrix.Multiply(FVector(SemiMajorAxis * CosHUCoord, SemiImaginaryAxis * SinHUCoord, 0.));

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = Matrix.MultiplyVector(FVector(SemiMajorAxis * SinHUCoord, SemiImaginaryAxis * CosHUCoord, 0.));

			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian = Matrix.MultiplyVector(FVector(SemiMajorAxis * CosHUCoord, SemiImaginaryAxis * SinHUCoord, 0.));
			}
		}
	}

	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 2);

		OutPoint.DerivativeOrder = DerivativeOrder;

		const double CosHUCoord = cosh(Coordinate);
		const double SinHUCoord = sinh(Coordinate);

		OutPoint.Point = Matrix.Multiply2D(FVector2d(SemiMajorAxis * CosHUCoord, SemiImaginaryAxis * SinHUCoord));

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = Matrix.MultiplyVector2D(FVector2d(SemiMajorAxis * SinHUCoord, SemiImaginaryAxis * CosHUCoord));

			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian = Matrix.MultiplyVector2D(FVector2d(SemiMajorAxis * CosHUCoord, SemiImaginaryAxis * SinHUCoord));
			}
		}
	}
};

} // namespace UE::CADKernel

