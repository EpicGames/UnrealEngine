// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/NURBSCurve.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Math/BSpline.h"
#include "Utils/Util.h"

#include "Algo/ForEach.h"
#include "Algo/Reverse.h"

namespace UE::CADKernel
{ 

FNURBSCurve::FNURBSCurve(int32 InDegre, const TArray<double>& InNodalVector, const TArray<FVector>& InPoles, const TArray<double>& InWeights, int8 InDimension)
	: FCurve(InDimension)
	, Degree(InDegre)
	, NodalVector(InNodalVector)
	, Weights(InWeights)
	, Poles(InPoles)
	, bIsRational(!InWeights.IsEmpty())
{
	Finalize();
}

FNURBSCurve::FNURBSCurve(FNurbsCurveData& NurbsCurveData)
	: FCurve(NurbsCurveData.Dimension)
	, Degree(NurbsCurveData.Degree)
	, NodalVector(NurbsCurveData.NodalVector)
	, Weights(NurbsCurveData.Weights)
	, Poles(NurbsCurveData.Poles)
	, bIsRational(NurbsCurveData.bIsRational)
{
	Finalize();
}

TSharedPtr<FEntityGeom> FNURBSCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TArray<FVector> TransformedPoles;

	TransformedPoles.Reserve(Poles.Num());
	for (const FVector& Pole : Poles) 
	{
		TransformedPoles.Emplace(InMatrix.Multiply(Pole));
	}

	return FEntity::MakeShared<FNURBSCurve>(Degree, NodalVector, TransformedPoles, Weights, Dimension);
}

void FNURBSCurve::Offset(const FVector& OffsetDirection)
{
	for (FVector& Pole : Poles)
	{
		Pole += OffsetDirection;
	}
	Finalize();
}

#ifdef CADKERNEL_DEV
FInfoEntity& FNURBSCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info).Add(TEXT("Degre"), Degree)
		.Add(TEXT("Nodal vector"), NodalVector)
		.Add(TEXT("Poles"), Poles)
		.Add(TEXT("Weights"), Weights);
}
#endif

void FNURBSCurve::Finalize()
{
	// Check if the curve is really rational, otherwise remove the weights
	if (bIsRational)
	{
		const double FirstWeigth = Weights[0];

		bool bIsReallyRational = false;
		for (const double& Weight : Weights)
		{
			if (!FMath::IsNearlyEqual(Weight, FirstWeigth))
			{
				bIsReallyRational = true;
				break;
			}
		}

		if (!bIsReallyRational)
		{
			if (!FMath::IsNearlyEqual(1., FirstWeigth))
			{
				for (FVector& Pole : Poles)
				{
					Pole /= FirstWeigth;
				}
			}
			bIsRational = false;
		}
	}

	PoleDimension = Dimension + (bIsRational ? 1 : 0);
	HomogeneousPoles.SetNum(Poles.Num() * PoleDimension);

	if (bIsRational)
	{
		if (Dimension == 2)
		{
			for (int32 Index = 0, Jndex = 0; Index < Poles.Num(); Index++)
			{
				HomogeneousPoles[Jndex++] = Poles[Index].X * Weights[Index];
				HomogeneousPoles[Jndex++] = Poles[Index].Y * Weights[Index];
				HomogeneousPoles[Jndex++] = Weights[Index];
			}
		}
		else
		{
			for (int32 Index = 0, Jndex = 0; Index < Poles.Num(); Index++)
			{
				HomogeneousPoles[Jndex++] = Poles[Index].X * Weights[Index];
				HomogeneousPoles[Jndex++] = Poles[Index].Y * Weights[Index];
				HomogeneousPoles[Jndex++] = Poles[Index].Z * Weights[Index];
				HomogeneousPoles[Jndex++] = Weights[Index];
			}
		}
	}
	else
	{
		int32 Jndex = 0;
		if (Dimension == 2)
		{
			for (FVector& Pole : Poles)
			{
				HomogeneousPoles[Jndex++] = Pole.X;
				HomogeneousPoles[Jndex++] = Pole.Y;
			}
		}
		else
		{
			memcpy(HomogeneousPoles.GetData(), Poles.GetData(), sizeof(FVector) * Poles.Num());
		}
	}

	Boundary.Set(NodalVector[Degree], NodalVector[NodalVector.Num() - 1 - Degree]);
}

void FNURBSCurve::ExtendTo(const FVector& Point)
{
	PolylineTools::ExtendTo(Poles, Point);
	Finalize();
}


void FNURBSCurve::SetStartNodalCoordinate(double NewStartBoundary)
{
	double Offset = NewStartBoundary - NodalVector[0];
	for (double& NodalValue : NodalVector)
	{
		NodalValue += Offset;
	}

	Boundary.Set(NodalVector[Degree], NodalVector[NodalVector.Num() - 1 - Degree]);
}

void FNURBSCurve::Invert()
{
	{
		TArray<FVector> NewPoles;
		NewPoles.Reserve(Poles.Num());
		for (int32 Index = Poles.Num() - 1; Index >= 0; --Index)
		{
			NewPoles.Emplace(Poles[Index]);
		}
		Swap(Poles, NewPoles);
	}

	{
		TArray<double> NewNodalVector;
		NewNodalVector.Reserve(NodalVector.Num());
		double LastNodalValue = NodalVector.Last();
		for (int32 Index = NodalVector.Num() - 1; Index >= 0; --Index)
		{
			NewNodalVector.Emplace(LastNodalValue - NodalVector[Index]);
		}
		Swap(NodalVector, NewNodalVector);

	}

	if (IsRational())
	{
		TArray<double> NewWeights;
		NewWeights.Reserve(Weights.Num());
		for (int32 Index = Weights.Num() - 1; Index >= 0; --Index)
		{
			NewWeights.Emplace(Weights[Index]);
		}
		Swap(Weights, NewWeights);
	}

	Finalize();
}

} // namespace UE::CADKernel