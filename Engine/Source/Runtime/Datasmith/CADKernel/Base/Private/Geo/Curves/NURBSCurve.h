// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"
#include "Math/BSpline.h"

namespace UE::CADKernel
{
struct FNurbsCurveData
{
	bool bIsRational = false;
	int32 Dimension = 0;

	int32 Degree = 0;
	TArray<double> NodalVector;

	TArray<double> Weights;
	TArray<FVector> Poles;
};

class CADKERNEL_API FNURBSCurve : public FCurve
{
	friend class FEntity;

protected:

	int32 Degree;

	TArray<double> NodalVector;
	TArray<double> Weights;

	TArray<FVector> Poles;

	bool bIsRational;

	/**
	 * Data generated at initialization which are not serialized
	 */
	TArray<double> HomogeneousPoles;
	int32 PoleDimension;

	FNURBSCurve(int32 InDegree, const TArray<double>& InNodalVector, const TArray<FVector>& InPoles, const TArray<double>& InWeights, int8 InDimension = 3);
	FNURBSCurve(FNurbsCurveData& NurbsCurveData);

	FNURBSCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		Ar << Degree;
		Ar.Serialize(NodalVector);
		Ar.Serialize(Weights);
		Ar.Serialize(Poles);
		Ar << bIsRational;

		if (Ar.IsLoading())
		{
			Finalize();
		}
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Nurbs;
	}

	int32 GetDegree() const
	{
		return Degree;
	}

	const int32 GetPoleCount() const
	{
		return Poles.Num();
	}

	const TArray<FVector>& GetPoles() const
	{
		return Poles;
	}

	const TArray<double>& GetWeights() const
	{
		return Weights;
	}

	const TArray<double>& GetHPoles() const
	{
		return HomogeneousPoles;
	}

	const TArray<double>& GetNodalVector() const
	{
		return NodalVector;
	}

	bool IsRational() const
	{
		return bIsRational;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override
	{
		BSpline::EvaluatePoint(*this, Coordinate, OutPoint, DerivativeOrder);
	}

	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		BSpline::Evaluate2DPoint(*this, Coordinate, OutPoint, DerivativeOrder);
	}

	virtual void FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const override
	{
		BSpline::FindNotDerivableParameters(*this, DerivativeOrder, InBoundary, OutNotDerivableCoordinates);
	}

	virtual void ExtendTo(const FVector& DesiredPosition) override;

	void Invert();
	void SetStartNodalCoordinate(double NewStartBoundary);

private:

	/**
	 * Fill homogeneous points and set bounds
	 */
	void Finalize();

};

} // namespace UE::CADKernel

