// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/ParabolaCurve.h"

#include "Math/Point.h"
#include "Math/MatrixH.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FParabolaCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FParabolaCurve>(NewMatrix, FocalDistance, Boundary, Dimension);
}

void FParabolaCurve::Offset(const FVector& OffsetDirection)
{
	FMatrixH Offset = FMatrixH::MakeTranslationMatrix(OffsetDirection);
	Matrix *= Offset;
}


#ifdef CADKERNEL_DEV
FInfoEntity& FParabolaCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info)
		.Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("focal dist"), FocalDistance);
}
#endif

}