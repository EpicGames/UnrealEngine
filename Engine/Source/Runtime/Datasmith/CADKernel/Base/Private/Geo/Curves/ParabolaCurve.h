// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"

namespace UE::CADKernel
{

class CADKERNEL_API FParabolaCurve : public FCurve
{
	friend class FIGESEntity104;
	friend class FEntity;

protected:
	FMatrixH Matrix;
	double FocalDistance;

	FParabolaCurve(const FMatrixH& InMatrix, double InFocalDistance, const FLinearBoundary& InBounds, int8 InDimension = 3)
		: FCurve(InBounds, InDimension)
		, Matrix(InMatrix)
		, FocalDistance(InFocalDistance)
	{
	}

	FParabolaCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		Ar << Matrix;
		Ar << FocalDistance;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Parabola;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void ExtendTo(const FVector& Point) override
	{
		ensureCADKernel(false);
	}

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 3);

		OutPoint.DerivativeOrder = DerivativeOrder;

		OutPoint.Point = Matrix.Multiply(FVector(Coordinate * Coordinate * FocalDistance, Coordinate, 0.));

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = Matrix.MultiplyVector(FVector(2.0 * Coordinate * FocalDistance, 1., 0.));

			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian = Matrix.MultiplyVector(FVector(2.0 * FocalDistance, 0., 0.));
			}
		}
	}

	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 2);

		OutPoint.DerivativeOrder = DerivativeOrder;

		OutPoint.Point = Matrix.Multiply2D(FVector2d(Coordinate * Coordinate * FocalDistance, Coordinate));

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = Matrix.MultiplyVector2D(FVector2d(2.0 * Coordinate * FocalDistance, 1.));

			if (DerivativeOrder > 1)
			{
				OutPoint.Laplacian = Matrix.MultiplyVector2D(FVector2d(2.0 * FocalDistance, 0.));
			}
		}
	}
};

} // namespace UE::CADKernel

