// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"
#include "Geo/Sampling/Polyline.h"
#include "Geo/Sampling/PolylineTools.h"

namespace UE::CADKernel
{
class FInfoEntity;
class FSurface;

template<class PointType, class PointCurveType>
class CADKERNEL_API TPolylineCurve : public FCurve, public TPolyline<PointType>
{
	friend class FEntity;
	friend class FPolylineTools;

protected:

	TPolylineApproximator<PointType> Approximator;

	TPolylineCurve(const TArray<PointType>& InPoints, const TArray<double>& InCoordinates, int8 InDimension)
		: FCurve(InDimension)
		, Approximator(this->Coordinates, this->Points)
	{
		this->Coordinates = InCoordinates;
		this->Points = InPoints;
		ensureCADKernel(this->Coordinates[0] < this->Coordinates.Last());
		Boundary.Set(this->Coordinates.HeapTop(), this->Coordinates.Last());
	}

	TPolylineCurve(const TArray<PointType>& InPoints, int8 InDimension)
		: FCurve(InDimension)
		, TPolyline<PointType>(InPoints)
		, Approximator(this->Coordinates, this->Points)
	{
		this->Coordinates.Reserve(InPoints.Num());
		this->Coordinates.Add(0.);

		double CurvilineLength = 0;
		for (int32 iPoint = 1; iPoint < this->Points.Num(); iPoint++)
		{
			CurvilineLength += PointType::Distance(this->Points[iPoint], this->Points[iPoint - 1]);
			this->Coordinates.Add(CurvilineLength);
		}

		Boundary.Set(0, CurvilineLength);
	}

	TPolylineCurve()
		: Approximator(this->Coordinates, this->Points)
	{
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		TPolyline<PointType>::Serialize(Ar);
	}

	virtual void EvaluateCurvesPoint(double InCoordinate, PointCurveType& OutPoint, int32 InDerivativeOrder = 0) const
	{
		Approximator.ApproximatePoint(InCoordinate, OutPoint, InDerivativeOrder);
	}

	PointType EvaluatePointAt(double InCoordinate) const
	{
		return Approximator.ApproximatePoint(InCoordinate);
	}

	virtual void EvaluateCurvesPoints(const TArray<double>& InCoordinates, TArray<PointCurveType>& OutPoints, int32 InDerivativeOrder = 0) const
	{
		Approximator.ApproximatePoints(InCoordinates, OutPoints, InDerivativeOrder);
	}

	virtual double ComputeSubLength(const FLinearBoundary& InBoundary) const
	{
		return Approximator.ComputeLengthOfSubPolyline(InBoundary);
	}

	const TArray<PointType>& GetPolylinePoints() const
	{
		return this->Points;
	}

	const TArray<double>& GetPolylineParameters() const
	{
		return this->Coordinates;
	}

	virtual void FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const override
	{
		const int32 CoordinateCount = this->Coordinates.Num();
		if (CoordinateCount > 2)
		{
			OutNotDerivableCoordinates.Reserve(CoordinateCount - 2);
			int32 Index = 1;
			for (; Index < CoordinateCount - 1 && this->Coordinates[Index] <= InBoundary.GetMin(); ++Index);

			for (; Index < CoordinateCount - 1 && this->Coordinates[Index] <= InBoundary.GetMax(); ++Index)
			{
				OutNotDerivableCoordinates.Emplace(this->Coordinates[Index]);
			}
		}
	}

	void SetPoints(const TArray<PointType>& InPoints)
	{
		this->Points = InPoints;
		GlobalLength.Empty();
	}

	template<class PolylineType>
	TSharedPtr<FEntityGeom> ApplyMatrixImpl(const FMatrixH& InMatrix) const
	{
		TArray<PointType> NewPoints;
		NewPoints.Reserve(this->Points.Num());

		for (PointType Point : this->Points)
		{
			NewPoints.Emplace(InMatrix.Multiply(Point));
		}

		return FEntity::MakeShared<PolylineType>(NewPoints, this->Coordinates);
	}

	virtual void Offset(const FVector& OffsetDirection) override
	{
		for (PointType& Pole : this->Points)
		{
			Pole += PointType(OffsetDirection);
		}
	}

	virtual void ExtendTo(const FVector& DesiredPoint) override
	{
		PolylineTools::ExtendTo(this->Points, (PointType)DesiredPoint);
	}
};

class CADKERNEL_API FPolylineCurve : public TPolylineCurve<FVector, FCurvePoint>
{
	friend class FEntity;

protected:
	FPolylineCurve(const TArray<FVector>& InPoints, const TArray<double>& InCoordinates)
		: TPolylineCurve<FVector, FCurvePoint>(InPoints, InCoordinates, 3)
	{
	}

	FPolylineCurve(const TArray<FVector>& InPoints)
		: TPolylineCurve<FVector, FCurvePoint>(InPoints, 3)
	{
	}

	FPolylineCurve() = default;

public:

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Polyline3D;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override
	{
		return ApplyMatrixImpl<FPolylineCurve>(InMatrix);
	}

	virtual FVector EvaluatePoint(double InCoordinate) const override
	{
		return EvaluatePointAt(InCoordinate);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity& Info) const override;
#endif

};

class CADKERNEL_API FPolyline2DCurve : public TPolylineCurve<FVector2d, FCurvePoint2D>
{
	friend class FEntity;

protected:
	FPolyline2DCurve(const TArray<FVector2d>& InPoints, const TArray<double>& InCoordinates)
		: TPolylineCurve<FVector2d, FCurvePoint2D>(InPoints, InCoordinates, 2)
	{
	}

	FPolyline2DCurve(const TArray<FVector2d>& InPoints)
		: TPolylineCurve<FVector2d, FCurvePoint2D>(InPoints, 2)
	{
	}

	FPolyline2DCurve() = default;

public:

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Polyline2D;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override
	{
		return ApplyMatrixImpl<FPolyline2DCurve>(InMatrix);
	}

	virtual void Evaluate2DPoint(double InCoordinate, FCurvePoint2D& OutPoint, int32 InDerivativeOrder = 0) const override
	{
		EvaluateCurvesPoint(InCoordinate, OutPoint, InDerivativeOrder);
	}

	virtual FVector2d Evaluate2DPoint(double InCoordinate) const override
	{
		return EvaluatePointAt(InCoordinate);
	}

	virtual void Evaluate2DPoints(const TArray<double>& InCoordinates, TArray<FCurvePoint2D>& OutPoints, int32 InDerivativeOrder = 0) const override
	{
		EvaluateCurvesPoints(InCoordinates, OutPoints, InDerivativeOrder);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity& Info) const override;
#endif

};

} // namespace UE::CADKernel

