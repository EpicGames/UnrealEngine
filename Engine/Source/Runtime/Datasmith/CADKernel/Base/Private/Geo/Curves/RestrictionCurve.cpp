// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/RestrictionCurve.h"

namespace UE::CADKernel
{

void FRestrictionCurve::ExtendTo(const FVector2d& Point)
{
	Curve2D->ExtendTo(Point);
	EvaluateSurfacicPolyline(Polyline);
}

void FRestrictionCurve::Offset2D(const FVector2d& Offset)
{
	Curve2D->Offset(FVector(Offset, 0.));
	EvaluateSurfacicPolyline(Polyline);
}


#ifdef CADKERNEL_DEV
FInfoEntity& FRestrictionCurve::GetInfo(FInfoEntity& Info) const
{
	return FSurfacicCurve::GetInfo(Info)
		.Add(TEXT("2D polyline"), Polyline.Points2D)
		.Add(TEXT("3D polyline"), Polyline.Points3D);
}
#endif

}
