// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/SegmentCurve.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FSegmentCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FVector TransformedStartPoint = InMatrix.Multiply(StartPoint);
	FVector TransformedEndPoint = InMatrix.Multiply(EndPoint);

	return FEntity::MakeShared<FSegmentCurve>(TransformedStartPoint, TransformedEndPoint, Dimension);
}

void FSegmentCurve::Offset(const FVector& OffsetDirection)
{
	StartPoint += OffsetDirection;
	EndPoint += OffsetDirection;
}

#ifdef CADKERNEL_DEV
FInfoEntity& FSegmentCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info).Add(TEXT("StartPoint"), StartPoint).Add(TEXT("pt2"), EndPoint);
}
#endif

}