// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"
#include "Geo/GeoEnum.h"

namespace UE::CADKernel
{

class CADKERNEL_API FSegmentCurve : public FCurve
{
	friend class FEntity;

protected:
	FVector StartPoint;
	FVector EndPoint;

	FSegmentCurve(const FVector& InStartPoint, const FVector& InEndPoint, int8 InDimension = 3)
		: FCurve(InDimension)
		, StartPoint(InStartPoint)
		, EndPoint(InEndPoint)
	{
	}

	FSegmentCurve(const FVector2d& InStartPoint, const FVector2d& InEndPoint, int8 InDimension = 3)
		: FCurve(InDimension)
		, StartPoint(InStartPoint, 0.)
		, EndPoint(InEndPoint, 0.)
	{
	}

	FSegmentCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		Ar << StartPoint;
		Ar << EndPoint;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Segment;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 3);
		Evaluate<FCurvePoint, FVector>(Coordinate, OutPoint, DerivativeOrder);
	}

	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		ensureCADKernel(Dimension == 2);
		Evaluate<FCurvePoint2D, FVector2d>(Coordinate, OutPoint, DerivativeOrder);
	}

	const FVector& GetStartPoint() const
	{
		return StartPoint;
	}

	const FVector& GetEndPoint() const
	{
		return EndPoint;
	}

	virtual void ExtendTo(const FVector& DesiredPosition) override
	{
		double DistanceToStartPoint = FVector::DistSquared(DesiredPosition, StartPoint);
		double DistanceToEndPoint = FVector::DistSquared(DesiredPosition, EndPoint);
		if (DistanceToEndPoint < DistanceToStartPoint)
		{
			EndPoint = DesiredPosition;
		}
		else
		{
			StartPoint = DesiredPosition;
		}
	}

private:
	void GetTangent(FVector& Tangent) const
	{
		Tangent = EndPoint - StartPoint;
	}

	void GetTangent(FVector2d& Tangent) const
	{
		Tangent = FVector2d(EndPoint.X - StartPoint.X, EndPoint.Y - StartPoint.Y);
	}

	template <typename CurvePointType, typename PointType>
	void Evaluate(double Coordinate, CurvePointType& OutPoint, int32 DerivativeOrder) const
	{
		OutPoint.DerivativeOrder = DerivativeOrder;

		PointType Tangent;
		GetTangent(Tangent);

		OutPoint.Point = Tangent * Coordinate + PointType(StartPoint);

		if (DerivativeOrder > 0)
		{
			OutPoint.Gradient = MoveTemp(Tangent);
		}
	}

};

} // namespace UE::CADKernel

