// Copyright Epic Games, Inc. All Rights Reserved.
#include "Geo/Curves/SplineCurve.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Math/BSpline.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FSplineCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	ensureCADKernel(false);
	//TArray<FVector> TransformedPoles;
	//TransformedPoles.Reserve(Poles.Num());

	//for (const FVector& Pole : Poles)
	//{
	//	TransformedPoles.Emplace(InMatrix.Multiply(Pole));
	//}

	return FEntity::MakeShared<FSplineCurve>(/*TransformedPoles*/);
}

void FSplineCurve::Offset(const FVector& OffsetDirection)
{
	ensureCADKernel(false);
	//for (FVector& Pole : Poles)
	//{
	//	Pole += OffsetDirection;
	//}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FSplineCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info);
// 		.Add(TEXT("degre"), Poles.Num() - 1)
// 		.Add(TEXT("poles"), Poles);
}
#endif

void FSplineCurve::ExtendTo(const FVector& Point)
{
	ensureCADKernel(false);
	//PolylineTools::ExtendTo(Poles, Point);
}

void FSplineCurve::SetSplinePoints(const TArray<FVector>& Points)
{
	const int32 NumPoints = Points.Num();
	Position.Points.Reset(NumPoints);
	//Rotation.Points.Reset(NumPoints);
	//Scale.Points.Reset(NumPoints);

	double InputKey = 0.0;
	for (const FVector& Point : Points)
	{
		Position.Points.Emplace(InputKey, Point, FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
		InputKey += 1.0;
	}

	UpdateSpline();

	Boundary.SetMin(0.);
	Boundary.SetMax(InputKey);
}

void FSplineCurve::SetSplinePoints(const TArray<FVector>& Points, const TArray<FVector>& Tangents)
{
	const int32 NumPoints = Points.Num();
	ensureCADKernel(Tangents.Num() == NumPoints);

	Position.Points.Reset(NumPoints);

	double InputKey = 0.0;
	for (int32 Index = 0; Index < Points.Num(); ++Index)
	{
		Position.Points.Emplace(InputKey, Points[Index], Tangents[Index], Tangents[Index], CIM_CurveAuto);
		InputKey += 1.0;
	}
	//UpdateSpline();

	Boundary.SetMin(0.);
	Boundary.SetMax(InputKey);
}

void FSplineCurve::SetSplinePoints(const TArray<FVector>& Points, const TArray<FVector>& ArriveTangents, const TArray<FVector>& LeaveTangents)
{
	const int32 NumPoints = Points.Num();
	Position.Points.Reset(NumPoints);
	ensureCADKernel(false);

	double InputKey = 0.0;
	for (int32 Index = 0; Index < Points.Num(); ++Index)
	{
		//Position.Points.Emplace(InputKey, Points[Index], ArriveTangents[Index], LeaveTangents[Index], ? CIM_CurveAuto : CIM_CurveBreak);
		InputKey += 1.0;
	}
	UpdateSpline();

	Boundary.SetMin(0.);
	Boundary.SetMax(InputKey);
}



void FSplineCurve::UpdateSpline(bool bInClosedLoop, bool bStationaryEndpoints, int32 ReparamStepsPerSegment)
{
	const int32 NumPoints = Position.Points.Num();
	//check(Rotation.Points.Num() == NumPoints && Scale.Points.Num() == NumPoints);

	// Ensure input keys are strictly ascending
	for (int32 Index = 1; Index < NumPoints; Index++)
	{
		ensureAlways(Position.Points[Index - 1].InVal < Position.Points[Index].InVal);
	}

	// Ensure splines' looping status matches with that of the spline component
	if (bInClosedLoop)
	{
		const double LastKey = Position.Points.Num() > 0 ? Position.Points.Last().InVal : 0.0;
		const double LoopKey = /*bLoopPositionOverride ? LoopPosition :*/ LastKey + 1.0;
		Position.SetLoopKey(LoopKey);
		//Rotation.SetLoopKey(LoopKey);
		//Scale.SetLoopKey(LoopKey);
	}
	else
	{
		Position.ClearLoopKey();
		//Rotation.ClearLoopKey();
		//Scale.ClearLoopKey();
	}

	// Automatically set the tangents on any CurveAuto keys
	Position.AutoSetTangents(0.0, bStationaryEndpoints);
	//Rotation.AutoSetTangents(0.0, bStationaryEndpoints);
	//Scale.AutoSetTangents(0.0, bStationaryEndpoints);

	// Now initialize the spline reparam table
	const int32 NumSegments = bInClosedLoop ? NumPoints : FMath::Max(0, NumPoints - 1);

	// Start by clearing it
	ReparamTable.Points.Reset(NumSegments * ReparamStepsPerSegment + 1);
	double AccumulatedLength = 0.0f;
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		for (int32 Step = 0; Step < ReparamStepsPerSegment; ++Step)
		{
			const double Param = static_cast<double>(Step) / ReparamStepsPerSegment;
			const double SegmentLength = (Step == 0) ? 0.0 : GetSegmentLength(SegmentIndex, Param, bInClosedLoop);

			ReparamTable.Points.Emplace(SegmentLength + AccumulatedLength, SegmentIndex + Param, 0.0, 0.0, CIM_Linear);
		}
		AccumulatedLength += GetSegmentLength(SegmentIndex, 1.0, bInClosedLoop);
	}

	ReparamTable.Points.Emplace(AccumulatedLength, static_cast<double>(NumSegments), 0.0, 0.0, CIM_Linear);
}

FVector FSplineCurve::GetLocationAtSplineInputKey(double Coordinate) const
{
	return Position.Eval((float)Coordinate, FVector::ZeroVector);
}

FVector FSplineCurve::GetTangentAtSplineInputKey(double Coordinate) const
{
	return Position.EvalDerivative((float)Coordinate, FVector::ZeroVector);
}

FVector FSplineCurve::GetDirectionAtSplineInputKey(double Coordinate) const
{
	return Position.EvalDerivative(Coordinate, FVector::ZeroVector).GetSafeNormal();
}

void FSplineCurve::EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder) const
{


	OutPoint.Point = GetLocationAtSplineInputKey(Coordinate);
	if (DerivativeOrder > 0)
	{
		OutPoint.Gradient = GetTangentAtSplineInputKey(Coordinate);
	}
}

double FSplineCurve::GetSplineLength() const
{
	const int32 NumPoints = ReparamTable.Points.Num();
	if (NumPoints > 0)
	{
		return ReparamTable.Points.Last().InVal;
	}
	return 0.0f;
}

double FSplineCurve::GetSegmentLength(const int32 Index, const double Param, bool bInClosedLoop) const
{
	const int32 NumPoints = Position.Points.Num();
	const int32 LastPoint = NumPoints - 1;

	check(Index >= 0 && ((bInClosedLoop && Index < NumPoints) || (!bInClosedLoop && Index < LastPoint)));
	check(Param >= 0.0 && Param <= 1.0);

	// Evaluate the length of a Hermite spline segment.
	// This calculates the integral of |dP/dt| dt, where P(t) is the spline equation with components (x(t), y(t), z(t)).
	// This isn't solvable analytically, so we use a numerical method (Legendre-Gauss quadrature) which performs very well
	// with functions of this type, even with very few samples.  In this case, just 5 samples is sufficient to yield a
	// reasonable result.

	struct FLegendreGaussCoefficient
	{
		double Abscissa;
		double Weight;
	};

	static const FLegendreGaussCoefficient LegendreGaussCoefficients[] =
	{
		{ 0.0f, 0.5688889f },
		{ -0.5384693f, 0.47862867f },
		{ 0.5384693f, 0.47862867f },
		{ -0.90617985f, 0.23692688f },
		{ 0.90617985f, 0.23692688f }
	};

	const auto& StartPoint = Position.Points[Index];
	const auto& EndPoint = Position.Points[Index == LastPoint ? 0 : Index + 1];

	const auto& P0 = StartPoint.OutVal;
	const auto& T0 = StartPoint.LeaveTangent;
	const auto& P1 = EndPoint.OutVal;
	const auto& T1 = EndPoint.ArriveTangent;

	// Special cases for linear or constant segments
	if (StartPoint.InterpMode == CIM_Linear)
	{
		return (P1 - P0).Length() * Param;
	}
	else if (StartPoint.InterpMode == CIM_Constant)
	{
		// Special case: constant interpolation acts like distance = 0 for all p in [0, 1[ but for p == 1, the distance returned is the linear distance between start and end
		return Param == 1. ? (P1 - P0).Length() : 0.0;
	}

	// Cache the coefficients to be fed into the function to calculate the spline derivative at each sample point as they are constant.
	const FVector Coeff1 = ((P0 - P1) * 2.0 + T0 + T1) * 3.0;
	const FVector Coeff2 = (P1 - P0) * 6.0 - T0 * 4.0f - T1 * 2.0;
	const FVector Coeff3 = T0;

	const double HalfParam = Param * 0.5;

	double Length = 0.0;
	for (const auto& LegendreGaussCoefficient : LegendreGaussCoefficients)
	{
		// Calculate derivative at each Legendre-Gauss sample, and perform a weighted sum
		const double Alpha = HalfParam * (1.0 + LegendreGaussCoefficient.Abscissa);
		const FVector Derivative = ((Coeff1 * Alpha + Coeff2) * Alpha + Coeff3);
		Length += Derivative.Length() * LegendreGaussCoefficient.Weight;
	}
	Length *= HalfParam;

	return Length;
}


} // namespace UE::CADKernel

