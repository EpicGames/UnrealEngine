// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"

#include "Math/Point.h"
#include "Math/InterpCurve.h"

DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveDouble, double)
DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveFVector, FVector)

namespace UE::CADKernel
{

class CADKERNEL_API FSplineCurve : public FCurve
{
	friend class FEntity;

protected:

	FInterpCurveFVector Position;

	/** Spline built from rotation data. */
	//FInterpCurveQuat Rotation;

	/** Spline built from scale data. */
	//FInterpCurveFPoint Scale;

	/** Input: distance along curve, output: parameter that puts you there. */
	FInterpCurveDouble ReparamTable;

	/**
	 * Whether the spline is to be considered as a closed loop.
	 * Use SetClosedLoop() to set this property, and IsClosedLoop() to read it.
	 */
	bool bClosedLoop;

	//bool bLoopPositionOverride;
	//double LoopPosition;

	FSplineCurve(const TArray<FVector>& InPoles)
	{
		SetSplinePoints(InPoles);
	}

	FSplineCurve(const TArray<FVector>& InPoles, const TArray<FVector>& InTangents)
	{
		SetSplinePoints(InPoles, InTangents);
	}

	FSplineCurve(const TArray<FVector>& InPoles, const TArray<FVector>& InArriveTangents, const TArray<FVector>& InLeaveTangents)
	{
		SetSplinePoints(InPoles, InArriveTangents, InLeaveTangents);
	}

	FSplineCurve() = default;

	/** Get location along spline at the provided input coordinate value */
	FVector GetLocationAtSplineInputKey(double InCoordinate) const;

	/** Get tangent along spline at the provided input coordinate value */
	FVector GetTangentAtSplineInputKey(double InCoordinate) const;

	/** Get unit direction (normalized tangent) along spline at the provided input coordinate value */
	FVector GetDirectionAtSplineInputKey(double InCoordinate) const;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		Ar << Position;
		Ar << ReparamTable;
		Ar << bClosedLoop;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Spline;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;
	virtual void Offset(const FVector& OffsetDirection) override;

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override;

	virtual void ExtendTo(const FVector& Point) override;

	/**
	 * Update the spline's internal data according to the passed-in params
	 * @param	bClosedLoop				Whether the spline is to be considered as a closed loop.
	 * @param	bStationaryEndpoints	Whether the endpoints of the spline are considered stationary when traversing the spline at non-constant velocity.  Essentially this sets the endpoints' tangents to zero vectors.
	 * @param	ReparamStepsPerSegment	Number of steps per spline segment to place in the re parameterization table
	 */
	void UpdateSpline(bool bClosedLoop = false, bool bStationaryEndpoints = false, int32 ReparamStepsPerSegment = 10);

	/** Returns the length of the specified spline segment up to the parametric value given */
	double GetSegmentLength(const int32 Index, const double Param, bool bClosedLoop = false) const;

	/** Returns total length along this spline */
	double GetSplineLength() const;

	/** Sets the spline to an array of points */
	void SetSplinePoints(const TArray<FVector>& InPoints);
	void SetSplinePoints(const TArray<FVector>& InPoints, const TArray<FVector>& InTangents);
	void SetSplinePoints(const TArray<FVector>& InPoints, const TArray<FVector>& InArriveTangents, const TArray<FVector>& InLeaveTangents);

	const FInterpCurveFVector& GetSplinePointsPosition() const { return Position; }
};

} // namespace UE::CADKernel

