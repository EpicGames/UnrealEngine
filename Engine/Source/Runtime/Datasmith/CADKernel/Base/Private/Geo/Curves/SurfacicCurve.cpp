// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Curves/SurfacicCurve.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/Polyline.h"
#include "Geo/Sampling/SurfacicPolyline.h"
#include "Utils/ArrayUtils.h"

namespace UE::CADKernel
{

void FSurfacicCurve::EvaluatePoint(double InCoordinate, FCurvePoint& OutPoint, int32 InDerivativeOrder) const
{
	FCurvePoint2D Point2D;
	Curve2D->Evaluate2DPoint(InCoordinate, Point2D, InDerivativeOrder);

	FSurfacicPoint SurfacicPoint;
	CarrierSurface->EvaluatePoint(Point2D.Point, SurfacicPoint, InDerivativeOrder);

	OutPoint.Combine(Point2D, SurfacicPoint);
}

void FSurfacicCurve::EvaluatePoints(const TArray<double>& InCoordinates, TArray<FCurvePoint>& OutPoints, int32 InDerivativeOrder) const
{
	TArray<FCurvePoint2D> SurfacicCoordinates;
	Curve2D->Evaluate2DPoints(InCoordinates, SurfacicCoordinates, InDerivativeOrder);
	TArray<FSurfacicPoint> SurfacicPoints3D;

	CarrierSurface->EvaluatePoints(SurfacicCoordinates, SurfacicPoints3D, InDerivativeOrder);

	OutPoints.SetNum(InCoordinates.Num());
	for (int32 Index = 0; Index < SurfacicCoordinates.Num(); ++Index)
	{
		OutPoints[Index].Combine(SurfacicCoordinates[Index], SurfacicPoints3D[Index]);
	}	
}

void FSurfacicCurve::EvaluateSurfacicPolylineWithNormalAndTangent(FSurfacicPolyline& OutPolyline) const
{
	OutPolyline.bWithNormals = true;

	TArray<FCurvePoint2D> Points2D;
	Curve2D->Evaluate2DPoints(OutPolyline.Coordinates, Points2D, 1);
	CarrierSurface->EvaluatePoints(Points2D, OutPolyline);
}

void FSurfacicCurve::EvaluateSurfacicPolyline(FSurfacicPolyline& OutPolyline) const
{
	if (OutPolyline.bWithTangent)
	{
		return EvaluateSurfacicPolylineWithNormalAndTangent(OutPolyline);
	}

	Curve2D->Evaluate2DPoints(OutPolyline.Coordinates, OutPolyline.Points2D);
	CarrierSurface->EvaluatePoints(OutPolyline);
}

TSharedPtr<FEntityGeom> FSurfacicCurve::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FSurface> transformedSurface = StaticCastSharedPtr<FSurface>(CarrierSurface->ApplyMatrix(InMatrix));
	if (transformedSurface.IsValid())
	{
		return FEntity::MakeShared<FSurfacicCurve>(Curve2D.ToSharedRef(), transformedSurface.ToSharedRef());
	}
	return TSharedPtr<FEntityGeom>();
}

#ifdef CADKERNEL_DEV
FInfoEntity& FSurfacicCurve::GetInfo(FInfoEntity& Info) const
{
	return FCurve::GetInfo(Info)
		.Add(TEXT("Surface"), CarrierSurface)
		.Add(TEXT("2D curve"), Curve2D);
}
#endif

void FSurfacicCurve::FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const
{
	Curve2D->FindNotDerivableCoordinates(InBoundary, DerivativeOrder, OutNotDerivableCoordinates);

	FCoordinateGrid SurfaceNotDerivableCoordinates;
	CarrierSurface->LinesNotDerivables(CarrierSurface->GetBoundary(), DerivativeOrder, SurfaceNotDerivableCoordinates);

	if (SurfaceNotDerivableCoordinates[EIso::IsoU].Num() == 0 && SurfaceNotDerivableCoordinates[EIso::IsoV].Num() == 0)
	{
		return;
	}

	double Tolerance = CarrierSurface->Get3DTolerance();
	FSurfacicPolyline SurfacicPolyline(CarrierSurface.ToSharedRef(), Curve2D.ToSharedRef(), Tolerance, Tolerance, false, false);

	FPolyline2D Polyline;
	Swap(Polyline.Coordinates, SurfacicPolyline.Coordinates);
	Swap(Polyline.Points, SurfacicPolyline.Points2D);

	TArray<double> NotDerivableCurveCoordinates;
	for (int32 Iso = 0; Iso < EIso::UndefinedIso; ++Iso)
	{
		for (const double& NotDerivableCoord : SurfaceNotDerivableCoordinates[(EIso) Iso])
		{
			Polyline.FindIntersectionsWithIso((EIso) Iso, NotDerivableCoord, NotDerivableCurveCoordinates);
		}
	}
	Algo::Sort(NotDerivableCurveCoordinates);

	double ToleranceGeoEdge = GetBoundary().ComputeMinimalTolerance();
	ArrayUtils::Complete(OutNotDerivableCoordinates, NotDerivableCurveCoordinates, ToleranceGeoEdge);
}

}