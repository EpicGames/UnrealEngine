// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/Surfaces/Surface.h"
#include "Geo/Curves/Curve.h"
#include "Math/Boundary.h"
#include "Math/Geometry.h"
#include "Math/Point.h"

//#define DEBUG_CURVE_SAMPLING 
//#define CHECK_RESULT_MAKE_ISO_POLYLINE

namespace UE::CADKernel
{

template<typename PolylineType, typename PointType>
class TCurveSamplerAbstract
{
public:

	struct TSamplingPoint
	{
		const PolylineType* Sampling;
		const PolylineType* Polyline;
		int32 Index;

		TSamplingPoint(const PolylineType& InSampling)
			: Sampling(&InSampling)
		{
		}

		void Set(const PolylineType& InPolyline, int32 InIndex)
		{
			Polyline = &InPolyline;
			Index = InIndex;
		}

		constexpr bool IsCandidatePoint()
		{
			return Sampling != Polyline;
		}

		constexpr bool IsSamplingPoint()
		{
			return Sampling == Polyline;
		}
	};

	TCurveSamplerAbstract(const FLinearBoundary& InBoundary, PolylineType& OutPolyline, double InDesiredChordError)
		: Boundary(InBoundary)
		, Sampling(OutPolyline)
		, TmpPolylineCoordinates(CandidatePoints.GetCoordinates())
		, SamplingCoordinates(Sampling.GetCoordinates())
		, EndStudySegment(OutPolyline)
		, DesiredChordError(InDesiredChordError)
	{
	}

	virtual ~TCurveSamplerAbstract() = default;

	/**
	 * Method to call to generate the curve sampling
	 */
	virtual void Sample()
	{
		SamplingInitializing();
		RunSampling();
	}

protected:

	/** Evaluate the new candidate points corresponding to the NextCoordinates */
	virtual void EvaluatesNewCandidatePoints() = 0;

	/**
	 * This function check the sub-polyline to verify if its respect the attended quality
	 * e.g. Curve sampler implementation of this method check the chord error
	 */
	virtual int32 CheckSamplingError(int32 FirstIndex, int32 EndIndex) = 0;

	/**
	 * OutNotDerivableCoordinates must not include the boundaries of the curve. They are added after
	 */
	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates)
	{
	}

	/**
	 * Method to call to generate the curve sampling
	 */
	virtual void SamplingInitializing()
	{
		IsOptimalSegments.Empty(100);
		Sampling.Empty(100);

		CandidatePoints.Empty(100);

		// Initialization of the algorithm with the not derivable coordinates of the curve and at least 5 temporary points
		int32 ComplementaryPointOffset = 0;
		{
			TArray<double> NotDerivableCoordinates;
			GetNotDerivableCoordinates(NotDerivableCoordinates);

			NotDerivableCoordinates.Insert(Boundary.Min, 0);
			NotDerivableCoordinates.Add(Boundary.Max);

			ComplementaryPointOffset = NotDerivableCoordinates.Num() < 10 ? 10 : 1;

			NextCoordinates.Empty(NotDerivableCoordinates.Num() * (ComplementaryPointOffset + 1));
			NextCoordinates.Add(Boundary.Min);

			for (int32 Index = 0; Index < NotDerivableCoordinates.Num() - 1; ++Index)
			{
				if (AddIntermediateCoordinates(NotDerivableCoordinates[Index], NotDerivableCoordinates[Index + 1], ComplementaryPointOffset))
				{
					NextCoordinates.Add(NotDerivableCoordinates[Index + 1]);
				}
			}
		}

		// #cadkernel_check: Could we do a MoveTem?
		CandidatePoints.SwapCoordinates(NextCoordinates);
		EvaluatesNewCandidatePoints();

		// Not Derivable points and boundaries initialize the sampling
		ComplementaryPointOffset++;
		for (int32 Index = 0, ISampling = 0; Index < CandidatePoints.Size(); Index += ComplementaryPointOffset, ++ISampling)
		{
			Sampling.EmplaceAt(ISampling, CandidatePoints, Index);
			IsOptimalSegments.Add(false);
		}
		// Remove last entered entry to IsOptimalSegments as this is to track segments
		IsOptimalSegments.Pop();

		CandidatePoints.RemoveComplementaryPoints(ComplementaryPointOffset);
	}

	/**
	 * Method to call to generate the curve sampling
	 */
	void RunSampling()
	{
		if (IsOptimalSegments.Num() == 0)
		{
			// #cadkernel_check: What does this mean that IsOptimalSegments is empty.
			//                   Why only one CandidatePoints?
			return;
		}

		// < 100000 : check to avoid that the process loops endlessly
		int32 CandidatePointsCount = CandidatePoints.Size();
		while (CandidatePointsCount && Sampling.Coordinates.Num() < 100000)
		{
			NextCoordinates.Empty();

			StartSamplingSegmentIndex = 0;
			EndStudySegment.Index = 0;

			// indices of TmpPoints and TmpCoordinates
			int32 FirstCandidateIndex = 0;
			int32 LastCandidateIndex = 1;

			int32 SegmentIndex = 0;

			do
			{
				if (IsOptimalSegments[StartSamplingSegmentIndex])
				{
					StartSamplingSegmentIndex++;
					continue;
				}

				EndSamplingSegmentIndex = StartSamplingSegmentIndex + 1;

				do
				{
					if ((LastCandidateIndex < CandidatePointsCount && TmpPolylineCoordinates[LastCandidateIndex] < Sampling.Coordinates[EndSamplingSegmentIndex]))
					{
						EndStudySegment.Set(CandidatePoints, LastCandidateIndex);
					}
					else
					{
						EndStudySegment.Set(Sampling, EndSamplingSegmentIndex);
					}

					// if the next candidate point is biggest than the next sampling point, no chord check can be done. 
					// an intermediate point is added. 
					// the next segment is processed
					if ((FirstCandidateIndex >= CandidatePointsCount) || (FirstCandidateIndex < CandidatePointsCount && TmpPolylineCoordinates[FirstCandidateIndex] > EndStudySegment.Polyline->Coordinates[EndStudySegment.Index]))
					{
						AddIntermediateCoordinates(Sampling.Coordinates[StartSamplingSegmentIndex], Sampling.Coordinates[EndSamplingSegmentIndex], 1);
						StartSamplingSegmentIndex = EndSamplingSegmentIndex;
						break;
					}

					int32 NeededPointNum = CheckSamplingError(FirstCandidateIndex, LastCandidateIndex);
					if (NeededPointNum == 0 || (NeededPointNum == 1 && ((LastCandidateIndex + 1) == CandidatePointsCount || (((LastCandidateIndex + 1) < CandidatePointsCount) && TmpPolylineCoordinates[LastCandidateIndex + 1] > Sampling.Coordinates[EndSamplingSegmentIndex]))))
					{
						if (EndStudySegment.IsSamplingPoint())
						{
							IsOptimalSegments[StartSamplingSegmentIndex] = true;
						}
						else
						{
							LastCandidateIndex++;
							continue;
						}
					}
					else
					{
						CompletesPolyline(NeededPointNum, LastCandidateIndex - 1);
					}

					// An existing point is reached 
					if (EndStudySegment.IsSamplingPoint())
					{
						StartSamplingSegmentIndex = EndSamplingSegmentIndex;
						FirstCandidateIndex = LastCandidateIndex;
						LastCandidateIndex++;
					}
					else
					{
						StartSamplingSegmentIndex = EndSamplingSegmentIndex - 1;
						FirstCandidateIndex = LastCandidateIndex + 1;
						LastCandidateIndex += 2;
					}

				} while (EndStudySegment.IsCandidatePoint());

			} while (StartSamplingSegmentIndex < IsOptimalSegments.Num() && EndStudySegment.Index < Sampling.Coordinates.Num() - 1);

			CandidatePoints.SwapCoordinates(NextCoordinates);
			EvaluatesNewCandidatePoints();

			CandidatePointsCount = CandidatePoints.Size();

#ifdef DEBUG_CURVE_SAMPLING
			DisplaySampling(CurveIndex == CurveToDisplay, 0);
#endif
		}

#ifdef CHECK_RESULT_MAKE_ISO_POLYLINE
		CheckResult();
#endif

	}

	/** Adds coordinates of the next candidate points in NextCoordinates array */
	bool AddIntermediateCoordinates(double UMin, double UMax, int32 PointNum)
	{
		if (FMath::IsNearlyEqual(UMin, UMax, UE_DOUBLE_SMALL_NUMBER))
		{
			return false;
		}
		double DeltaCoord = (UMax - UMin) / (PointNum + 1);
		double UCoord = UMin;
		for (int32 Index = 0; Index < PointNum; Index++)
		{
			UCoord += DeltaCoord;
			NextCoordinates.Add(UCoord);
		}

		return true;
	};

	/**
	 * Completes Polyline and add new candidate coordinates for the next step.
	 * @param NeededPointCount estimated number of needed points to respect the sampling error.
	 * NeededPointNumber = 1: only the last point is needed
	 * NeededPointNumber = 2: the intermediate point and last point is needed
	 * NeededPointNumber > 2: the both points are not enought. Candidate points are added on each side of the intermediate point
	 * @param IntermediateIndex
	 */
	void CompletesPolyline(int32 NeededPointNumber, int32 IntermediateIndex)
	{
		if (NeededPointNumber > 50)
		{
			NeededPointNumber = 50;
		}

		const TArray<double>& CandidateCoordinates = CandidatePoints.GetCoordinates();

		if (NeededPointNumber > 1)
		{
			Sampling.EmplaceAt(EndSamplingSegmentIndex, CandidatePoints, IntermediateIndex);
			IsOptimalSegments.EmplaceAt(EndSamplingSegmentIndex - 1, NeededPointNumber == 2 ? true : false);
			EndSamplingSegmentIndex++;
		}

		if (EndStudySegment.IsCandidatePoint())
		{
			Sampling.EmplaceAt(EndSamplingSegmentIndex, CandidatePoints, EndStudySegment.Index);
			IsOptimalSegments.EmplaceAt(EndSamplingSegmentIndex - 1, NeededPointNumber <= 2 ? true : false);
			EndSamplingSegmentIndex++;
		}
		else
		{
			EndStudySegment.Index = EndSamplingSegmentIndex;
			if (NeededPointNumber <= 2)
			{
				IsOptimalSegments[EndSamplingSegmentIndex - 1] = true;
			}
		}

		if (NeededPointNumber > 2)
		{
			NeededPointNumber -= 2;
			AddIntermediateCoordinates(Sampling.Coordinates[StartSamplingSegmentIndex], CandidateCoordinates[IntermediateIndex], NeededPointNumber);
			AddIntermediateCoordinates(CandidateCoordinates[IntermediateIndex], EndStudySegment.Polyline->Coordinates[EndStudySegment.Index], NeededPointNumber);
		}
	};

	int32 GetFirstNeighbor(int32 NeighborIndex, const double StartCoordinate, const PolylineType& Points, const int32 Increment, double& NeighborCoordinate)
	{
		NeighborCoordinate = Points.Coordinates[NeighborIndex];
		if (FMath::IsNearlyEqual(NeighborCoordinate, StartCoordinate) && Points.Coordinates.IsValidIndex(NeighborIndex + Increment))
		{
			NeighborIndex += Increment;
			NeighborCoordinate = Points.Coordinates[NeighborIndex];;
		}
		return NeighborIndex;
	};

	int32 CountOfNeededPointsToRespectChordError(const PointType& PointA, const PointType& PointB, double ChordError)
	{
		double ABLength = PointType::Distance(PointA, PointB) / 2;
		// Max segment length to respect the chord error
		double LengthToHave = sqrt(FMath::Square(ABLength) * DesiredChordError / ChordError);
		return (int32)(2 + (ABLength / LengthToHave + 0.5));
	};

	int32 CheckTangentError(const PointType& APoint, double ACoordinate, const PointType& BPoint, double BCoordinate, int32 FirstIndex, int32 EndIndex, int32 InStartSamplingSegmentIndex)
	{
		PointType Middle = CandidatePoints.GetPoints()[FirstIndex];

		int32 CountOfNeededPoints = 2;

		if (FirstIndex > 0 && InStartSamplingSegmentIndex > 0)
		{
			// get the first previous point in Sampling or CandidatePoints
			double PreviousSampingCoordinate;
			int32 PreviousSampingIndex = GetFirstNeighbor(InStartSamplingSegmentIndex - 1, ACoordinate, Sampling, -1, PreviousSampingCoordinate);

			double PreviousCandidateCoordinate;
			int32 PreviousCandidateIndex = GetFirstNeighbor(FirstIndex - 1, ACoordinate, CandidatePoints, -1, PreviousCandidateCoordinate);

			const PointType& PreviousPoint = PreviousSampingCoordinate > PreviousCandidateCoordinate ? Sampling.GetPoints()[PreviousSampingIndex] : CandidatePoints.GetPoints()[PreviousCandidateIndex];

			double ChordA = DistanceOfPointToSegment<PointType>(APoint, PreviousPoint, Middle);
			if (ChordA > 4 * DesiredChordError)
			{
				CountOfNeededPoints = CountOfNeededPointsToRespectChordError(APoint, PreviousPoint, ChordA);
			}
		}

		Middle = CandidatePoints.GetPoints()[EndIndex - 1];
		if (EndIndex + 1 < CandidatePoints.Coordinates.Num())
		{
			double NextSampingCoordinate;
			int32 NextSampingIndex = GetFirstNeighbor(StartSamplingSegmentIndex + 1, BCoordinate, Sampling, +1, NextSampingCoordinate);

			double NextCandidateCoordinate;
			int32 NextCandidateIndex = GetFirstNeighbor(EndIndex, BCoordinate, CandidatePoints, +1, NextCandidateCoordinate);

			const PointType& NextPoint = NextSampingCoordinate < NextCandidateCoordinate ? Sampling.GetPoints()[NextSampingIndex] : CandidatePoints.GetPoints()[NextCandidateIndex];

			double ChordB = DistanceOfPointToSegment<PointType>(BPoint, NextPoint, Middle);
			if (ChordB > 4 * DesiredChordError)
			{
				CountOfNeededPoints = FMath::Max(CountOfNeededPointsToRespectChordError(BPoint, NextPoint, ChordB), CountOfNeededPoints);
			}
		}

		return CountOfNeededPoints;
	};

protected:

	FLinearBoundary Boundary;

	// Final sampling of the curve. This polyline will be completed at each iteration
	PolylineType& Sampling;

	PolylineType CandidatePoints;

	const TArray<double>& TmpPolylineCoordinates;
	const TArray<double>& SamplingCoordinates;

	/** Array to indicated if the segment at Index defined by Sampling[Index] and Sampling[Index + 1] respects the desired criteria */
	TArray<char> IsOptimalSegments;

	/**
	 * StartSamplingPointIndex and EndSamplingPointIndex defined the extremity of the segment that is currently enriched with candidate points if needed
	 */
	int32 StartSamplingSegmentIndex;

	/**
	 * EndSamplingPointIndex changes (+1) each time a candidate point is inserted in the sampling
	 */
	int32 EndSamplingSegmentIndex;

	/**
	 * The study segment starting at StartSamplingPoint and ending at EndStudySegment that is either a point of Sampling or a point of CandidatePoints
	 * The sampling error is compute between the intermediate candidate points (candidate points localized between StartSamplingPoint and EndStudySegment) and this segment.
	 * FirstCandidateIndex and (LastCandidateIndex - 1) are the index of the first and the last intermediate points
	 */
	TSamplingPoint EndStudySegment;

	/** Coordinates of the new candidate points for the next iteration */
	TArray<double> NextCoordinates;

	double DesiredChordError;

#ifdef DEBUG_CURVE_SAMPLING
public:
	int32 CurveIndex = 0;
	int32 CurveToDisplay = 1;

	void DisplaySampling(bool bDisplay, int32 Step)
	{
		if (bDisplay)
		{
			{
				int32 Index = 0;
				F3DDebugSession G(FString::Printf(TEXT("Curve %d"), Step));
				for (const FVector& Point : Sampling.GetPoints())
				{
					UE::CADKernel::DisplayPoint(Point, EVisuProperty::BluePoint, Index++);
				}

				for (int32 Index = 0; Index < Sampling.GetPoints().Num() - 1; ++Index)
				{
					if (IsOptimalSegments[Index])
					{
						DisplaySegment(Sampling.GetPoints()[Index], Sampling.GetPoints()[Index + 1], Index, EVisuProperty::BlueCurve);
					}
					else
					{
						DisplaySegment(Sampling.GetPoints()[Index], Sampling.GetPoints()[Index + 1], Index, EVisuProperty::YellowCurve);
					}
				}
			}
			{
				int32 Index = 0;
				F3DDebugSession G(FString::Printf(TEXT("Next Point Tmp %d"), Step));
				for (const FVector& Point : CandidatePoints.GetPoints())
				{
					UE::CADKernel::DisplayPoint(Point, EVisuProperty::YellowPoint, Index++);
				}
			}
			Wait();
		}
	}
#endif

#ifdef CHECK_RESULT_MAKE_ISO_POLYLINE
	void CheckResult()
	{
		// Check the result
		NextCoordinates.Empty(Sampling.Coordinates.Num());

		for (int32 Index = 0; Index < Sampling.Coordinates.Num() - 1; ++Index)
		{
			AddIntermediateCoordinates(Sampling.Coordinates[Index], Sampling.Coordinates[Index + 1], 1);
		}
		CandidatePoints.SwapCoordinates(NextCoordinates);
		EvaluatesNewCandidatePoints();

		double MinError[2] = { HUGE_VALUE, HUGE_VALUE };
		double MaxError[2] = { 0., 0. };
		double ErrorSum[2] = { 0., 0. };
		double SquareErrorSum[2] = { 0., 0. };

		int32 EvaluationCount = CandidatePoints.Coordinates.Num();

		for (int32 Index = 0; Index < EvaluationCount; ++Index)
		{
			double CurvilinearAbscise;
			PointType ProjectedPoint = ProjectPointOnSegment<PointType>(CandidatePoints.GetPoints()[Index], Sampling.GetPoints()[Index], Sampling.GetPoints()[Index + 1], CurvilinearAbscise);
			double ChordError = ProjectedPoint.Distance(CandidatePoints.GetPoints()[Index]);

			MinError[0] = FMath::Min(MinError[0], ChordError);
			MaxError[0] = FMath::Max(MaxError[0], ChordError);
			ErrorSum[0] += ChordError;
			SquareErrorSum[0] += FMath::Square(ChordError);

			double ABCoordinate = Sampling.Coordinates[Index + 1] - Sampling.Coordinates[Index];
			double AMCoordinate = CandidatePoints.Coordinates[Index] - Sampling.Coordinates[Index];
			PointType MApproximate = Sampling.GetPoints()[Index] + (AMCoordinate / ABCoordinate) * (Sampling.GetPoints()[Index + 1] - Sampling.GetPoints()[Index]);
			double ParametrizationError = MApproximate.Distance(ProjectedPoint);

			MinError[1] = FMath::Min(MinError[1], ParametrizationError);
			MaxError[1] = FMath::Max(MaxError[1], ParametrizationError);
			ErrorSum[1] += ParametrizationError;
			SquareErrorSum[1] += FMath::Square(ParametrizationError);
		}
		double ChordMed = ErrorSum[0] / EvaluationCount;
		double ChordStandartDeviation = sqrt(SquareErrorSum[0] / EvaluationCount - FMath::Square(ChordMed));

		double ParamMed = ErrorSum[1] / EvaluationCount;
		double ParamStandartDeviation = sqrt(SquareErrorSum[1] / EvaluationCount - FMath::Square(ParamMed));

		UE_LOGF(LogCADKernelBase, Verbose, "Chord Desired Med Sd Min Max %f %f %f %f %f", MaxError, ChordMed, ChordStandartDeviation, MinError[0], MaxError[0]);
		UE_LOGF(LogCADKernelBase, Verbose, "Param Desired Med Sd Min Max %f %f %f %f %f", MaxError, ParamMed, ParamStandartDeviation, MinError[1], MaxError[1]);
	}
#endif
};

} // ns UE::CADKernel