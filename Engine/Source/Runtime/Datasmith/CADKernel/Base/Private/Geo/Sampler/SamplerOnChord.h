// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/Surfaces/Surface.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Sampler/SamplerAbstract.h"
#include "Geo/Sampling/Polyline.h"
#include "Geo/Sampling/SurfacicSampling.h"
#include "Math/Boundary.h"
#include "Math/Point.h"
#include "Utils/ArrayUtils.h"

//#define DEBUG_CHECKCHORDERROR

namespace UE::CADKernel
{

template<class PointType>
class TSamplerBasedOnChordError : public TCurveSamplerAbstract<TPolyline<PointType>, PointType>
{
public:
	TSamplerBasedOnChordError(const FLinearBoundary& InBoundary, double InMaxAllowedError, TPolyline<PointType>& OutPolyline)
		: TCurveSamplerAbstract<TPolyline<PointType>, PointType>(InBoundary, OutPolyline, InMaxAllowedError)
	{
	};

protected:

	/**
	 * Check the chord error of the AB segment for each point of TmpPoints between FirstIndex and EndIndex
	 * If the MaxChord is biggest to the DesiredChordError, an estimation of the needed point number to insert is done.
	 * This estimation is bases on the approximation of local curve radius (R) :
	 * R = square(L) / 8Sag
	 * L2 = sqrt(square(L1) * S2 / S1)
	 * S2 = S1 * square(L2 / L1)
	 *
	 * Sag & Angle criterion.pdf
	 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	 *
	 * @return the number of point to add after PointA to PointB to respect the chord error.
	 * if PointNum = 0, PointB is not necessary
	 * if PointNum = 1, PointB has to be add to respect the chord error and the segment AB is respecting the chord error
	 * if PointNum = 2, Point at FirstIndex and PointB have to be add to respect the chord error and both segments are respecting the chord error
	 *                  In this case, a complementary check is done at each extremity to verify that the curve do not have a fast modification
	 * if PointNum > 2, Point at FirstIndex and PointB have to be add but at least (PointNum - 2) points have to be added in both segments to respect the chord error
	 */
	virtual int32 CheckSamplingError(int32 FirstIndex, int32 EndIndex) override
	{
		const PointType& APoint = this->Sampling.Points[this->StartSamplingSegmentIndex];
		double ACoordinate = this->Sampling.Coordinates[this->StartSamplingSegmentIndex];

		const PointType& BPoint = this->EndStudySegment.Polyline->Points[this->EndStudySegment.Index];
		double BCoordinate = this->EndStudySegment.Polyline->Coordinates[this->EndStudySegment.Index];

		double MaxChord = 0;
		for (int32 Index = FirstIndex; Index < EndIndex; ++Index)
		{
			const PointType& Middle = this->CandidatePoints.GetPointAt(Index);
			double Chord = DistanceOfPointToSegment<PointType>(Middle, APoint, BPoint);
			if (Chord > MaxChord)
			{
				MaxChord = Chord;
			}

			if (MaxChord > this->DesiredChordError)
			{
				break;
			}
		}

#ifdef DEBUG_CHECKCHORDERROR
		if (bDisplay)
		{
			{
				int32 Index = 0;
				F3DDebugSession G(FString::Printf(TEXT("CheckChord %f %d"), MaxChord, FirstIndex));

				UE::CADKernel::Display(APoint, EVisuProperty::BluePoint);
				UE::CADKernel::Display(BPoint, EVisuProperty::BluePoint);

				for (int32 Index = FirstIndex; Index < EndIndex; ++Index)
				{
					const PointType& Middle = CandidatePoints.GetPoints()[Index];
					UE::CADKernel::Display(Middle, EVisuProperty::YellowPoint);
				}

				DisplaySegment(APoint, BPoint, 0, EVisuProperty::BlueCurve);
				DisplaySegment(APoint, CandidatePoints.GetPoints()[FirstIndex], 0, EVisuProperty::YellowCurve);
				DisplaySegment(BPoint, CandidatePoints.GetPoints()[EndIndex - 1], 0, EVisuProperty::YellowCurve);

				for (int32 Index = FirstIndex; Index < EndIndex - 1; ++Index)
				{
					DisplaySegment(CandidatePoints.GetPoints()[Index], CandidatePoints.GetPoints()[Index + 1], 0, EVisuProperty::YellowCurve);
				}
			}
		}
#endif

		if (MaxChord < this->DesiredChordError / 4)
		{
			// S2 = S1 * sqare(L2 / L1)
			// if the length is doubled, the chord error is still smaller than the desired chord error.
			// the end point is not needed
			return 0;
		}
		else if (MaxChord < this->DesiredChordError)
		{
			// the end point is needed because the Chord is smaller but nearly equal to the desired chord error.
			return 1;
		}
		else if (MaxChord < 4 * this->DesiredChordError)
		{
			// if the length is divided by two, the chord error is divided by 4 and becomes smaller than the desired chord error.
			// Add First index and Index
			return this->CheckTangentError(APoint, ACoordinate, BPoint, BCoordinate, FirstIndex, EndIndex, this->StartSamplingSegmentIndex);
		}
		else
		{
			return this->CountOfNeededPointsToRespectChordError(APoint, BPoint, MaxChord);
		}
	};
#ifdef DEBUG_CHECKCHORDERROR
	bool bDisplay = true;
#endif
};

/**
 * Sampler of curve based on parametrization and chord error control
 */
class FCurveSamplerOnChord : public TSamplerBasedOnChordError<FVector>
{
public:
	FCurveSamplerOnChord(const FCurve& InCurve, const FLinearBoundary& InBoundary, double InMaxAllowedError, FPolyline3D& OutPolyline)
		: TSamplerBasedOnChordError<FVector>(InBoundary, InMaxAllowedError, OutPolyline)
		, Curve(InCurve)
	{
	};

protected:

	const FCurve& Curve;

	virtual void EvaluatesNewCandidatePoints() override
	{
		Curve.EvaluatePoints(CandidatePoints.Coordinates, CandidatePoints.Points);
	};

	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates) override
	{
		Curve.FindNotDerivableCoordinates(Boundary, 2, OutNotDerivableCoordinates);
	}

};

class FCurve2DSamplerOnChord : public TSamplerBasedOnChordError<FVector2d>
{
public:
	FCurve2DSamplerOnChord(const FCurve& InCurve, const FLinearBoundary& InBoundary, double InMaxAllowedError, FPolyline2D& OutPolyline)
		: TSamplerBasedOnChordError<FVector2d>(InBoundary, InMaxAllowedError, OutPolyline)
		, Curve(InCurve)
	{
	};

protected:

	const FCurve& Curve;

	virtual void EvaluatesNewCandidatePoints() override
	{
		Curve.Evaluate2DPoints(CandidatePoints.Coordinates, CandidatePoints.Points);
	};

	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates) override
	{
		Curve.FindNotDerivableCoordinates(Boundary, 2, OutNotDerivableCoordinates);
	}

};

class FIsoCurve3DSamplerOnChord : public TSamplerBasedOnChordError<FVector>
{
public:
	FIsoCurve3DSamplerOnChord(const FSurface& InSurface, double InMaxAllowedError, FPolyline3D& OutPolyline)
		: TSamplerBasedOnChordError<FVector>(FLinearBoundary(), InMaxAllowedError, OutPolyline)
		, Surface(InSurface)
	{
	};

	void Set(EIso InIsoType, double InIsoCoordinate, const FLinearBoundary& CurveBounds)
	{
		IsoType = InIsoType;
		IsoCoordinate.Empty(1);
		IsoCoordinate.Add(InIsoCoordinate);
		Boundary = CurveBounds;
	}

protected:

	const FSurface& Surface;
	FCoordinateGrid NotDerivableCoordinates;
	bool bNotDerivableFound = false;

	mutable TArray<double> IsoCoordinate;
	EIso IsoType;

	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates) override
	{
		if (!bNotDerivableFound)
		{
			Surface.LinesNotDerivables(Surface.GetBoundary(), 2, NotDerivableCoordinates);
			bNotDerivableFound = true;
		}

		EIso CurveIso = IsoType == EIso::IsoU ? EIso::IsoV : EIso::IsoU;
		const TArray<double>& SurfaceIsoNotDerivableCoordinates = NotDerivableCoordinates[CurveIso];
		double Tolerance = Surface.GetIsoTolerance(CurveIso);
		ArrayUtils::SubArrayWithoutBoundary(SurfaceIsoNotDerivableCoordinates, Boundary, Tolerance, OutNotDerivableCoordinates);
	}

	virtual void EvaluatesNewCandidatePoints() override
	{
		TFunction<void(TArray<double>&, TArray<double>&)> Evaluates = [&](TArray<double>& UCoordinates, TArray<double>& VCoordinates)
		{
			FSurfacicSampling PointGrid;

			FCoordinateGrid Grid;
			Grid.Swap(UCoordinates, VCoordinates);
			Surface.EvaluatePointGrid(Grid, PointGrid);
			Grid.Swap(UCoordinates, VCoordinates);

			Swap(CandidatePoints.Points, PointGrid.Points3D);
		};

		if (CandidatePoints.Coordinates.Num() > 0)
		{
			if (IsoType == EIso::IsoU)
			{
				Evaluates(IsoCoordinate, CandidatePoints.Coordinates);
			}
			else
			{
				Evaluates(CandidatePoints.Coordinates, IsoCoordinate);
			}
		}
		else
		{
			CandidatePoints.Points.SetNum(0);
		}
	}

};

} // ns UE::CADKernel
