// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/Surfaces/Surface.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Curves/RestrictionCurve.h"
#include "Geo/Sampler/SamplerAbstract.h"
#include "Geo/Sampling/Polyline.h"   
#include "Geo/Sampling/SurfacicPolyline.h"   
#include "Geo/Sampling/SurfacicSampling.h"
#include "Math/Boundary.h"
#include "Math/Point.h"
#include "Utils/ArrayUtils.h"

namespace UE::CADKernel
{

template<typename PolylineType, typename PointType>
class TSamplerBasedOnParametrizationAndChordError : public TCurveSamplerAbstract<PolylineType, PointType>
{
public:
	TSamplerBasedOnParametrizationAndChordError(const FLinearBoundary& InBoundary, double InMaxChordError, double InMaxParameterizationError, PolylineType& OutPolyline)
		: TCurveSamplerAbstract<PolylineType, PointType>(InBoundary, OutPolyline, InMaxChordError)
		, DesiredParameterizationError(InMaxParameterizationError)
	{
	};

protected:

	double  DesiredParameterizationError;

	/**
	 * This function check the chord error and the parametrization error
	 *
	 * Chord error:
	 *   Check the chord error of the AB segment for each point of TmpPoints between FirstIndex and EndIndex
	 *   If the MaxChord is biggest to the DesiredChordError, an estimation of the needed point number to insert is done.
	 *   This estimation is bases on the approximation of local curve radius (R) :
	 *   R = square(L) / 8Sag
	 *   L2 = sqrt(square(L1) * S2 / S1)
	 *   S2 = S1 * square(L2 / L1)
	 *
	 *   Sag & Angle criterion.pdf
	 *   https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	 *
	 *   @return the number of point to add after PointA to PointB to respect the chord error.
	 *   if PointNum = 0, PointB is not necessary
	 *   if PointNum = 1, PointB has to be add to respect the chord error and the segment AB is respecting the chord error
	 *   if PointNum = 2, Point at FirstIndex and PointB have to be add to respect the chord error and both segments are respecting the chord error
	 *   if PointNum > 2, Point at FirstIndex and PointB have to be add but at least (PointNum - 2) points have to be added in both segments to respect the chord error
	 *
	 * Parametrization error:
	 *   This function check that for inner point of the segment AB, point M(u) is near its approximation = A + AB * (u - uA) / (uB - uA) than the MaxAllowedError
	 *   To Remove Chord error, M approximation is compare to the projection of M(u) on AB
	 */

	virtual int32 CheckSamplingError(int32 FirstIndex, int32 EndIndex) override
	{
		const FVector& APoint = this->Sampling.GetPointAt(this->StartSamplingSegmentIndex);
		double ACoordinate = this->Sampling.Coordinates[this->StartSamplingSegmentIndex];

		const FVector& BPoint = this->EndStudySegment.Polyline->GetPointAt(this->EndStudySegment.Index);
		double BCoordinate = this->EndStudySegment.Polyline->Coordinates[this->EndStudySegment.Index];

		double ABCoordinate = BCoordinate - ACoordinate;
		FVector ABVector = BPoint - APoint;

		double MaxChord = 0;
		double MaxParametrizationError = 0;

		for (int32 Index = FirstIndex; Index < EndIndex; ++Index)
		{
			const FVector& PointM = this->CandidatePoints.GetPointAt(Index);

			double CurvilinearAbscise;
			FVector ProjectedPoint = ProjectPointOnSegment<FVector>(PointM, APoint, BPoint, CurvilinearAbscise, /*bRestrictCoodinateToInside*/ true);

			double Chord = FVector::Distance(ProjectedPoint, PointM);
			double AMCoordinate = this->CandidatePoints.Coordinates[Index] - ACoordinate;
			FVector MApproximate = APoint + (AMCoordinate / ABCoordinate) * ABVector;
			double ParametrizationError = FVector::Distance(MApproximate, ProjectedPoint);

			if (Chord > MaxChord)
			{
				MaxChord = Chord;
			}

			if (ParametrizationError > MaxParametrizationError)
			{
				MaxParametrizationError = ParametrizationError;
			}
		}

		int32 SegmentNumToRespectParametrizationError = (int32)(MaxParametrizationError / DesiredParameterizationError);
		if (SegmentNumToRespectParametrizationError > 2)
		{
			SegmentNumToRespectParametrizationError = (int32)((SegmentNumToRespectParametrizationError - 1) / 2) + 2;
		}

		int32 SegmentNumToRespectChordError = 0;
		if (MaxChord < this->DesiredChordError / 4)
		{
			// S2 = S1 * sqare(L2 / L1)
			// if the length is doubled, the chord error is still smaller than the desired chord error.
			// the end point is not needed
			return 0;
		}
		else if (MaxChord < this->DesiredChordError)
		{
			// the end point is needed because the Chord is smaller but nearly equal to the desired chord error.
			SegmentNumToRespectChordError = 1;
		}
		else if (MaxChord < 4 * this->DesiredChordError)
		{
			// if the length is divided by two, the chord error is divided by 4 and becomes smaller than the desired chord error.
			// Add First index and Index
			SegmentNumToRespectChordError = this->CheckTangentError(APoint, ACoordinate, BPoint, BCoordinate, FirstIndex, EndIndex, this->StartSamplingSegmentIndex);
		}
		else
		{
			SegmentNumToRespectChordError = this->CountOfNeededPointsToRespectChordError(APoint, BPoint, MaxChord);
		}

		return FMath::Max(SegmentNumToRespectChordError, SegmentNumToRespectParametrizationError);
	};
};

/**
 * Sampler of surfacic curve based on parametrization and chord error control
 */
class FSurfacicCurveSamplerOnParam : public TSamplerBasedOnParametrizationAndChordError<FSurfacicPolyline, FVector>
{
public:
	FSurfacicCurveSamplerOnParam(const FRestrictionCurve& InCurve, const FLinearBoundary& InBoundary, double InMaxSagError, double InMaxParameterizationError, FSurfacicPolyline& OutPolyline)
		: TSamplerBasedOnParametrizationAndChordError<FSurfacicPolyline, FVector>(InBoundary, InMaxSagError, InMaxParameterizationError, OutPolyline)
		, Surface(*InCurve.GetCarrierSurface())
		, Curve(*InCurve.Get2DCurve())
	{
	};

	FSurfacicCurveSamplerOnParam(const FSurface& InSurface, const FCurve& InCurve, const FLinearBoundary& InBoundary, double InMaxSagError, double InMaxParameterizationError, FSurfacicPolyline& OutPolyline)
		: TSamplerBasedOnParametrizationAndChordError<FSurfacicPolyline, FVector>(InBoundary, InMaxSagError, InMaxParameterizationError, OutPolyline)
		, Surface(InSurface)
		, Curve(InCurve)
	{
	};

protected:

	const FSurface& Surface;
	const FCurve& Curve;
	double MaxParameterizationError;

	virtual void EvaluatesNewCandidatePoints() override
	{
		CandidatePoints.bWithNormals = Sampling.bWithNormals;
		CandidatePoints.bWithTangent = Sampling.bWithTangent;
		if (CandidatePoints.bWithTangent)
		{
			TArray<FCurvePoint2D> OutPoints;
			Curve.Evaluate2DPoints(CandidatePoints.Coordinates, OutPoints, 1);
			Surface.EvaluatePoints(OutPoints, CandidatePoints);
		}
		else
		{
			Curve.Evaluate2DPoints(CandidatePoints.Coordinates, CandidatePoints.Points2D);
			Surface.ValidateUVPoints(CandidatePoints.Points2D);
			Surface.EvaluatePoints(CandidatePoints);
		}
	};

	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates) override
	{
		Curve.FindNotDerivableCoordinates(Boundary, 2, OutNotDerivableCoordinates);
	}

};

/**
 * Sampler of curve based on parametrization and chord error control
 */
class FCurveSamplerOnParam : public TSamplerBasedOnParametrizationAndChordError<FPolyline3D, FVector>
{
public:
	FCurveSamplerOnParam(const FCurve& InCurve, const FLinearBoundary& InBoundary, double InMaxSagError, double InMaxParameterizationError, FPolyline3D& OutPolyline)
		: TSamplerBasedOnParametrizationAndChordError<FPolyline3D, FVector>(InBoundary, InMaxSagError, InMaxParameterizationError, OutPolyline)
		, Curve(InCurve)
	{
	};

protected:

	const FCurve& Curve;
	double MaxParameterizationError;

	virtual void EvaluatesNewCandidatePoints() override
	{
		Curve.EvaluatePoints(CandidatePoints.Coordinates, CandidatePoints.Points);
	};

	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates) override
	{
		Curve.FindNotDerivableCoordinates(Boundary, 2, OutNotDerivableCoordinates);
	}
};

/**
 * Sampler of surface based on parametrization and chord error control
 */
class FSurfaceSamplerOnParam : public TSamplerBasedOnParametrizationAndChordError<FPolyline3D, FVector>
{

public:
	FSurfaceSamplerOnParam(const FSurface& InSurface, const FSurfacicBoundary& InBoundary, double InMaxSagError, double InMaxParameterizationError, FPolyline3D& TemporaryPolyline, FCoordinateGrid& OutSampling)
		: TSamplerBasedOnParametrizationAndChordError<FPolyline3D, FVector>(FLinearBoundary(), InMaxSagError, InMaxParameterizationError, TemporaryPolyline)
		, Surface(InSurface)
		, SurfaceBoundary(InBoundary)
		, SurfaceSampling(OutSampling)
	{
	};

	void Set(EIso InIsoType, double InIsoCoordinate, const FLinearBoundary& CurveBounds)
	{
		IsoType = InIsoType;
		IsoCoordinate.Empty(1);
		IsoCoordinate.Add(InIsoCoordinate);
		Boundary = CurveBounds;
	}

	virtual void Sample() override
	{
		Sample(EIso::IsoU);
		Sample(EIso::IsoV);
	}

	void Sample(EIso InIsoType)
	{
#ifdef DEBUG_CURVE_SAMPLING
		CurveIndex = CurveToDisplay - 1;
#endif

		double Middle = SurfaceBoundary[InIsoType].GetMiddle();

		Set(InIsoType, Middle, SurfaceBoundary[Other(InIsoType)]);

		SamplingInitializing();
		RunSampling();

		double Step = SurfaceBoundary[InIsoType].Length() / 6;
		IsoCoordinate[0] = SurfaceBoundary[InIsoType].GetMin();
		for (int32 IsoIndex = 1; IsoIndex < 6; ++IsoIndex)
		{
			IsoCoordinate[0] += Step;
			if (IsoIndex == 3)
			{
				continue;
			}

			Swap(CandidatePoints.Coordinates, Sampling.Coordinates);
			EvaluatesNewCandidatePoints();
			Swap(CandidatePoints.Coordinates, Sampling.Coordinates);
			Swap(CandidatePoints.Points, Sampling.Points);

			// Check the result
			CandidatePoints.Coordinates.Empty(Sampling.Coordinates.Num());

			for (int32 Index = 0; Index < Sampling.Coordinates.Num() - 1; ++Index)
			{
				double UCoord = Sampling.Coordinates[Index] + (Sampling.Coordinates[Index + 1] - Sampling.Coordinates[Index]) * 0.5;
				CandidatePoints.Coordinates.Add(UCoord);
			}

			EvaluatesNewCandidatePoints();

#ifdef DEBUG_CURVE_SAMPLING
			CurveToDisplay = CurveIndex;
			DisplaySampling(CurveIndex == CurveToDisplay, 0);
#endif

			RunSampling();
		}

		Swap(SurfaceSampling[Other(InIsoType)], Sampling.Coordinates);
	}

	virtual void SamplingInitializing() override
	{

#ifdef DEBUG_CURVE_SAMPLING
		++CurveIndex;
#endif
		IsOptimalSegments.Empty(100);
		Sampling.Empty(100);

		CandidatePoints.Empty(100);

		// Initialization of the algorithm with the not derivable coordinates of the curve and at least 5 temporary points
		int32 ComplementaryPointOffset = 0;
		{
			TArray<double> LocalNotDerivableCoordinates;
			GetNotDerivableCoordinates(LocalNotDerivableCoordinates);
			LocalNotDerivableCoordinates.Insert(Boundary.Min, 0);
			LocalNotDerivableCoordinates.Add(Boundary.Max);

			ComplementaryPointOffset = LocalNotDerivableCoordinates.Num() < 5 ? 5 : 1;

			NextCoordinates.Empty(LocalNotDerivableCoordinates.Num() * (ComplementaryPointOffset + 1));
			NextCoordinates.Add(Boundary.Min);

			for (int32 Index = 0; Index < LocalNotDerivableCoordinates.Num() - 1; ++Index)
			{
				if (AddIntermediateCoordinates(LocalNotDerivableCoordinates[Index], LocalNotDerivableCoordinates[Index + 1], ComplementaryPointOffset))
				{
					NextCoordinates.Add(LocalNotDerivableCoordinates[Index + 1]);
				}
			}
		}

		CandidatePoints.SwapCoordinates(NextCoordinates);
		EvaluatesNewCandidatePoints();

		// Not Derivable point initialize the sampling
		ComplementaryPointOffset++;
		for (int32 Index = 0, ISampling = 0; Index < CandidatePoints.Size(); Index += ComplementaryPointOffset, ++ISampling)
		{
			Sampling.EmplaceAt(ISampling, CandidatePoints, Index);
			IsOptimalSegments.Add(false);
		}
		IsOptimalSegments.Pop();

		CandidatePoints.RemoveComplementaryPoints(ComplementaryPointOffset);
	}

protected:

	const FSurface& Surface;
	const FSurfacicBoundary& SurfaceBoundary;

	FCoordinateGrid& SurfaceSampling;

	FCoordinateGrid NotDerivableCoordinates;

	bool bNotDerivableFound = false;

	TArray<double> IsoCoordinate;
	EIso IsoType;

	virtual void GetNotDerivableCoordinates(TArray<double>& OutNotDerivableCoordinates) override
	{
		if (!bNotDerivableFound)
		{
			Surface.LinesNotDerivables(Surface.GetBoundary(), 2, NotDerivableCoordinates);
			bNotDerivableFound = true;
		}

		EIso CurveIso = IsoType == EIso::IsoU ? EIso::IsoV : EIso::IsoU;
		const TArray<double>& SurfaceIsoNotDerivableCoordinates = NotDerivableCoordinates[CurveIso];
		double Tolerance = Surface.GetIsoTolerance(CurveIso);
		ArrayUtils::SubArrayWithoutBoundary(SurfaceIsoNotDerivableCoordinates, Boundary, Tolerance, OutNotDerivableCoordinates);
	}

	virtual void EvaluatesNewCandidatePoints() override
	{
		if (CandidatePoints.Coordinates.Num() > 0)
		{
			TArray<double>& UCoordinates = IsoType == EIso::IsoU ? IsoCoordinate : CandidatePoints.Coordinates;
			TArray<double>& VCoordinates = IsoType == EIso::IsoU ? CandidatePoints.Coordinates : IsoCoordinate;

			FSurfacicSampling PointGrid;
			FCoordinateGrid Grid;

			Grid.Swap(UCoordinates, VCoordinates);
			Surface.EvaluatePointGrid(Grid, PointGrid);
			Grid.Swap(UCoordinates, VCoordinates);

			Swap(CandidatePoints.Points, PointGrid.Points3D);
		}
		else
		{
			CandidatePoints.Points.SetNum(0);
		}
	}

};

} // ns UE::CADKernel