// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Sampling/SurfacicPolyline.h"

#include "Geo/Curves/Curve.h"
#include "Geo/Surfaces/Surface.h"
#include "Geo/Sampler/SamplerOnParam.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Math/SlopeUtils.h"

#include "Algo/BinarySearch.h"

namespace UE::CADKernel
{

FSurfacicPolyline::FSurfacicPolyline(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> Curve2D, const double InTolerance)
	: bWithNormals(true)
	, bWithTangent(false)
{
	FSurfacicCurveSamplerOnParam Sampler(InCarrierSurface.Get(), Curve2D.Get(), Curve2D->GetBoundary(), InTolerance, InTolerance, *this);
	Sampler.Sample();
	BoundingBox.Set(Points2D);
}

FSurfacicPolyline::FSurfacicPolyline(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> Curve2D)
	: FSurfacicPolyline(InCarrierSurface, Curve2D, InCarrierSurface->Get3DTolerance())
{
}

FSurfacicPolyline::FSurfacicPolyline(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> Curve2D, const double ChordTolerance, const double ParamTolerance, bool bInWithNormals, bool bInWithTangents)
	: bWithNormals(bInWithNormals)
	, bWithTangent(bInWithTangents)
{
	FSurfacicCurveSamplerOnParam Sampler(InCarrierSurface.Get(), Curve2D.Get(), Curve2D->GetBoundary(), ChordTolerance, ParamTolerance, *this);
	Sampler.Sample();
	BoundingBox.Set(Points2D);
}

void FSurfacicPolyline::CheckIfDegenerated(const double Tolerance3D, const FSurfacicTolerance& ToleranceIso, const FLinearBoundary& Boudary, bool& bDegeneration2D, bool& bDegeneration3D, double& Length3D) const
{
	TPolylineApproximator<FVector> Approximator(Coordinates, Points3D);
	int32 BoundaryIndices[2];
	Approximator.GetStartEndIndex(Boudary, BoundaryIndices);

	TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
	Length3D = Approximator3D.ComputeLengthOfSubPolyline(BoundaryIndices, Boudary);

	if (!FMath::IsNearlyZero(Length3D, Tolerance3D))
	{
		bDegeneration3D = false;
		bDegeneration2D = false;
		return;
	}

	bDegeneration3D = true;
	Length3D = 0.;

	// Tolerance along Iso U/V is very costly to compute and not accurate.
	// To test if a curve is degenerated, its 2d bounding box is compute and its compare to the surface boundary along U and along V
	// Indeed, defining a Tolerance2D has no sense has the boundary length along an Iso could be very very huge compare to the boundary length along the other Iso like [[0, 1000] [0, 1]]
	// The tolerance along ans iso is the length of the boundary along this iso divide by 100 000: if the curve length in 3d is 10m, the tolerance is 0.01mm

	FAABB2D Aabb;
	TPolylineApproximator<FVector2d> Approximator2D(Coordinates, Points2D);
	Approximator2D.ComputeBoundingBox<2>(BoundaryIndices, Boudary, Aabb);

	bDegeneration2D = (Aabb.GetSize(0) < ToleranceIso[EIso::IsoU] && Aabb.GetSize(1) < ToleranceIso[EIso::IsoV]);
}

void FSurfacicPolyline::GetExtremities(const FLinearBoundary& InBoundary, const double Tolerance3D, const FSurfacicTolerance& MinToleranceIso, FSurfacicCurveExtremities& Extremities) const
{
	FDichotomyFinder Finder(Coordinates);
	const int32 StartIndex = Finder.Find(InBoundary.Min);
	const int32 EndIndex = Finder.Find(InBoundary.Max);

	Extremities[0].Point2D = PolylineTools::ComputePoint(Coordinates, Points2D, StartIndex, InBoundary.Min);
	Extremities[0].Point = PolylineTools::ComputePoint(Coordinates, Points3D, StartIndex, InBoundary.Min);
	Extremities[0].Tolerance = ComputeTolerance(Tolerance3D, MinToleranceIso, StartIndex);

	Extremities[1].Point2D = PolylineTools::ComputePoint(Coordinates, Points2D, EndIndex, InBoundary.Max);
	Extremities[1].Point = PolylineTools::ComputePoint(Coordinates, Points3D, EndIndex, InBoundary.Max);
	if (EndIndex == StartIndex)
	{
		Extremities[1].Tolerance = Extremities[0].Tolerance;
	}
	else
	{
		Extremities[1].Tolerance = ComputeTolerance(Tolerance3D, MinToleranceIso, EndIndex);
	}
}

void FSurfacicPolyline::ComputeIntersectionsWithIsos(const FLinearBoundary& InBoundary, const TArray<double>& IsoCoordinates, const EIso IsoType, const FSurfacicTolerance& ToleranceIso, TArray<double>& Intersection) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FSurfacicPolyline::ComputeIntersectionsWithIsos)

	const double SurfaceIsoTolerance = ToleranceIso[IsoType];

	if (BoundingBox.Length(IsoType) < SurfaceIsoTolerance)
	{
		// the edge is on an IsoCurve on Iso axis
		return;
	}

	const int32 IsoCoordinateCount = IsoCoordinates.Num();
	Intersection.Reserve(IsoCoordinateCount + Coordinates.Num());

	double LastIntersection = UE_MAX_FLT;

	TFunction<void(double, double, int32, double)> InsertIntersection;
	InsertIntersection = [this, &Intersection, &LastIntersection](double Start, double End, int32 SegmentIndex, double IsoCoordinate) -> void
		{
			const double LocalCoord = (IsoCoordinate - Start) / (End - Start);
			double EdgeCoord = this->Coordinates[SegmentIndex] + (LocalCoord * (this->Coordinates[SegmentIndex + 1] - this->Coordinates[SegmentIndex]));

			if (!FMath::IsNearlyEqual(EdgeCoord, LastIntersection))
			{
				Intersection.Add(EdgeCoord);
			}
		};

	// Find the largest surface's iso coordinate that is less or equal to the start iso value,
	// within the surface's tolerance on the given iso type
	TFunction<int32(double)> FindIsoCoordinateIndex;
	FindIsoCoordinateIndex = [&IsoCoordinates, &SurfaceIsoTolerance, &IsoCoordinateCount](double Value) -> int32
		{
			// #cad_kernel: Algo::LowerBound ???
			int32 Index = 0;
			const double IsoValueStartMinusTol = Value - SurfaceIsoTolerance;
			for (; Index < IsoCoordinateCount - 1 && IsoValueStartMinusTol > IsoCoordinates[Index]; ++Index);

			return Index;
		};

	for (int32 Index = 0, NextIndex = 1; NextIndex < Points2D.Num(); ++NextIndex, ++Index)
	{
		// Check that the segment to consider is within the curve's boundaries
		// used for trimming
		{
			// if the segment is outside the minimum edge boundary, go to the next one
			if (Coordinates[NextIndex] < InBoundary.GetMin())
			{
				continue;
			}

			// if the segment is outside the maximum edge boundary, no need to go any further
			if (Coordinates[Index] > InBoundary.GetMax())
			{
				break;
			}
		}

		const double IsoValueStart = Points2D[Index][IsoType];
		const double IsoValueEnd = Points2D[NextIndex][IsoType];

		// Skip this segment if it is degenerated along the given iso type
		if (FMath::IsNearlyEqual(IsoValueStart, IsoValueEnd, SurfaceIsoTolerance))
		{
			continue;
		}

		int32 IsoCoordinateIndex = FindIsoCoordinateIndex(IsoValueStart);

		// If the start point is equal to the iso coordinate within the surface's tolerance
		// on the given iso direction, add the polyline's coordinate and continue
		if (FMath::IsNearlyEqual(IsoValueStart, IsoCoordinates[IsoCoordinateIndex], SurfaceIsoTolerance))
		{
			Intersection.Add(Coordinates[Index]);
			LastIntersection = Intersection.Last();
		}

		// Segment is forward in surface's given iso type's direction
		if (IsoValueStart < IsoValueEnd)
		{
			// Insert an intersection as long as the iso values are less than the end value
			// Intentionally do not check whether the end iso value is within surface's tolerance
			// on an iso type's coordinate. It will be checked on the next iteration
			const double IsoValueEndMinusTol = IsoValueEnd - SurfaceIsoTolerance;
			for (++IsoCoordinateIndex; IsoCoordinateIndex < IsoCoordinateCount && IsoCoordinates[IsoCoordinateIndex] < IsoValueEndMinusTol; ++IsoCoordinateIndex)
			{
				InsertIntersection(IsoValueStart, IsoValueEnd, Index, IsoCoordinates[IsoCoordinateIndex]);
				LastIntersection = Intersection.Last();
			}
		}
		// Segment is backward in surface's given iso type's direction
		else if(IsoCoordinateIndex > 0)
		{
			// Insert an intersection as long as the iso values are more than the end value
			// Intentionally do not check whether the end iso value is within surface's tolerance
			// on an iso type's coordinate. It will be checked on the next iteration
			const double IsoValueEndMinusTol = IsoValueEnd + SurfaceIsoTolerance;
			for (--IsoCoordinateIndex; IsoCoordinateIndex >= 0 && IsoCoordinates[IsoCoordinateIndex] > IsoValueEndMinusTol; --IsoCoordinateIndex)
			{
				InsertIntersection(IsoValueStart, IsoValueEnd, Index, IsoCoordinates[IsoCoordinateIndex]);
				LastIntersection = Intersection.Last();
			}
		}
		// Lowest surface's iso coordinate is strictly between the end value and the start value
		// Insert an intersection there
		else
		{
			InsertIntersection(IsoValueStart, IsoValueEnd, Index, IsoCoordinates[IsoCoordinateIndex]);
			LastIntersection = Intersection.Last();
		}
	}

	// Process last point
	{
		const double IsoValue = Points2D.Last()[IsoType];

		int32 IsoCoordinateIndex = FindIsoCoordinateIndex(IsoValue);

		// If the start point is equal to the iso coordinate within the surface's tolerance
		// on the given iso direction, add the polyline's coordinate and continue
		if (FMath::IsNearlyEqual(IsoValue, IsoCoordinates[IsoCoordinateIndex], SurfaceIsoTolerance))
		{
			Intersection.Add(Coordinates.Last());
		}
	}
}

} //namespace UE::CADKernel
