// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Math/Point.h"

namespace UE::CADKernel
{

struct FSurfacicSampling
{
	bool bWithNormals = false;
	TArray<FVector2d> Points2D;
	TArray<FVector> Points3D;
	TArray<FVector3f> Normals;

	int32 Count()
	{
		return Points3D.Num();
	}

	void SetNum(int32 Number)
	{
		Points2D.SetNum(Number);
		Points3D.SetNum(Number);
		if (bWithNormals)
		{
			Normals.SetNum(Number);
		}
	}

	void Reserve(int32 Number)
	{
		Points2D.Empty(Number);
		Points3D.Empty(Number);
		if (bWithNormals)
		{
			Normals.Empty(Number);
		}
	}

	void NormalizeNormals()
	{
		for (FVector3f& Normal : Normals)
		{
			Normal.Normalize();
		}
	}

	void Set2DCoordinates(const FCoordinateGrid& Coordinates)
	{
		Reserve(Coordinates.Count());
		for (double VCoord : Coordinates[EIso::IsoV])
		{
			for (double UCoord : Coordinates[EIso::IsoU])
			{
				Points2D.Emplace(UCoord, VCoord);
			}
		}
	}
};

} // ns UE::CADKernel