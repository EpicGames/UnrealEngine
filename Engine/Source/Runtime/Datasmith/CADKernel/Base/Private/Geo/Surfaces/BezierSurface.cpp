// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/BezierSurface.h"

#include "Geo/Sampling/PolylineTools.h"
#include "Geo/GeoPoint.h"
#include "Math/BSpline.h"

namespace UE::CADKernel
{

void FBezierSurface::EvaluatePoint(const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	TArray<FVector> VCurvePoles;
	TArray<FVector> VCurveUGradient;
	TArray<FVector> VCurveULaplacian;

	VCurvePoles.SetNum(VPoleNum);

	OutPoint3D.DerivativeOrder = InDerivativeOrder;
	
	if (InDerivativeOrder > 0)
	{
		VCurveUGradient.Init(FVector::ZeroVector, VPoleNum);
	}

	if (InDerivativeOrder > 1)
	{
		VCurveULaplacian.SetNum(VPoleNum);
	}

	{
		double Coordinate = InPoint2D.X;

		TArray<FVector> UAuxiliaryPoles;
		UAuxiliaryPoles.SetNum(UPoleNum);

		TArray<FVector> UAuxiliaryGradient;
		TArray<FVector> UAuxiliaryLaplacian;

		// For each iso V Curve compute Point, Gradient and Laplacian at U coordinate
		for (int32 Vndex = 0, PoleIndex = 0; Vndex < VPoleNum; Vndex++, PoleIndex += UPoleNum)
		{
			UAuxiliaryPoles.Empty(UPoleNum);
			UAuxiliaryPoles.Append(Poles.GetData() + PoleIndex, UPoleNum);

			if (InDerivativeOrder > 0)
			{
				UAuxiliaryGradient.Init(FVector::ZeroVector, UPoleNum);
			}
			if (InDerivativeOrder > 1)
			{
				UAuxiliaryLaplacian.Init(FVector::ZeroVector, UPoleNum);
			}

			// Compute Point, Gradient and Laplacian at U coordinate with De Casteljau's algorithm, 
			for (int32 Undex = UPoleNum - 2; Undex >= 0; Undex--)
			{
				for (int32 Index = 0; Index <= Undex; Index++)
				{
					const FVector PointI = UAuxiliaryPoles[Index];
					const FVector& PointB = UAuxiliaryPoles[Index + 1];
					const FVector VectorIB = PointB - PointI;

					UAuxiliaryPoles[Index] = PointI + VectorIB * Coordinate;
					if (InDerivativeOrder > 0)
					{
						const FVector GradientI = UAuxiliaryGradient[Index];
						const FVector& GradientB = UAuxiliaryGradient[Index + 1];
						const FVector VectorGradientIB = GradientB - GradientI;

						UAuxiliaryGradient[Index] = GradientI + VectorGradientIB * Coordinate + VectorIB;
						if (InDerivativeOrder > 1)
						{
							//UAuxiliaryLaplacian[Index] = UAuxiliaryLaplacian[Index] + (UAuxiliaryLaplacian[Index + 1] - UAuxiliaryLaplacian[Index]) * Coordinate + 2.0 * VectorGradientIB;
							UAuxiliaryLaplacian[Index] = PolylineTools::LinearInterpolation(UAuxiliaryLaplacian, Index, Coordinate) + 2.0 * VectorGradientIB;
						}
					}
				}
			}

			// Point, Gradient and Laplacian of the iso v curve are saved to defined the VCurve
			VCurvePoles[Vndex] = UAuxiliaryPoles[0];
			if (InDerivativeOrder > 0)
			{
				VCurveUGradient[Vndex] = UAuxiliaryGradient[0];

				if (InDerivativeOrder > 1)
				{
					VCurveULaplacian[Vndex] = UAuxiliaryLaplacian[0];
				}
			}
		}
	}

	TArray<FVector> VAuxiliaryGradient;
	TArray<FVector> UVAuxiliaryLaplacian;
	TArray<FVector> VVAuxiliaryLaplacian;
	if (InDerivativeOrder > 0)
	{
		VAuxiliaryGradient.Init(FVector::ZeroVector, VPoleNum);
		if (InDerivativeOrder > 1)
		{
			VVAuxiliaryLaplacian.Init(FVector::ZeroVector, VPoleNum);
			UVAuxiliaryLaplacian.Init(FVector::ZeroVector, VPoleNum);
		}
	}

	double Coordinate = InPoint2D.Y;

	// Compute Point, Gradient and Laplacian at V coordinate with De Casteljau's algorithm,
	for (int32 Vndex = VPoleNum - 2; Vndex >= 0; Vndex--)
	{
		for (int32 Index = 0; Index <= Vndex; Index++)
		{
			const FVector PointI = VCurvePoles[Index];
			const FVector& PointB = VCurvePoles[Index + 1];
			const FVector VectorIB = PointB - PointI;
			VCurvePoles[Index] = PointI + VectorIB * Coordinate;

			if (InDerivativeOrder > 0) 
			{
				const FVector UGradientI = VCurveUGradient[Index];
				const FVector& UGradientB = VCurveUGradient[Index + 1];
				const FVector VectorUGradientIB = UGradientB - UGradientI;

				VCurveUGradient[Index] = UGradientI + VectorUGradientIB * Coordinate;

				const FVector VGradientI = VAuxiliaryGradient[Index]; 
				const FVector& VGradientB = VAuxiliaryGradient[Index + 1];
				const FVector VectorVGradientIB = VGradientB - VGradientI;

				VAuxiliaryGradient[Index] = VGradientI + VectorVGradientIB * Coordinate + VectorIB;

				if (InDerivativeOrder > 1)
				{
					//UVAuxiliaryLaplacian[Index] = UVAuxiliaryLaplacian[Index] + (UVAuxiliaryLaplacian[Index + 1] - UVAuxiliaryLaplacian[Index]) * Coordinate + VectorUGradientIB;
					UVAuxiliaryLaplacian[Index] = PolylineTools::LinearInterpolation(UVAuxiliaryLaplacian, Index, Coordinate) + VectorUGradientIB;

					//VCurveULaplacian[Index] = VCurveULaplacian[Index] + (VCurveULaplacian[Index + 1] - VCurveULaplacian[Index]) * Coordinate;
					VCurveULaplacian[Index] = PolylineTools::LinearInterpolation(VCurveULaplacian, Index, Coordinate);

					//VVAuxiliaryLaplacian[Index] = VVAuxiliaryLaplacian[Index] + (VVAuxiliaryLaplacian[Index + 1] - VVAuxiliaryLaplacian[Index]) * Coordinate + 2.0 * VectorVGradientIB;
					VVAuxiliaryLaplacian[Index] = PolylineTools::LinearInterpolation(VVAuxiliaryLaplacian, Index, Coordinate) + 2.0 * VectorVGradientIB;;
				}
			}
		}
	}

	OutPoint3D.Point = VCurvePoles[0];

	if (InDerivativeOrder > 0) 
	{
		OutPoint3D.GradientU = VCurveUGradient[0];
		OutPoint3D.GradientV = VAuxiliaryGradient[0];
	}

	if (InDerivativeOrder > 1) 
	{
		OutPoint3D.LaplacianU = VCurveULaplacian[0];
		OutPoint3D.LaplacianV = VVAuxiliaryLaplacian[0];
		OutPoint3D.LaplacianUV = UVAuxiliaryLaplacian[0];
	}
}

void FBezierSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates)
{
	ensureCADKernel(false);
}

TSharedPtr<FEntityGeom> FBezierSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TArray<FVector> TransformedPoles;
	TransformedPoles.Reserve(Poles.Num());
	for (const FVector& Pole : Poles) 
	{
		TransformedPoles.Emplace(InMatrix.Multiply(Pole));
	}
	return FEntity::MakeShared<FBezierSurface>(Tolerance3D, UDegre, VDegre, TransformedPoles);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FBezierSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info).Add(TEXT("degre U"), UDegre)
		.Add(TEXT("degre V"), VDegre)
		.Add(TEXT("poles"), Poles);
}
#endif

}