// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{

class CADKERNEL_API FBezierSurface : public FSurface
{
	friend FEntity;

protected:

	int32 UDegre;
	int32 VDegre;

	int32 UPoleNum;
	int32 VPoleNum;

	TArray<FVector> Poles;

	FBezierSurface(const double InToleranceGeometric, int32 InUDegre, int32 InVDegre, const TArray<FVector>& InPoles)
		: FSurface(InToleranceGeometric)
		, UDegre(InUDegre)
		, VDegre(InVDegre)
		, UPoleNum(InUDegre + 1)
		, VPoleNum(InVDegre + 1)
		, Poles(InPoles)
	{
		ensureCADKernel((UDegre + 1) * (VDegre + 1) == Poles.Num());
		ComputeDefaultMinToleranceIso();
	}

	FBezierSurface() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << UDegre;
		Ar << VDegre;
		Ar << UPoleNum;
		Ar << VPoleNum;
		Ar.Serialize(Poles);
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Bezier;
	}

	int32 GetUDegree() const
	{
		return UDegre;
	}

	int32 GetVDegree() const
	{
		return VDegre;
	}

	virtual const TArray<FVector>& GetPoles() const
	{
		return Poles;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif
};

} // namespace UE::CADKernel

