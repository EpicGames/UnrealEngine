// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/CompositeSurface.h"

#include "Utils/ArrayUtils.h"
#include "Utils/IndexOfCoordinateFinder.h"

namespace UE::CADKernel
{

FCompositeSurface::FCompositeSurface(const double InToleranceGeometric, int32 USurfaceNum, int32 VSurfaceNum, const TArray<double>& UCoordinates, const TArray<double>& VCoordinates, const TArray<TSharedPtr<FSurface>>& InSurfaces)
	: FSurface(InToleranceGeometric)
	, Surfaces(InSurfaces)
{
	ensureCADKernel(USurfaceNum * VSurfaceNum == (int32)InSurfaces.Num());

	NativeUVBoundaries.Reserve(InSurfaces.Num());

	GlobalCoordinates[EIso::IsoU] = UCoordinates;
	GlobalCoordinates[EIso::IsoV] = VCoordinates;

	for (const TSharedPtr<FSurface>& Surface : InSurfaces)
	{
		const FSurfacicBoundary& Bounds = Surface->GetBoundary();
		NativeUVBoundaries.Emplace(Bounds);
	}
	ComputeDefaultMinToleranceIso();
}

void FCompositeSurface::InitBoundary()
{
	Boundary[EIso::IsoU].Min = GlobalCoordinates[EIso::IsoU][0];
	Boundary[EIso::IsoV].Min = GlobalCoordinates[EIso::IsoV][0];
	Boundary[EIso::IsoU].Max = GlobalCoordinates[EIso::IsoU][GetSurfNum(EIso::IsoU)];
	Boundary[EIso::IsoV].Max = GlobalCoordinates[EIso::IsoV][GetSurfNum(EIso::IsoV)];
}


void FCompositeSurface::LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const
{
	for (int32 Iso = EIso::IsoU; Iso <= EIso::IsoV; Iso++)
	{
		for (int32 i = 0; i < GlobalCoordinates[Iso].Num(); i++)
		{
			OutNotDerivableCoordinates[(EIso) Iso].Add(GlobalCoordinates[Iso][i]);
		}
	}

	int32 SurfaceIndex = 0;
	for (const TSharedPtr<FSurface>& Surface : Surfaces)
	{
		FCoordinateGrid SurfaceNotDerivableCoordinates;
		Surface->LinesNotDerivables(InDerivativeOrder, SurfaceNotDerivableCoordinates);

		for (int32 Iso = EIso::IsoU; Iso <= EIso::IsoV; Iso++)
		{
			for (double Coord : SurfaceNotDerivableCoordinates[(EIso) Iso])
			{
				OutNotDerivableCoordinates[(EIso) Iso].Add(LocalToGlobalCoordinate(SurfaceIndex, Coord, (EIso) Iso));
			}
		}

		SurfaceIndex++;
	}

	for (int32 Iso = EIso::IsoU; Iso <= EIso::IsoV; Iso++)
	{
		Algo::Sort(OutNotDerivableCoordinates[(EIso)Iso]);
		ArrayUtils::RemoveDuplicates(OutNotDerivableCoordinates[(EIso)Iso], GetIsoTolerances()[Iso]);
	}
}

void FCompositeSurface::EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	FVector2d Coordinate = InSurfacicCoordinate;

	Boundary.MoveInsideIfNot(Coordinate);

	FVector2d Point2DPatch;
	FDichotomyFinder FinderU(GlobalCoordinates[EIso::IsoU]);
	int32 PatchUIndex = FinderU.Find(Coordinate.X);
	Point2DPatch.X = (Coordinate.X - GlobalCoordinates[EIso::IsoU][PatchUIndex]) / (GlobalCoordinates[EIso::IsoU][PatchUIndex + 1] - GlobalCoordinates[EIso::IsoU][PatchUIndex]);

	FDichotomyFinder FinderV(GlobalCoordinates[EIso::IsoV]);
	int32 PatchVIndex = FinderV.Find(Coordinate.Y);
	Point2DPatch.Y = (Coordinate.Y - GlobalCoordinates[EIso::IsoV][PatchVIndex]) / (GlobalCoordinates[EIso::IsoV][PatchVIndex + 1] - GlobalCoordinates[EIso::IsoV][PatchVIndex]);

	int32 SurfaceIndex = PatchVIndex * GetSurfNum(EIso::IsoU) + PatchUIndex;

	ensureCADKernel(SurfaceIndex == Surfaces.Num());

	Point2DPatch.X = NativeUVBoundaries[SurfaceIndex][EIso::IsoU].GetAt(Point2DPatch.X);
	Point2DPatch.Y = NativeUVBoundaries[SurfaceIndex][EIso::IsoV].GetAt(Point2DPatch.Y);

	Surfaces[SurfaceIndex]->EvaluatePoint(Point2DPatch, OutPoint3D, InDerivativeOrder);
}

void FCompositeSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates)
{
	ensureCADKernel(false);
}

double FCompositeSurface::LocalToGlobalCoordinate(int32 SurfaceIndex, double Coordinate, EIso Iso) const
{
	ensureCADKernel(!(RealCompare(Coordinate, NativeUVBoundaries[SurfaceIndex][Iso].Min) < 0 || RealCompare(Coordinate, NativeUVBoundaries[SurfaceIndex][Iso].Max) > 0));

	double GlobalCoord = (Coordinate - NativeUVBoundaries[SurfaceIndex][Iso].Min);
	if (!FMath::IsNearlyZero(NativeUVBoundaries[SurfaceIndex][Iso].Max - NativeUVBoundaries[SurfaceIndex][Iso].Min))
	{
		GlobalCoord = GlobalCoord / (NativeUVBoundaries[SurfaceIndex][Iso].Max - NativeUVBoundaries[SurfaceIndex][Iso].Min);
	}
	GlobalCoord = GlobalCoordinates[Iso][SurfaceIndex] + GlobalCoord * (GlobalCoordinates[Iso][SurfaceIndex + 1] - GlobalCoordinates[Iso][SurfaceIndex]);
	return GlobalCoord;
}

TSharedPtr<FEntityGeom> FCompositeSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FSurface> TransformedSurface;
	TArray<TSharedPtr<FSurface>> TransformedSurfaces;
	TransformedSurfaces.Reserve(Surfaces.Num());
	for (const TSharedPtr<FSurface>& Surface : Surfaces)
	{
		TransformedSurface = StaticCastSharedPtr<FSurface>(Surface->ApplyMatrix(InMatrix));
		TransformedSurfaces.Add(TransformedSurface);
	}

	return FEntity::MakeShared<FCompositeSurface>(Tolerance3D, GetSurfNum(EIso::IsoU), GetSurfNum(EIso::IsoV), GlobalCoordinates[EIso::IsoU], GlobalCoordinates[EIso::IsoV], TransformedSurfaces);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FCompositeSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("U Surf count"), GetSurfNum(EIso::IsoU))
		.Add(TEXT("V Surf count"), GetSurfNum(EIso::IsoV))
		.Add(TEXT("Surfaces"), Surfaces);
}
#endif

}