// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{

class CADKERNEL_API FCompositeSurface : public FSurface
{
	friend FEntity;

protected:

	/**
	 * U & V Coordinates of the composite surface's parametric space at each change of elementary surface
	 */
	TArray<double> GlobalCoordinates[2];

	/**
	 * Elementary surfaces
	 */
	TArray<TSharedPtr<FSurface>> Surfaces;

	/**
	 * Native UV Loops of each surface composing the composite surface
	 */
	TArray<FSurfacicBoundary> NativeUVBoundaries;

	FCompositeSurface(const double InToleranceGeometric, int32 USurfaceNum, int32 VSurfaceNum, const TArray<double>& UCoordinates, const TArray<double>& VCoordinates, const TArray<TSharedPtr<FSurface>>& Surfaces);

	FCompositeSurface() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << GlobalCoordinates[EIso::IsoU];
		Ar << GlobalCoordinates[EIso::IsoV];
		SerializeIdents(Ar, Surfaces);
		Ar << NativeUVBoundaries;
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities(Surfaces, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities(Surfaces);
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Composite;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	int32 GetSurfNum(EIso Iso) const
	{
		return GlobalCoordinates[Iso].Num() - 1;
	}

	virtual void InitBoundary() override;

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override;

	double LocalToGlobalCoordinate(int32 SurfaceIndex, double CoordinateU, EIso Iso) const;
};

} // namespace UE::CADKernel


