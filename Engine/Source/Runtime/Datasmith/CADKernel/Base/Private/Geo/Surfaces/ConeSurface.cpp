// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/ConeSurface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicSampling.h"
#include "Math/Boundary.h"

namespace UE::CADKernel
{

void FConeSurface::EvaluatePoint(const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	OutPoint3D.DerivativeOrder = InDerivativeOrder;

	double DeltaVR = tan(ConeAngle);
	double Radius = StartRadius + InPoint2D.Y * DeltaVR;

	OutPoint3D.Point.Set(Radius * cos(InPoint2D.X), Radius * sin(InPoint2D.X), InPoint2D.Y);

	OutPoint3D.Point = Matrix.Multiply(OutPoint3D.Point);

	if (InDerivativeOrder > 0)
	{
		OutPoint3D.GradientU = FVector(-Radius * sin(InPoint2D.X), Radius * cos(InPoint2D.X), 0.);
		OutPoint3D.GradientV = FVector(DeltaVR * cos(InPoint2D.X), DeltaVR * sin(InPoint2D.X), 1.0);

		OutPoint3D.GradientU = Matrix.MultiplyVector(OutPoint3D.GradientU);
		OutPoint3D.GradientV = Matrix.MultiplyVector(OutPoint3D.GradientV);

		if (InDerivativeOrder > 1)
		{
			OutPoint3D.LaplacianU = FVector(-Radius * cos(InPoint2D.X), -Radius * sin(InPoint2D.X), 0.);
			OutPoint3D.LaplacianV = FVector(0., 0., 0.);

			OutPoint3D.LaplacianUV = FVector(-DeltaVR * sin(InPoint2D.X), DeltaVR * cos(InPoint2D.X), 0.);

			OutPoint3D.LaplacianU = Matrix.MultiplyVector(OutPoint3D.LaplacianU);
			OutPoint3D.LaplacianV = Matrix.MultiplyVector(OutPoint3D.LaplacianV);
			OutPoint3D.LaplacianUV = Matrix.MultiplyVector(OutPoint3D.LaplacianUV);
		}
	}

}

void FConeSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;

	int32 PointNum = Coordinates.Count();
	OutPoints.Reserve(PointNum);

	OutPoints.Set2DCoordinates(Coordinates);

	int32 UCount = Coordinates.IsoCount(EIso::IsoU);
	int32 VCount = Coordinates.IsoCount(EIso::IsoV);

	TArray<double> CosU;
	TArray<double> SinU;
	CosU.Reserve(UCount);
	SinU.Reserve(UCount);

	double DeltaVR = tan(ConeAngle);

	for (double Angle : Coordinates[EIso::IsoU])
	{
		CosU.Emplace(cos(Angle));
	}

	for (double Angle : Coordinates[EIso::IsoU])
	{
		SinU.Emplace(sin(Angle));
	}

	for (int32 Vndex = 0, Index = 0; Vndex < VCount; ++Vndex)
	{
		double Radius = StartRadius + Coordinates[EIso::IsoV][Vndex] * DeltaVR;

		for (int32 Undex = 0; Undex < UCount; Undex++)
		{
			FVector& Point = OutPoints.Points3D.Emplace_GetRef(Radius * CosU[Undex], Radius * SinU[Undex], Coordinates[EIso::IsoV][Vndex]);
			Point = Matrix.Multiply(Point);
		}
	}

	if (bComputeNormals)
	{
		for (int32 Vndex = 0, Index = 0; Vndex < VCount; ++Vndex)
		{
			double Radius = StartRadius + Vndex * DeltaVR;
			for (int32 Undex = 0; Undex < UCount; Undex++)
			{
				FVector GradientU(-Radius * SinU[Undex], Radius * CosU[Undex], 0.);
				FVector GradientV(DeltaVR * CosU[Undex], DeltaVR * SinU[Undex], 1.0);
				GradientU = Matrix.MultiplyVector(GradientU);
				GradientV = Matrix.MultiplyVector(GradientV);
				const FVector Normal = GradientU ^ GradientV;
				OutPoints.Normals.Emplace(Normal);
			}
		}

		OutPoints.NormalizeNormals();
	}
}

TSharedPtr<FEntityGeom> FConeSurface::ApplyMatrix(const FMatrixH& NewMatrix) const
{
	FMatrixH Mat = NewMatrix * Matrix;
	return FEntity::MakeShared<FConeSurface>(Tolerance3D, Mat, StartRadius, ConeAngle, Boundary[EIso::IsoU].Min, Boundary[EIso::IsoU].Max, Boundary[EIso::IsoV].Min, Boundary[EIso::IsoV].Max);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FConeSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("StartRadius"), StartRadius)
		.Add(TEXT("ConeAngle"), ConeAngle)
		.Add(TEXT("StartAngle"), Boundary[EIso::IsoU].Min)
		.Add(TEXT("EndAngle"), Boundary[EIso::IsoU].Max)
		.Add(TEXT("StartRuleLength"), Boundary[EIso::IsoV].Min)
		.Add(TEXT("EndRuleLength"), Boundary[EIso::IsoV].Max);
}
#endif

}