// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class CADKERNEL_API FConeSurface : public FSurface
{
	friend FEntity;

protected:
	FMatrixH Matrix;
	double   StartRadius;
	double   ConeAngle;

	/**
	 * A Cone surface is defined by a point on the axis of the cone, the direction of the axis of the cone,
	 * the radius of the cone at the axis point and the cone semi-angle.
	 *
	 * The local coordinate system is defined with the origin at the axis point and the Z axis in the axis direction,
	 * The equation of the surface in this system is x2 + y2 – (StartRadius + z*tan(ConeAngle) )2 = 0
	 * where ConeAngle is the cone semi-angle and StartRadius is the given cone radius.
	 *
	 * The cone is placed at its final position and orientation by the Matrix
	 */
	FConeSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InStartRadius, double InConeAngle, double InStartRuleLength = -HUGE_VALUE, double InEndRuleLength = HUGE_VALUE, double InStartAngle = 0.0, double InEndAngle = DOUBLE_TWO_PI)
		: FConeSurface(InToleranceGeometric, InMatrix, InStartRadius, InConeAngle, FSurfacicBoundary(InStartAngle, InEndAngle, InStartRuleLength, InEndRuleLength))
	{
	}

	/**
	 * A Cone surface is defined by a point on the axis of the cone, the direction of the axis of the cone,
	 * the radius of the cone at the axis point and the cone semi-angle.
	 *
	 * The local coordinate system is defined with the origin at the axis point and the Z axis in the axis direction,
	 * The equation of the surface in this system is x2 + y2 – (StartRadius + z*tan(ConeAngle) )2 = 0
	 * where ConeAngle is the cone semi-angle and StartRadius is the given cone radius.
	 *
	 * The cone is placed at its final position and orientation by the Matrix
	 *
	 * The bounds of the cone are defined as follow:
	 * Bounds[EIso::IsoU].Min = StartRuleLength;
	 * Bounds[EIso::IsoU].Max = EndRuleLength;
	 * Bounds[EIso::IsoV].Min = StartAngle;
	 * Bounds[EIso::IsoV].Max = EndAngle;
	 */
	FConeSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InStartRadius, double InConeAngle, const FSurfacicBoundary& InBoundary)
		: FSurface(InToleranceGeometric, InBoundary)
		, Matrix(InMatrix)
		, StartRadius(InStartRadius)
		, ConeAngle(InConeAngle)
	{
		ComputeMinToleranceIso();
	}

	FConeSurface() = default;

	void ComputeMinToleranceIso()
	{
		FVector Origin = Matrix.Multiply(FVector::ZeroVector);

		double DeltaVR = tan(ConeAngle);
		double Radius1 = FMath::Abs(StartRadius + Boundary[EIso::IsoV].Max * DeltaVR);
		double Radius2 = FMath::Abs(StartRadius + Boundary[EIso::IsoV].Min * DeltaVR);
		double Radius = FMath::Max(Radius2, Radius1);

		FVector Point2DU{ 1 , 0, 0 };
		double ScaleU = ComputeScaleAlongAxis(Point2DU, Matrix, Origin);
		double ToleranceU = Tolerance3D / Radius;
		ToleranceU /= ScaleU;

		FVector Point2DV{ 0, 1, 0 };
		double ScaleV = ComputeScaleAlongAxis(Point2DV, Matrix, Origin);
		double ToleranceV = Tolerance3D / FMath::Sqrt(FMath::Square(DeltaVR) + 1);
		ToleranceV /= ScaleV;

		MinToleranceIso.Set(ToleranceU, ToleranceV);
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << Matrix;
		Ar << StartRadius;
		Ar << ConeAngle;
	}


	ESurface GetSurfaceType() const
	{
		return ESurface::Cone;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override
	{
		PresampleIsoCircle(InBoundaries, OutCoordinates, EIso::IsoU);

		OutCoordinates[EIso::IsoV].Empty(3);
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Min);
		OutCoordinates[EIso::IsoV].Add((InBoundaries[EIso::IsoV].Max + InBoundaries[EIso::IsoV].Min) / 2.0);
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Max);
	}

};
}

