// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/CoonsSurface.h"

#include "Geo/Curves/Curve.h"
#include "Utils/ArrayUtils.h"

namespace UE::CADKernel
{

FCoonsSurface::FCoonsSurface(const double InToleranceGeometric, TSharedPtr<FCurve> InCurves[4])
	: FCoonsSurface(InToleranceGeometric, InCurves[0], InCurves[1], InCurves[2], InCurves[3])
{
	ComputeDefaultMinToleranceIso();
}

FCoonsSurface::FCoonsSurface(const double InToleranceGeometric, TSharedPtr<FCurve> InCurve1, TSharedPtr<FCurve> InCurve2, TSharedPtr<FCurve> InCurve3, TSharedPtr<FCurve> InCurve4)
	: FSurface(InToleranceGeometric)
{
	Curves[0] = InCurve1;
	Curves[1] = InCurve2;
	Curves[2] = InCurve3;
	Curves[3] = InCurve4;

	Corners.SetNum(4);
	Corners[0] = Curves[0]->EvaluatePoint(Curves[0]->GetUMin()) * 0.5 + Curves[2]->EvaluatePoint(Curves[2]->GetUMin()) * 0.5;
	Corners[1] = Curves[0]->EvaluatePoint(Curves[0]->GetUMax()) * 0.5 + Curves[3]->EvaluatePoint(Curves[3]->GetUMin()) * 0.5;
	Corners[2] = Curves[1]->EvaluatePoint(Curves[1]->GetUMin()) * 0.5 + Curves[2]->EvaluatePoint(Curves[2]->GetUMax()) * 0.5;
	Corners[3] = Curves[1]->EvaluatePoint(Curves[1]->GetUMax()) * 0.5 + Curves[3]->EvaluatePoint(Curves[3]->GetUMax()) * 0.5;

	ComputeDefaultMinToleranceIso();
}

void FCoonsSurface::LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const
{
	TFunction<void(const TSharedPtr<FCurve>&, int32, TArray<double>&)> FindLinesNotDerivables = [&](const TSharedPtr<FCurve>& Curve, int32 CurveIndex, TArray<double>& NotDerivables)
	{
		TArray<double> TmpCoords;
		Curve->FindNotDerivableCoordinates(InDerivativeOrder, TmpCoords);
		double Length = Curve->GetUMax() - Curve->GetUMin();
		for (int32 Index = 0; Index < TmpCoords.Num(); Index++)
		{
			NotDerivables.Add((TmpCoords[Index] - Curve->GetUMin()) / Length);
		}
	};

	FindLinesNotDerivables(Curves[0], 0, OutNotDerivableCoordinates[EIso::IsoU]);
	FindLinesNotDerivables(Curves[1], 1, OutNotDerivableCoordinates[EIso::IsoU]);
	FindLinesNotDerivables(Curves[2], 2, OutNotDerivableCoordinates[EIso::IsoV]);
	FindLinesNotDerivables(Curves[3], 3, OutNotDerivableCoordinates[EIso::IsoV]);

	for (int32 Iso = EIso::IsoU; Iso <= EIso::IsoV; Iso++)
	{
		Algo::Sort(OutNotDerivableCoordinates[(EIso)Iso]);
		ArrayUtils::RemoveDuplicates(OutNotDerivableCoordinates[(EIso)Iso], GetIsoTolerances()[Iso]);
	}
}

void FCoonsSurface::EvaluatePoint(const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	TFunction<void(const TSharedPtr<FCurve>*, const FVector2d&, EIso, FVector&, FCurvePoint[])> \
	ComputePointOnRuledSurface = [&](const TSharedPtr<FCurve>* InCurves, const FVector2d& Coord, EIso Iso, FVector& OutPoint, FCurvePoint OutCurvePoints[2])
	{
		const int32 IndexCurve0 = Iso * 2;
		const int32 IndexCurve1 = Iso * 2 + 1;

		EIso OtherIso = Iso == EIso::IsoU ? EIso::IsoV : EIso::IsoU;

		double CoordinateCurve0 = InCurves[IndexCurve0]->GetUMin() + Coord[Iso] * (InCurves[IndexCurve0]->GetUMax() - InCurves[IndexCurve0]->GetUMin());
		double CoordinateCurve1 = InCurves[IndexCurve1]->GetUMin() + Coord[Iso] * (InCurves[IndexCurve1]->GetUMax() - InCurves[IndexCurve1]->GetUMin());

		InCurves[IndexCurve0]->EvaluatePoint(CoordinateCurve0, OutCurvePoints[0], InDerivativeOrder);
		InCurves[IndexCurve1]->EvaluatePoint(CoordinateCurve1, OutCurvePoints[1], InDerivativeOrder);

		OutPoint = OutCurvePoints[0].Point + (OutCurvePoints[1].Point - OutCurvePoints[0].Point) * Coord[OtherIso];

		if (InDerivativeOrder > 0)
		{
			OutCurvePoints[0].Gradient *= (InCurves[IndexCurve0]->GetUMax() - InCurves[IndexCurve0]->GetUMin());
			OutCurvePoints[1].Gradient *= (InCurves[IndexCurve1]->GetUMax() - InCurves[IndexCurve1]->GetUMin());
			if (InDerivativeOrder > 1)
			{
				OutCurvePoints[0].Laplacian *= FMath::Square(InCurves[IndexCurve0]->GetUMax() - InCurves[IndexCurve0]->GetUMin());
				OutCurvePoints[1].Laplacian *= FMath::Square(InCurves[IndexCurve1]->GetUMax() - Curves[IndexCurve1]->GetUMin());
			}
		}
	};

	// 1. First ruled Surface
	FVector RuledPoint1;
	FCurvePoint Curve12Points[2];
	ComputePointOnRuledSurface(Curves, InPoint2D, EIso::IsoU, RuledPoint1, Curve12Points);
	
	// 2. Second ruled Surface
	FVector RuledPoint2;
	FCurvePoint Curve34Points[2];
	ComputePointOnRuledSurface(Curves+2, InPoint2D, EIso::IsoV, RuledPoint2, Curve34Points);

	// 3. bilinear surface
	FVector BilinearPoint = (Corners[0]*(1.0-InPoint2D.X) + Corners[1]*InPoint2D.X)*(1.0-InPoint2D.Y) + (Corners[2]*(1.0-InPoint2D.X) + Corners[3]*InPoint2D.X)*InPoint2D.Y;

	OutPoint3D.DerivativeOrder = InDerivativeOrder;

	// Mixed of ruled surfaces et bilinear Surface
	OutPoint3D.Point = RuledPoint1 + RuledPoint2 - BilinearPoint;

	if (InDerivativeOrder > 0)
	{
		FVector DeltaU1 = Curve12Points[0].Gradient + InPoint2D.Y * (Curve12Points[1].Gradient - Curve12Points[0].Gradient);
		FVector DeltaU2 = Curve34Points[1].Point - Curve34Points[0].Point;
		FVector DeltaU3 = (Corners[1] * (1.0 - InPoint2D.Y) + Corners[3] * InPoint2D.Y) - (Corners[0] * (1.0 - InPoint2D.Y) + Corners[2] * InPoint2D.Y);
		OutPoint3D.GradientU = DeltaU1 + DeltaU2 - DeltaU3;

		FVector DeltaV1 = Curve12Points[1].Point - Curve12Points[0].Point;;
		FVector DeltaV2 = Curve34Points[0].Gradient + InPoint2D.X * (Curve34Points[1].Gradient - Curve34Points[0].Gradient);
		FVector DeltaV3 = (Corners[2] * (1.0 - InPoint2D.X) + Corners[3] * InPoint2D.X) - (Corners[0] * (1.0 - InPoint2D.X) + Corners[1] * InPoint2D.X);
		OutPoint3D.GradientV = DeltaV1 + DeltaV2 - DeltaV3;

		if (InDerivativeOrder > 1)
		{
			FVector LaplacianUV1 = (Curve12Points[1].Laplacian - Curve12Points[0].Laplacian);
			FVector LaplacianUV2 = (Curve34Points[1].Laplacian - Curve34Points[0].Laplacian);

			OutPoint3D.LaplacianU = Curve12Points[0].Laplacian + InPoint2D.Y * LaplacianUV1;
			OutPoint3D.LaplacianV = Curve34Points[0].Laplacian + InPoint2D.X * LaplacianUV2;

			OutPoint3D.LaplacianUV = LaplacianUV1 + LaplacianUV2;
		}
	}
}

TSharedPtr<FEntityGeom> FCoonsSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FCurve> TransformedCurves[4];

	for (int32 Index = 0; Index < 4; ++Index)
	{
		TransformedCurves[Index] = StaticCastSharedPtr<FCurve>(Curves[Index]->ApplyMatrix(InMatrix));
		if (!TransformedCurves[Index].IsValid())
		{
			return TSharedPtr<FEntityGeom>();
		}
	}

	return FEntity::MakeShared<FCoonsSurface>(Tolerance3D, TransformedCurves);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FCoonsSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info).Add(TEXT("curve 1"), Curves[0])
							      .Add(TEXT("curve 2"), Curves[1])
							      .Add(TEXT("curve 3"), Curves[2])
							      .Add(TEXT("curve 4"), Curves[3]);
}
#endif

}