// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class FCurve;
class FNURBSSurface;

class CADKERNEL_API FCoonsSurface : public FSurface
{
	friend FEntity;

protected:
	TSharedPtr<FCurve> Curves[4];
	TArray<FVector> Corners;

	FCoonsSurface(const double InToleranceGeometric, TSharedPtr<FCurve> InCurve1, TSharedPtr<FCurve> InCurve2, TSharedPtr<FCurve> InCurve3, TSharedPtr<FCurve> InCurve4);
	FCoonsSurface(const double InToleranceGeometric, TSharedPtr<FCurve> InCurves[4]);
	FCoonsSurface() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		SerializeIdent(Ar, Curves[0]);
		SerializeIdent(Ar, Curves[1]);
		SerializeIdent(Ar, Curves[2]);
		SerializeIdent(Ar, Curves[3]);
		Ar << Corners;
	}


#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ESurface GetSurfaceType() const override
	{
		return ESurface::Coons;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const override;
};

} // namespace UE::CADKernel

