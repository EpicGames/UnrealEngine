// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/CylinderSurface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{

FCylinderSurface::FCylinderSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, double InStartLength, double InEndLength, double InStartAngle, double InEndAngle)
	: FCylinderSurface(InToleranceGeometric, InMatrix, InRadius, FSurfacicBoundary(InStartAngle, InEndAngle, InStartLength, InEndLength))
{
}

FCylinderSurface::FCylinderSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const double InRadius, const FSurfacicBoundary& InBoundary)
: FSurface(InToleranceGeometric, InBoundary)
, Matrix(InMatrix)
, Radius(InRadius)
{
	ComputeMinToleranceIso();
}

void FCylinderSurface::InitBoundary()
{
}

void FCylinderSurface::EvaluatePoint(const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	OutPoint3D.DerivativeOrder = InDerivativeOrder;

	OutPoint3D.Point.Set(Radius*(cos(InPoint2D.X)), Radius*(sin(InPoint2D.X)), InPoint2D.Y);
	OutPoint3D.Point = Matrix.Multiply(OutPoint3D.Point);

	if(InDerivativeOrder>0) 
	{
		OutPoint3D.GradientU = FVector(Radius*(-sin(InPoint2D.X)), Radius*(cos(InPoint2D.X)), 0.0);
		OutPoint3D.GradientV = FVector(0.0, 0.0, 1.0);

		OutPoint3D.GradientU = Matrix.MultiplyVector(OutPoint3D.GradientU);
		OutPoint3D.GradientV = Matrix.MultiplyVector(OutPoint3D.GradientV);

		if (InDerivativeOrder > 1) 
		{
			OutPoint3D.LaplacianU = FVector(Radius * (-cos(InPoint2D.X)), Radius * (-sin(InPoint2D.X)), 0.0);
			OutPoint3D.LaplacianU = Matrix.MultiplyVector(OutPoint3D.LaplacianU);

			OutPoint3D.LaplacianV = FVector(0.0, 0.0, 0.0);
			OutPoint3D.LaplacianUV = FVector(0.0, 0.0, 0.0);
		}
	}
}

TSharedPtr<FEntityGeom> FCylinderSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FCylinderSurface>(Tolerance3D, NewMatrix, Radius, Boundary[EIso::IsoV].Min, Boundary[EIso::IsoU].Max, Boundary[EIso::IsoU].Min, Boundary[EIso::IsoU].Max);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FCylinderSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info).Add(TEXT("Matrix"), Matrix)
							 .Add(TEXT("Radius"), Radius)
							 .Add(TEXT("StartAngle"), Boundary[EIso::IsoU].Min)
							 .Add(TEXT("EndAngle"), Boundary[EIso::IsoU].Max)
							 .Add(TEXT("StartLength"), Boundary[EIso::IsoV].Min)
							 .Add(TEXT("EndLength"), Boundary[EIso::IsoV].Max);
}
#endif

}