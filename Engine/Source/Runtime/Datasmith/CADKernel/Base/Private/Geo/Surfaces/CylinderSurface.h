// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class FNURBSSurface;

class CADKERNEL_API FCylinderSurface : public FSurface
{
	friend FEntity;

protected:
	FMatrixH Matrix;
	double Radius;

	/**
	 * The Cylinder surface is a right circular cylinder surface starting at the axis origin and with z axis as the axis direction.
	 * It is defined by its Radius and its bounds (StartAngle, EndAngle, StartLength, EndLength)
	 *
	 * The surface is placed at its final position and orientation by the Matrix
	 *
	 * The bounds of the cylinder are defined as follow:
	 * Bounds[EIso::IsoU].Min = StartAngle;
	 * Bounds[EIso::IsoU].Max = EndAngle;
	 * Bounds[EIso::IsoV].Min = StartLength;
	 * Bounds[EIso::IsoV].Max = EndLength;
	 */
	FCylinderSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, double InStartLength = -HUGE_VALUE, double InEndLength = HUGE_VALUE, double InStartAngle = 0.0, double InEndAngle = DOUBLE_TWO_PI);
	FCylinderSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const double InRadius, const FSurfacicBoundary& Boundary);

	FCylinderSurface() = default;

	void ComputeMinToleranceIso()
	{
		FVector Origin = Matrix.Multiply(FVector::ZeroVector);

		FVector Point2DU{ 1 , 0, 0 };
		double ToleranceU = Tolerance3D / Radius;

		double ScaleU = ComputeScaleAlongAxis(Point2DU, Matrix, Origin);
		ToleranceU /= ScaleU;

		FVector Point2DV{ 0, 1, 0 };
		double ToleranceV = Tolerance3D / ComputeScaleAlongAxis(Point2DV, Matrix, Origin);

		MinToleranceIso.Set(ToleranceU, ToleranceV);
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << Matrix;
		Ar << Radius;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	ESurface GetSurfaceType() const
	{
		return ESurface::Cylinder;
	}

	virtual void InitBoundary() override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override
	{
		PresampleIsoCircle(InBoundaries, OutCoordinates, EIso::IsoU);

		OutCoordinates[EIso::IsoV].Empty(3);
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Min);
		OutCoordinates[EIso::IsoV].Add((InBoundaries[EIso::IsoV].Max + InBoundaries[EIso::IsoV].Min) / 2.0);
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Max);
	}

};
}

