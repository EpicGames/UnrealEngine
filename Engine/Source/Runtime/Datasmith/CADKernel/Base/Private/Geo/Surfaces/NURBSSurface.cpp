// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/NURBSSurface.h"

#include "Geo/GeoPoint.h"
#include "Math/Aabb.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FNURBSSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TArray<FVector> TransformedPoles;
	TransformedPoles.Reserve(Poles.Num());

	for (const FVector& Pole : Poles)
	{
		TransformedPoles.Emplace(InMatrix.Multiply(Pole));
	}

	return FEntity::MakeShared<FNURBSSurface>(Tolerance3D, PoleUCount, PoleVCount, UDegree, VDegree, UNodalVector, VNodalVector, TransformedPoles, Weights);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FNURBSSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("Degre"), UDegree, VDegree)
		.Add(TEXT("Is Rational"), bIsRational)
		.Add(TEXT("Poles Num"), PoleUCount, PoleVCount)
		.Add(TEXT("Nodal Vector U"), UNodalVector)
		.Add(TEXT("Nodal Vector V"), VNodalVector)
		.Add(TEXT("Poles"), Poles)
		.Add(TEXT("Weights"), Weights);
}
#endif

void FNURBSSurface::Finalize()
{
	if (bIsRational && Weights.IsEmpty())
	{
		bIsRational = false;
	}

	if (bIsRational)
	{
		const double FirstWeigth = Weights[0];

		bool bIsReallyRational = false;
		for (const double& Weight : Weights)
		{
			if (!FMath::IsNearlyEqual(Weight, FirstWeigth))
			{
				bIsReallyRational = true;
				break;
			}
		}

		if (!bIsReallyRational)
		{
			if (!FMath::IsNearlyEqual(1., FirstWeigth))
			{
				for (FVector& Pole : Poles)
				{
					Pole /= FirstWeigth;
				}
			}
			bIsRational = false;
		}
	}

	if (bIsRational)
	{
		HomogeneousPoles.SetNum(Poles.Num() * 4);
		for (int32 Index = 0, Jndex = 0; Index < Poles.Num(); Index++)
		{
			HomogeneousPoles[Jndex++] = Poles[Index].X * Weights[Index];
			HomogeneousPoles[Jndex++] = Poles[Index].Y * Weights[Index];
			HomogeneousPoles[Jndex++] = Poles[Index].Z * Weights[Index];
			HomogeneousPoles[Jndex++] = Weights[Index];
		}
	}
	else
	{
		HomogeneousPoles.SetNum(Poles.Num() * 3);
		memcpy(HomogeneousPoles.GetData(), Poles.GetData(), sizeof(FVector) * Poles.Num());
	}

	double UMin = UNodalVector[UDegree];
	double UMax = UNodalVector[UNodalVector.Num() - 1 - UDegree];
	double VMin = VNodalVector[VDegree];
	double VMax = VNodalVector[VNodalVector.Num() - 1 - VDegree];

	Boundary.Set(UMin, UMax, VMin, VMax);
	ComputeMinToleranceIso();
}

void FNURBSSurface::FillNurbs(FNurbsSurfaceHomogeneousData& NurbsData)
{
	bIsRational = NurbsData.bIsRational;

	PoleUCount = NurbsData.PoleUCount;
	PoleVCount = NurbsData.PoleVCount;

	UDegree = NurbsData.UDegree;
	VDegree = NurbsData.VDegree;

	Swap(UNodalVector, NurbsData.UNodalVector);
	Swap(VNodalVector, NurbsData.VNodalVector);

	int32 PoleCount = PoleUCount * PoleVCount;
	Poles.Reserve(PoleCount);

	const int32 Dimension = bIsRational ? 4 : 3;
	const int32 UOffSet = Dimension * PoleVCount;
	const double* HomogeneousPolesPtr = NurbsData.HomogeneousPoles.GetData();
	for (int32 Vndex = 0, VOffset = 0; Vndex < PoleVCount; ++Vndex, VOffset += Dimension)
	{
		const double* RawPolesPtr = HomogeneousPolesPtr + VOffset;
		for (int32 Undex = 0; Undex < PoleUCount; ++Undex, RawPolesPtr += UOffSet)
		{
			Poles.Emplace(RawPolesPtr[0], RawPolesPtr[1], RawPolesPtr[2]);
		}
	}

	if(bIsRational)
	{
		Weights.Reserve(PoleCount);
		for (int32 Vndex = 3, VOffset = 3; Vndex < (PoleVCount * Dimension); Vndex += Dimension, VOffset += Dimension)
		{
			const double* RawPolesPtr = HomogeneousPolesPtr + VOffset;
			for (int32 Undex = 0; Undex < PoleUCount; ++Undex, RawPolesPtr += UOffSet)
			{
				Weights.Emplace(*RawPolesPtr);
			}
		}
	}
	Finalize();
}

void FNURBSSurface::ComputeMinToleranceIso()
{
	double LengthU = 0;
	double LengthV = 0;

	for (int32 IndexV = 0, Index = 0; IndexV < PoleVCount; IndexV++)
	{
		FAABB ControlPolygonAABB;
		for (int32 IndexU = 0; IndexU < PoleUCount; IndexU++, Index++)
		{
			ControlPolygonAABB += Poles[Index];
		}
		double Length = ControlPolygonAABB.DiagonalLength();
		if (Length > LengthU)
		{
			LengthU = Length;
		}
	}

	for (int32 IndexU = 0; IndexU < PoleUCount; IndexU++)
	{
		FAABB ControlPolygonAABB;
		for (int32 IndexV = 0, Index = IndexU; IndexV < PoleVCount; IndexV++, Index += PoleUCount)
		{
			ControlPolygonAABB += Poles[Index];
		}
		double Length = ControlPolygonAABB.DiagonalLength();
		if (Length > LengthV)
		{
			LengthV = Length;
		}
	}

	double ToleranceU = Tolerance3D * Boundary[EIso::IsoU].Length() / LengthU * 0.1;
	double ToleranceV = Tolerance3D * Boundary[EIso::IsoV].Length() / LengthV * 0.1;

	MinToleranceIso.Set(ToleranceU, ToleranceV);
}

}
