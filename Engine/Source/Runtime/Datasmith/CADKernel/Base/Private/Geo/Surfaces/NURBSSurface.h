// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/NurbsSurfaceData.h"
#include "Geo/Surfaces/Surface.h"
#include "Math/BSpline.h"

namespace UE::CADKernel
{
class FNURBSSurface : public FSurface
{
	friend FEntity;

protected:
	int32 PoleUCount;
	int32 PoleVCount;

	int32 UDegree;
	int32 VDegree;

	TArray<double> UNodalVector;
	TArray<double> VNodalVector;

	TArray<double> Weights;

	TArray<FVector> Poles;

	bool bIsRational;

	/**
	 * Data generated at initialization which are not serialized
	 */
	TArray<double> HomogeneousPoles;

	/**
	 * Build a Non uniform B-Spline surface
	 * @param NodalVectorU Its size is the number of poles in U + the surface degree in U + 1 (PoleUNum + UDegre + 1)
	 * @param NodalVectorV Its size is the number of poles in V + the surface degree in V + 1 (PoleVNum + VDegre + 1)
	 */
	FNURBSSurface(const double InToleranceGeometric, int32 InPoleUCount, int32 InPoleVCount, int32 InDegreU, int32 InDegreV, const TArray<double>& InNodalVectorU, const TArray<double>& InNodalVectorV, const TArray<FVector>& InPoles)
		: FSurface(InToleranceGeometric)
		, PoleUCount(InPoleUCount)
		, PoleVCount(InPoleVCount)
		, UDegree(InDegreU)
		, VDegree(InDegreV)
		, UNodalVector(InNodalVectorU)
		, VNodalVector(InNodalVectorV)
		, Poles(InPoles)
		, bIsRational(false)
	{
		Finalize();
	}

	/**
	 * Build a Non uniform rational B-Spline surface
	 * @param NodalVectorU Its size is the number of poles in U + the surface degree in U + 1 (PoleUNum + UDegre + 1)
	 * @param NodalVectorV Its size is the number of poles in V + the surface degree in V + 1 (PoleVNum + VDegre + 1)
	 */
	FNURBSSurface(const double InToleranceGeometric, int32 InPoleUCount, int32 InPoleVCount, int32 InDegreU, int32 InDegreV, const TArray<double>& InNodalVectorU, const TArray<double>& InNodalVectorV, const TArray<FVector>& InPoles, const TArray<double>& InWeights)
		: FSurface(InToleranceGeometric)
		, PoleUCount(InPoleUCount)
		, PoleVCount(InPoleVCount)
		, UDegree(InDegreU)
		, VDegree(InDegreV)
		, UNodalVector(InNodalVectorU)
		, VNodalVector(InNodalVectorV)
		, Weights(InWeights)
		, Poles(InPoles)
		, bIsRational(true)
	{
		ComputeMinToleranceIso();
		Finalize();
	}

	/**
	 * Build a Non uniform B-Spline surface
	 * @param NodalVectorU Its size is the number of poles in U + the surface degree in U + 1 (PoleUNum + UDegre + 1)
	 * @param NodalVectorV Its size is the number of poles in V + the surface degree in V + 1 (PoleVNum + VDegre + 1)
	 */
	FNURBSSurface(const double InToleranceGeometric, FNurbsSurfaceData NurbsData)
		: FSurface(InToleranceGeometric)
		, PoleUCount(NurbsData.PoleUCount)
		, PoleVCount(NurbsData.PoleVCount)
		, UDegree(NurbsData.UDegree)
		, VDegree(NurbsData.VDegree)
		, UNodalVector(NurbsData.UNodalVector)
		, VNodalVector(NurbsData.VNodalVector)
		, Weights(NurbsData.Weights)
		, Poles(NurbsData.Poles)
		, bIsRational(true)
	{
		ComputeMinToleranceIso();
		Finalize();
	}

	/**
	 * Build a Non uniform B-Spline surface
	 * @param NodalVectorU Its size is the number of poles in U + the surface degree in U + 1 (PoleUNum + UDegre + 1)
	 * @param NodalVectorV Its size is the number of poles in V + the surface degree in V + 1 (PoleVNum + VDegre + 1)
	 */
	FNURBSSurface(const double InToleranceGeometric, FNurbsSurfaceHomogeneousData NurbsData)
		: FSurface(InToleranceGeometric)
	{
		FillNurbs(NurbsData);
	}

	FNURBSSurface() = default;

public:


	virtual void ValidateUVPoints(TArray<FVector2d>& UVPoints) const
	{
		const double ToleranceU = GetIsoTolerance(EIso::IsoU);
		const double ToleranceV = GetIsoTolerance(EIso::IsoV);

		auto ValidateValue = [](double& Value, const double& Reference, const double& Tolerance)
			{
				if (FMath::IsNearlyEqual(Value, Reference, Tolerance))
				{
					Value = Reference;
					return true;
				}

				return false;
			};

		for (FVector2d& UVPoint : UVPoints)
		{
			for (const double& NodalValue : UNodalVector)
			{
				if (ValidateValue(UVPoint.X, NodalValue, ToleranceU))
				{
					break;
				}
			}

			for (const double& NodalValue : VNodalVector)
			{
				if (ValidateValue(UVPoint.Y, NodalValue, ToleranceV))
				{
					break;
				}
			}
		}
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << PoleUCount;
		Ar << PoleVCount;
		Ar << UDegree;
		Ar << VDegree;
		Ar.Serialize(UNodalVector);
		Ar.Serialize(VNodalVector);
		Ar.Serialize(Weights);
		Ar.Serialize(Poles);
		Ar << bIsRational;

		if (Ar.IsLoading())
		{
			Finalize();
		}
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Nurbs;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	constexpr const int32 GetDegree(EIso Iso) const
	{
		switch (Iso)
		{
		case EIso::IsoU:
			return UDegree;
		case EIso::IsoV:
		default:
			return VDegree;
		}
	}

	constexpr const int32 GetPoleCount(EIso Iso) const
	{
		switch (Iso)
		{
		case EIso::IsoU:
			return PoleUCount;
		case EIso::IsoV:
		default:
			return PoleVCount;
		}
	}

	const TArray<FVector>& GetPoles() const
	{
		return Poles;
	}

	const TArray<double>& GetWeights() const
	{
		return Weights;
	}

	const TArray<double>& GetHPoles() const
	{
		return HomogeneousPoles;
	}

	constexpr const TArray<double>& GetNodalVector(EIso Iso) const
	{
		switch (Iso)
		{
		case EIso::IsoU:
			return UNodalVector;
		case EIso::IsoV:
		default:
			return VNodalVector;
		}
	}

	bool IsRational() const
	{
		return bIsRational;
	}


	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override
	{
		BSpline::EvaluatePoint(*this, InSurfacicCoordinate, OutPoint3D, InDerivativeOrder);
	}

	void EvaluatePointGrid(const FCoordinateGrid& InSurfacicCoordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const override
	{
		BSpline::EvaluatePointGrid(*this, InSurfacicCoordinates, OutPoints, bComputeNormals);
	}

	virtual void LinesNotDerivables(const FSurfacicBoundary& InBoundary, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const override
	{
		BSpline::FindNotDerivableParameters(*this, InDerivativeOrder, InBoundary, OutNotDerivableCoordinates);
	}

	void ComputeMinToleranceIso();

private:
	void Finalize();
	void FillNurbs(FNurbsSurfaceHomogeneousData& NurbsData);
};

} // namespace UE::CADKernel

