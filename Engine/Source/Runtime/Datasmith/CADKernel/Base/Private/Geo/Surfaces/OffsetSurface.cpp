// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/OffsetSurface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{

void FOffsetSurface::InitBoundary()
{
	Boundary = BaseSurface->GetBoundary();
}

void FOffsetSurface::LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 derivatedOrder, FCoordinateGrid& NotDerivables) const
{
	BaseSurface->LinesNotDerivables(Bounds, derivatedOrder, NotDerivables);
}

TSharedPtr<FEntityGeom> FOffsetSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FSurface> TransformedBaseSurface;

	TransformedBaseSurface = StaticCastSharedPtr<FSurface>(BaseSurface->ApplyMatrix(InMatrix));
	if (!TransformedBaseSurface.IsValid())
	{
		return TSharedPtr<FEntityGeom>();
	}

	return FEntity::MakeShared<FOffsetSurface>(Tolerance3D, TransformedBaseSurface.ToSharedRef(), Offset);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FOffsetSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info).Add(TEXT("base Surface"), BaseSurface)
		.Add(TEXT("distance"), Offset);
}
#endif

void FOffsetSurface::EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	if (InDerivativeOrder == 0)
	{
		++InDerivativeOrder;
	}
	BaseSurface->EvaluatePoint(InSurfacicCoordinate, OutPoint3D, InDerivativeOrder);

	FVector Normal = OutPoint3D.GradientU ^ OutPoint3D.GradientV;
	Normal.Normalize();
	OutPoint3D.Point += Normal*Offset;
}

void FOffsetSurface::EvaluatePoints(const TArray<FVector2d>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder) const
{
	if (InDerivativeOrder == 0)
	{
		++InDerivativeOrder;
	}

	BaseSurface->EvaluatePoints(InSurfacicCoordinates, OutPoint3D, InDerivativeOrder);

	for (FSurfacicPoint& Point : OutPoint3D)
	{
		FVector Normal = Point.GradientU ^ Point.GradientV;
		Normal.Normalize();
		Point.Point += Normal * Offset;
	}
}

void FOffsetSurface::EvaluatePointGrid(const FCoordinateGrid& Coords, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;
	BaseSurface->EvaluatePointGrid(Coords, OutPoints, true);

	for (int32 Index = 0; Index < OutPoints.Count(); ++Index)
	{
		OutPoints.Points3D[Index] += FVector(OutPoints.Normals[Index]) * Offset;
	}
}

void FOffsetSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates)
{
	BaseSurface->Presample(InBoundaries, Coordinates);
}

}

