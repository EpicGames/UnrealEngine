// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{

class CADKERNEL_API FOffsetSurface : public FSurface
{
	friend FEntity;

protected:

	TSharedPtr<FSurface> BaseSurface;
	double Offset;

	FOffsetSurface(const double InToleranceGeometric, TSharedRef<FSurface> InBaseSurface, double InOffset)
		: FSurface(InToleranceGeometric)
		, BaseSurface(InBaseSurface)
		, Offset(InOffset)
	{
		ComputeMinToleranceIso();
	}

	FOffsetSurface() = default;

	void ComputeMinToleranceIso()
	{
		MinToleranceIso = BaseSurface->GetIsoTolerances();
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		SerializeIdent(Ar, BaseSurface);
		Ar << Offset;
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		BaseSurface->SpawnIdent(Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		BaseSurface->ResetMarkersRecursively();
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Offset;
	}

	const double GetOffset() const
	{
		return Offset;
	}

	TSharedPtr<FSurface> GetBaseSurface() const
	{
		return BaseSurface;
	}

	virtual void InitBoundary() override;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const override;

	/**
	 * The Gradient and Laplacian are approximated by their value on the carrier surface.
	 */
	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePoints(const TArray<FVector2d>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override;
};

} // namespace UE::CADKernel

