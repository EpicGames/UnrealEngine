// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/PlaneSurface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{

FPlaneSurface::FPlaneSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const FSurfacicBoundary& InBoundary)
	: FSurface(InToleranceGeometric, InBoundary)
{
	Matrix = InMatrix;

	ensureCADKernel(FMath::IsNearlyZero(Matrix.Get(3, 0)) && FMath::IsNearlyZero(Matrix.Get(3, 1)) && FMath::IsNearlyZero(Matrix.Get(3, 2)));

	InverseMatrix = Matrix;
	InverseMatrix.Inverse();
	ComputeMinToleranceIso();
}

FPlaneSurface::FPlaneSurface(const double InToleranceGeometric, const FVector& InPosition, FVector InNormal, const FSurfacicBoundary& InBoundary)
	: FSurface(InToleranceGeometric, InBoundary)
{
	InNormal.Normalize();
	Matrix.FromAxisOrigin(InNormal, InPosition);

	InverseMatrix = Matrix;
	InverseMatrix.Inverse();
	ComputeMinToleranceIso();
}

FPlane FPlaneSurface::GetPlane() const
{
	FSurfacicPoint Point3D;
	EvaluatePoint(FVector2d::ZeroVector, Point3D, 0);
	FVector Normal = Matrix.Column(2);
	return FPlane(Point3D.Point, Normal);
}

void FPlaneSurface::EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	OutPoint3D.DerivativeOrder = InDerivativeOrder;
	OutPoint3D.Point = Matrix.Multiply(InSurfacicCoordinate);
	if(InDerivativeOrder>0) 
	{
		OutPoint3D.GradientU = Matrix.Column(0);
		OutPoint3D.GradientV = Matrix.Column(1);
	}

	if(InDerivativeOrder>1) 
	{
		OutPoint3D.LaplacianU = FVector::ZeroVector;
		OutPoint3D.LaplacianV = FVector::ZeroVector;
		OutPoint3D.LaplacianUV = FVector::ZeroVector;
	}
}

void FPlaneSurface::EvaluatePoints(const TArray<FVector2d>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder) const
{
	int32 PointNum = InSurfacicCoordinates.Num();
	OutPoint3D.SetNum(PointNum);

	for (int32 Index = 0; Index < PointNum; ++Index)
	{
		OutPoint3D[Index].DerivativeOrder = InDerivativeOrder;
	}

	for (int32 Index = 0; Index < PointNum; ++Index)
	{
		OutPoint3D[Index].Point = Matrix.Multiply(InSurfacicCoordinates[Index]);
	}

	if (InDerivativeOrder > 0)
	{
		FVector GradientU(Matrix.Column(0));
		FVector GradientV(Matrix.Column(1));

		for (int32 Index = 0; Index < PointNum; ++Index)
		{
			OutPoint3D[Index].GradientU = GradientU;
		}

		for (int32 Index = 0; Index < PointNum; ++Index)
		{
			OutPoint3D[Index].GradientV = GradientV;
		}
	}
}

void FPlaneSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;

	int32 PointNum = Coordinates.Count();
	OutPoints.Reserve(PointNum);

	OutPoints.Set2DCoordinates(Coordinates);

	for (FVector2d& Point : OutPoints.Points2D)
	{
		OutPoints.Points3D.Emplace(Matrix.Multiply(Point));
	}

	if(bComputeNormals)
	{
		FVector Normal(Matrix.Column(2));
		OutPoints.Normals.Init((FVector3f)Normal, PointNum);
	}
}

FVector FPlaneSurface::ProjectPoint(const FVector& Point, FVector* OutProjectedPoint) const
{
	FVector PointCoordinate = InverseMatrix.Multiply(Point);
	PointCoordinate.Z = 0.0;

	if(OutProjectedPoint)
	{
		*OutProjectedPoint = Matrix.Multiply(PointCoordinate);
	}

	return PointCoordinate;
}

void FPlaneSurface::ProjectPoints(const TArray<FVector>& Points, TArray<FVector>* PointCoordinates, TArray<FVector>* OutProjectedPointS) const
{
	PointCoordinates->Reserve(Points.Num());
	if(OutProjectedPointS) 
	{
		OutProjectedPointS->Reserve(Points.Num());
	}

	for (const FVector& Point : Points)
	{
		FVector& PointCoordinate = PointCoordinates->Emplace_GetRef(InverseMatrix.Multiply(Point));
		PointCoordinate.Z = 0.0;
	}

	if (OutProjectedPointS)
	{
		for (const FVector& PointCoordinate : *PointCoordinates)
		{
			OutProjectedPointS->Emplace(Matrix.Multiply(PointCoordinate));
		}
	}
}

TSharedPtr<FEntityGeom> FPlaneSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FPlaneSurface>(Tolerance3D, NewMatrix, Boundary);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FPlaneSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("Inverse"), InverseMatrix);
}
#endif

}