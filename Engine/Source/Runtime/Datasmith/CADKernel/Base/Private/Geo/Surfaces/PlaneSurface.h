// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class CADKERNEL_API FPlaneSurface : public FSurface
{
	friend FEntity;

protected:

	FMatrixH Matrix;
	FMatrixH InverseMatrix;

	/**
	 * The plane surface is the plane XY.
	 * The surface is placed at its final position and orientation by the Matrix
	 */
	FPlaneSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const FSurfacicBoundary& InBoundary);

	/**
	 * The plane surface is the plane XY.
	 * The surface is placed at its final position and orientation by the Matrix,
	 * The matrix is calculated from the normal plane at its final position and its distance from the origin along the normal.
	 */
	FPlaneSurface(const double InToleranceGeometric, double InDistanceFromOrigin, FVector InNormal, const FSurfacicBoundary& InBoundary)
		: FPlaneSurface(InToleranceGeometric, InNormal* InDistanceFromOrigin, InNormal, InBoundary)
	{
	}

	/**
	 * The plane surface is the plane XY.
	 * The surface is placed at its final position and orientation by the Matrix
	 * The matrix is calculated from the plan origin at its final position and its final normal
	 */
	FPlaneSurface(const double InToleranceGeometric, const FVector& Position, FVector Normal, const FSurfacicBoundary& InBoundary);

	FPlaneSurface() = default;

	void ComputeMinToleranceIso()
	{
		FVector Origin = Matrix.Multiply(FVector::ZeroVector);

		FVector Point2DU{ 1 , 0, 0 };
		FVector Point2DV{ 0, 1, 0 };

		double ToleranceU = Tolerance3D / ComputeScaleAlongAxis(Point2DU, Matrix, Origin);
		double ToleranceV = Tolerance3D / ComputeScaleAlongAxis(Point2DV, Matrix, Origin);

		MinToleranceIso.Set(ToleranceU, ToleranceV);
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << Matrix;
		Ar << InverseMatrix;
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Plane;
	}

	const FMatrixH& GetMatrix() const
	{
		return Matrix;
	}

	FPlane GetPlane() const;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePoints(const TArray<FVector2d>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual FVector ProjectPoint(const FVector& InPoint, FVector* OutProjectedPoint = nullptr) const;// override;
	virtual void ProjectPoints(const TArray<FVector>& InPoints, TArray<FVector>* OutProjectedPointCoordinates, TArray<FVector>* OutProjectedPoints = nullptr) const;// override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override
	{
		OutCoordinates[EIso::IsoU].Empty(3);
		OutCoordinates[EIso::IsoU].Add(InBoundaries[EIso::IsoU].Min);
		OutCoordinates[EIso::IsoU].Add(InBoundaries[EIso::IsoU].GetMiddle());
		OutCoordinates[EIso::IsoU].Add(InBoundaries[EIso::IsoU].Max);

		OutCoordinates[EIso::IsoV].Empty(3);
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Min);
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].GetMiddle());
		OutCoordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Max);
	}

	virtual void IsSurfaceClosed(bool& bOutClosedAlongU, bool& bOutClosedAlongV) const override
	{
		bOutClosedAlongU = false;
		bOutClosedAlongV = false;
	}

};

} // namespace UE::CADKernel
