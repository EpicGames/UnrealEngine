// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/RevolutionSurface.h"

#include "Geo/Curves/SegmentCurve.h"
#include "Geo/Sampling/SurfacicSampling.h"

#include <math.h>

namespace UE::CADKernel
{

FRevolutionSurface::FRevolutionSurface(const double InToleranceGeometric, TSharedRef<FSegmentCurve> InAxe, TSharedRef<FCurve> InGeneratrix, double InMinAngle, double InMaxAngle)
	: FSurface(InToleranceGeometric)
	, Axis(InAxe)
	, Generatrix(InGeneratrix)
{
	const FLinearBoundary& GeneratrixBounds = Generatrix->GetBoundary();
	Boundary.Set(GeneratrixBounds.Min, GeneratrixBounds.Max, InMinAngle, InMaxAngle);

	// Compute rotation axis
	RotationAxis = Axis->GetEndPoint() - Axis->GetStartPoint();
	RotationAxis.Normalize();
	ComputeDefaultMinToleranceIso();
}

void FRevolutionSurface::EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	FCurvePoint GeneratrixPoint;
	Generatrix->EvaluatePoint(InSurfacicCoordinate.X, GeneratrixPoint, InDerivativeOrder);

	FMatrixH Matrix = FMatrixH::MakeRotationMatrix(InSurfacicCoordinate.Y, RotationAxis);

	OutPoint3D.DerivativeOrder = InDerivativeOrder;
	OutPoint3D.Point = Matrix.PointRotation(GeneratrixPoint.Point, Axis->GetStartPoint());

	if (InDerivativeOrder > 0) 
	{
		FVector Vector = OutPoint3D.Point - Axis->GetStartPoint();
		OutPoint3D.GradientU = Matrix.MultiplyVector(GeneratrixPoint.Gradient);

		OutPoint3D.GradientV = RotationAxis ^ Vector;

		if (InDerivativeOrder > 1)
		{
			OutPoint3D.LaplacianU = Matrix.MultiplyVector(GeneratrixPoint.Laplacian);
			OutPoint3D.LaplacianV = RotationAxis ^ OutPoint3D.GradientV;
			OutPoint3D.LaplacianUV = RotationAxis ^ OutPoint3D.GradientU;
		}
	}
}

void FRevolutionSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;

	int32 PointNum = Coordinates.Count();
	OutPoints.Reserve(PointNum);

	OutPoints.Set2DCoordinates(Coordinates);

	int32 DerivativeOrder = bComputeNormals ? 1 : 0;
	TArray<FCurvePoint> GeneratrixPoints;
	TArray<FVector> GeneratrixGradientU;
	Generatrix->EvaluatePoints(Coordinates[EIso::IsoU], GeneratrixPoints, DerivativeOrder);

	TArray<FMatrixH> Matrix;
	Matrix.SetNum(Coordinates.IsoCount(EIso::IsoV));

	for (int32 Vndex = 0, Index = 0; Vndex < Coordinates.IsoCount(EIso::IsoV); ++Vndex)
	{
		Matrix[Vndex] = FMatrixH::MakeRotationMatrix(Coordinates[EIso::IsoV][Vndex], RotationAxis);
	}

	for (int32 Vndex = 0, Index = 0; Vndex < Coordinates.IsoCount(EIso::IsoV); ++Vndex)
	{
		for (FCurvePoint GeneratrixPoint : GeneratrixPoints)
		{
			OutPoints.Points3D.Emplace(Matrix[Vndex].PointRotation(GeneratrixPoint.Point, Axis->GetStartPoint()));
		}
	}

	if (bComputeNormals)
	{
		for (int32 Vndex = 0, Index = 0; Vndex < Coordinates.IsoCount(EIso::IsoV); ++Vndex)
		{
			for (FCurvePoint GeneratrixPoint : GeneratrixPoints)
			{
				FVector GradientU = Matrix[Vndex].MultiplyVector(GeneratrixPoint.Gradient);

				FVector Vector = OutPoints.Points3D[Index] - Axis->GetStartPoint();
				FVector GradientV = RotationAxis ^ Vector;

				OutPoints.Normals.Emplace(GradientU ^ GradientV);
				Index++;
			}
		}
		OutPoints.NormalizeNormals();
	}

}

void FRevolutionSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates)
{
	Generatrix->Presample(Coordinates[EIso::IsoU], Tolerance3D);
	PresampleIsoCircle(InBoundaries, Coordinates, EIso::IsoV);
}

void FRevolutionSurface::LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutCoordinates) const
{
	Generatrix->FindNotDerivableCoordinates(InDerivativeOrder, OutCoordinates[EIso::IsoU]);
}

TSharedPtr<FEntityGeom> FRevolutionSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FSegmentCurve> TransformedAxe = StaticCastSharedPtr<FSegmentCurve>(Axis->ApplyMatrix(InMatrix));
	if (!TransformedAxe.IsValid())
	{
		return TSharedPtr<FEntityGeom>();
	}

	TSharedPtr<FCurve> TransformedGeneratrix = StaticCastSharedPtr<FCurve>(Generatrix->ApplyMatrix(InMatrix));
	if (!TransformedGeneratrix.IsValid()) 
	{
		return TSharedPtr<FEntityGeom>();
	}

	return FEntity::MakeShared<FRevolutionSurface>(Tolerance3D, TransformedAxe.ToSharedRef(), TransformedGeneratrix.ToSharedRef(), Boundary[EIso::IsoU].Min, Boundary[EIso::IsoU].Max);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FRevolutionSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info).Add(TEXT("axis"), Axis)
		.Add(TEXT("Generatrix"), Generatrix)
		.Add(TEXT("Min angle"), Boundary[EIso::IsoU].Min)
		.Add(TEXT("Max angle"), Boundary[EIso::IsoU].Max);
}
#endif

void FRevolutionSurface::SpawnIdent(FDatabase& Database)
{
	if (!FEntity::SetId(Database))
	{
		return;
	}

	Axis->SpawnIdent(Database);
	Generatrix->SpawnIdent(Database);
}

void FRevolutionSurface::ResetMarkersRecursively() const
{
	ResetMarkers();
	Axis->ResetMarkersRecursively();
	Generatrix->ResetMarkersRecursively();
}

}