// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class FCurve;
class FSegmentCurve;

class CADKERNEL_API FRevolutionSurface : public FSurface
{
	friend FEntity;

protected:

	TSharedPtr<FSegmentCurve> Axis;
	TSharedPtr<FCurve> Generatrix;

	FVector RotationAxis;

	/**
	 * A surface of revolution is defined by an axis of rotation (Axis) and a generatrix.
	 * The start and terminate rotation angles can limited the surface.
	 * The surface is created by rotating the generatrix about the axis of rotation through the start and terminating angles.
	 *
	 * The bounds of the surface of revolution are defined as follow:
	 * Bounds[EIso::IsoU].Min = Start rotation angle;
	 * Bounds[EIso::IsoU].Max = Terminate rotation angle;
	 * Bounds[EIso::IsoV].Min = Min Generatrix bounds;
	 * Bounds[EIso::IsoV].Max = Max Generatrix bounds;
	 */
	FRevolutionSurface(const double InToleranceGeometric, TSharedRef<FSegmentCurve> Axe, TSharedRef<FCurve> Generatrix, double MinAngle, double MaxAngle);

	FRevolutionSurface() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		SerializeIdent(Ar, Axis);
		SerializeIdent(Ar, Generatrix);
		Ar << RotationAxis;
	}

	virtual void SpawnIdent(FDatabase& Database) override;
	virtual void ResetMarkersRecursively() const override;

	ESurface GetSurfaceType() const
	{
		return ESurface::Revolution;
	}

	TSharedPtr<FCurve> GetGeneratrix() const
	{
		return Generatrix;
	}

	virtual TSharedPtr<FSegmentCurve> GetAxe() const
	{
		return Axis;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutCoordinates) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override;

};

} // namespace UE::CADKernel

