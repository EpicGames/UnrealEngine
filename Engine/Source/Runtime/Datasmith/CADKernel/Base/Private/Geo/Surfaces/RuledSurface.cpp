// Copyright Epic Games, Inc. All Rights Reserved.InBoundary

#include "Geo/Surfaces/RuledSurface.h"

#include "Geo/Curves/Curve.h"
#include "Geo/Sampling/SurfacicSampling.h"
#include "Utils/ArrayUtils.h"

namespace UE::CADKernel
{

void FRuledSurface::LinesNotDerivables(const FSurfacicBoundary& InBoundary, int32 InDerivativeOrder, FCoordinateGrid& OutCoordinates) const
{
	TFunction<void(int32, TArray<double>&)> FindNotDerivableCoordinates = [&](int32 CurveIndex, TArray<double>& NotDerivables)
	{
		Curves[CurveIndex]->FindNotDerivableCoordinates(InDerivativeOrder, NotDerivables);

		double ULength = Curves[CurveIndex]->GetUMax() - Curves[CurveIndex]->GetUMin();
		double UMin = Curves[CurveIndex]->GetUMin();

		for (double& NotDerivableU : NotDerivables)
		{
			NotDerivableU = (NotDerivableU - UMin) / ULength;
		}
	};

	FindNotDerivableCoordinates(0, OutCoordinates[EIso::IsoU]);
	TArray<double> LinesNotDerivablesCurve1;
	FindNotDerivableCoordinates(1, LinesNotDerivablesCurve1);

	// remove duplicated
	ArrayUtils::Complete(OutCoordinates[EIso::IsoU], LinesNotDerivablesCurve1, GetIsoTolerances()[IsoU]);
}

void FRuledSurface::EvaluatePoint(const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	double CoordinateCurve0 = Curves[0]->GetBoundary().Min + InPoint2D.X * (Curves[0]->GetBoundary().Max - Curves[0]->GetBoundary().Min);
	double CoordinateCurve1 = Curves[1]->GetBoundary().Min + InPoint2D.X * (Curves[1]->GetBoundary().Max - Curves[1]->GetBoundary().Min);

	FCurvePoint CurvePoint[2];
	Curves[0]->EvaluatePoint(CoordinateCurve0, CurvePoint[0], InDerivativeOrder);
	Curves[1]->EvaluatePoint(CoordinateCurve1, CurvePoint[1], InDerivativeOrder);

	OutPoint3D.GradientV = CurvePoint[1].Point - CurvePoint[0].Point;

	OutPoint3D.DerivativeOrder = InDerivativeOrder;
	OutPoint3D.Point = CurvePoint[0].Point + OutPoint3D.GradientV * InPoint2D.Y;

	if (InDerivativeOrder > 0)
	{
		OutPoint3D.LaplacianUV = CurvePoint[1].Gradient - CurvePoint[0].Gradient;
		OutPoint3D.GradientU = CurvePoint[0].Gradient + InPoint2D.Y * OutPoint3D.LaplacianUV;

		if (InDerivativeOrder > 1)
		{
			OutPoint3D.LaplacianU = CurvePoint[0].Laplacian + InPoint2D.Y * (CurvePoint[1].Laplacian - CurvePoint[0].Laplacian);
			OutPoint3D.LaplacianV = FVector(0.0, 0.0, 0.0);
		}
	}
}

void FRuledSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;

	int32 PointCount = Coordinates.Count();

	OutPoints.Set2DCoordinates(Coordinates);

	int32 DerivativeOrder = bComputeNormals ? 1 : 0;

	TFunction<void(TSharedPtr<FCurve>, TArray<FCurvePoint>&)> ComputeCurveCoords = [&](TSharedPtr<FCurve> Curve, TArray<FCurvePoint>& Points)
	{
		TArray<double> OutCurveCoords;
		OutCurveCoords.Empty(Coordinates.IsoCount(EIso::IsoU));
		const double Min = Curve->GetBoundary().Min;
		const double ParametricLength = Curve->GetBoundary().Length();

		for (const double Coord : Coordinates[EIso::IsoU])
		{
			OutCurveCoords.Emplace(Min + Coord * ParametricLength);
		}
		Curve->EvaluatePoints(OutCurveCoords, Points, DerivativeOrder);
	};


	TArray<FCurvePoint> CurveCoords[2];
	TArray<FCurvePoint> CurveGradients[2];
	ComputeCurveCoords(Curves[0], CurveCoords[0]);
	ComputeCurveCoords(Curves[1], CurveCoords[1]);

	int32 UCount = Coordinates.IsoCount(EIso::IsoU);
	int32 VCount = Coordinates.IsoCount(EIso::IsoV);

	OutPoints.SetNum(PointCount);

	for (int32 Undex = 0; Undex < Coordinates.IsoCount(EIso::IsoU); Undex++)
	{
		FVector GradientV = CurveCoords[1][Undex].Point - CurveCoords[0][Undex].Point;

		for (int32 Vndex = 0, Index = Undex; Vndex < VCount; ++Vndex)
		{
			OutPoints.Points3D[Index] = CurveCoords[0][Undex].Point + GradientV * Coordinates[EIso::IsoV][Vndex];
			Index += UCount;
		}

		if (bComputeNormals)
		{
			for (int32 Vndex = 0, Index = Undex; Vndex < VCount; ++Vndex)
			{
				FVector GradientU = CurveCoords[0][Undex].Gradient + Coordinates[EIso::IsoV][Vndex] * (CurveCoords[1][Undex].Gradient - CurveCoords[0][Undex].Gradient);
				OutPoints.Normals[Index] = (FVector3f)(GradientU ^ GradientV);
				Index += UCount;
			}
		}
		OutPoints.NormalizeNormals();
	}
}

void FRuledSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates)
{
	TFunction<void(int32, TArray<double>&)> PresampleCurve = [&](int32 CurveIndex, TArray<double>& Sample)
	{
		Curves[CurveIndex]->Presample(Sample, Tolerance3D);

		double ULength = Curves[CurveIndex]->GetUMax() - Curves[CurveIndex]->GetUMin();
		double UMin = Curves[CurveIndex]->GetUMin();

		for (double& Coordinate : Sample)
		{
			Coordinate = (Coordinate - UMin) / ULength;
		}
	};

	PresampleCurve(0, Coordinates[EIso::IsoU]);
	TArray<double> Curve1Sample;
	PresampleCurve(1, Curve1Sample);

	// remove duplicated
	ArrayUtils::InsertInside(Coordinates[EIso::IsoU], Curve1Sample, GetIsoTolerance(IsoU));

	Coordinates[EIso::IsoV].Empty(3);
	Coordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Min);
	Coordinates[EIso::IsoV].Add((InBoundaries[EIso::IsoV].Max + InBoundaries[EIso::IsoV].Min) / 2.0);
	Coordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Max);
}

TSharedPtr<FEntityGeom> FRuledSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FCurve> TransformedCurveU = StaticCastSharedPtr<FCurve>(Curves[0]->ApplyMatrix(InMatrix));
	if (!TransformedCurveU.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FCurve> TransformedCurveV = StaticCastSharedPtr<FCurve>(Curves[1]->ApplyMatrix(InMatrix));
	if (!TransformedCurveV.IsValid())
	{
		return nullptr;
	}

	return FEntity::MakeShared<FRuledSurface>(Tolerance3D, TransformedCurveU, TransformedCurveV);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FRuledSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info).Add(TEXT("Curve 0"), Curves[0])
		.Add(TEXT("Curve 1"), Curves[1]);
}
#endif

void FRuledSurface::SpawnIdent(FDatabase& Database)
{
	if (!FEntity::SetId(Database))
	{
		return;
	}

	Curves[0]->SpawnIdent(Database);
	Curves[1]->SpawnIdent(Database);
}

void FRuledSurface::ResetMarkersRecursively() const
{
	ResetMarkers();
	Curves[0]->ResetMarkersRecursively();
	Curves[1]->ResetMarkersRecursively();
}

}
