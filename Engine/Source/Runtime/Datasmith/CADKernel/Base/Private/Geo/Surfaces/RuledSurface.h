// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class FCurve;
class FNURBSSurface;

class CADKERNEL_API FRuledSurface : public FSurface
{
	friend FEntity;

protected:
	TSharedPtr<FCurve> Curves[2];

	FRuledSurface(const double InToleranceGeometric, TSharedPtr<FCurve> InCurveU, TSharedPtr<FCurve> InCurveV)
		: FSurface(InToleranceGeometric, 0., 1., 0., 1.)
	{
		Curves[0] = InCurveU;
		Curves[1] = InCurveV;
		ComputeDefaultMinToleranceIso();
	}

	FRuledSurface() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		SerializeIdent(Ar, Curves[0]);
		SerializeIdent(Ar, Curves[1]);
	}

	virtual void SpawnIdent(FDatabase& Database) override;
	virtual void ResetMarkersRecursively() const override;

	ESurface GetSurfaceType() const
	{
		return ESurface::Ruled;
	}

	const TSharedPtr<FCurve>& GetCurve(EIso Iso) const
	{
		return Curves[Iso];
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutCoordinates) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override;

};

} // namespace UE::CADKernel

