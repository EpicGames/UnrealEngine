// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/SphericalSurface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{

void FSphericalSurface::EvaluatePointGridInCylindricalSpace(const FCoordinateGrid& Coordinates, TArray<FVector2d>& OutPoints) const
{
	int32 PointNum = Coordinates.Count();
	OutPoints.Empty(PointNum);

	int32 UCount = Coordinates.IsoCount(EIso::IsoU);

	TArray<double> CosU;
	TArray<double> SinU;
	CosU.Reserve(UCount);
	SinU.Reserve(UCount);

	for (double Angle : Coordinates[EIso::IsoU])
	{
		CosU.Emplace(cos(Angle));
	}

	for (double Angle : Coordinates[EIso::IsoU])
	{
		SinU.Emplace(sin(Angle));
	}

	for (double VAngle : Coordinates[EIso::IsoV])
	{
		double CosV = cos(VAngle);
		double Rho = Radius * CosV;
		double SwapOrientation = (VAngle < DOUBLE_PI && VAngle >= 0) ? 1.0 : -1.0;

		for (int32 Undex = 0; Undex < UCount; Undex++)
		{
			OutPoints.Emplace(Rho * CosU[Undex] * SwapOrientation, Rho * SinU[Undex]);
		}
	}
}

void FSphericalSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;
	
	int32 PointNum = Coordinates.Count();
	OutPoints.Reserve(PointNum);

	OutPoints.Set2DCoordinates(Coordinates);

	int32 UCount = Coordinates.IsoCount(EIso::IsoU);

	TArray<double> CosU;
	TArray<double> SinU;
	CosU.Reserve(UCount);
	SinU.Reserve(UCount);

	for (double Angle : Coordinates[EIso::IsoU])
	{
		CosU.Emplace(cos(Angle));
	}

	for (double Angle : Coordinates[EIso::IsoU])
	{
		SinU.Emplace(sin(Angle));
	}

	for (double VAngle : Coordinates[EIso::IsoV])
	{
		double CosV = cos(VAngle);
		double SinV = sin(VAngle);
		double Rho = Radius * CosV;
		double Height = Radius * SinV;

		for (int32 Undex = 0; Undex < UCount; Undex++)
		{
			OutPoints.Points3D.Emplace(Rho * CosU[Undex], Rho * SinU[Undex], Height);
		}
	}

	for (FVector& Point : OutPoints.Points3D)
	{
		Point = Matrix.Multiply(Point);
	}

	if (bComputeNormals)
	{
		FVector Center = Matrix.Column(3);
		for (FVector& Point : OutPoints.Points3D)
		{
			OutPoints.Normals.Emplace(Point - Center);
		}
		OutPoints.NormalizeNormals();
	}
}

TSharedPtr<FEntityGeom> FSphericalSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FSphericalSurface>(Tolerance3D, NewMatrix, Radius, 
		Boundary[EIso::IsoU].Min, Boundary[EIso::IsoU].Max,
		Boundary[EIso::IsoV].Min, Boundary[EIso::IsoV].Max);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FSphericalSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("Radius"), Radius)
		.Add(TEXT("MeridianStartAngle"), Boundary[EIso::IsoU].Min)
		.Add(TEXT("MeridianEndAngle"),   Boundary[EIso::IsoU].Max)
		.Add(TEXT("ParallelStartAngle"), Boundary[EIso::IsoV].Min)
		.Add(TEXT("ParallelEndAngle"),   Boundary[EIso::IsoV].Max);
}
#endif

}
