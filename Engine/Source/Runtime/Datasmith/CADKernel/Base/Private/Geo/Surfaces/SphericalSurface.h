// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class CADKERNEL_API FSphericalSurface : public FSurface
{
	friend FEntity;

protected:
	FMatrixH Matrix;
	double Radius;

	/**
	 * The spherical surface is defined its radius.
	 *
	 * It's defined as the rotation around Z axis of an semicircle defined in the plan XY centered at the origin.
	 *
	 * The spherical surface is placed at its final position and orientation by the Matrix
	 */
	FSphericalSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, double InParallelStartAngle = 0.0, double InParallelEndAngle = DOUBLE_TWO_PI, double InMeridianStartAngle = -DOUBLE_HALF_PI, double InMeridianEndAngle = DOUBLE_HALF_PI)
		: FSphericalSurface(InToleranceGeometric, InMatrix, InRadius, FSurfacicBoundary(InParallelStartAngle, InParallelEndAngle, InMeridianStartAngle, InMeridianEndAngle))
	{
	}

	/**
	 * The spherical surface is defined its radius.
	 *
	 * It's defined as the rotation around Z axis of an semicircle defined in the plan XY centered at the origin.
	 *
	 * The spherical surface is placed at its final position and orientation by the Matrix
	 *
	 * The bounds of the spherical surface are defined as follow:
	 * Bounds[EIso::IsoU].Min = MeridianStartAngle
	 * Bounds[EIso::IsoU].Max = MeridianEndAngle
	 * Bounds[EIso::IsoV].Min = ParallelStartAngle
	 * Bounds[EIso::IsoV].Max = ParallelEndAngle
	 */
	FSphericalSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, const FSurfacicBoundary& Boundary)
		: FSurface(InToleranceGeometric, Boundary)
		, Matrix(InMatrix)
		, Radius(InRadius)
	{
		ComputeMinToleranceIso();
	}

	FSphericalSurface() = default;

	void ComputeMinToleranceIso()
	{
		double Tolerance2D = Tolerance3D / Radius;

		FVector Origin = Matrix.Multiply(FVector::ZeroVector);

		FVector Point2DU{ 1 , 0, 0 };
		FVector Point2DV{ 0, 1, 0 };

		double ToleranceU = Tolerance2D / ComputeScaleAlongAxis(Point2DU, Matrix, Origin);
		double ToleranceV = Tolerance2D / ComputeScaleAlongAxis(Point2DV, Matrix, Origin);

		MinToleranceIso.Set(ToleranceU, ToleranceV);
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << Matrix;
		Ar << Radius;
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Sphere;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override
	{
		double CosU = cos(InSurfacicCoordinate.X);
		double CosV = cos(InSurfacicCoordinate.Y);

		double SinU = sin(InSurfacicCoordinate.X);
		double SinV = sin(InSurfacicCoordinate.Y);

		OutPoint3D.DerivativeOrder = InDerivativeOrder;
		OutPoint3D.Point.Set(Radius * CosV * CosU, Radius * CosV * SinU, Radius * SinV);
		OutPoint3D.Point = Matrix.Multiply(OutPoint3D.Point);

		if (InDerivativeOrder > 0)
		{
			OutPoint3D.GradientU = FVector(-Radius * CosV * SinU, Radius * CosV * CosU, 0.0);
			OutPoint3D.GradientV = FVector(-Radius * SinV * CosU, -Radius * SinV * SinU, Radius * CosV);

			OutPoint3D.GradientU = Matrix.MultiplyVector(OutPoint3D.GradientU);
			OutPoint3D.GradientV = Matrix.MultiplyVector(OutPoint3D.GradientV);
		}

		if (InDerivativeOrder > 1)
		{
			OutPoint3D.LaplacianU = FVector(-Radius * CosV * CosU, -Radius * CosV * SinU, 0.0);
			OutPoint3D.LaplacianV = FVector(OutPoint3D.LaplacianU.X, OutPoint3D.LaplacianU.Y, -Radius * SinV);
			OutPoint3D.LaplacianUV = FVector(Radius * SinV * SinU, -Radius * SinV * CosU, 0.);

			OutPoint3D.LaplacianU = Matrix.MultiplyVector(OutPoint3D.LaplacianU);
			OutPoint3D.LaplacianV = Matrix.MultiplyVector(OutPoint3D.LaplacianV);
			OutPoint3D.LaplacianUV = Matrix.MultiplyVector(OutPoint3D.LaplacianUV);
		}
	}

	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual FVector2d EvaluatePointInCylindricalSpace(const FVector2d& InSurfacicCoordinate) const override
	{
		double CosU = cos(InSurfacicCoordinate.X);
		double CosV = cos(InSurfacicCoordinate.Y);

		double SinU = sin(InSurfacicCoordinate.X);
		double SwapOrientation = (InSurfacicCoordinate.Y < DOUBLE_PI&& InSurfacicCoordinate.Y >= 0) ? 1.0 : -1.0;

		return FVector2d(Radius * CosV * CosU * SwapOrientation, Radius * CosV * SinU);
	}

	virtual void EvaluatePointGridInCylindricalSpace(const FCoordinateGrid& Coordinates, TArray<FVector2d>&) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override
	{
		PresampleIsoCircle(InBoundaries, OutCoordinates, EIso::IsoU);
		PresampleIsoCircle(InBoundaries, OutCoordinates, EIso::IsoV);
	}
};
}
