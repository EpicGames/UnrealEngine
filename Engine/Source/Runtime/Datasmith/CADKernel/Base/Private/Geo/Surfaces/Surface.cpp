// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/Surface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampler/SamplerOnParam.h"
#include "Geo/Sampling/SurfacicPolyline.h"
#include "Geo/Sampling/SurfacicSampling.h"

#include "Mesh/Structure/Grid.h"

namespace UE::CADKernel
{
#ifdef CADKERNEL_DEV
FInfoEntity& FSurface::GetInfo(FInfoEntity& Info) const
{
	return FEntityGeom::GetInfo(Info)
		.Add(TEXT("Surface type"), SurfacesTypesNames[(uint8)GetSurfaceType()])
		.Add(TEXT("Boundary"), Boundary)
		.Add(TEXT("MinToleranceIso"), (FVector2d)*MinToleranceIso);
}
#endif

void FSurface::Sample(const FSurfacicBoundary& InBoundary, int32 NumberOfSubdivisions[2], FCoordinateGrid& OutCoordinateSampling) const
{
	for (int32 Index = EIso::IsoU; Index <= EIso::IsoV; Index++)
	{
		EIso Iso = (EIso)Index;
		double Step = (InBoundary[Iso].Max - InBoundary[Iso].Min) / ((double)(NumberOfSubdivisions[Iso] - 1));

		OutCoordinateSampling[Iso].Empty(NumberOfSubdivisions[Iso]);
		double Value = InBoundary[Iso].Min;
		for (int32 Kndex = 0; Kndex < NumberOfSubdivisions[Iso]; Kndex++)
		{
			OutCoordinateSampling[Iso].Add(Value);
			Value += Step;
		}
	}
}

void FSurface::Sample(const FSurfacicBoundary& Bounds, int32 NumberOfSubdivisions[2], FSurfacicSampling& OutPointSampling) const
{
	FCoordinateGrid CoordinateSampling;
	Sample(Bounds, NumberOfSubdivisions, CoordinateSampling);
	EvaluatePointGrid(CoordinateSampling, OutPointSampling);
}

void FSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates)
{
	FPolyline3D TemporaryPolyline;
	FSurfaceSamplerOnParam Sampler(*this, InBoundaries, Tolerance3D * 10, Tolerance3D, TemporaryPolyline, OutCoordinates);
	Sampler.Sample();
}

void FSurface::EvaluatePoints(const TArray<FVector2d>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder) const
{
	OutPoint3D.SetNum(InSurfacicCoordinates.Num());
	for (int32 Index = 0; Index < InSurfacicCoordinates.Num(); ++Index)
	{
		EvaluatePoint(InSurfacicCoordinates[Index], OutPoint3D[Index], InDerivativeOrder);
	}
}

void FSurface::EvaluatePoints(const TArray<FCurvePoint2D>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder) const
{
	OutPoint3D.SetNum(InSurfacicCoordinates.Num());
	for (int32 Index = 0; Index < InSurfacicCoordinates.Num(); Index++)
	{
		EvaluatePoint(InSurfacicCoordinates[Index].Point, OutPoint3D[Index], InDerivativeOrder);
	}
}

void FSurface::EvaluatePoints(const TArray<FCurvePoint2D>& InSurfacicCoordinates, TArray<FCurvePoint>& OutPoints3D, int32 InDerivativeOrder) const
{
	OutPoints3D.SetNum(InSurfacicCoordinates.Num());
	TArray<FSurfacicPoint> SurfacicPoints3D;
	EvaluatePoints(InSurfacicCoordinates, SurfacicPoints3D, InDerivativeOrder);
	for (int32 Index = 0; Index < InSurfacicCoordinates.Num(); ++Index)
	{
		OutPoints3D[Index].Combine(InSurfacicCoordinates[Index], SurfacicPoints3D[Index]);
	}
}

void FSurface::EvaluatePoints(FSurfacicPolyline& Polyline) const
{
	int32 DerivativeOrder = Polyline.bWithNormals ? 1 : 0;

	TArray<FSurfacicPoint> Point3D;
	EvaluatePoints(Polyline.Points2D, Point3D, DerivativeOrder);

	Polyline.Points3D.Empty(Polyline.Points2D.Num());
	for (FSurfacicPoint Point : Point3D)
	{
		Polyline.Points3D.Emplace(Point.Point);
	}

	if (Polyline.bWithNormals)
	{
		Polyline.Normals.Empty(Polyline.Points2D.Num());
		for (FSurfacicPoint Point : Point3D)
		{
			Polyline.Normals.Emplace(Point.GradientU ^ Point.GradientV);
		}
		for (FVector3f Normal : Polyline.Normals)
		{
			Normal.Normalize();
		}
	}
}

void FSurface::EvaluatePoints(const TArray<FCurvePoint2D>& InPoints2D, FSurfacicPolyline& Polyline) const
{
	int32 DerivativeOrder = 1;

	TArray<FSurfacicPoint> Points3D;
	EvaluatePoints(InPoints2D, Points3D, DerivativeOrder);

	Polyline.Points2D.Empty(InPoints2D.Num());
	Polyline.Points3D.Empty(InPoints2D.Num());
	Polyline.Tangents.Empty(InPoints2D.Num());
	Polyline.Normals.Empty(InPoints2D.Num());

	for (const FCurvePoint2D& Point : InPoints2D)
	{
		Polyline.Points2D.Emplace(Point.Point);
	}

	for (const FSurfacicPoint& Point : Points3D)
	{
		Polyline.Points3D.Emplace(Point.Point);
	}

	for (int32 Index = 0; Index < InPoints2D.Num(); ++Index)
	{
		Polyline.Tangents.Emplace(Points3D[Index].GradientU * InPoints2D[Index].Gradient.X + Points3D[Index].GradientV * InPoints2D[Index].Gradient.Y);
	}

	for (const FSurfacicPoint& Point : Points3D)
	{
		Polyline.Normals.Emplace(Point.GradientU ^ Point.GradientV);
	}

	for (FVector3f& Normal : Polyline.Normals)
	{
		Normal.Normalize();
	}
}

void FSurface::EvaluateNormals(const TArray<FVector2d>& InPoints2D, TArray<FVector3f>& Normals) const
{
	int32 DerivativeOrder = 1;
	TArray<FSurfacicPoint> Points3D;
	EvaluatePoints(InPoints2D, Points3D, DerivativeOrder);

	Normals.Empty(InPoints2D.Num());

	for (const FSurfacicPoint& Point : Points3D)
	{
		Normals.Emplace(Point.GradientU ^ Point.GradientV);
	}

	for (FVector3f& Normal : Normals)
	{
		Normal.Normalize();
	}
}

void FSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.Reserve(Coordinates.Count());

	OutPoints.Set2DCoordinates(Coordinates);

	int32 DerivativeOrder = bComputeNormals ? 1 : 0;
	TArray<FSurfacicPoint> Point3D;
	EvaluatePoints(OutPoints.Points2D, Point3D, DerivativeOrder);

	OutPoints.bWithNormals = bComputeNormals;

	for (FSurfacicPoint Point : Point3D)
	{
		OutPoints.Points3D.Emplace(Point.Point);
	}

	if (bComputeNormals)
	{
		for (FSurfacicPoint Point : Point3D)
		{
			const FVector Normal = Point.GradientU ^ Point.GradientV;
			OutPoints.Normals.Emplace(Normal);
		}
		OutPoints.NormalizeNormals();
	}
}

void FSurface::EvaluateGrid(FGrid& Grid) const
{
	FSurfacicSampling OutPoints;

	const FCoordinateGrid& CoordinateGrid = Grid.GetCuttingCoordinates();
	EvaluatePointGrid(CoordinateGrid, OutPoints, true);

	Grid.GetInner3DPoints() = MoveTemp(OutPoints.Points3D);
	Grid.GetNormals() = MoveTemp(OutPoints.Normals);
}

} // namespace UE::CADKernel

