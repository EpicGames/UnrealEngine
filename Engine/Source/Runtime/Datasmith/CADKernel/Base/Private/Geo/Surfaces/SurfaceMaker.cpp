// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/Surface.h"

#include "Geo/Surfaces/PlaneSurface.h"
#include "Geo/Surfaces/BezierSurface.h"
#include "Geo/Surfaces/NURBSSurface.h"
#include "Geo/Surfaces/RuledSurface.h"
#include "Geo/Surfaces/RevolutionSurface.h"
#include "Geo/Surfaces/TabulatedCylinderSurface.h"
#include "Geo/Surfaces/CylinderSurface.h"
#include "Geo/Surfaces/OffsetSurface.h"
#include "Geo/Surfaces/CompositeSurface.h"
#include "Geo/Surfaces/CoonsSurface.h"
#include "Geo/Surfaces/SphericalSurface.h"
#include "Geo/Surfaces/TorusSurface.h"
#include "Geo/Surfaces/ConeSurface.h"


namespace UE::CADKernel
{
TSharedPtr<FSurface> FSurface::MakeBezierSurface(const double InToleranceGeometric, int32 InUDegre, int32 InVDegre, const TArray<FVector>& InPoles)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FBezierSurface>(InToleranceGeometric, InUDegre, InVDegre, InPoles);
}

TSharedPtr<FSurface> FSurface::MakeConeSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InStartRadius, double InConeAngle, const FSurfacicBoundary& InBoundary)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FConeSurface>(InToleranceGeometric, InMatrix, InStartRadius, InConeAngle, InBoundary);
}

TSharedPtr<FSurface> FSurface::MakeCylinderSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const double InRadius, const FSurfacicBoundary& InBoundary)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FCylinderSurface>(InToleranceGeometric, InMatrix, InRadius, InBoundary);
}

TSharedPtr<FSurface> FSurface::MakeNurbsSurface(const double InToleranceGeometric, const FNurbsSurfaceHomogeneousData& NurbsData)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FNURBSSurface>(InToleranceGeometric, NurbsData);
}

TSharedPtr<FSurface> FSurface::MakeNurbsSurface(const double InToleranceGeometric, const FNurbsSurfaceData& NurbsData)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FNURBSSurface>(InToleranceGeometric, NurbsData);
}

TSharedPtr<FSurface> FSurface::MakePlaneSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const FSurfacicBoundary& InBoundary)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FPlaneSurface>(InToleranceGeometric, InMatrix, InBoundary);
}

TSharedPtr<FSurface> FSurface::MakeSphericalSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, const FSurfacicBoundary& InBoundary)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FSphericalSurface>(InToleranceGeometric, InMatrix, InRadius, InBoundary);
}

TSharedPtr<FSurface> FSurface::MakeTorusSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InMajorRadius, double InMinorRadius, const FSurfacicBoundary& InBoundary)
{
	return UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FTorusSurface>(InToleranceGeometric, InMatrix, InMajorRadius, InMinorRadius, InBoundary);
}

} // namespace UE::CADKernel

