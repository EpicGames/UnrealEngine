// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/Surface.h"

#include "Core/CADKernelArchive.h"

#include "Geo/Surfaces/PlaneSurface.h"
#include "Geo/Surfaces/BezierSurface.h"
#include "Geo/Surfaces/NURBSSurface.h"
#include "Geo/Surfaces/RuledSurface.h"
#include "Geo/Surfaces/RevolutionSurface.h"
#include "Geo/Surfaces/TabulatedCylinderSurface.h"
#include "Geo/Surfaces/CylinderSurface.h"
#include "Geo/Surfaces/OffsetSurface.h"
#include "Geo/Surfaces/CompositeSurface.h"
#include "Geo/Surfaces/CoonsSurface.h"
#include "Geo/Surfaces/SphericalSurface.h"
#include "Geo/Surfaces/TorusSurface.h"
#include "Geo/Surfaces/ConeSurface.h"


namespace UE::CADKernel
{
TSharedPtr<FSurface> FSurface::Deserialize(FCADKernelArchive& Archive)
{
	ESurface SurfaceType = ESurface::None;
	Archive << SurfaceType;

	switch (SurfaceType)
	{
	case ESurface::Bezier:
		return FEntity::MakeShared<FBezierSurface>(Archive);
	case ESurface::Cone:
		return FEntity::MakeShared<FConeSurface>(Archive);
	case ESurface::Composite:
		return FEntity::MakeShared<FCompositeSurface>(Archive);
	case ESurface::Coons:
		return FEntity::MakeShared<FCoonsSurface>(Archive);
	case ESurface::Cylinder:
		return FEntity::MakeShared<FCylinderSurface>(Archive);
	case ESurface::Nurbs:
		return FEntity::MakeShared<FNURBSSurface>(Archive);
	case ESurface::Offset:
		return FEntity::MakeShared<FOffsetSurface>(Archive);
	case ESurface::Plane:
		return FEntity::MakeShared<FPlaneSurface>(Archive);
	case ESurface::Revolution:
		return FEntity::MakeShared<FRevolutionSurface>(Archive);
	case ESurface::Ruled:
		return FEntity::MakeShared<FRuledSurface>(Archive);
	case ESurface::Sphere:
		return FEntity::MakeShared<FSphericalSurface>(Archive);
	case ESurface::TabulatedCylinder:
		return FEntity::MakeShared<FTabulatedCylinderSurface>(Archive);
	case ESurface::Torus:
		return FEntity::MakeShared<FTorusSurface>(Archive);
	default:
		return TSharedPtr<FSurface>();
	}
}

} // namespace UE::CADKernel

