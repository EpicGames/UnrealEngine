// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/SurfaceUtilities.h"

#include "Geo/Surfaces/BezierSurface.h"
#include "Geo/Surfaces/NURBSSurface.h"
#include "Geo/Surfaces/OffsetSurface.h"
#include "Geo/Surfaces/Surface.h"

#include "Geo/GeoEnum.h"

namespace UE::CADKernel::SurfaceUtilities
{
	TArray<TArray<FVector3f>> GetPoles(const UE::CADKernel::FSurface& Surface)
	{
		using namespace UE::CADKernel;

		TArray<TArray<FVector3f>> Poles;
		
		return Poles;
	}

	bool AreControlPointsPlanar(const TArray<FVector>& Poles, int32 UPoleCount)
	{
		using namespace UE::CADKernel;

		const FVector Origin(Poles[0].X, Poles[0].Y, Poles[0].Z);
		const FVector UDir(Poles[1].X, Poles[1].Y, Poles[1].Z);
		const FVector VDir(Poles[UPoleCount].X, Poles[UPoleCount].Y, Poles[UPoleCount].Z);
		const FVector Normal = ((UDir - Origin) ^ (VDir - Origin)).GetSafeNormal();

		for (int32 Index = 1; Index < Poles.Num(); ++Index)
		{
			const double Dot = Normal | (FVector(Poles[Index].X, Poles[Index].Y, Poles[Index].Z) - Origin).GetSafeNormal();
			if (FMath::Abs(Dot) > 0.01745241 /* Angle between Normal and segment is more than 89 degrees */)
			{
				return false;
			}
		}

		return true;
	}

	bool IsSurfacePlanar(const UE::CADKernel::FBezierSurface& Surface)
	{
		if (Surface.GetUDegree() == 1 && Surface.GetVDegree() == 1)
		{
			return true;
		}

		return AreControlPointsPlanar(Surface.GetPoles(), Surface.GetUDegree() + 1);
	}

	bool IsSurfacePlanar(const UE::CADKernel::FNURBSSurface& Surface)
	{
		if (Surface.GetDegree(EIso::IsoU) == 1 && Surface.GetDegree(EIso::IsoV) == 1)
		{
			return true;
		}

		// #cad_kernel: Might be rational but still planar
		if (Surface.IsRational())
		{
			return false;
		}

		return AreControlPointsPlanar(Surface.GetPoles(), Surface.GetPoleCount(EIso::IsoU));
	}

	bool IsPlanar(const UE::CADKernel::FSurface& Surface)
	{
		using namespace UE::CADKernel;

		switch (Surface.GetSurfaceType())
		{
		case ESurface::Nurbs:
			return IsSurfacePlanar(static_cast<const FNURBSSurface&>(Surface));
		
		case ESurface::Bezier:
			return IsSurfacePlanar(static_cast<const FBezierSurface&>(Surface));

		case ESurface::Offset:
			return IsPlanar(*static_cast<const FOffsetSurface&>(Surface).GetBaseSurface());

		case ESurface::Composite:
		case ESurface::Cone:
		case ESurface::Coons:
		case ESurface::Cylinder:
		case ESurface::Revolution:
		case ESurface::Ruled:
		case ESurface::Sphere:
		case ESurface::TabulatedCylinder:
		case ESurface::Torus:
			return false;

		case ESurface::Plane:
			return true;

		default:
			ensureCADKernel(false);
			break;
		}

		return false;
	}
}
