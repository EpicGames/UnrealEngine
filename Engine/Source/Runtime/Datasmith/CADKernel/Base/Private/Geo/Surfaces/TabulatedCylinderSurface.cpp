// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/TabulatedCylinderSurface.h"

#include "Math/Point.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{

void FTabulatedCylinderSurface::SpawnIdent(FDatabase& Database)
{
	if (!FEntity::SetId(Database))
	{
		return;
	}

	GuideCurve->SpawnIdent(Database);
}

void FTabulatedCylinderSurface::ResetMarkersRecursively() const
{
	ResetMarkers();
	GuideCurve->ResetMarkersRecursively();
}

void FTabulatedCylinderSurface::EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder) const
{
	const FLinearBoundary& CurveBounds = GuideCurve->GetBoundary();

	FCurvePoint PointU;
	GuideCurve->EvaluatePoint(CurveBounds.GetAt(InSurfacicCoordinate.X), PointU, InDerivativeOrder);

	OutPoint3D.DerivativeOrder = InDerivativeOrder;
	OutPoint3D.Point = PointU.Point + DirectorVector * InSurfacicCoordinate.Y;

	if (InDerivativeOrder > 0) 
	{
		OutPoint3D.GradientU = PointU.Gradient * CurveBounds.Length();
		OutPoint3D.GradientV = DirectorVector;

		if (InDerivativeOrder > 1)
		{
			OutPoint3D.LaplacianU = PointU.Laplacian * FMath::Square(CurveBounds.Length());
			OutPoint3D.LaplacianV = FVector::ZeroVector;
			OutPoint3D.LaplacianUV = FVector::ZeroVector;
		}
	}
}

void FTabulatedCylinderSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	OutPoints.bWithNormals = bComputeNormals;

	int32 PointNum = Coordinates.Count();
	OutPoints.Reserve(PointNum);

	OutPoints.Set2DCoordinates(Coordinates);

	int32 DerivativeOrder = bComputeNormals ? 1 : 0;
	TArray<FCurvePoint> GuidePoints;
	GuideCurve->EvaluatePoints(Coordinates[EIso::IsoU], GuidePoints, DerivativeOrder);

	for (double VCoordinate : Coordinates[EIso::IsoV])
	{
		for (const FCurvePoint& Point : GuidePoints)
		{
			OutPoints.Points3D.Emplace(Point.Point + DirectorVector * VCoordinate);
		}
	}

	if (bComputeNormals)
	{
		OutPoints.Normals.SetNum(PointNum);
		int32 NumU = Coordinates.IsoCount(EIso::IsoU);
		int32 NumV = Coordinates.IsoCount(EIso::IsoV);
		int32 Undex = 0;
		for (const FCurvePoint& Point : GuidePoints)
		{
			int32 Index = Undex++;
			FVector3f Normal = (FVector3f)(Point.Gradient ^ DirectorVector);
			for (int32 Vndex = 0; Vndex < NumV; ++Vndex)
			{
				OutPoints.Normals[Index] = Normal;
				Index += NumU;
			}
		}
		OutPoints.NormalizeNormals();
	}

}

void FTabulatedCylinderSurface::Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates)
{
	GuideCurve->Presample(Coordinates[EIso::IsoU], Tolerance3D);

	Coordinates[EIso::IsoV].Empty(3);
	Coordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Min);
	Coordinates[EIso::IsoV].Add((InBoundaries[EIso::IsoV].Max + InBoundaries[EIso::IsoV].Min) / 2.0);
	Coordinates[EIso::IsoV].Add(InBoundaries[EIso::IsoV].Max);
}

void FTabulatedCylinderSurface::LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivables) const
{
	const FLinearBoundary& CurveBounds = GuideCurve->GetBoundary();
	double Length = CurveBounds.Length();

	GuideCurve->FindNotDerivableCoordinates(InDerivativeOrder, OutNotDerivables[EIso::IsoU]);

	for (double& NotDerivable : OutNotDerivables[EIso::IsoU])
	{
		NotDerivable = (NotDerivable - CurveBounds.Min) / Length;
	}
}

TSharedPtr<FEntityGeom> FTabulatedCylinderSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	TSharedPtr<FEntityGeom> TransformedGuideCurve = GuideCurve->ApplyMatrix(InMatrix);

	if (!TransformedGuideCurve.IsValid())
	{
		return TSharedPtr<FEntityGeom>();
	}

	double GuideUMin = GuideCurve->GetUMin();
	FVector Point = GuideCurve->EvaluatePoint(GuideUMin);
	Point += DirectorVector;

	FVector TransformedPoint = InMatrix.Multiply(Point);
	Point = StaticCastSharedPtr<FCurve>(TransformedGuideCurve)->EvaluatePoint(GuideUMin);
	FVector NewDirector = TransformedPoint - Point;

	return FEntity::MakeShared<FTabulatedCylinderSurface>(Tolerance3D, StaticCastSharedPtr<FCurve>(TransformedGuideCurve), NewDirector, Boundary[EIso::IsoV].Min, Boundary[EIso::IsoV].Max);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FTabulatedCylinderSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("guide curve"), GuideCurve)
		.Add(TEXT("direction vector"), DirectorVector)
		.Add(TEXT("VMin"), Boundary[EIso::IsoV].Min)
		.Add(TEXT("VMax"), Boundary[EIso::IsoV].Max);
}
#endif

}
