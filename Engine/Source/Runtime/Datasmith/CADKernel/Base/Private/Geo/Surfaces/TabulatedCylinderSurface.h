// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
class FCurve;

class CADKERNEL_API FTabulatedCylinderSurface : public FSurface
{
	friend FEntity;

protected:
	TSharedPtr<FCurve> GuideCurve;
	FVector DirectorVector;

	/**
	 * A tabulated cylinder is a surface formed by moving a curve segment called GuideCurve parallel to
	 * itself along a vector called DirectorVector.
	 *
	 * The bounds of the cylinder are defined as follow:
	 * Bounds[EIso::IsoU] = GuideCurve bounds;
	 * Bounds[EIso::IsoV].Min = InVMin;
	 * Bounds[EIso::IsoV].Max = InVMax;
	 */
	FTabulatedCylinderSurface(const double InToleranceGeometric, TSharedPtr<FCurve> InGuideCurve, const FVector& InDirectorVector, double InVMin = 0., double InVMax = 1.)
		: FSurface(InToleranceGeometric, 0.0, 1.0, InVMin, InVMax)
		, GuideCurve(InGuideCurve)
		, DirectorVector(InDirectorVector)
	{
		ComputeDefaultMinToleranceIso();
	}

	FTabulatedCylinderSurface() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		SerializeIdent(Ar, GuideCurve);
		Ar << DirectorVector;
	}

	virtual void SpawnIdent(FDatabase& Database) override;
	virtual void ResetMarkersRecursively() const override;

	ESurface GetSurfaceType() const
	{
		return ESurface::TabulatedCylinder;
	}

	TSharedPtr<FCurve> GetGuideCurve() const
	{
		return GuideCurve;
	}

	FVector GetDirectorVector() const
	{
		return DirectorVector;
	}

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override;
	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivables) const override;

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override;

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

};

} // namespace UE::CADKernel
