// Copyright Epic Games, Inc. All Rights Reserved.

#include "Geo/Surfaces/TorusSurface.h"

#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{

TSharedPtr<FEntityGeom> FTorusSurface::ApplyMatrix(const FMatrixH& InMatrix) const
{
	FMatrixH NewMatrix = InMatrix * Matrix;
	return FEntity::MakeShared<FTorusSurface>(Tolerance3D, NewMatrix, MajorRadius, MinorRadius, Boundary[EIso::IsoU].Min, Boundary[EIso::IsoU].Max, Boundary[EIso::IsoV].Min, Boundary[EIso::IsoV].Max);
}

#ifdef CADKERNEL_DEV
FInfoEntity& FTorusSurface::GetInfo(FInfoEntity& Info) const
{
	return FSurface::GetInfo(Info)
		.Add(TEXT("Matrix"), Matrix)
		.Add(TEXT("MajorRadius"), MajorRadius)
		.Add(TEXT("MinorRadius"), MinorRadius)
		.Add(TEXT("MajorStartAngle"), Boundary[EIso::IsoU].Min)
		.Add(TEXT("MajorEndAngle"), Boundary[EIso::IsoU].Max)
		.Add(TEXT("MinorStartAngle"), Boundary[EIso::IsoV].Min)
		.Add(TEXT("MinorEndAngle"), Boundary[EIso::IsoV].Max);
}
#endif

void FTorusSurface::EvaluatePointGridInCylindricalSpace(const FCoordinateGrid& Coordinates, TArray<FVector2d>& OutPoints) const
{
	int32 PointNum = Coordinates.Count();

	OutPoints.Empty(PointNum);

	int32 UCount = Coordinates.IsoCount(EIso::IsoU);

	TArray<double> CosU;
	TArray<double> SinU;
	CosU.Reserve(UCount);
	SinU.Reserve(UCount);

	for (double Angle : Coordinates[EIso::IsoU])
	{
		CosU.Emplace(cos(Angle));
	}

	for (double Angle : Coordinates[EIso::IsoU])
	{
		SinU.Emplace(sin(Angle));
	}

	for (double Angle : Coordinates[EIso::IsoV])
	{
		double CosV = cos(Angle);
		double Rho = MajorRadius + MinorRadius * CosV;

		double SwapOrientation = (Angle < DOUBLE_PI && Angle >= 0) ? 1.0 : -1.0;
		for (int32 Undex = 0; Undex < UCount; Undex++)
		{
			OutPoints.Emplace(Rho * CosU[Undex] * SwapOrientation, Rho * SinU[Undex]);
		}
	}
}

void FTorusSurface::EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals) const
{
	int32 PointNum = Coordinates.Count();

	OutPoints.bWithNormals = bComputeNormals;

	OutPoints.Reserve(PointNum);
	OutPoints.Set2DCoordinates(Coordinates);

	int32 UCount = Coordinates.IsoCount(EIso::IsoU);

	TArray<double> CosU;
	TArray<double> SinU;
	CosU.Reserve(UCount);
	SinU.Reserve(UCount);

	for (double Angle : Coordinates[EIso::IsoU])
	{
		CosU.Emplace(cos(Angle));
	}

	for (double Angle : Coordinates[EIso::IsoU])
	{
		SinU.Emplace(sin(Angle));
	}


	for (double Angle : Coordinates[EIso::IsoV])
	{
		double CosV = cos(Angle);
		double SinV = sin(Angle);
		double Rho = MajorRadius + MinorRadius * CosV;
		double Height = MinorRadius * SinV;

		for (int32 Undex = 0; Undex < UCount; Undex++)
		{
			OutPoints.Points3D.Emplace(Rho * CosU[Undex], Rho * SinU[Undex], Height);
		}

		if (bComputeNormals)
		{
			double DeltaHeight = MinorRadius * CosV;
			for (int32 Undex = 0; Undex < UCount; Undex++)
			{
				FVector GradientU = FVector(Rho * (-SinU[Undex]), Rho * CosU[Undex], 0.0);
				FVector GradientV = FVector(-Height * CosU[Undex], -Height * SinU[Undex], DeltaHeight);
				OutPoints.Normals.Emplace(GradientU ^ GradientV);
			}
		}
	}

	for (FVector& Point : OutPoints.Points3D)
	{
		Point = Matrix.Multiply(Point);
	}

	if (bComputeNormals)
	{
		FVector3f Center = (FVector3f)Matrix.Column(3);
		for (FVector3f& Normal : OutPoints.Normals)
		{
			 Normal = Matrix.MultiplyVector(Normal);
		}
		OutPoints.NormalizeNormals();
	}
}

}
