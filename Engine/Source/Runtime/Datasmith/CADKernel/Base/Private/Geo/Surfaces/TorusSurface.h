// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Surfaces/Surface.h"

namespace UE::CADKernel
{
class CADKERNEL_API FTorusSurface : public FSurface
{
	friend FEntity;

protected:
	FMatrixH Matrix;
	double MajorRadius;
	double MinorRadius;

	/**
	 * A torus is the solid formed by revolving a circular disc about a specified coplanar axis.
	 * MajorRadius is the distance from the axis to the center of the defining disc, and MinorRadius is the radius of the defining disc,
	 * where MajorRadius > MinorRadius > 0.0.
	 *
	 * The torus computed at the origin with Z axis.
	 * It is placed at its final position and orientation by the Matrix
	 */
	FTorusSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InMajorRadius, double InMinorRadius, double InMajorStartAngle = 0.0, double InMajorEndAngle = DOUBLE_TWO_PI, double InMinorStartAngle = 0.0, double InMinorEndAngle = DOUBLE_TWO_PI)
		: FTorusSurface(InToleranceGeometric, InMatrix, InMajorRadius, InMinorRadius, FSurfacicBoundary(InMajorStartAngle, InMajorEndAngle, InMinorStartAngle, InMinorEndAngle))
	{
	}

	/**
	 * A torus is the solid formed by revolving a circular disc about a specified coplanar axis.
	 * MajorRadius is the distance from the axis to the center of the defining disc, and MinorRadius is the radius of the defining disc,
	 * where MajorRadius > MinorRadius > 0.0.
	 *
	 * The torus computed at the origin with Z axis.
	 * It is placed at its final position and orientation by the Matrix
	 *
	 * The bounds of the cone are defined as follow:
	 * Bounds[EIso::IsoU].Min = MajorStartAngle
	 * Bounds[EIso::IsoU].Max = MajorEndAngle
	 * Bounds[EIso::IsoV].Min = MinorStartAngle
	 * Bounds[EIso::IsoV].Max = MinorEndAngle
	 */
	FTorusSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InMajorRadius, double InMinorRadius, const FSurfacicBoundary& InBoundary)
		: FSurface(InToleranceGeometric, InBoundary)
		, Matrix(InMatrix)
		, MajorRadius(InMajorRadius)
		, MinorRadius(InMinorRadius)
	{
		ComputeMinToleranceIso();
	}

	FTorusSurface() = default;

	void ComputeMinToleranceIso()
	{
		double Tolerance2DU = Tolerance3D / MajorRadius;
		double Tolerance2DV = Tolerance3D / MinorRadius;

		FVector Origin = Matrix.Multiply(FVector::ZeroVector);

		FVector Point2DU{ 1 , 0, 0 };
		FVector Point2DV{ 0, 1, 0 };

		Tolerance2DU /= ComputeScaleAlongAxis(Point2DU, Matrix, Origin);
		Tolerance2DV /= ComputeScaleAlongAxis(Point2DV, Matrix, Origin);

		MinToleranceIso.Set(FMath::Abs(Tolerance2DU), FMath::Abs(Tolerance2DV));
	}

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurface::Serialize(Ar);
		Ar << Matrix;
		Ar << MajorRadius;
		Ar << MinorRadius;
	}

	ESurface GetSurfaceType() const
	{
		return ESurface::Torus;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const override
	{
		const double CosU = cos(InSurfacicCoordinate.X);
		const double CosV = cos(InSurfacicCoordinate.Y);

		const double SinU = sin(InSurfacicCoordinate.X);
		const double SinV = sin(InSurfacicCoordinate.Y);
		double Rho = (MajorRadius + MinorRadius * CosV);

		OutPoint3D.DerivativeOrder = InDerivativeOrder;
		OutPoint3D.Point.Set(Rho * CosU, Rho * SinU, MinorRadius * SinV);

		OutPoint3D.Point = Matrix.Multiply(OutPoint3D.Point);

		if (InDerivativeOrder > 0)
		{
			OutPoint3D.GradientU = FVector((MajorRadius + MinorRadius * CosV) * -SinU, (MajorRadius + MinorRadius * CosV) * CosU, 0.0);
			OutPoint3D.GradientV = FVector((MinorRadius * -SinV) * CosU, (MinorRadius * -SinV) * SinU, MinorRadius * CosV);

			OutPoint3D.GradientU = Matrix.MultiplyVector(OutPoint3D.GradientU);
			OutPoint3D.GradientV = Matrix.MultiplyVector(OutPoint3D.GradientV);

			if (InDerivativeOrder > 1)
			{
				OutPoint3D.LaplacianU = FVector((MajorRadius + MinorRadius * CosV) * -CosU, (MajorRadius + MinorRadius * CosV) * -SinU, 0.0);
				OutPoint3D.LaplacianV = FVector((MinorRadius * -CosV) * CosU, (MinorRadius * -CosV) * SinU, MinorRadius * -SinV);
				OutPoint3D.LaplacianUV = FVector((MinorRadius * -SinV) * -SinU, (MinorRadius * -SinV) * CosU, 0.0);

				OutPoint3D.LaplacianU = Matrix.MultiplyVector(OutPoint3D.LaplacianU);
				OutPoint3D.LaplacianV = Matrix.MultiplyVector(OutPoint3D.LaplacianV);
				OutPoint3D.LaplacianUV = Matrix.MultiplyVector(OutPoint3D.LaplacianUV);
			}
		}
	}

	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const override;

	virtual void EvaluatePointGridInCylindricalSpace(const FCoordinateGrid& Coordinates, TArray<FVector2d>&) const override;

	virtual FVector2d EvaluatePointInCylindricalSpace(const FVector2d& InSurfacicCoordinate) const override
	{
		const double CosU = cos(InSurfacicCoordinate.X);
		const double CosV = cos(InSurfacicCoordinate.Y);

		const double SinU = sin(InSurfacicCoordinate.X);

		double Rho = (MajorRadius + MinorRadius * CosV);

		double SwapOrientation = (InSurfacicCoordinate.Y < DOUBLE_PI&& InSurfacicCoordinate.Y >= 0) ? 1.0 : -1.0;

		return FVector2d(Rho * CosU * SwapOrientation, Rho * SinU);
	}

	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates) override
	{
		PresampleIsoCircle(InBoundaries, OutCoordinates, EIso::IsoU);
		PresampleIsoCircle(InBoundaries, OutCoordinates, EIso::IsoV);
	}

};
}

