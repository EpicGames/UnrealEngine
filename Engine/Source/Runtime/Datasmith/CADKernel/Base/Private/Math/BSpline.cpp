// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/BSpline.h"

#include "Geo/Curves/NURBSCurve.h"
#include "Geo/Sampling/SurfacicSampling.h"
#include "Geo/Surfaces/NURBSSurface.h"
#include "Math/MatrixH.h"
#include "Mesh/Structure/Grid.h"
#include "Utils/Util.h"

#include <algorithm>

namespace UE::CADKernel
{
const FLinearBoundary FLinearBoundary::DefaultBoundary(0., 1.);
const FSurfacicBoundary FSurfacicBoundary::DefaultBoundary(0., 1., 0., 1.);

namespace BSpline
{

// DeBoor Valeurs B-Spline
void DeBoorValeursBSpline(const int32 degre, const double* const knot, const int32 segment, const int32 ndim, const int32 derivee, const double u, double* const pole_aux, double* const grad_aux, double* const lap_aux, double* const valeur, double* const Gradient, double* const Laplacian);
void InsertKnot(TArray<double>& vn, TArray<FVector>& poles, TArray<double>& weights, double u, double* newU = nullptr);
void Blossom(int32 degre, const TArray<FVector>& poles, const TArray<double>& vecteurNodal, const TArray<double>& poids, int32 seg, TArray<double>& params, FVector& pnt, double& weight);
void DuplicateNurbsCurveWithHigherDegree(int32 degre, const TArray<FVector>& poles, const TArray<double>& nodalVector, const TArray<double>& weights, TArray<FVector>& newPoles, TArray<double>& newNodalVector, TArray<double>& newWeights);

#define DEGRE_MAX_POLY 9
namespace
{ // anonymous ns for static
int32 CoefficientsBinomiaux[][10] =
{
	{ 1 },
	{ 1, 1 },
	{ 1, 2, 1},
	{ 1, 3, 3, 1 },
	{ 1, 4, 6, 4, 1 },
	{ 1, 5, 10, 10, 5, 1 },
	{ 1, 6, 15, 20, 15, 6, 1 },
	{ 1, 7, 21, 35, 35, 21, 7, 1},
	{ 1, 8, 28, 56, 70, 56, 28, 8, 1},
	{ 1, 9, 36, 84, 126, 126, 84, 36, 9, 1}
};
}

int32 fact_r(int32 n)
{
	if (n <= 0) return 1;
	return n * fact_r(n - 1);
}

int32 fact(int32 n)
{
	if (n <= 0) return 1;
	int32 k = n;
	int32 result = k--;
	while (k > 0)
	{
		result *= k;
		k--;
	}
	return result;
}

int32 CoefficientBinomial(int32 n, int32 k)
{
	if (n <= DEGRE_MAX_POLY) return CoefficientsBinomiaux[n][k];
	return fact(n) / (fact(k) * fact(n - k));
}

// #cadkernel_check: Precompute Bernstein values.
void Bernstein(int32 Degree, double InCoordinateU, double* BernsteinValuesAtU, double* BernsteinGradientsAtU, double* BernsteinLaplaciansAtU)
{
	int32 i;
	double t, s;

	const int32 Order = Degree + 1;
	TArray<double> Buffer;
	Buffer.SetNum(3 * Order);

	double* ti = Buffer.GetData();
	double* si = ti + Order;
	double* ci = si + Order;

	// calculer les coefficient ci
	ci[0] = 1;
	ci[Degree] = 1;

	for (i = 1; i < Degree; i++)
	{
		ci[i] = ci[i - 1] * (Order - i) / (double)i;
	}

	ti[0] = 1.;
	si[0] = 1.;

	// calculer les monomes t**i et s**i=(1-t)**i
	t = InCoordinateU;
	s = 1. - InCoordinateU;
	for (i = 1; i <= Degree; i++)
	{
		ti[i] = ti[i - 1] * t;
		si[i] = si[i - 1] * s;
	}

	// calculer les polynomes de bernstein
	for (i = 0; i <= Degree; i++)
	{
		BernsteinValuesAtU[i] = ci[i] * ti[i] * si[Degree - i];
	}

	if (BernsteinGradientsAtU)
	{
		// calculer les derivees premieres
		BernsteinGradientsAtU[0] = -Degree * si[Degree - 1];
		BernsteinGradientsAtU[Degree] = Degree * ti[Degree - 1];
		for (i = 1; i < Degree; i++)
		{
			BernsteinGradientsAtU[i] = ci[i] * (i * ti[i - 1] * si[Degree - i] - (Degree - i) * ti[i] * si[Degree - i - 1]);
		}

		if (BernsteinLaplaciansAtU)
		{
			// calculer les derivees secondes
			if (Degree > 1)
			{
				BernsteinLaplaciansAtU[0] = Degree * (Degree - 1) * si[Degree - 2];
				BernsteinLaplaciansAtU[1] = ci[1] * (-2) * (Degree - 1) * si[Degree - 2];
				if (Degree > 2) BernsteinLaplaciansAtU[1] += ci[1] * (Degree - 1) * (Degree - 2) * ti[1] * si[Degree - 3];
				BernsteinLaplaciansAtU[Degree] = Degree * (Degree - 1) * ti[Degree - 2];
				BernsteinLaplaciansAtU[Degree - 1] = ci[Degree - 1] * (-2) * (Degree - 1) * ti[Degree - 2];
				if (Degree > 2) BernsteinLaplaciansAtU[Degree - 1] += ci[Degree - 1] * (Degree - 1) * (Degree - 2) * ti[Degree - 3] * si[1];

				for (i = 2; i < Degree - 1; i++)
				{
					BernsteinLaplaciansAtU[i] = ci[i] * (i * (i - 1) * ti[i - 2] * si[Degree - i]
						- 2 * i * (Degree - i) * ti[i - 1] * si[Degree - i - 1]
						+ (Degree - i) * (Degree - i - 1) * ti[i] * si[Degree - i - 2]);
				}
			}
			else
			{
				BernsteinLaplaciansAtU[0] = 0.0f;
			}
		}
	}
}

int32 Dichotomy(const double* const tab, double key, int32 first, int32 last)
{
	int32 center;
	int32 a = first;
	int32 b = last;
	while (a < b) {
		center = (a + b) >> 1;
		if (key > tab[center] && key <= tab[center + 1]) {
			return center;
		}
		else if (key <= tab[center]) {
			b = center;
		}
		else {
			a = center + 1;
		}
	}
	return a;
}

/******************************description fonction*****************************/
/* Auteur:						 - Fichier: bspline.c										*/
/* Role:- Calculer l'interpolation par fonctions BSplines 1D d'ordre "degre"	*/
/*		 d'une liste de nb_pole poles de dimensionnalite ndim stokes dans	*/
/*		 tab_pole, par rapport au vecteur nodal knot , le calcul portant sur*/
/*		 nb_pt points definis dans tab_u .												*/
/*		 On recupere en sortie les valeurs interpolees dans						*/
/*		 valeur[point][composante] et selon l'ordre de derivation desire		*/
/*		 (0, 1 ou 2) leurs derivees premieres et secondes dans					*/
/*		 Gradient[point][composante] et Laplacian[point][composante].						*/
/*******************************fin fonction************************************/
void Interpolate1DBSpline(
	const int32 Degre,						/* E-Odre de la bspline*/
	const int32 PoleNum,						/* E-Nombre de poles de la bspline*/
	const double* const NodalVector,		/* E-Le vecteur nodale de la bspline*/
	const int32 SpaceDimension,			/* E-Dimension des poles (1D,2D,3D,4D=3D+poids...)*/
	const double* const Poles,				/* E-La liste des poles de dimension ndim*/
	const int32 PointNum,					/* E-Le nombre de valeurs a calculer*/
	const double* const Coordinate,		/* E-La liste des parametres a calculer*/
	const int32 Derivee,						/* E-L'odre de derivation*/
	double* const OutPoints,					/* S-Les points calcules*/
	double* const OutGradient,				/* S-Les derivees premieres*/
	double* const OutLaplacian				/* S-Les derivees secondes*/
)
{
	TArray<double> pole_aux;
	TArray<double> grad_aux;
	TArray<double> lap_aux;

	int32 Index, IndexK, StartIndex, EndIndex, Segment, PreviousSegment, IndexSegment, DegrePlus1, SquareDegre;

	TArray<double> tab_u;
	tab_u.SetNum(PointNum);
	std::copy(Coordinate, Coordinate + PointNum, tab_u.GetData());

	DegrePlus1 = Degre + 1;
	SquareDegre = DegrePlus1 * DegrePlus1;
	/* allocations des tables de travail*/

	/* Allocations pour Algorithme De Boor*/
	pole_aux.Init(0., SquareDegre * SpaceDimension);

	if (Derivee >= 1)
	{
		grad_aux.Init(0., SquareDegre * SpaceDimension);
	}

	if (Derivee >= 2)
	{
		lap_aux.Init(0., SquareDegre * SpaceDimension);
	}

	/* initialiser les index de debut et fin du domaine nodal*/
	StartIndex = Degre;
	EndIndex = PoleNum;
	PreviousSegment = 0;

	/* calcul des points apres definition du segment BSpline*/
	for (Index = 0; Index < PointNum; Index++)
	{
		double& currentU = tab_u[Index];

		/* Verifier la coordonnee parametrique*/
		if (currentU < NodalVector[Degre]) currentU = NodalVector[Degre];
		if (currentU > NodalVector[EndIndex]) currentU = NodalVector[EndIndex];

		/* Calculer le segment auquel appartient la coordonnee parametrique*/
		if (currentU < NodalVector[StartIndex]) StartIndex = Degre;
		//for(iseg = ideb ; iseg <ifin && currentU > knot[iseg +1]; ++iseg);
		IndexSegment = Dichotomy(NodalVector, currentU, StartIndex, EndIndex);
		if (IndexSegment > EndIndex) IndexSegment = EndIndex;

		Segment = IndexSegment;

		/* mise a jour du segment courant pour un point suivant a evaluer*/
		StartIndex = IndexSegment;

		/* calcul par la methode de De Boor*/
		if (Segment != PreviousSegment)
		{
			/* reinitialiser le vecteur des poles du segment courant*/
			for (IndexK = 0; IndexK <= Degre; ++IndexK)
			{
				memcpy(&pole_aux[IndexK * SpaceDimension], &Poles[(Segment - Degre + IndexK) * SpaceDimension], SpaceDimension * sizeof(pole_aux[0]));
			}
		}

		DeBoorValeursBSpline(Degre, NodalVector, Segment, SpaceDimension, Derivee, currentU, &(pole_aux[0]), Derivee > 0 ? &(grad_aux[0]) : nullptr, Derivee > 1 ? &(lap_aux[0]) : nullptr, OutPoints + (Index * SpaceDimension), OutGradient + (Index * SpaceDimension), OutLaplacian + (Index * SpaceDimension));

		PreviousSegment = Segment;
	}
}

/******************************description fonction*****************************/
/* Auteur:						 - Fichier: bspline.c										*/
/* Role:- Calculer l'interpolation par fonctions BSplines 2D d'une liste de	*/
/*		 (n_pole_u*n_pole_v) poles de dimensionnalite ndim stokees dans pole,*/
/*		 par rapport aux vecteurs nodaux knot_u,knot_v ,le calcul s'effectuant*/
/*		 sur un pave tab_u*tab_v comprenant nb_pt_u*nb_pt_v points.			*/
/*		 On recupere en sortie les valeurs interpolees dans valeur, et selon*/
/*		 l'ordre de derivation desire 0, 1 ou 2 leurs derivees premieres	*/
/*		 deru,derv et secondes deruu, dervv, deruv.									*/
/*******************************fin fonction************************************/
void InterpolerBSpline2D(const FNURBSSurface& Nurbs, FGrid& Grid)
{
}

/******************************description fonction*****************************/
/* Auteur:						 - Fichier: bspline.c										*/
/* Role:- Calculer l'interpolation par fonctions BSplines 2D d'une liste de	*/
/*		 (n_pole_u*n_pole_v) poles de dimensionnalite ndim stokees dans pole,*/
/*		 par rapport aux vecteurs nodaux knot_u,knot_v ,le calcul s'effectuant*/
/*		 sur un pave tab_u*tab_v comprenant nb_pt_u*nb_pt_v points.			*/
/*		 On recupere en sortie les valeurs interpolees dans valeur, et selon*/
/*		 l'ordre de derivation desire 0, 1 ou 2 leurs derivees premieres	*/
/*		 deru,derv et secondes deruu, dervv, deruv.									*/
/*******************************fin fonction************************************/
void Interpolate2DBSpline(
	const int32 UDegre,					/* E-Le degre en U de la bspline*/
	const int32 VDegre,					/* E-Le degre en V de la bspline*/
	const int32 PoleUNum,				/* E-Le nombre de poles en U de la bspline*/
	const int32 PoleVNum,				/* E-Le nombre de poles en V de la bspline*/
	const double* const UNodalVector,	/* E-Le vecteur nodale en U de la bspline*/
	const double* const VNodalVector,	/* E-Le vecteur nodale en V de la bspline*/
	const int32 ndim,					/* E-Dimension des poles (1D,2D,3D,4D=3D+poids...)*/
	const double* const PoleUV,			/* E-La liste des poles*/

	const int32 nb_pt_u,				/* E-Le nombre de parametre U de la grille a calculer*/
	const double* const tab_u,			/* E-La liste des paramletres U de la grille a calculer*/
	const int32 nb_pt_v,				/* E-Le nombre de parametre V de la grille a calculer*/
	const double* const tab_v,			/* E-La liste des paramletres V de la grille a calculer*/

	const int32 derivee,				/* E-L'ordre desiree de derivation*/
	double* const valeur,				/* S-La liste des valeurs*/
	double* const deru,					/* S-Les derivee premiere par rapport a U*/
	double* const derv,					/* S-Les derivee premiere par rapport a V*/
	double* const deruu,				/* S-Les derivee seconde par rapport a U*/
	double* const dervv,				/* S-Les derivee seconde par rapport a V*/
	double* const deruv					/* S-Les derivee seconde croise (dUdV)*/
)
{
	EIso IsoType;
	int32 deg1, deg2, nb_pt1, nb_pt2, nb_pole1, nb_pole2;
	bool inversion_uv;
	int32 IndexV, IndexU;

	/* initialisations*/
	TArray<double> tab_t1;
	TArray<double> tab_t2;
	TArray<double> pole1;
	TArray<double> pole2;
	TArray<double> pole_aux;
	TArray<double> val_aux;
	TArray<double> d1pole_aux;
	TArray<double> d1pole2;
	TArray<double> d1val_aux;
	TArray<double> d2val_aux;
	TArray<double> d11pole_aux;
	TArray<double> d11pole2;
	TArray<double> d11val_aux;
	TArray<double> d22val_aux;
	TArray<double> d21val_aux;

	double const* knot1;
	double const* knot2;

	/* Calculer le nombre d'evaluation necessaires suivant les 2 strategies
	isoV ==> on cree nb_pt_v iso_v d'abord puis calcul de nb_pt_u points
	isoU ==> on cree nb_pt_u iso_u d'abord puis calcul de nb_pt_v points
	Choisir le nombre Minimal d'evaluations*/

	if (((1 + derivee) * (nb_pt_v * PoleUNum * VDegre * (VDegre + 1)) + (1 + 2 * derivee) * (nb_pt_u * nb_pt_v * UDegre * (UDegre + 1))) < ((1 + derivee) * (nb_pt_u * PoleVNum * UDegre * (UDegre + 1)) + (1 + 2 * derivee) * (nb_pt_u * nb_pt_v * VDegre * (VDegre + 1))))
	{
		/* choisir la strategie isoV*/
		nb_pt1 = nb_pt_v;
		knot1 = VNodalVector;
		nb_pt2 = nb_pt_u;
		knot2 = UNodalVector;
		deg1 = VDegre;
		deg2 = UDegre;
		nb_pole1 = PoleVNum;
		nb_pole2 = PoleUNum;
		IsoType = EIso::IsoV;
		inversion_uv = true; /* derivation par rapport a v d'abord*/
	}
	else
	{
		/* choisir la strategie isoU*/
		nb_pt1 = nb_pt_u;
		knot1 = UNodalVector;
		nb_pt2 = nb_pt_v;
		knot2 = VNodalVector;
		deg1 = UDegre;
		deg2 = VDegre;
		nb_pole1 = PoleUNum;
		nb_pole2 = PoleVNum;
		IsoType = EIso::IsoU;
		inversion_uv = false; /* derivation par rapport a u d'abord*/
	}

	/* recopie des parametres en entree*/
	tab_t1.SetNum(nb_pt1);
	tab_t2.SetNum(nb_pt2);

	if (IsoType == EIso::IsoV)
	{
		std::copy(tab_v, tab_v + nb_pt1, tab_t1.GetData());
		std::copy(tab_u, tab_u + nb_pt2, tab_t2.GetData());
	}
	else // (IsoType == EIso::IsoU)
	{
		std::copy(tab_u, tab_u + nb_pt1, tab_t1.GetData());
		std::copy(tab_v, tab_v + nb_pt2, tab_t2.GetData());
	}

	/* constituer le reseau de pole pour chaque courbe de construction*/
	pole1.SetNum(nb_pole2 * nb_pole1 * ndim);

	/* on constitue un vecteur de nb_pole1 comportant (nb_pole2*ndim) composantes (ces composantes seront ensuite interpretes comme une sucession de (point(x,y,z),poids) a la suite les uns des autres)*/
	if (IsoType == EIso::IsoV)
	{
		for (IndexV = 0; IndexV < nb_pole1; IndexV++)
		{
			for (IndexU = 0; IndexU < nb_pole2; IndexU++)
			{
				std::copy(PoleUV + ((IndexV * PoleUNum + IndexU) * ndim), PoleUV + ((IndexV * PoleUNum + IndexU) + 1) * ndim, pole1.GetData() + ((IndexV * nb_pole2 + IndexU) * ndim));
			}
		}
	}
	else // (IsoType == EIso::IsoU)
	{
		for (IndexV = 0; IndexV < nb_pole1; IndexV++)
		{
			for (IndexU = 0; IndexU < nb_pole2; IndexU++)
			{
				std::copy(PoleUV + (IndexU * PoleUNum + IndexV) * ndim, PoleUV + ((IndexU * PoleUNum + IndexV) + 1) * ndim, pole1.GetData() + ((IndexV * nb_pole2 + IndexU) * ndim));
			}
		}
	}

	/* Allouer pour chaque valeur du premier parametre (nb_pt1 points) les tableaux des poles fictifs interpoles et derivees correspondants a la courbe iso-parametre generee (nb_pt1 courbes)*/
	pole_aux.SetNum(nb_pt1 * nb_pole2 * ndim);

	/* derivee premiere*/
	if (derivee >= 1)
	{
		d1pole_aux.SetNum(nb_pt1 * nb_pole2 * ndim);
	}

	/* derivee seconde*/
	if (derivee >= 2)
	{
		d11pole_aux.SetNum(nb_pt1 * nb_pole2 * ndim);
	}

	/* allouer les tables de travail necessaires a l'interpolation par rapport au second parametre de vecteurs de (nb_pt1 * ndim) composantes*/

	pole2.SetNum(nb_pole2 * nb_pt1 * ndim);
	val_aux.SetNum(nb_pt2 * nb_pt1 * ndim);

	/* derivee premiere*/
	if (derivee >= 1)
	{
		d1pole2.SetNum(nb_pole2 * nb_pt1 * ndim);
		d1val_aux.SetNum(nb_pt2 * nb_pt1 * ndim);
		d2val_aux.SetNum(nb_pt2 * nb_pt1 * ndim);
	}

	/* derivee seconde*/
	if (derivee >= 2)
	{
		d11pole2.SetNum(nb_pole2 * nb_pt1 * ndim);
		d11val_aux.SetNum(nb_pt2 * nb_pt1 * ndim);
		d22val_aux.SetNum(nb_pt2 * nb_pt1 * ndim);
		d21val_aux.SetNum(nb_pt2 * nb_pt1 * ndim);
	}

	/* interpoler et deriver le vecteur pole fictif de dimension (nb_pole2*ndim) par rapport au premier parametre*/
	Interpolate1DBSpline(deg1, nb_pole1, knot1, nb_pole2 * ndim, &(pole1[0]), nb_pt1, &(tab_t1[0]), derivee, &(pole_aux[0]), derivee > 0 ? &(d1pole_aux[0]) : nullptr, derivee > 1 ? &(d11pole_aux[0]) : nullptr);

	/* Constituer les vecteurs de travail de (nb_pt1*ndim) composantes a interpoler par rapport au 2 eme parametre*/

	/* valeurs*/
	for (IndexV = 0; IndexV < nb_pole2; IndexV++)
	{
		for (IndexU = 0; IndexU < nb_pt1; IndexU++)
		{
			std::copy(pole_aux.GetData() + (IndexU * nb_pole2 + IndexV) * ndim, pole_aux.GetData() + ((IndexU * nb_pole2 + IndexV) + 1) * ndim, pole2.GetData() + (IndexV * nb_pt1 + IndexU) * ndim);
		}
	}

	/* derivee premiere des valeurs par rapport au premier parametre*/
	if (derivee >= 1)
	{
		for (IndexV = 0; IndexV < nb_pole2; IndexV++)
		{
			for (IndexU = 0; IndexU < nb_pt1; IndexU++)
			{
				//for(k=0; k<ndim; k++) d1pole2[(i*nb_pt1+j)*ndim+k] = d1pole_aux[(j*nb_pole2 +i)*ndim+k];
				std::copy(d1pole_aux.GetData() + (IndexU * nb_pole2 + IndexV) * ndim, d1pole_aux.GetData() + ((IndexU * nb_pole2 + IndexV) + 1) * ndim, d1pole2.GetData() + (IndexV * nb_pt1 + IndexU) * ndim);
			}
		}
	}

	/* derivee seconde des valeurs par rapport au premier parametre*/
	if (derivee >= 2)
	{
		for (IndexV = 0; IndexV < nb_pole2; IndexV++)
		{
			for (IndexU = 0; IndexU < nb_pt1; IndexU++)
			{
				//for(k=0; k<ndim; k++) d11pole2[(i*nb_pt1+j)*ndim+k] = d11pole_aux[(j*nb_pole2 +i)*ndim+k];
				std::copy(d11pole_aux.GetData() + (IndexU * nb_pole2 + IndexV) * ndim, d11pole_aux.GetData() + ((IndexU * nb_pole2 + IndexV) + 1) * ndim, d11pole2.GetData() + (IndexV * nb_pt1 + IndexU) * ndim);
			}
		}
	}

	/* interpoler et deriver le vecteur des valeurs par rapport au second parametre*/
	Interpolate1DBSpline(deg2, nb_pole2, knot2, nb_pt1 * ndim, &(pole2[0]), nb_pt2, &(tab_t2[0]), derivee, &(val_aux[0]), derivee > 0 ? &(d2val_aux[0]) : nullptr, derivee > 1 ? &(d22val_aux[0]) : nullptr);

	/* sauver les valeurs en sortie*/
	if (inversion_uv)
	{
		for (IndexV = 0; IndexV < nb_pt2; IndexV++)
		{
			for (IndexU = 0; IndexU < nb_pt1; IndexU++)
			{
				std::copy(val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, valeur + (IndexU * nb_pt2 + IndexV) * ndim);
			}
		}
	}
	else
	{
		for (IndexV = 0; IndexV < nb_pt2; IndexV++)
		{
			for (IndexU = 0; IndexU < nb_pt1; IndexU++)
			{
				std::copy(val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, valeur + (IndexV * nb_pt1 + IndexU) * ndim);
			}
		}
	}

	/* calculer les derivees premieres*/
	if (derivee >= 1)
	{
		/* interpoler et deriver une seule fois (eventuellement) le vecteur a (nb_pt1*ndim) composantes des derivees premieres par rapport au second parametre*/
		Interpolate1DBSpline(deg2, nb_pole2, knot2, nb_pt1 * ndim, &(d1pole2[0]), nb_pt2, &(tab_t2[0]), derivee - 1, &(d1val_aux[0]), derivee > 1 ? &(d21val_aux[0]) : nullptr, nullptr);

		/* sauver les derivees premieres en sortie*/
		if (IsoType == EIso::IsoV)
		{
			for (IndexV = 0; IndexV < nb_pt2; IndexV++)
			{
				for (IndexU = 0; IndexU < nb_pt1; IndexU++)
				{
					std::copy(d2val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d2val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, deru + (IndexU * nb_pt2 + IndexV) * ndim);
					std::copy(d1val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d1val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, derv + (IndexU * nb_pt2 + IndexV) * ndim);
				}
			}
		}
		else // (IsoType == EIso::IsoU)
		{
			for (IndexV = 0; IndexV < nb_pt2; IndexV++)
			{
				for (IndexU = 0; IndexU < nb_pt1; IndexU++)
				{
					std::copy(d1val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d1val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, deru + (IndexV * nb_pt1 + IndexU) * ndim);
					std::copy(d2val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d2val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, derv + (IndexV * nb_pt1 + IndexU) * ndim);
				}
			}
		}
	}

	/* calculer les derivees secondes*/
	if (derivee >= 2)
	{
		/* interpoler le vecteur des derivees secondes a (nb_pt1*ndim) composantes par rapport au second parametre*/
		Interpolate1DBSpline(deg2, nb_pole2, knot2, nb_pt1 * ndim, &(d11pole2[0]), nb_pt2, &(tab_t2[0]), (int32)0, &(d11val_aux[0]), nullptr, nullptr);

		/* sauver les derivees secondes en sortie*/
		if (IsoType == EIso::IsoV)
		{
			for (IndexV = 0; IndexV < nb_pt2; IndexV++)
			{
				for (IndexU = 0; IndexU < nb_pt1; IndexU++)
				{
					std::copy(d22val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d22val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, deruu + (IndexU * nb_pt2 + IndexV) * ndim);
					std::copy(d11val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d11val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, dervv + (IndexU * nb_pt2 + IndexV) * ndim);
					std::copy(d21val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d21val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, deruv + (IndexU * nb_pt2 + IndexV) * ndim);
				}
			}
		}
		else // (IsoType == EIso::IsoU)
		{
			for (IndexV = 0; IndexV < nb_pt2; IndexV++)
			{
				for (IndexU = 0; IndexU < nb_pt1; IndexU++)
				{
					std::copy(d11val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d11val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, deruu + (IndexV * nb_pt1 + IndexU) * ndim);
					std::copy(d22val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d22val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, dervv + (IndexV * nb_pt1 + IndexU) * ndim);
					std::copy(d21val_aux.GetData() + (IndexV * nb_pt1 + IndexU) * ndim, d21val_aux.GetData() + ((IndexV * nb_pt1 + IndexU) + 1) * ndim, deruv + (IndexV * nb_pt1 + IndexU) * ndim);
				}
			}
		}
	}
}

void EvaluatePoint(const FNURBSCurve& Nurbs, double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder)
{
	ensureCADKernel(Nurbs.GetDimension() == 3);

	OutPoint.DerivativeOrder = DerivativeOrder;

	double ValueH[4];
	double GradientH[4];
	double LaplacianH[4];

	int32 PoleDimension = Nurbs.GetDimension() + (Nurbs.IsRational() ? 1 : 0);

	BSpline::Interpolate1DBSpline(Nurbs.GetDegree(), Nurbs.GetPoleCount(), Nurbs.GetNodalVector().GetData(), PoleDimension, Nurbs.GetHPoles().GetData(), 1, &Coordinate, DerivativeOrder, ValueH, GradientH, LaplacianH);

	if (Nurbs.IsRational())
	{
		OutPoint.Point.Set(ValueH[0] / ValueH[3], ValueH[1] / ValueH[3], ValueH[2] / ValueH[3]);
	}
	else
	{
		OutPoint.Point.Set(ValueH[0], ValueH[1], ValueH[2]);
	}

	if (DerivativeOrder > 0)
	{
		if (Nurbs.IsRational())
		{
			OutPoint.Gradient.X = (GradientH[0] - GradientH[3] * ValueH[0] / ValueH[3]) / ValueH[3];
			OutPoint.Gradient.Y = (GradientH[1] - GradientH[3] * ValueH[1] / ValueH[3]) / ValueH[3];
			OutPoint.Gradient.Z = (GradientH[2] - GradientH[3] * ValueH[2] / ValueH[3]) / ValueH[3];
		}
		else
		{
			OutPoint.Gradient.Set(GradientH[0], GradientH[1], GradientH[2]);
		}

		if (DerivativeOrder > 1)
		{
			if (Nurbs.IsRational())
			{
				OutPoint.Laplacian.X = (LaplacianH[0] * ValueH[3] - LaplacianH[3] * ValueH[0] - 2.0 * GradientH[3] * (GradientH[0] - GradientH[3] * ValueH[0] / ValueH[3])) / (ValueH[3] * ValueH[3]);
				OutPoint.Laplacian.Y = (LaplacianH[1] * ValueH[3] - LaplacianH[3] * ValueH[1] - 2.0 * GradientH[3] * (GradientH[1] - GradientH[3] * ValueH[1] / ValueH[3])) / (ValueH[3] * ValueH[3]);
				OutPoint.Laplacian.Z = (LaplacianH[2] * ValueH[3] - LaplacianH[3] * ValueH[2] - 2.0 * GradientH[3] * (GradientH[2] - GradientH[3] * ValueH[2] / ValueH[3])) / (ValueH[3] * ValueH[3]);
			}
			else
			{
				OutPoint.Laplacian.Set(LaplacianH[0], LaplacianH[1], LaplacianH[2]);
			}
		}
	}
}

void Evaluate2DPoint(const FNURBSCurve& Nurbs, double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder)
{
	ensureCADKernel(Nurbs.GetDimension() == 2);

	OutPoint.DerivativeOrder = DerivativeOrder;

	double ValueH[3];
	double GradientH[3];
	double LaplacianH[3];

	int32 PoleDimension = Nurbs.GetDimension() + (Nurbs.IsRational() ? 1 : 0);
	BSpline::Interpolate1DBSpline(Nurbs.GetDegree(), Nurbs.GetPoleCount(), Nurbs.GetNodalVector().GetData(), PoleDimension, Nurbs.GetHPoles().GetData(), 1, &Coordinate, DerivativeOrder, ValueH, GradientH, LaplacianH);

	const double InvWeight = 1. / ValueH[2];

	if (Nurbs.IsRational())
	{
		OutPoint.Point.Set(ValueH[0] * InvWeight, ValueH[1] * InvWeight);
	}
	else
	{
		OutPoint.Point.Set(ValueH[0], ValueH[1]);
	}

	if (DerivativeOrder > 0)
	{
		if (Nurbs.IsRational())
		{
			OutPoint.Gradient.X = (GradientH[0] - GradientH[2] * ValueH[0] * InvWeight) * InvWeight;
			OutPoint.Gradient.Y = (GradientH[1] - GradientH[2] * ValueH[1] * InvWeight) * InvWeight;
		}
		else
		{
			OutPoint.Gradient.Set(GradientH[0], GradientH[1]);
		}

		if (DerivativeOrder > 1)
		{
			if (Nurbs.IsRational())
			{
				const double InvWeightSquared = InvWeight * InvWeight;

				OutPoint.Laplacian.X = (LaplacianH[0] * ValueH[2] - LaplacianH[2] * ValueH[0] - 2.0 * GradientH[2] * (GradientH[0] - GradientH[2] * ValueH[0] * InvWeight)) * InvWeightSquared;
				OutPoint.Laplacian.Y = (LaplacianH[1] * ValueH[2] - LaplacianH[2] * ValueH[1] - 2.0 * GradientH[2] * (GradientH[1] - GradientH[2] * ValueH[1] * InvWeight)) * InvWeightSquared;
			}
			else
			{
				OutPoint.Laplacian.Set(LaplacianH[0], LaplacianH[1]);
			}
		}
	}
}

TSharedPtr<FNURBSCurve> DuplicateNurbsCurveWithHigherDegree(int32 Degre, const FNURBSCurve& InCurve)
{
	// To avoid crashes while waiting for the fix (jira UETOOL-5046)
	return TSharedPtr<FNURBSCurve>();

#ifdef UETOOL_5046_WIP
	TArray<FVector> NewPoles;
	TArray<double> NewNodalVector;
	TArray<double> NewWeights;

	DuplicateNurbsCurveWithHigherDegree(Degre, InCurve.GetPoles(), InCurve.GetNodalVector(), InCurve.GetWeights(), NewPoles, NewNodalVector, NewWeights);
	if (NewPoles.IsEmpty())
	{
		return TSharedPtr<FNURBSCurve>();
	}
	return FEntity::MakeShared<FNURBSCurve>(Degre, NewNodalVector, NewPoles, NewWeights, InCurve.GetDimension());
#endif
}


void EvaluatePoint(const FNURBSSurface& Nurbs, const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder)
{
	double Point4[4];
	double GradientU[4];
	double GradientV[4];
	double LaplacianU[4];
	double LaplacianV[4];
	double LaplacianUV[4];

	BSpline::Interpolate2DBSpline(Nurbs.GetDegree(EIso::IsoU), Nurbs.GetDegree(EIso::IsoV), Nurbs.GetPoleCount(EIso::IsoU), Nurbs.GetPoleCount(EIso::IsoV),
		Nurbs.GetNodalVector(EIso::IsoU).GetData(), Nurbs.GetNodalVector(EIso::IsoV).GetData(), Nurbs.IsRational() ? 4 : 3, Nurbs.GetHPoles().GetData(),
		1, &InPoint2D.X, 1, &InPoint2D.Y, InDerivativeOrder, Point4, GradientU, GradientV, LaplacianU, LaplacianV, LaplacianUV);

	OutPoint3D.DerivativeOrder = InDerivativeOrder;

	if (Nurbs.IsRational())
	{
		OutPoint3D.Point.Set(Point4[0] / Point4[3], Point4[1] / Point4[3], Point4[2] / Point4[3]);

		double PointHSquare = FMath::Square(Point4[3]);

		if (InDerivativeOrder > 0)
		{
			OutPoint3D.GradientU.X = (GradientU[0] * Point4[3] - Point4[0] * GradientU[3]) / PointHSquare;
			OutPoint3D.GradientU.Y = (GradientU[1] * Point4[3] - Point4[1] * GradientU[3]) / PointHSquare;
			OutPoint3D.GradientU.Z = (GradientU[2] * Point4[3] - Point4[2] * GradientU[3]) / PointHSquare;

			OutPoint3D.GradientV.X = (GradientV[0] * Point4[3] - Point4[0] * GradientV[3]) / PointHSquare;
			OutPoint3D.GradientV.Y = (GradientV[1] * Point4[3] - Point4[1] * GradientV[3]) / PointHSquare;
			OutPoint3D.GradientV.Z = (GradientV[2] * Point4[3] - Point4[2] * GradientV[3]) / PointHSquare;
		}

		if (InDerivativeOrder > 1)
		{
			double PointHCubic = Point4[3] * Point4[3] * Point4[3];
			double GradientUHSquare = FMath::Square(GradientU[3]);
			double GradientVHSquare = FMath::Square(GradientV[3]);
			double GradientUHVH = GradientU[3] * GradientV[3];

			OutPoint3D.LaplacianU.X = (LaplacianU[0] * Point4[3] - Point4[0] * LaplacianU[3] - 2.0 * GradientU[0] * GradientU[3]) / PointHSquare + 2.0 * Point4[0] * GradientUHSquare / PointHCubic;
			OutPoint3D.LaplacianU.Y = (LaplacianU[1] * Point4[3] - Point4[1] * LaplacianU[3] - 2.0 * GradientU[1] * GradientU[3]) / PointHSquare + 2.0 * Point4[1] * GradientUHSquare / PointHCubic;
			OutPoint3D.LaplacianU.Z = (LaplacianU[2] * Point4[3] - Point4[2] * LaplacianU[3] - 2.0 * GradientU[2] * GradientU[3]) / PointHSquare + 2.0 * Point4[2] * GradientUHSquare / PointHCubic;

			OutPoint3D.LaplacianV.X = (LaplacianV[0] * Point4[3] - Point4[0] * LaplacianV[3] - 2.0 * GradientV[0] * GradientV[3]) / PointHSquare + 2.0 * Point4[0] * GradientVHSquare / PointHCubic;
			OutPoint3D.LaplacianV.Y = (LaplacianV[1] * Point4[3] - Point4[1] * LaplacianV[3] - 2.0 * GradientV[1] * GradientV[3]) / PointHSquare + 2.0 * Point4[1] * GradientVHSquare / PointHCubic;
			OutPoint3D.LaplacianV.Z = (LaplacianV[2] * Point4[3] - Point4[2] * LaplacianV[3] - 2.0 * GradientV[2] * GradientV[3]) / PointHSquare + 2.0 * Point4[2] * GradientVHSquare / PointHCubic;

			OutPoint3D.LaplacianUV.X = (LaplacianUV[0] * Point4[3] - Point4[0] * LaplacianUV[3] - GradientU[0] * GradientV[3] - GradientV[0] * GradientU[3]) / PointHSquare + 2.0 * Point4[0] * GradientUHVH / PointHCubic;
			OutPoint3D.LaplacianUV.Y = (LaplacianUV[1] * Point4[3] - Point4[1] * LaplacianUV[3] - GradientU[1] * GradientV[3] - GradientV[1] * GradientU[3]) / PointHSquare + 2.0 * Point4[1] * GradientUHVH / PointHCubic;
			OutPoint3D.LaplacianUV.Z = (LaplacianUV[2] * Point4[3] - Point4[2] * LaplacianUV[3] - GradientU[2] * GradientV[3] - GradientV[2] * GradientU[3]) / PointHSquare + 2.0 * Point4[2] * GradientUHVH / PointHCubic;
		}
	}
	else
	{
		OutPoint3D.Point.Set(Point4[0], Point4[1], Point4[2]);

		if (InDerivativeOrder > 0)
		{
			OutPoint3D.GradientU.Set(GradientU[0], GradientU[1], GradientU[2]);
			OutPoint3D.GradientV.Set(GradientV[0], GradientV[1], GradientV[2]);

			if (InDerivativeOrder > 1)
			{
				OutPoint3D.LaplacianU.Set(LaplacianU[0], LaplacianU[1], LaplacianU[2]);
				OutPoint3D.LaplacianV.Set(LaplacianV[0], LaplacianV[1], LaplacianV[2]);
				OutPoint3D.LaplacianUV.Set(LaplacianUV[0], LaplacianUV[1], LaplacianUV[2]);
			}
		}
	}
}

void EvaluatePointGrid(const FNURBSSurface& Nurbs, const FCoordinateGrid& Coords, FSurfacicSampling& OutPoints, bool bComputeNormals)
{
	const int32 ULineCount = (int32)Coords.IsoCount(EIso::IsoU);
	const int32 VLineCount = (int32)Coords.IsoCount(EIso::IsoV);
	const int32 PointCount = ULineCount * VLineCount;

	OutPoints.bWithNormals = bComputeNormals;

	OutPoints.Set2DCoordinates(Coords);


	TArray<double> HPoints;
	int32 PointDim = Nurbs.IsRational() ? 4 : 3;
	HPoints.SetNum(PointCount * PointDim);

	TArray<double> UGradients;
	TArray<double> VGradients;

	int32 DerivativeOrder = 0;
	if (bComputeNormals)
	{
		OutPoints.bWithNormals = true;
		UGradients.SetNum(PointCount * PointDim);
		VGradients.SetNum(PointCount * PointDim);
		DerivativeOrder = 1;
	}

	BSpline::Interpolate2DBSpline(Nurbs.GetDegree(EIso::IsoU), Nurbs.GetDegree(EIso::IsoV), Nurbs.GetPoleCount(EIso::IsoU), Nurbs.GetPoleCount(EIso::IsoV),
		Nurbs.GetNodalVector(EIso::IsoU).GetData(), Nurbs.GetNodalVector(EIso::IsoV).GetData(), Nurbs.IsRational() ? 4 : 3, Nurbs.GetHPoles().GetData(),
		ULineCount, Coords[EIso::IsoU].GetData(), VLineCount, Coords[EIso::IsoV].GetData(),
		DerivativeOrder, HPoints.GetData(), UGradients.GetData(), VGradients.GetData(), nullptr, nullptr, nullptr);

	OutPoints.Points3D.SetNum(PointCount);
	if (Nurbs.IsRational())
	{
		for (int32 Index = 0, Hndex = 0; Index < PointCount; Index++, Hndex += 4)
		{
			double* HPoint = HPoints.GetData() + Hndex;
			OutPoints.Points3D[Index].Set(HPoint[0] / HPoint[3], HPoint[1] / HPoint[3], HPoint[2] / HPoint[3]);
		}
	}
	else
	{
		memcpy(OutPoints.Points3D.GetData(), HPoints.GetData(), PointCount * sizeof(FVector));
	}

	if (bComputeNormals)
	{
		OutPoints.Normals.SetNum(PointCount);

		if (Nurbs.IsRational())
		{
			FVector DeltaU;
			FVector DeltaV;
			for (int32 Index = 0; Index < PointCount; Index++)
			{
				double* HPoint = HPoints.GetData() + Index * 4;
				double  SquarePointH = FMath::Square(HPoint[3]);
				double* UGradient = UGradients.GetData() + Index * 4;
				double* VGradient = VGradients.GetData() + Index * 4;

				DeltaU.X = (UGradient[0] * HPoint[3] - HPoint[0] * UGradient[3]) / SquarePointH;
				DeltaU.Y = (UGradient[1] * HPoint[3] - HPoint[1] * UGradient[3]) / SquarePointH;
				DeltaU.Z = (UGradient[2] * HPoint[3] - HPoint[2] * UGradient[3]) / SquarePointH;

				DeltaV.X = (VGradient[0] * HPoint[3] - HPoint[0] * VGradient[3]) / SquarePointH;
				DeltaV.Y = (VGradient[1] * HPoint[3] - HPoint[1] * VGradient[3]) / SquarePointH;
				DeltaV.Z = (VGradient[2] * HPoint[3] - HPoint[2] * VGradient[3]) / SquarePointH;

				OutPoints.Normals[Index] = (FVector3f)(DeltaU ^ DeltaV);
			}
		}
		else
		{
			double* UGradientsPtr = UGradients.GetData();
			double* VGradientsPtr = VGradients.GetData();
			for (int32 Index = 0; Index < PointCount; Index++, UGradientsPtr += 3, VGradientsPtr += 3)
			{
				const FVector UGradient(UGradientsPtr[0], UGradientsPtr[1], UGradientsPtr[2]);
				const FVector VGradient(VGradientsPtr[0], VGradientsPtr[1], VGradientsPtr[2]);

				OutPoints.Normals[Index] = (FVector3f)(UGradient ^ VGradient);
			}
		}

		OutPoints.NormalizeNormals();
	}
}

/******************************description fonction*****************************/
/* Auteur:						 - Fichier: bspline.c										*/
/* Role:- Calculer selon la methode de "DeBoor" l'interpolation par fonctions*/
/*		 BSplines d'ordre "degre" sur l'intervalle numero "segment" d'une	*/
/*		 liste de poles de dimensionnalite ndim stokes dans pole_aux[0], par*/
/*		 rapport au vecteur nodal knot.													*/
/*		 On recupere en sortie les valeurs interpolees dans valeur[composante]*/
/*		 et selon l'ordre de derivation desire (0, 1 ou 2) leurs derivees	*/
/*		 premieres et secondes dans Gradient[composante] et Laplacian[composante].		*/
/*		 Toutes les tables de travail (pole_aux, grad_aux, lap_aux) et les	*/
/*		 tables de resultats doivent etre allouees par l'appelant.				*/
/*******************************fin fonction************************************/
void DeBoorValeursBSpline(
	const int32 Degre,					/* E-Ordre de la bspline*/
	const double* const NodalVector,	/* E-Vecteur nodale de la bspline*/
	const int32 SegmentIndex,			/* E-Segment sur lequel faire le calcul*/
	const int32 PointDimension,		/* E-Dimension des poles (1D,2D,3D,4D=3D+poids...)*/
	const int32 DeriveOrder,			/* E-Ordre de derivation desire*/
	const double Coordinate,			/* E-*/
	double* const InterationPoles,		/* S-Les valeurs tempo*/
	double* const InterationGardient,	/* S-Les derivee premieres tempo*/
	double* const lap_aux,				/* S-Les derivee secondes tempo*/
	double* const OutPoint,				/* S-Les valeurs*/
	double* const OutGradient,			/* S-Les derivee premieres*/
	double* const OutLaplacien			/* S-Les derivee secondes*/
)
{
	double DeltaU, OneOnDeltaU, tu, tu1;
	int32 Index, IndexJ, i_aux, IndexK, deg1;

	deg1 = Degre + 1;
	/*Calculer par recurrence les valeurs*/

	for (IndexK = 1; IndexK <= Degre; IndexK++)
	{
		for (Index = SegmentIndex - Degre + IndexK + 1, i_aux = IndexK; Index <= SegmentIndex + 1; Index++, i_aux++)
		{
			DeltaU = NodalVector[Index + Degre - IndexK] - NodalVector[Index - 1];
			if (DeltaU <= 1.e-13)
			{
				tu = 0.;
				OneOnDeltaU = 0.;
			}
			else
			{
				tu = (Coordinate - NodalVector[Index - 1]) / DeltaU;
				OneOnDeltaU = 1. / DeltaU;
			}

			tu1 = 1. - tu;

			///*Calcul du point courant*/
			int32 A = (i_aux + IndexK * deg1) * PointDimension;
			int32 B = A - deg1 * PointDimension;
			for (IndexJ = 0; IndexJ < PointDimension; IndexJ++, A++, B++)
			{
				InterationPoles[A] = InterationPoles[B - PointDimension] * tu1 + InterationPoles[B] * tu;
			}


			/*Calcul de la derivee premiere*/
			if (DeriveOrder >= 1)
			{
				A = (i_aux + IndexK * deg1) * PointDimension;
				B = A - deg1 * PointDimension;
				for (IndexJ = 0; IndexJ < PointDimension; IndexJ++, A++, B++)
				{
					InterationGardient[A] = (InterationPoles[B] - InterationPoles[B - PointDimension]) * OneOnDeltaU + InterationGardient[B - PointDimension] * tu1 + InterationGardient[B] * tu;
					//InterationGardient[(i_aux + IndexK * deg1)*PointDimension + IndexJ] = InterationGardient[(i_aux - 1 + (IndexK - 1)*deg1)*PointDimension + IndexJ] * tu1 + InterationGardient[(i_aux + (IndexK - 1)*deg1)*PointDimension + IndexJ] * tu + (InterationPoles[(i_aux + (IndexK - 1)*deg1)*PointDimension + IndexJ] - InterationPoles[(i_aux - 1 + (IndexK - 1)*deg1)*PointDimension + IndexJ])* OneOnDeltaU;
				}
			}

			/*Calcul de la derivee seconde*/
			if (DeriveOrder >= 2)
			{
				for (IndexJ = 0; IndexJ < PointDimension; IndexJ++)
				{
					lap_aux[(i_aux + IndexK * deg1) * PointDimension + IndexJ] = lap_aux[(i_aux - 1 + (IndexK - 1) * deg1) * PointDimension + IndexJ] * tu1 + lap_aux[(i_aux + (IndexK - 1) * deg1) * PointDimension + IndexJ] * tu + 2. * (InterationGardient[(i_aux + (IndexK - 1) * deg1) * PointDimension + IndexJ] - InterationGardient[(i_aux - 1 + (IndexK - 1) * deg1) * PointDimension + IndexJ]) * OneOnDeltaU;
				}
			}
		}
	}

	/*stocker le point courant*/
	std::copy(InterationPoles + (deg1 * deg1 - 1) * PointDimension, InterationPoles + ((deg1 * deg1 - 1) + 1) * PointDimension, OutPoint);

	/*stocker la derivee premiere*/
	if (DeriveOrder > 0)
	{
		std::copy(InterationGardient + (deg1 * deg1 - 1) * PointDimension, InterationGardient + ((deg1 * deg1 - 1) + 1) * PointDimension, OutGradient);
	}

	/*stocker la derivee seconde*/
	if (DeriveOrder > 1)
	{
		std::copy(lap_aux + (deg1 * deg1 - 1) * PointDimension, lap_aux + ((deg1 * deg1 - 1) + 1) * PointDimension, OutLaplacien);
	}
}

/******************************description fonction*****************************/
/* Auteur:						 - Fichier: bspline.c										*/
/* Role:- Calcul des valeurs des fonctions de base BSpline ou de leurs derivees*/
/*		 jusqu'a l'ordre derivee.															*/
/*		 Les fonctions sont definies par:												*/
/*		 - degre: degre des fonctions													*/
/*		 - nb_pole: nombre total de fonctions											*/
/*		 - knot[]: table representant le vecteur nodal								*/
/*		 Le calcul est a effectuer sur nb_pt points de coordonnees curvilignes*/
/*		 definis dans tab_u[] .															*/
/*		 On recupere en sortie les valeurs interpolees dans valeur[point]	*/
/*		 et selon l'ordre de derivation desire (0, 1 ou 2) leurs derivees	*/
/*		 premieres et secondes dans Gradient[point] et Laplacian[point].					*/
/*******************************fin fonction************************************/
void calculerBaseBSpline(
	const int32 degre,				 /* E-degre des fonctions*/
	const int32 derivee,				/* E-Ordre de derivation souhaite*/
	const int32 nb_pole,				/* E-nombre total de fonctions*/
	const double* const knot,	 /* E-table representant le vecteur nodal*/
	const int32 nb_pt,				 /* E-Nombre de parametre a evaluer*/
	const double* const tab_u,	/* E-La table des parametres*/
	double* const valeur,		 /* S-Les points calcules*/
	double* const Gradient,			 /* S-Les derivees premeieres*/
	double* const Laplacian				/* S-Les derivees secondes*/
)
{
	TArray<double> tab_p, val, gra, la;
	int32 i, ndim;

	ndim = 1;

	/* dimensionner les tables necessaires*/

	tab_p.SetNum(nb_pole);
	val.SetNum(nb_pt);

	if (derivee > 0)
	{
		gra.SetNum(nb_pt);
		if (derivee > 1)
		{
			la.SetNum(nb_pt);
		}
	}

	for (i = 0; i < nb_pole; i++)
	{
		//for(j=0;j<nb_pole;j++) tab_p[j] = 0.0;
		tab_p.Init(0.0, nb_pole);
		tab_p[i] = 1.0;

		Interpolate1DBSpline(degre, nb_pole, knot, ndim, &tab_p[0], nb_pt, tab_u, derivee, &val[0], derivee > 0 ? &gra[0] : nullptr, derivee > 1 ? &la[0] : nullptr);

		//for(j=0;j<nb_pt;j++) valeur[i*nb_pt+j] = val[j];
		std::copy(valeur + i * nb_pt, valeur + i * nb_pt + nb_pt, val.GetData());

		if (derivee > 0)
		{
			//for(j=0;j<nb_pt;j++) Gradient[i*nb_pt+j] = gra[j];
			std::copy(Gradient + i * nb_pt, Gradient + i * nb_pt + nb_pt, gra.GetData());
			if (derivee > 1)
			{
				//for(j=0;j<nb_pt;j++) Laplacian[i*nb_pt+j] = la[j];
				std::copy(Laplacian + i * nb_pt, Laplacian + i * nb_pt + nb_pt, la.GetData());
			}
		}
	}
}

double calculateN_HR(double u, const double* knots, int32 n, int32 l)
{
	if (n == 0)
	{
		if (FMath::IsNearlyEqual(knots[l - 1], knots[l]) && FMath::IsNearlyEqual(u, knots[l])) return 1;
		else if (knots[l - 1] <= u && u < knots[l]) return 1;
		return 0;
	}

	double n1 = calculateN_HR(u, knots, n - 1, l);
	double n2 = calculateN_HR(u, knots, n - 1, l + 1);

	double c1 = 0, c2 = 0;
	if (!FMath::IsNearlyZero(n1)) c1 = (u - knots[l - 1]) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(n2)) c2 = (knots[l + n] - u) / (knots[l + n] - knots[l]);

	return c1 * n1 + c2 * n2;
}

double calculateNd1_HR(double u, const double* knots, int32 n, int32 l)
{
	if (n == 0) return 0;

	double n1 = calculateN_HR(u, knots, n - 1, l);
	double n2 = calculateN_HR(u, knots, n - 1, l + 1);
	double n1d1 = calculateNd1_HR(u, knots, n - 1, l);
	double n2d1 = calculateNd1_HR(u, knots, n - 1, l + 1);

	double c1 = 0, c2 = 0, c1d1 = 0, c2d1 = 0;
	if (!FMath::IsNearlyZero(n1)) c1d1 = (1. - 0.) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(n1d1)) c1 = (u - knots[l - 1]) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(n2)) c2d1 = (0. - 1.) / (knots[l + n] - knots[l]);
	if (!FMath::IsNearlyZero(n2d1)) c2 = (knots[l + n] - u) / (knots[l + n] - knots[l]);

	return c1d1 * n1 + c1 * n1d1 +
		c2d1 * n2 + c2 * n2d1;
}

double calculateNd2_HR(double u, const double* knots, int32 n, int32 l)
{
	if (n == 0)
	{
		return 0;
	}

	double n1 = calculateN_HR(u, knots, n - 1, l);
	double n2 = calculateN_HR(u, knots, n - 1, l + 1);
	double n1d1 = calculateNd1_HR(u, knots, n - 1, l);
	double n2d1 = calculateNd1_HR(u, knots, n - 1, l + 1);
	double n1d2 = calculateNd2_HR(u, knots, n - 1, l);
	double n2d2 = calculateNd2_HR(u, knots, n - 1, l + 1);

	double c1 = 0, c2 = 0, c1d1 = 0, c2d1 = 0, c1d2 = 0, c2d2 = 0;
	if (!FMath::IsNearlyZero(knots[l + n - 1] - knots[l - 1])) c1d1 = (1. - 0.) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(knots[l + n - 1] - knots[l - 1])) c1 = (u - knots[l - 1]) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(knots[l + n] - knots[l])) c2d1 = (0. - 1.) / (knots[l + n] - knots[l]);
	if (!FMath::IsNearlyZero(knots[l + n] - knots[l])) c2 = (knots[l + n] - u) / (knots[l + n] - knots[l]);

	return c1d2 * n1 + c1d1 * n1d1 +
		c1d1 * n1d1 + c1 * n1d2 +
		c2d2 * n2 + c2d1 * n2d1 +
		c2d1 * n2d1 + c2 * n2d2;
}

double calculateNd3_HR(double u, const double* knots, int32 n, int32 l)
{
	if (n == 0) return 0;

	double n1 = calculateN_HR(u, knots, n - 1, l);
	double n2 = calculateN_HR(u, knots, n - 1, l + 1);
	double n1d1 = calculateNd1_HR(u, knots, n - 1, l);
	double n2d1 = calculateNd1_HR(u, knots, n - 1, l + 1);
	double n1d2 = calculateNd2_HR(u, knots, n - 1, l);
	double n2d2 = calculateNd2_HR(u, knots, n - 1, l + 1);
	double n1d3 = calculateNd3_HR(u, knots, n - 1, l);
	double n2d3 = calculateNd3_HR(u, knots, n - 1, l + 1);

	double c1 = 0, c2 = 0, c1d1 = 0, c2d1 = 0, c1d2 = 0, c2d2 = 0, c1d3 = 0, c2d3 = 0;
	if (!FMath::IsNearlyZero(knots[l + n - 1] - knots[l - 1])) c1d1 = (1. - 0.) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(knots[l + n - 1] - knots[l - 1])) c1 = (u - knots[l - 1]) / (knots[l + n - 1] - knots[l - 1]);
	if (!FMath::IsNearlyZero(knots[l + n] - knots[l])) c2d1 = (0. - 1.) / (knots[l + n] - knots[l]);
	if (!FMath::IsNearlyZero(knots[l + n] - knots[l])) c2 = (knots[l + n] - u) / (knots[l + n] - knots[l]);

	return c1d3 * n1 + c1d2 * n1d1 +
		c1d2 * n1d1 + c1d1 * n1d2 +
		c1d2 * n1d1 + c1d1 * n1d2 +
		c1d1 * n1d2 + c1 * n1d3 +
		c2d3 * n2 + c2d2 * n2d1 +
		c2d2 * n2d1 + c2d1 * n2d2 +
		c2d2 * n2d1 + c2d1 * n2d2 +
		c2d1 * n2d2 + c2 * n2d3;
}

void calculerBaseBSpline_HR(
	const int32 degre,				 /* E-degre des fonctions*/
	const int32 derivee,				/* E-Ordre de derivation souhaite*/
	const int32 nb_pole,				/* E-nombre total de fonctions*/
	const double* const knot,	 /* E-table representant le vecteur nodal*/
	const int32 nb_pt,				 /* E-Nombre de parametre a evaluer*/
	const double* const tab_u,	/* E-La table des parametres*/
	TArray<double>& valeur,		 /* S-Les points calcules*/
	TArray<double>& Gradient,			 /* S-Les derivees premeieres*/
	TArray<double>& Laplacian,			 /* S-Les derivees secondes*/
	TArray<double>& der3			 /* S-Les derivees secondes*/
)
{
	int32 cnt = 0;

	for (int32 p = 0; p < nb_pole; p++)
	{
		for (int32 v = 0; v < nb_pt; v++)
		{
			//Calculate the value basis for each pole
			valeur[cnt] = calculateN_HR(tab_u[v], knot + 1, degre, p);

			if (derivee >= 1)
			{
				Gradient[cnt] = calculateNd1_HR(tab_u[v], knot + 1, degre, p);
			}
			if (derivee >= 2)
			{
				Laplacian[cnt] = calculateNd2_HR(tab_u[v], knot + 1, degre, p);
			}
			if (derivee >= 3)
			{
				der3[cnt] = calculateNd3_HR(tab_u[v], knot + 1, degre, p);
			}
			cnt++;
		}
	}
}

void matricePassageMonomeBernstein(int32 ordre, TArray<double>& matrice)
{
	int32 i, j, n, signe;
	TArray<double> C;

	n = ordre - 1;

	// allouer la table permettant le calcul des C(i,j)
	C.SetNum(ordre * ordre);

	// initialiser les C(i,j) et matrice(i,j)
	for (i = 0; i < ordre * ordre; i++)
	{
		C[i] = 0.0;
		matrice[i] = 0.0;
	}

	// calculer les C(i,j)
	for (i = 0; i < ordre; i++)
	{
		C[i * ordre + 0] = 1.0;
		C[i * ordre + i] = 1.0;

		for (j = 1; j < i; j++)
		{
			C[i * ordre + j] = C[i * ordre + j - 1] * (i - j + 1) / j;
		}
	}

	// Calculer les coefficients de la matrice
	for (i = 0; i < ordre; i++)
	{
		signe = 1;
		for (j = i; j < ordre; j++)
		{
			matrice[i * ordre + j] = signe * C[n * ordre + j] * C[j * ordre + i];
			signe *= -1;
		}
	}
}

void coeffCourbeToPolesBezier(int32 order, int32 nbDim, TArray<double>& tabCoeff, TArray<double>& tabPoles)
{
	TArray<double> C;

	C.SetNum(order * order);

	// calcul de la matrice de passage
	matricePassageMonomeBernstein(order, C);

	// inversion de cette matrice
	InverseMatrixN(C.GetData(), order);

	// effectuer le produit tab_pole[] = tab_coeff[] * C1[]
	MatrixProduct(nbDim, order, order, tabCoeff.GetData(), C.GetData(), tabPoles.GetData());
}

void coeffSurfaceToPolesBezier(int32 orderU, int32 orderV, int32 nbDim, const TArray<double>& tabCoeff, TArray<double>& tabPoles)
{
	TArray<double> Cu, Cv, Cvt, Mtp;
	int32 nbVal = orderU * orderV;
	TArray<double> vecCoeff;
	TArray<double> vecPoles;

	Cu.SetNum(orderU * orderU);
	Cv.SetNum(orderV * orderV);
	Cvt.SetNum(orderV * orderV);
	Mtp.SetNum(orderU * orderV);

	vecPoles.SetNum(nbVal);

	// calcul de la matrice de passage
	matricePassageMonomeBernstein(orderU, Cu);

	// calcul de la matrice de passage
	matricePassageMonomeBernstein(orderV, Cv);

	// transposer la matrice Cv[]
	TransposeMatrix(orderV, orderV, Cv.GetData(), Cvt.GetData());

	// inversion de ces matrices
	InverseMatrixN(Cu.GetData(), orderU);
	InverseMatrixN(Cvt.GetData(), orderV);

	// pour chaque coordonnees effectuer le produit :
	// tab_pole[] = Cvt[]* tab_coeff[] * Cu[]
	for (int32 i = 0; i < nbDim; i++)
	{
		vecCoeff.Empty();
		vecCoeff.Reserve(nbVal);
		for (int32 k = 0; k < nbVal; k++)
		{
			vecCoeff.Add(tabCoeff[i * nbVal + k]);
		}

		MatrixProduct(orderV, orderV, orderU, Cvt.GetData(), vecCoeff.GetData(), Mtp.GetData());
		MatrixProduct(orderV, orderU, orderU, Mtp.GetData(), Cu.GetData(), vecPoles.GetData());

		for (int32 k = 0; k < nbVal; k++)
		{
			tabPoles[i * nbVal + k] = vecPoles[k];
		}
	}
}

void FindNotDerivableParameters(int32 Degree, int32 PoleCount, const TArray<double>& NodalVector, int32 DerivativeOrder, const FLinearBoundary& Boundary, TArray<double>& OutNotDerivableParameters)
{
	double udeb;
	int32 i, multiplicite;

	udeb = NodalVector[0];
	i = 1;
	multiplicite = 1;

	while (i < Degree + PoleCount + 1)
	{
		// noter la valeur courante du noeud
		udeb = NodalVector[i];
		multiplicite = 1;

		// definir l'ordre de multiplicite du noeud
		while (i < Degree + PoleCount)
		{
			if (FMath::Abs(NodalVector[i + 1] - udeb) > DOUBLE_SMALL_NUMBER) break;
			multiplicite++;
			i++;
		}

		// si le noeud est different du noeud debut ou fin tester la discontinuite
		if ((udeb > NodalVector[0]) && (udeb < NodalVector[Degree + PoleCount]))
		{
			// ne rechercher la discontinuite que dans l'intervalle specifie
			if ((udeb > Boundary.Min) && (udeb < Boundary.Max))
			{
				// verifier la condition necessaire de derivabilite
				if (multiplicite > Degree - DerivativeOrder)
				{
					OutNotDerivableParameters.Add(udeb);
				}
			}
		}
		i++;
	}
}

void FindNotDerivableParameters(const FNURBSCurve& Curve, int32 InDerivativeOrder, const FLinearBoundary& Boundary, TArray<double>& OutNotDerivableParameters)
{
	FindNotDerivableParameters(Curve.GetDegree(), Curve.GetPoleCount(), Curve.GetNodalVector(), InDerivativeOrder, Boundary, OutNotDerivableParameters);
}

void FindNotDerivableParameters(const FNURBSSurface& Nurbs, int32 InDerivativeOrder, const FSurfacicBoundary& Boundary, FCoordinateGrid& OutNotDerivableParameters)
{
	FindNotDerivableParameters(Nurbs.GetDegree(EIso::IsoU), Nurbs.GetPoleCount(EIso::IsoU), Nurbs.GetNodalVector(EIso::IsoU), InDerivativeOrder, Boundary[EIso::IsoU], OutNotDerivableParameters[EIso::IsoU]);
	FindNotDerivableParameters(Nurbs.GetDegree(EIso::IsoV), Nurbs.GetPoleCount(EIso::IsoV), Nurbs.GetNodalVector(EIso::IsoV), InDerivativeOrder, Boundary[EIso::IsoV], OutNotDerivableParameters[EIso::IsoV]);
}

void SampleNodalVector(double UMin, double UMax, TArray<double>& nodalVector, int32 nbPoles, int32 degre, TArray<double>* TabU)
{
	int32 ideb = degre;
	int32 ifin = nbPoles;

	double uCourant = nodalVector[ideb];
	double u1 = UMin;
	double u2 = UMax;

	TabU->Empty();

	for (int32 i = ideb; i < ifin; i++)
	{
		if (nodalVector[i + 1] < UMin)
		{
			uCourant = nodalVector[i + 1];
			continue;
		}

		// passer sur les noeuds multiples
		if (FMath::Abs(nodalVector[i + 1] - nodalVector[i]) < DOUBLE_SMALL_NUMBER) continue;
		else uCourant = nodalVector[i];

		// definir les bornes de l'intervalle a discretiser
		if (uCourant > u1 + DOUBLE_SMALL_NUMBER) u1 = uCourant;
		if (nodalVector[i + 1] < u2 - DOUBLE_SMALL_NUMBER) u2 = nodalVector[i + 1];

		// repartir les parametres dans l'intervalle courant*/
		if (degre <= 1)
		{
			// prendre le point milieu si on n'est pas dans le premier intervalle
			// ou dans le dernier intervalle
			if (u1 > UMin && u2 < UMax)
			{
				TabU->Add((u1 + u2) / 2.0);
			}
			else if (FMath::IsNearlyEqual(u1, UMin))
			{
				TabU->Add(u1);
			}
		}
		else
		{
			TabU->Reserve(TabU->Num() + degre);
			for (int32 j = 0; j < degre; j++)
			{
				TabU->Add(u1 + (u2 - u1) * (double)j / (double)degre);
			}
		}

		// tester s'il s'agit du dernier intervalle
		if (u2 > UMax - DOUBLE_SMALL_NUMBER) break;

		// passer a l'intervalle suivant
		u1 = u2;
		u2 = UMax;
	}

	// dernier point
	TabU->Add(UMax);
}

void DuplicateNurbsCurveWithHigherDegree(int32 degre, const TArray<FVector>& poles, const TArray<double>& nodalVector, const TArray<double>& weights,
	TArray<FVector>& newPoles, TArray<double>& newNodalVector, TArray<double>& newWeights)
{
	//Count number of polynomial segments and
	//create new knot vector
	TArray<int32> segMaps;
	newNodalVector.Empty();
	int32 nbSegs = 0;
	for (int32 i = 0; i < nodalVector.Num(); i++)
	{
		newNodalVector.Add(nodalVector[i]);
		if (i == nodalVector.Num() - 1)
		{
			newNodalVector.Add(nodalVector[i]);
			break;
		}
		if (!FMath::IsNearlyEqual(nodalVector[i], nodalVector[i + 1]))
		{
			segMaps.Add(i);
			segMaps.Add((int32)newNodalVector.Num());
			newNodalVector.Add(nodalVector[i]);
			nbSegs++;
		}
	}

	newPoles.SetNum(poles.Num() + nbSegs);
	newWeights.SetNum(poles.Num() + nbSegs);

	//int32 newN=n+1;
	int32 newN = degre + 1;

	if (newPoles.Num() + newN - 1 != newNodalVector.Num())
	{
		// To avoid crashes while waiting for the fix (jira UETOOL-5046)
		newPoles.Empty();
		return;
	}

	TArray<bool> polesDone;
	polesDone.Init(false, newPoles.Num());

	//Calculate all poles for each segment.
	for (int32 l = 0; l < segMaps.Num(); l += 2)
	{
		int32 seg = segMaps[l];
		int32 newSeg = segMaps[l + 1];

		int32 i;
		for (i = 0; i < newN + 1; i++)
		{
			if (polesDone[newSeg - newN + 1 + i]) continue;
			TArray<TArray<double>> params; params.SetNum(newN);

			for (int32 j = 0; j < newN; j++)
			{
				for (int32 k = 0; k < newN; k++)
				{
					if (k != j)
					{
						params[j].Add(newNodalVector[newSeg - newN + 1 + k + i]);
					}
				}
			}

			TArray<FVector> poles_;
			poles_.SetNum(newN);
			TArray<double> weightsTmp;
			weightsTmp.SetNum(newN);
			for (int32 k = 0; k < newN; k++)
			{
				Blossom(degre, poles, nodalVector, weights, seg, params[k], poles_[k], weightsTmp[k]);
			}

			FVector pole(0, 0, 0);
			double w = 0;
			for (int32 k = 0; k < newN; k++)
			{
				pole[0] += poles_[k][0] * weightsTmp[k];
				pole[1] += poles_[k][1] * weightsTmp[k];
				pole[2] += poles_[k][2] * weightsTmp[k];
				w += weightsTmp[k];
			}
			pole[0] *= 1.0 / newN;
			pole[1] *= 1.0 / newN;
			pole[2] *= 1.0 / newN;

			w /= newN;
			pole[0] *= 1.0 / w;
			pole[1] *= 1.0 / w;
			pole[2] *= 1.0 / w;
			newPoles[newSeg - newN + 1 + i][0] = pole[0];
			newPoles[newSeg - newN + 1 + i][1] = pole[1];
			newPoles[newSeg - newN + 1 + i][2] = pole[2];
			newWeights[newSeg - newN + 1 + i] = w;
			polesDone[newSeg - newN + 1 + i] = true;
		}
	}
}

int32 getKnotMultiplicity(const TArray<double>& knots, double u)
{
	int32 m = 0;
	for (int32 i = 0; i < (int32)knots.Num(); i++)
		if (FMath::IsNearlyEqual(knots[i], u)) m++;

	return m;
}


void insertKnotInKnots(TArray<double>& vn, double u, double* newU = nullptr)
{
	TArray<double> newKnots;
	newKnots.Reserve(vn.Num() + 1);
	bool done = false;
	for (int32 i = 0; i < (int32)vn.Num() - 1; i++)
	{
		newKnots.Add(vn[i]);
		if ((!done) && vn[i] <= u && u <= vn[i + 1])
		{
			newKnots.Add(newU ? *newU : u);
			done = true;
		}
	}
	newKnots.Add(vn[vn.Num() - 1]);
	Swap(vn, newKnots);
}

void InsertKnot(TArray<double>& vn, TArray<FVector>& poles, TArray<double>& weights, double u, double* newU)
{
	int32 i = 0;

	int32 n = (int32)vn.Num() - (int32)poles.Num() + 1; //Degree
	TArray<FVector> newPoles;
	TArray<double> newWeights;

	newPoles.Reserve(poles.Num() + vn.Num());
	newWeights.Reserve(poles.Num() + vn.Num());

	newPoles.Add(poles[0]);
	newWeights.Add(weights[0]);
	int32 front = 1;
	for (i = 0; i < (int32)poles.Num() - 1; i++)
	{
		if (vn[i] <= u && u < vn[i + n])
		{
			double w1 = weights[i];
			double w2 = weights[i + 1];
			double c1 = ((vn[i + n] - u) / (vn[i + n] - vn[i]));
			double c2 = ((u - vn[i]) / (vn[i + n] - vn[i]));
			double w = w1 * c1 + w2 * c2;
			FVector p((poles[i][0] * w1 * c1 + poles[i + 1][0] * w2 * c2) / w,
				(poles[i][1] * w1 * c1 + poles[i + 1][1] * w2 * c2) / w,
				(poles[i][2] * w1 * c1 + poles[i + 1][2] * w2 * c2) / w);
			newPoles.Add(p);
			newWeights.Add(w);
			front = 0;
			continue;
		}
		newPoles.Add(poles[i + front]);
		newWeights.Add(weights[i + front]);
	}
	newPoles.Add(poles[poles.Num() - 1]);
	newWeights.Add(weights[poles.Num() - 1]);

	Swap(poles, newPoles);
	Swap(weights, newWeights);

	insertKnotInKnots(vn, u, newU);
}

void insertKnotVInPatch(double v, int32 nbTime, TArray<double>& knotsV, TArray<TArray<FVector>>& poles, TArray<TArray<double>>& weights)
{
	TArray<double> knotsTmp;
	int32 nbCurveV = (int32)poles.Num();
	for (int32 i = 0; i < nbCurveV; i++)
	{
		knotsTmp = knotsV;
		for (int32 j = 0; j < nbTime; j++)
		{
			InsertKnot(knotsTmp, poles[i], weights[i], v);
		}
	}
	knotsV = knotsTmp;
}

void insertKnotUInPatch(double u, int32 nbTime, TArray<double>& knotsU, TArray<TArray<FVector>>& poles, TArray<TArray<double>>& weights)
{
	int32 nbCurveU = (int32)poles[0].Num();
	int32 nbPoleCurveU = (int32)poles.Num();
	int32 i, j;
	TArray<FVector> polesCurveU;
	TArray<double> weightsCurveU;
	TArray<double> knotsTmp;

	for (j = 0; j < nbTime; j++)
	{
		TArray<FVector>& SubPoles = poles.Emplace_GetRef();
		SubPoles.SetNum(nbCurveU);
		TArray<double>& SubWeights = weights.Emplace_GetRef();
		SubWeights.Reserve(nbCurveU);
	}

	for (i = 0; i < nbCurveU; i++)
	{
		knotsTmp = knotsU;
		polesCurveU.SetNum(nbPoleCurveU);
		weightsCurveU.SetNum(nbPoleCurveU);
		for (j = 0; j < nbPoleCurveU; j++)
		{
			polesCurveU[j] = poles[j][i];
			weightsCurveU[j] = weights[j][i];
		}

		for (j = 0; j < nbTime; j++)
		{
			InsertKnot(knotsTmp, polesCurveU, weightsCurveU, u);
		}

		for (j = 0; j < (int32)polesCurveU.Num(); j++)
		{
			poles[j][i] = polesCurveU[j];
			weights[j][i] = weightsCurveU[j];
		}
	}
	knotsU = knotsTmp;
}

void rebound(TArray<double>& knots, TArray<FVector>& poles, TArray<double>& weights, double u1, double u2)
{
	// TD 21/09/2016 : corrige un crash à l'import de certaines pièces
	if (u1 > u2)
	{
		//si les points ne sont pas ordonnés correctement, on inverse les listes
		Algo::Reverse(knots);
		Algo::Reverse(poles);
		Algo::Reverse(weights);
	}

	int32 deg = (int32)knots.Num() - (int32)poles.Num() + 1;

	//Rebound curve by knot insertion of u1 then u2...
	while (getKnotMultiplicity(knots, u1) < deg)
	{
		InsertKnot(knots, poles, weights, u1);
	}
	while (getKnotMultiplicity(knots, u2) < deg)
	{
		InsertKnot(knots, poles, weights, u2);
	}

	//Now find the corresponding greville abscissas
	int32 u1Ind = -1, u2Ind = -1;
	for (int32 i = 0; i < (int32)knots.Num(); i++)
	{
		if (FMath::IsNearlyEqual(knots[i], u1) && u1Ind == -1) u1Ind = i;
		if (FMath::IsNearlyEqual(knots[i], u2) && u2Ind == -1) u2Ind = i;
	}

	//Now trim the NURBS around these abscissas
	TArray<double> tKnots;
	tKnots.Append(knots.GetData() + u1Ind, u2Ind + 1 - u1Ind);
	TArray<FVector> tPoles;
	tPoles.Append(poles.GetData() + u1Ind, u2Ind + 1 - u1Ind);
	TArray<double> tWeights;
	tWeights.Append(weights.GetData() + u1Ind, u2Ind + 1 - u1Ind);

	//Add the deg-1 knots to end the knot vector
	tKnots.Reserve(tKnots.Num() + deg - 1);
	for (int32 i = 0; i < deg - 1; i++) tKnots.Add(knots[u2Ind]);

	Swap(poles, tPoles);
	Swap(weights, tWeights);
	Swap(knots, tKnots);
}

TArray<double> homogeniseCurveKnots(TArray<TArray<double>>& cbKnots, double uMin, double uMax)
{
	TArray<double> knots;
	int32 j = 0;

	for (int32 i = 0; i < (int32)cbKnots.Num(); i++)
	{
		TArray<double>& cbKnot = cbKnots[i];
		if (cbKnot.Num() == 0) continue;
		double cbUMin = cbKnot[0], cbUMax = cbKnot[0];
		for (j = 1; j < (int32)cbKnot.Num(); j++)
		{
			if (cbUMin > cbKnot[j]) cbUMin = cbKnot[j];
			if (cbUMax < cbKnot[j]) cbUMax = cbKnot[j];
		}
		//Normalize between uMin and uMax.
		for (j = 0; j < (int32)cbKnot.Num(); j++)
			cbKnot[j] = uMin + (uMax - uMin) * (cbKnot[j] - cbUMin) / (cbUMax - cbUMin);
		for (j = 0; j < (int32)cbKnot.Num(); j++)
		{
			while (getKnotMultiplicity(knots, cbKnot[j]) < getKnotMultiplicity(cbKnot, cbKnot[j]))
			{ //This not doesn't exist with this multiplicity... So add it
				knots.Add(cbKnot[j]);
			}
		}
		for (int32 k = 0; k < (int32)knots.Num(); k++)
		{
			for (j = k; j < (int32)knots.Num(); j++)
			{
				if (knots[j] < knots[k])
				{
					double t = knots[j];
					knots[j] = knots[k];
					knots[k] = t;
				}
			}
		}
	}

	return knots;
}

void Blossom(int32 degre, const TArray<FVector>& poles, const TArray<double>& nodalVector, const TArray<double>& weights, int32 seg, TArray<double>& params, FVector& pnt, double& weight)
{
	int32 n = degre;
	int32 i = 0;

	//There must be n params.
	//Check interval is valid
	if (seg < n - 1) seg = n - 1;
	if (seg > (int32)nodalVector.Num() - n - 1) seg = (int32)nodalVector.Num() - n - 1;

	//Make recursion table so that (with first concerned pole =2
	//because for example n=3 and seg=4)
	//	d(2,0)
	//	d(3,0) d(3,1)
	//	d(4,0) d(4,1) d(4,2)
	//	d(5,0) d(5,1) d(5,2) d(5,3)
	//So with firstPole=2, recursion[4-firstPole][1]=d(4,1);

	TArray<TArray<FVector>> recursion;
	recursion.SetNum(n + 1);	//We have n+1 lines
	for (i = 0; i < n + 1; i++)
	{
		recursion[i].SetNum(n + 1); //Make the triangle
	}

	TArray<TArray<double>> recWeights;
	recWeights.SetNum(n + 1);	//We have n+1 lines
	for (i = 0; i < n + 1; i++)
	{
		recWeights[i].SetNum(n + 1); //Make the triangle
	}

	//For each pole startParam is the param value if the pole
	//is the first on the polygon segment and endParam is the
	//param value if the pole is at the end of the polygone
	//segment.

	TArray<TArray<double>> startParam;
	startParam.SetNum(n + 1);	//We have n+1 lines
	for (i = 0; i < n + 1; i++)
	{
		startParam[i].SetNum(n + 1); //Make the triangle
	}

	TArray<TArray<double>> endParam;
	endParam.SetNum(n + 1);	//We have n+1 lines
	for (i = 0; i < n + 1; i++)
	{
		endParam[i].SetNum(n + 1); //Make the triangle
	}

	//Calculate first and last concerned initial poles.
	int32 firstPole = seg - n + 1;
	int32 lastPole = firstPole + n;

	//Fill first colonne
	for (i = firstPole; i <= lastPole; i++)
	{
		recursion[i - firstPole][0][0] = poles[i][0] * weights[i];
		recursion[i - firstPole][0][1] = poles[i][1] * weights[i];
		recursion[i - firstPole][0][2] = poles[i][2] * weights[i];
		recWeights[i - firstPole][0] = weights[i];
	}
	//Set start and end params
	for (i = firstPole; i <= lastPole; i++)
	{ //For example pole 0 starts at u0 and ends at u3 (for n=3)
		startParam[i - firstPole][0] = nodalVector[i];
		endParam[i - firstPole][0] = nodalVector[i + n - 1];
	}

	//Apply de Boor algorithm for each param, there must be n params.
	//For colonne 1 we use param 1, for colonne 2 we use param 2, ...
	for (int32 c = 1; c <= n; c++)
	{
		//colonne goes from 1 to n
		//Calculate pole
		for (int32 l = c; l <= n; l++)
		{
			//line goes from line to n
			//Evaluate intermediate pole.
			double dt = (endParam[l - 0][c - 1] - startParam[l - 1][c - 1]);
			FVector pole;
			double div = (params[c - 1] - startParam[l - 1][c - 1]) / dt;
			pole[0] = recursion[l - 1][c - 1][0] * 1.0 + (-recursion[l - 1][c - 1][0] + recursion[l][c - 1][0]) * div;
			pole[1] = recursion[l - 1][c - 1][1] * 1.0 + (-recursion[l - 1][c - 1][1] + recursion[l][c - 1][1]) * div;
			pole[2] = recursion[l - 1][c - 1][2] * 1.0 + (-recursion[l - 1][c - 1][2] + recursion[l][c - 1][2]) * div;
			double w = recWeights[l - 1][c - 1] * 1.0 + (-recWeights[l - 1][c - 1] + recWeights[l][c - 1]) * div;

			recursion[l][c][0] = pole[0];
			recursion[l][c][1] = pole[1];
			recursion[l][c][2] = pole[2];
			recWeights[l][c] = w;

			//For FParameters, the new pole takes startParam of pole c-1 and endParam of pole l-1,c-1 (because new param is inserted here).
			startParam[l][c] = startParam[l][c - 1];
			endParam[l][c] = endParam[l - 1][c - 1];
		}
	}

	pnt[0] = recursion[n][n][0] / recWeights[n][n];
	pnt[1] = recursion[n][n][1] / recWeights[n][n];
	pnt[2] = recursion[n][n][2] / recWeights[n][n];
	weight = recWeights[n][n];
}

void composeNodalVector(double UMin, double UMax, int32 degre, int32 nbPoles, TArray<double>* nodalVector)
{
	nodalVector->Empty();
	nodalVector->Reserve(2 * (degre + 1) + nbPoles - degre + 1);

	for (int32 k = 0; k < degre + 1; k++)
	{
		nodalVector->Add(UMin);
	}

	int32 nbInter = nbPoles + 1 - degre;
	for (int32 k = 1; k < nbInter - 1; k++)
	{
		double val = UMin + ((double)k) / (nbInter - 1) * (UMax - UMin);
		nodalVector->Add(val);
	}

	for (int32 k = 0; k < degre + 1; k++) nodalVector->Add(UMax);
}

void hermite(double t, double* H, double* dH, double* ddH)
{
	H[0] = 2.0 * t * t * t - 3.0 * t * t + 1.0;
	H[1] = t * t * t - 2.0 * t * t + t;
	H[2] = t * t * t - t * t;
	H[3] = -2.0 * t * t * t + 3.0 * t * t;

	if (dH)
	{
		dH[0] = 6. * t * t - 6. * t;
		dH[1] = 3. * t * t - 4. * t + 1.;
		dH[2] = 3. * t * t - 2. * t;
		dH[3] = -6. * t * t + 6. * t;
	}

	if (ddH)
	{
		ddH[0] = 12. * t - 6.;
		ddH[1] = 6. * t - 4.;
		ddH[2] = 6. * t - 2.;
		ddH[3] = -12. * t + 6.;
	}
}
} // namespace Bspline
} // namespace UE::CADKernel
