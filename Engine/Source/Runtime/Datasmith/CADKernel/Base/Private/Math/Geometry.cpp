// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Geometry.h"

#include "Math/Geometry.h"
#include "Math/Point.h"
#include "Math/MatrixH.h"
#include "Utils/Util.h"

namespace UE::CADKernel
{
namespace IntersectionTool
{
static double IntersectionToolTolerance = 0.01;

void SetTolerance(const double Tolerance)
{
	IntersectionToolTolerance = Tolerance;
}

struct FIntersectionContext
{
	const FSegment2D& SegmentAB;
	const FSegment2D& SegmentCD;
	const FVector2d AB;
	const FVector2d CD;
	const FVector2d CA;

	double NormAB = 0;
	double NormCD = 0;

	FIntersectionContext(const FSegment2D& InSegmentAB, const FSegment2D& InSegmentCD)
		: SegmentAB(InSegmentAB)
		, SegmentCD(InSegmentCD)
		, AB(SegmentAB.GetVector())
		, CD(SegmentCD.GetVector())
		, CA(SegmentAB[0] - SegmentCD[0])
	{
	}
};

bool DoCoincidentSegmentsIntersectInside(double A, double B, double C, double D)
{
	return !((D < A + DOUBLE_KINDA_SMALL_NUMBER) || (B < C + DOUBLE_KINDA_SMALL_NUMBER));
}

bool DoCoincidentSegmentsIntersect(double A, double B, double C, double D)
{
	return !((D < A) || (B < C));
}

constexpr double MinValue(bool OnlyInside)
{
	return OnlyInside ? DOUBLE_KINDA_SMALL_NUMBER : -DOUBLE_KINDA_SMALL_NUMBER;
}

constexpr double MaxValue(bool OnlyInside)
{
	return OnlyInside ? 1. - DOUBLE_KINDA_SMALL_NUMBER : 1. + DOUBLE_KINDA_SMALL_NUMBER;
}

bool ConfirmIntersectionWhenNearlyCoincident(const FVector2d& AB, const FVector2d& AC, const FVector2d& AD, const double NormAB)
{
	double HeightC = (AB ^ AC) / NormAB;
	double HeightD = (AB ^ AD) / NormAB;

	if (fabs(HeightC) < IntersectionToolTolerance || fabs(HeightD) < IntersectionToolTolerance)
	{
		return true;
	}
	return HeightC * HeightD < 0;
};

bool ConfirmIntersectionWhenNearlyCoincident(const FIntersectionContext& Context)
{
	if (Context.NormAB > Context.NormCD)
	{
		const FVector2d DA = Context.SegmentAB[0] - Context.SegmentCD[1];
		return ConfirmIntersectionWhenNearlyCoincident(Context.AB, Context.CA, DA, Context.NormAB);
	}
	else
	{
		const FVector2d CB = Context.SegmentAB[1] - Context.SegmentCD[0];
		return ConfirmIntersectionWhenNearlyCoincident(Context.CD, Context.CA, CB, Context.NormCD);
	}
}

}


double ComputeCurvature(const FVector& Gradient, const FVector& Laplacian)
{
	const FVector GradientCopy = Gradient.GetSafeNormal();
	const FVector LaplacianCopy = Laplacian.GetSafeNormal();
	const FVector Normal = GradientCopy ^ LaplacianCopy;
	return (Normal.Length() * Laplacian.Length()) / (Gradient.Length() * Gradient.Length());
}

double ComputeCurvature(const FVector& Normal, const FVector& Gradient, const FVector& Laplacian)
{
	const FVector GradientCopy = Gradient.GetSafeNormal();
	const FVector LaplacianCopy = Laplacian.GetSafeNormal();
	const FVector NormalCopy = Normal.GetSafeNormal();
	const FVector Coef = (LaplacianCopy ^ GradientCopy) ^ NormalCopy;
	return (Coef.Length() * Laplacian.Length()) / Gradient.SquaredLength();
}

void FindLoopIntersectionsWithIso(const EIso Iso, const double IsoParameter, const TArray<TArray<FVector2d>>& Loops, TArray<double>& OutIntersections)
{
	TArray<double> LocalIntersections;
	LocalIntersections.Reserve(8);

	const int32 UIndex = Iso == EIso::IsoU ? 0 : 1;
	const int32 VIndex = Iso == EIso::IsoU ? 1 : 0;

	TFunction<void(const FVector2d&, const FVector2d&)> ComputeIntersection = [&](const FVector2d& Point1, const FVector2d& Point2)
	{
		if (IsoParameter > Point1[UIndex] && IsoParameter <= Point2[UIndex])
		{
			const double Intersection = (IsoParameter - Point1[UIndex]) / (Point2[UIndex] - Point1[UIndex]) * (Point2[VIndex] - Point1[VIndex]) + Point1[VIndex];
			LocalIntersections.Add(Intersection);
		}
	};

	for (const TArray<FVector2d>& Loop : Loops)
	{
		const FVector2d* Point1 = &Loop.Last();
		for (const FVector2d& Point2 : Loop)
		{
			if (!FMath::IsNearlyEqual((*Point1)[UIndex], Point2[UIndex]))
			{
				if ((*Point1)[UIndex] < Point2[UIndex])
				{
					ComputeIntersection(*Point1, Point2);
				}
				else
				{
					ComputeIntersection(Point2, *Point1);
				}
			}
			Point1 = &Point2;
		}
	}

	if (LocalIntersections.Num() == 0)
	{
		OutIntersections.Empty();
		return;
	}

	Algo::Sort(LocalIntersections);

	// Remove any duplicates
	OutIntersections.Empty(LocalIntersections.Num());
	OutIntersections.Add(LocalIntersections[0]);
	for (int32 Index = 1; Index < LocalIntersections.Num(); ++Index)
	{
		if (FMath::IsNearlyEqual(LocalIntersections[Index], OutIntersections.Last(), UE_DOUBLE_SMALL_NUMBER))
		{
			continue;
		}
		OutIntersections.Add(LocalIntersections[Index]);
	}
}

bool DoIntersect(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD, TFunction<bool(double, double, double, double)> DoCoincidentSegmentsIntersect, const double Min, const double Max)
{
	using namespace IntersectionTool;

	TFunction<bool(double, double, double, double)> FastIntersectionTestWhenCoincident = [&DoCoincidentSegmentsIntersect](double A, double B, double C, double D) -> bool
	{
		if (A < B)
		{
			if (C < D)
			{
				return DoCoincidentSegmentsIntersect(A, B, C, D);
			}
			else
			{
				return DoCoincidentSegmentsIntersect(A, B, D, C);
			}
		}
		else
		{
			if (C < D)
			{
				return DoCoincidentSegmentsIntersect(B, A, C, D);
			}
			else
			{
				return DoCoincidentSegmentsIntersect(B, A, D, C);
			}
		}
	};

	FIntersectionContext Context(SegmentAB, SegmentCD);

	const double ParallelCoef = Context.CD ^ Context.AB;
	if (FMath::IsNearlyZero(ParallelCoef, DOUBLE_KINDA_SMALL_NUMBER))
	{
		// double check with normalized vectors
		{
			FVector2d NormalizedAB = Context.AB;
			FVector2d NormalizedCD = Context.CD;
			FVector2d NormalizedCA = Context.CA;

			NormalizedAB.Normalize(Context.NormAB);
			NormalizedCD.Normalize(Context.NormCD);
			NormalizedCA.Normalize();

			const double NormalizedParallelCoef = NormalizedCD ^ NormalizedAB;
			if (FMath::IsNearlyZero(NormalizedParallelCoef, DOUBLE_KINDA_SMALL_NUMBER))
			{
				const double NormalizedParallelCoef2 = NormalizedCA ^ NormalizedAB;
				if (!FMath::IsNearlyZero(NormalizedParallelCoef2, DOUBLE_KINDA_SMALL_NUMBER))
				{
					return false;
				}

				if (fabs(Context.AB.X) > fabs(Context.AB.Y))
				{
					if (FastIntersectionTestWhenCoincident(SegmentAB[0].X, SegmentAB[1].X, SegmentCD[0].X, SegmentCD[1].X))
					{
						return ConfirmIntersectionWhenNearlyCoincident(Context);
					}
					else
					{
						return false;
					}
				}
				else
				{
					if (FastIntersectionTestWhenCoincident(SegmentAB[0].Y, SegmentAB[1].Y, SegmentCD[0].Y, SegmentCD[1].Y))
					{
						return ConfirmIntersectionWhenNearlyCoincident(Context);
					}
					else
					{
						return false;
					}
				}
			}
		}
	}

	const double ABIntersectionCoordinate = (Context.CA ^ Context.CD) / ParallelCoef;
	const double CDIntersectionCoordinate = (Context.CA ^ Context.AB) / ParallelCoef;
	return (ABIntersectionCoordinate <= Max && ABIntersectionCoordinate >= Min && CDIntersectionCoordinate <= Max && CDIntersectionCoordinate >= Min);
}

bool DoIntersectInside(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD)
{
	return DoIntersect(SegmentAB, SegmentCD, IntersectionTool::DoCoincidentSegmentsIntersectInside, IntersectionTool::MinValue(true), IntersectionTool::MaxValue(true));
}

bool DoIntersect(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD)
{
	return DoIntersect(SegmentAB, SegmentCD, IntersectionTool::DoCoincidentSegmentsIntersect, IntersectionTool::MinValue(false), IntersectionTool::MaxValue(false));
}

}
