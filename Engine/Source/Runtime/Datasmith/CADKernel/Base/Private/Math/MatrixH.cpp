// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/MatrixH.h"

#include "Math/MathConst.h"
#include "UI/Message.h"

namespace UE::CADKernel
{

const FMatrixH FMatrixH::Identity;

void FMatrixH::BuildChangeOfCoordinateSystemMatrix(const FVector& Xaxis, const FVector& Yaxis, const FVector& Zaxis, const FVector& Origin)
{
	Get(0, 0) = Xaxis[0];
	Get(1, 0) = Xaxis[1];
	Get(2, 0) = Xaxis[2];
	Get(3, 0) = 0.0;

	Get(0, 1) = Yaxis[0];
	Get(1, 1) = Yaxis[1];
	Get(2, 1) = Yaxis[2];
	Get(3, 1) = 0.0;

	Get(0, 2) = Zaxis[0];
	Get(1, 2) = Zaxis[1];
	Get(2, 2) = Zaxis[2];
	Get(3, 2) = 0.0;

	Get(0, 3) = Origin[0];
	Get(1, 3) = Origin[1];
	Get(2, 3) = Origin[2];
	Get(3, 3) = 1.0;
}

void FMatrixH::FromAxisOrigin(const FVector& Axis, const FVector& Origin)
{
	FVector Zaxis = FVector(0, 1, 0);
	FVector Xaxis = Zaxis ^ Axis;

	//on cherche le vecteur ox
	Xaxis.Normalize();
	if (FMath::Abs(Xaxis.Length()) < DOUBLE_SMALL_NUMBER)
	{
		Zaxis = FVector(1, 0, 0);
		Xaxis = Axis ^ Zaxis;
		Xaxis.Normalize();
		if (FMath::Abs(Xaxis.Length()) < DOUBLE_SMALL_NUMBER)
		{
			Zaxis = FVector(0, 0, 1);
			Xaxis = Axis ^ Zaxis;

			Xaxis.Normalize();
			ensureCADKernel(FMath::Abs(Xaxis.Length()) > DOUBLE_SMALL_NUMBER);
		}
	}

	FVector YAxis = Axis ^ Xaxis;
	Zaxis = Axis;

	YAxis.Normalize();
	Zaxis.Normalize();

	BuildChangeOfCoordinateSystemMatrix(Xaxis, YAxis, Zaxis, Origin);
}

FMatrixH FMatrixH::MakeRotationMatrix(double Angle, FVector Axe)
{
	FMatrixH Matrix;
	Matrix.SetIdentity();

	ensureCADKernel(Axe.Length() > DOUBLE_SMALL_NUMBER);
	Axe.Normalize();

	Matrix.Get(0, 0) = Axe[0] * Axe[0] + (double)cos(Angle) * ((double)1.0 - Axe[0] * Axe[0]);
	Matrix.Get(0, 1) = (double)(1.0 - cos(Angle)) * Axe[0] * Axe[1] - (double)sin(Angle) * Axe[2];
	Matrix.Get(0, 2) = (double)(1.0 - cos(Angle)) * Axe[0] * Axe[2] + (double)sin(Angle) * Axe[1];

	Matrix.Get(1, 0) = (double)(1.0 - cos(Angle)) * Axe[1] * Axe[0] + (double)sin(Angle) * Axe[2];
	Matrix.Get(1, 1) = Axe[1] * Axe[1] + (double)cos(Angle) * ((double)1.0 - Axe[1] * Axe[1]);
	Matrix.Get(1, 2) = (double)(1.0 - cos(Angle)) * Axe[1] * Axe[2] - (double)sin(Angle) * Axe[0];

	Matrix.Get(2, 0) = (double)(1.0 - cos(Angle)) * Axe[2] * Axe[0] - (double)sin(Angle) * Axe[1];
	Matrix.Get(2, 1) = (double)(1.0 - cos(Angle)) * Axe[2] * Axe[1] + (double)sin(Angle) * Axe[0];
	Matrix.Get(2, 2) = Axe[2] * Axe[2] + (double)cos(Angle) * ((double)1.0 - Axe[2] * Axe[2]);
	return Matrix;
}

FMatrixH FMatrixH::MakeTranslationMatrix(const FVector& Point)
{
	FMatrixH Matrix;
	Matrix.SetIdentity();
	Matrix.Get(0, 3) = Point.X;
	Matrix.Get(1, 3) = Point.Y;
	Matrix.Get(2, 3) = Point.Z;
	return Matrix;
}

FMatrixH FMatrixH::MakeScaleMatrix(double XScale, double YScale, double ZScale)
{
	FMatrixH Matrix;
	Matrix.SetIdentity();
	Matrix.Get(0, 0) = XScale;
	Matrix.Get(1, 1) = YScale;
	Matrix.Get(2, 2) = ZScale;
	return Matrix;
}

FMatrixH FMatrixH::MakeScaleMatrix(FVector& Scale)
{
	return MakeScaleMatrix(Scale.X, Scale.Y, Scale.Z);
}

void FMatrixH::Inverse()
{
	InverseMatrixN(Matrix, 4);
}

void InverseMatrixN(double* Matrice, int32 Rank)
{
	const double One = 1.0;
	const double Zero = 0.0;

	TArray<double> TempMatrix;
	TempMatrix.Append(Matrice, Rank * Rank);

	double Determinant = One;

	TArray<int32> ColumnToRow;
	ColumnToRow.SetNum(Rank);
	for (int32 Index = 0; Index < Rank; Index++)
	{
		ColumnToRow[Index] = Index;
	}

	for (int32 Column = 0; Column < Rank; ++Column)
	{
		double Pivot = 0;

		int32 Row = Column;
		while (Row < Rank)
		{
			Pivot = TempMatrix[Rank * Row + Column];
			if (!FMath::IsNearlyZero(Pivot))
			{
				break;
			}
			Row++;
		}

		Determinant = Determinant * Pivot;
		if (Row != Column)
		{
			Swap(ColumnToRow[Column], ColumnToRow[Row]);
			for (int32 Index = 0; Index < Rank; Index++)
			{
				Swap(TempMatrix[Row * Rank + Index], TempMatrix[Column * Rank + Index]);
			}
			Determinant = -Determinant;
		}

		double InvPivot = One / Pivot;
		TempMatrix[Column * Rank + Column] = One;

		for (int32 Index = 0; Index < Rank; Index++)
		{
			TempMatrix[Column * Rank + Index] = TempMatrix[Column * Rank + Index] * InvPivot;
		}

		for (Row = 0; Row < Rank; Row++)
		{
			if (Row == Column)
			{
				continue;
			}

			double ValueRC = TempMatrix[Row * Rank + Column];

			TempMatrix[Row * Rank + Column] = Zero;
			for (int32 Index = 0; Index < Rank; Index++)
			{
				TempMatrix[Row * Rank + Index] = TempMatrix[Row * Rank + Index] - ValueRC * TempMatrix[Column * Rank + Index];
			}
		}
	}

	for (int32 Column = 0; Column < Rank; Column++)
	{
		int32 Row = Column;
		while (Row < Rank)
		{
			if (ColumnToRow[Row] == Column)
			{
				break;
			}
			Row++;
		}

		if (Column == Row)
		{
			continue;
		}

		ColumnToRow[Row] = ColumnToRow[Column];
		for (int32 Index = 0; Index < Rank; Index++)
		{
			Swap(TempMatrix[Index * Rank + Column], TempMatrix[Index * Rank + Row]);
		}
	}

	memcpy(Matrice, TempMatrix.GetData(), Rank * Rank * sizeof(double));
}

void MatrixProduct(int32 ARowNum, int32 AColumnNum, int32 ResultRank, const double* MatrixA, const double* MatrixB, double* MatrixResult)
{
	for (int32 RowA = 0; RowA < ARowNum; RowA++)
	{
		for (int32 ColumnB = 0; ColumnB < ResultRank; ColumnB++)
		{
			int32 ResultIndex = RowA * ResultRank + ColumnB;

			MatrixResult[ResultIndex] = 0.0;
			for (int32 k = 0; k < AColumnNum; k++)
			{
				MatrixResult[ResultIndex] = MatrixResult[ResultIndex] + MatrixA[RowA * AColumnNum + k] * MatrixB[k * ResultRank + ColumnB];
			}
		}
	}
}

void TransposeMatrix(int32 RowNum, int32 ColumnNum, const double* InMatrix, double* OutMatrix)
{
	for (int32 Row = 0; Row < RowNum; ++Row)
	{
		for (int32 Column = 0; Column < ColumnNum; ++Column)
		{
			OutMatrix[RowNum * Column + Row] = InMatrix[ColumnNum * Row + Column];
		}
	}
}
} // namespace UE::CADKernel
