// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Point.h"

#include "Utils/Util.h"
#include "Math/MatrixH.h"

namespace UE::CADKernel
{
const FVector FVectorUtil::FarawayPoint3D(HUGE_VALUE, HUGE_VALUE, HUGE_VALUE);
const FVector2d FVectorUtil::FarawayPoint2D(HUGE_VALUE, HUGE_VALUE);

//double FVector::SignedAngle(const FVector & Other, const FVector & Normal) const
//{
//	FVector Vector1 = *this; 
//	FVector Vector2 = Other; 
//	FVector Vector3 = Normal; 
//
//	Vector1.Normalize();
//	Vector2.Normalize();
//	Vector3.Normalize();
//
//	double ScalarProduct = Vector1 * Vector2;
//
//	if (ScalarProduct >= 1 - DOUBLE_SMALL_NUMBER)
//	{
//		return 0.;
//	}
//
//	if (ScalarProduct <= -1 + DOUBLE_SMALL_NUMBER)
//	{
//		return DOUBLE_PI;
//	}
//
//	return MixedTripleProduct(Vector1, Vector2, Vector3) > 0 ? acos(ScalarProduct) : -acos(ScalarProduct);
//}


}