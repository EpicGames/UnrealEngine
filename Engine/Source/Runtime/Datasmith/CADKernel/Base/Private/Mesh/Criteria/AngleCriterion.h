// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Mesh/Criteria/Criterion.h"

/**
 * Sag & Angle criterion.pdf
 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
*/

namespace UE::CADKernel
{

class FAngleCriterion : public FCriterion
{
	friend class FEntity;

protected:
	double AngleCriterionValue;
	double SinMaxAngle;

	/**
	 * @param DegreeAngle the max allowed angle between tow elements in degree
	 */
	FAngleCriterion(double DegreeAngle)
	{
		AngleCriterionValue = FMath::DegreesToRadians(DegreeAngle);
		SinMaxAngle = sin(AngleCriterionValue * 0.5);
	}

	FAngleCriterion() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCriterion::Serialize(Ar);
		Ar << AngleCriterionValue;
		Ar << SinMaxAngle;
	}

	virtual double Value() const override
	{
		return AngleCriterionValue;
	}

	static double DefaultValue()
	{
		return FMath::DegreesToRadians(15.0);
	}

	virtual bool IsAppliedBetweenBreaks() const override
	{
		return true;
	}

	virtual ECriterion GetCriterionType() const override
	{
		return ECriterion::Angle;
	}


protected:

	/**
	 * Sag & Angle criterion.pdf
	 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	*/
	virtual double ComputeDeltaU(double ChordLength, double DeltaU, double Sag) const override
	{
		return 0.25 * SinMaxAngle * ChordLength * DeltaU / Sag;
	}
};

} // namespace UE::CADKernel

