// Copyright Epic Games, Inc. All Rights Reserved.
#include "Mesh/Criteria/CriteriaGrid.h"

#include "Mesh/Criteria/Criterion.h"
#include "Geo/Surfaces/Surface.h"
#include "Topo/TopologicalFace.h"

namespace UE::CADKernel
{

void FCriteriaGrid::Init()
{
	TFunction<void(EIso)> ComputeMiddlePointsCoordinates = [&](EIso Iso)
	{
		const TArray<double>& Tab = Face.GetCrossingPointCoordinates(Iso);
		TArray<double>& Tab2 = CoordinateGrid[Iso];

		CuttingCount[Iso] = Tab.Num() * 2 - 1;
		ensureCADKernel(CuttingCount[Iso]);
		Tab2.SetNum(CuttingCount[Iso]);

		Tab2[0] = Tab[0];
		for (int32 Index = 1; Index < Tab.Num(); Index++)
		{
			Tab2[2 * Index - 1] = (Tab[Index - 1] + Tab[Index]) / 2.0;
			Tab2[2 * Index] = Tab[Index];
		}
	};

	ComputeMiddlePointsCoordinates(EIso::IsoU);
	ComputeMiddlePointsCoordinates(EIso::IsoV);

	EvaluatePointGrid(CoordinateGrid, false);
}

FCriteriaGrid::FCriteriaGrid(FTopologicalFace& InFace)
	: FGridBase(InFace)
	, CoordinateGrid(InFace.GetCrossingPointCoordinates())
{
	Init();
	FaceMinMax.Init();

#ifdef DISPLAY_CRITERIA_GRID
	Display();
#endif
}

void FCriteriaGrid::ComputeFaceMinMaxThicknessAlongIso()
{
	using FGetPointFunc = TFunction<const FVector& (int32, int32)>;

	TFunction<void(EIso, FGetPointFunc, FGetPointFunc, int32, int32)> ComputeFaceThicknessAlong = [this](EIso Iso, FGetPointFunc GetPointAlong, FGetPointFunc GetIntermediatePointAlong, int32 IsoUCount, int32 IsoVCount)
	{
		for (int32 VIndex = 0; VIndex < IsoVCount; ++VIndex)
		{
			const FVector& StartPoint = GetPointAlong(0, VIndex);
			const FVector* PreviousPoint = &GetIntermediatePointAlong(0, VIndex);
			double Length = FVector::Distance(StartPoint, *PreviousPoint);

			for (int32 UIndex = 1; UIndex < IsoUCount - 1; ++UIndex)
			{
				const FVector& Point = GetPointAlong(UIndex, VIndex);

				PreviousPoint = &GetIntermediatePointAlong(UIndex, VIndex);
				Length += FVector::Distance(Point, *PreviousPoint);
			}
			const FVector& LastPoint = GetPointAlong(IsoUCount - 1, VIndex);
			Length += FVector::Distance(LastPoint, *PreviousPoint);

			this->FaceMinMax[Iso].ExtendTo(Length);
		}
	};

	FGetPointFunc GetPointAlongU = [this](int32 UIndex, int32 VIndex) -> const FVector&
	{
		return GetPoint(UIndex, VIndex);
	};

	FGetPointFunc GetPointAlongV = [this](int32 VIndex, int32 UIndex) -> const FVector&
	{
		return GetPoint(UIndex, VIndex);
	};

	FGetPointFunc GetIntermediatePointAlongU = [this](int32 UIndex, int32 VIndex) -> const FVector&
	{
		return GetIntermediateU(UIndex, VIndex);
	};

	FGetPointFunc GetIntermediatePointAlongV = [this](int32 VIndex, int32 UIndex) -> const FVector&
	{
		return GetIntermediateV(UIndex, VIndex);
	};

	TFunction<int32(EIso)> GetCoordinateCount = [&](EIso Iso) -> int32
	{
		return Face.GetCrossingPointCoordinates().IsoCount(Iso);
	};

	const int32 IsoUCount = GetCoordinateCount(EIso::IsoU);
	const int32 IsoVCount = GetCoordinateCount(EIso::IsoV);

	ComputeFaceThicknessAlong(EIso::IsoU, GetPointAlongU, GetIntermediatePointAlongU, IsoUCount, IsoVCount);
	ComputeFaceThicknessAlong(EIso::IsoV, GetPointAlongV, GetIntermediatePointAlongV, IsoVCount, IsoUCount);

	Face.SetEstimatedMinimalElementLength(FMath::Min(FaceMinMax[EIso::IsoU].GetMax(), FaceMinMax[EIso::IsoV].GetMax()));
}

bool FCriteriaGrid::CheckIfIsDegenerate()
{
	ensureCADKernel(FaceMinMax.IsValid());
	const double MinFaceThickness = Tolerance3D * 3.;
	return (FaceMinMax[EIso::IsoU].Max < MinFaceThickness) || (FaceMinMax[EIso::IsoV].Max < MinFaceThickness);
}


} // namespace UE::CADKernel

