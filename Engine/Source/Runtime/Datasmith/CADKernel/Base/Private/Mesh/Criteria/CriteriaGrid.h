// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Boundary.h"
#include "Mesh/Structure/GridBase.h"

#include "Geo/Sampling/SurfacicSampling.h"

namespace UE::CADKernel
{
class FTopologicalFace;

class FCriteriaGrid : public FGridBase
{

private:

	/*
	 * Cutting coordinates of the face respecting the meshing criteria
	 */
	FCoordinateGrid CoordinateGrid;

	FSurfacicBoundary FaceMinMax;

protected:

	virtual const FCoordinateGrid& GetCoordinateGrid() const override
	{
		return CoordinateGrid;
	}

	int32 GetIndex(int32 UIndex, int32 VIndex, bool bIsInternalU, bool bIsInternalV) const
	{
		int32 TrueUIndex = UIndex * 2;
		int32 TrueVIndex = VIndex * 2;
		if (bIsInternalU)
		{
			++TrueUIndex;
		}
		if (bIsInternalV)
		{
			++TrueVIndex;
		}
		return TrueVIndex * CuttingCount[EIso::IsoU] + TrueUIndex;
	}

	const FVector& GetPoint(int32 UIndex, int32 VIndex, bool bIsInternalU, bool bIsInternalV) const
	{
		int32 Index = GetIndex(UIndex, VIndex, bIsInternalU, bIsInternalV);
		ensureCADKernel(Points3D.IsValidIndex(Index));
		return Points3D[Index];
	}

	void Init();

	double GetCoordinate(EIso Iso, int32 ind) const
	{
		return CoordinateGrid[Iso][ind];
	}

public:

	FCriteriaGrid(FTopologicalFace& InFace);


	const FVector& GetPoint(int32 iU, int32 iV) const
	{
		return GetPoint(iU, iV, false, false);
	}

	const FVector& GetIntermediateU(int32 iU, int32 iV) const
	{
		return GetPoint(iU, iV, true, false);
	}

	const FVector& GetIntermediateV(int32 iU, int32 iV) const
	{
		return GetPoint(iU, iV, false, true);
	}

	const FVector& GetIntermediateUV(int32 iU, int32 iV) const
	{
		return GetPoint(iU, iV, true, true);
	}

	void ComputeFaceMinMaxThicknessAlongIso();

	double GetCharacteristicThicknessOfFace() const
	{
		return FMath::Max(FaceMinMax[EIso::IsoU].GetMax(), FaceMinMax[EIso::IsoV].GetMax());
	}

	/**
	 * @return true if all the iso lines are smaller than geometric tolerance
	 */
	bool CheckIfIsDegenerate();

#ifdef CADKERNEL_DEV
	void Display() const;
#endif
};
}

