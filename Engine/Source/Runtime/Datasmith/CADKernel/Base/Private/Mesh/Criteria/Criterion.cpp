// Copyright Epic Games, Inc. All Rights Reserved.
#include "Mesh/Criteria/Criterion.h"

#include "Mesh/Criteria/AngleCriterion.h"
#include "Mesh/Criteria/CriteriaGrid.h"
#include "Mesh/Criteria/CurvatureCriterion.h"
#include "Mesh/Criteria/SagCriterion.h"
#include "Mesh/Criteria/SizeCriterion.h"

#include "Geo/GeoPoint.h"
#include "Topo/TopologicalEdge.h"

namespace UE::CADKernel
{

void FCriterion::ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const
{
	TArray<double>& DeltaUMaxs = Edge.GetDeltaUMaxs();
	TArray<double>& DeltaUMins = Edge.GetDeltaUMins();

	double NumericPrecision = Edge.GetTolerance3D();
	NumericPrecision /= 10;

	if (Edge.Length() <= NumericPrecision)
	{
		return;
	}

	int32 PreviousIndex = 0;
	for (int32 Index = 1; Index < Coordinates.Num(); Index++)
	{
		const int32 MiddleIndex = PreviousIndex + Index;
		const double DeltaU = Coordinates[Index] - Coordinates[PreviousIndex];

		const FVector Start = Points[2 * PreviousIndex].Point;
		const FVector End = Points[2 * Index].Point;
		const FVector Middle = Points[MiddleIndex].Point;

		double ChordLength;
		const double SagFromStart = EvaluateSag(Start, End, Middle, ChordLength);
		const double SagFromEnd = EvaluateSag(End, Start, Middle, ChordLength);
		const double Sag = FMath::Max(SagFromStart, SagFromEnd);

		if (ChordLength > NumericPrecision && Sag > DOUBLE_SMALL_NUMBER)
		{
			const double NewDeltaUMax = ComputeDeltaU(ChordLength, DeltaU, Sag);
			UpdateWithUMaxValue(NewDeltaUMax, DeltaUMaxs[PreviousIndex], DeltaUMins[PreviousIndex]);
		}

		PreviousIndex = Index;
	}
}

TSharedPtr<FCriterion> FCriterion::Deserialize(FCADKernelArchive& Archive)
{
	ECriterion CriterionType = ECriterion::None;
	Archive << CriterionType;

	TSharedPtr<FEntity> Entity;
	switch (CriterionType)
	{
	case ECriterion::MinSize:
		return FEntity::MakeShared<FMinSizeCriterion>(Archive);
	case ECriterion::MaxSize:
		return FEntity::MakeShared<FMaxSizeCriterion>(Archive);
	case ECriterion::CADCurvature:
		return FEntity::MakeShared<FCurvatureCriterion>(Archive);
	case ECriterion::Sag:
		return FEntity::MakeShared<FSagCriterion>(Archive);
	case ECriterion::Angle:
		return FEntity::MakeShared<FAngleCriterion>(Archive);
	default:
		return TSharedPtr<FCriterion>();
	}
}

TSharedPtr<FCriterion> FCriterion::CreateCriterion(ECriterion Type, double Value)
{
	switch (Type)
	{
	case ECriterion::MinSize:
		return FEntity::MakeShared<FMinSizeCriterion>(Value);
	case ECriterion::MaxSize:
		return FEntity::MakeShared<FMaxSizeCriterion>(Value);
	case ECriterion::Sag:
		return FEntity::MakeShared<FSagCriterion>(Value);
	case ECriterion::Angle:
		return FEntity::MakeShared<FAngleCriterion>(Value);
	case ECriterion::CADCurvature:
		return FEntity::MakeShared<FCurvatureCriterion>();
	default:
		UE_LOGF(LogCADKernelBase, Warning, "CALL of \"FCriterion::CreateCriterion\" NOT EXPECTED");
		return TSharedPtr<FCriterion>();
	}
}

double FCriterion::DefaultValue(ECriterion Type)
{
	switch (Type)
	{
	case ECriterion::MinSize:
		return FSizeCriterion::DefaultValue(Type);
	case ECriterion::MaxSize:
		return FSizeCriterion::DefaultValue(Type);
	case ECriterion::Sag:
		return FSagCriterion::DefaultValue();
	case ECriterion::Angle:
		return FAngleCriterion::DefaultValue();
	default:
		UE_LOGF(LogCADKernelBase, Warning, "CALL of \"FCriterion::DefaultValue\" NOT EXPECTED");
		return 0.0;
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FCriterion::GetInfo(FInfoEntity& Info) const
{
	FEntity::GetInfo(Info)
		.Add(TEXT("CriterionType"), CriterionTypeNames[(uint8)GetCriterionType()])
		.Add(TEXT("Value"), Value());
	return Info;
}
#endif

const TCHAR* CriterionTypeNames[] = {
	TEXT("Size"),
	TEXT("MaxSize"),
	TEXT("MinSize"),
	TEXT("Angle"),
	TEXT("Chord Error"),
	TEXT("CAD Curvature"),
	TEXT("")
};

// Defined for Python purpose
const char* CriterionTypeConstNames[] = {
	"CRITERION_SIZE",
	"CRITERION_MAX_SIZE",
	"CRITERION_MIN_SIZE",
	"CRITERION_ANGLE",
	"CRITERION_SAG",
	"CRITERION_CAD_CURVATURE",
	nullptr
};

// Defined for Python purpose
const char* CriterionTypeConstDescHelp[] = {
	": Size of elements (only for check)",
	": maximum size of elements",
	": minimum size of elements",
	": angle between the normal of elements",
	": distance between elements and CAD surfaces/curves",
	": CAD curvature (only for check)",
	nullptr
};
}