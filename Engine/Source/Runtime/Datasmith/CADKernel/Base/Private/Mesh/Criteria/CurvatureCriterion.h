// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Curvature.h"
#include "Mesh/Criteria/Criterion.h"

namespace UE::CADKernel
{
struct FCurvePoint;

class FCurvatureCriterion : public FCriterion
{
	friend class FEntity;

protected:

	FCurvatureCriterion() = default;

public:

	void Serialize(FCADKernelArchive& Ar)
	{
		FCriterion::Serialize(Ar);
	}

	/**
	 * Sag & Angle criterion.pdf
	 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	 */
	virtual void UpdateDelta(double InDeltaU, double InUSag, double InDiagonalSag, double InVSag, double ChordLength, double DiagonalLength, double& OutSagDeltaUMax, double& OutSagDeltaUMin, FIsoCurvature& IsoCurvature) const override
	{
		if (DiagonalLength > DOUBLE_SMALL_NUMBER)
		{
			double Curvature = 8 * InUSag / FMath::Square(DiagonalLength);
			if (IsoCurvature.Max < Curvature)
			{
				IsoCurvature.Max = Curvature;
			}
			if (IsoCurvature.Min > Curvature)
			{
				IsoCurvature.Min = Curvature;
			}
		}
	}

	virtual double Value() const override
	{
		return 0.;
	}

	virtual void ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const override
	{
		// Do nothing
	}

	virtual ECriterion GetCriterionType() const override
	{
		return ECriterion::CADCurvature;
	}


protected:

	/**
	 * Sag & Angle criterion.pdf
	 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	 */
	virtual bool IsAppliedBetweenBreaks() const override
	{
		return true;
	}

	virtual double ComputeDeltaU(double ChordLength, double DeltaU, double Sag) const override
	{
		return 0.0;
	};

};

} // namespace UE::CADKernel

