// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Mesh/Criteria/Criterion.h"

namespace UE::CADKernel
{

class FSagCriterion : public FCriterion
{
	friend class FEntity;

protected:
	double MaxSag;

	FSagCriterion(double InSag = 0.2)
		: MaxSag(InSag)
	{
	}

public:

	void Serialize(FCADKernelArchive& Ar)
	{
		FCriterion::Serialize(Ar);
		Ar << MaxSag;
	}

	virtual double Value() const override
	{
		return MaxSag;
	}

	static double DefaultValue()
	{
		return 0.2;
	}

	virtual ECriterion GetCriterionType() const override
	{
		return ECriterion::Sag;
	}


protected:

	/**
	 * Sag & Angle criterion.pdf
	 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	*/
	virtual double ComputeDeltaU(double ChordLength, double DeltaU, double Sag) const override
	{
		return sqrt(MaxSag / Sag) * DeltaU;
	}

	virtual bool IsAppliedBetweenBreaks() const override
	{
		return true;
	}
};

} // namespace UE::CADKernel

