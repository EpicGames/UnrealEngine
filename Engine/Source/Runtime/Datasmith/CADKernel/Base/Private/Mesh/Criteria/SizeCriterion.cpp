// Copyright Epic Games, Inc. All Rights Reserved.
#include "Mesh/Criteria/SizeCriterion.h"

#include "Geo/GeoPoint.h"
#include "Topo/TopologicalEdge.h"

namespace UE::CADKernel
{

/**
 * In some cases as a huge plan, the MaxSizeCriteria can generate a lot of triangles e.g. a 100m side plan with a MaxSizeCriteria of 3cm will need 3e3 elements by side so 2e7 triangles.
 * This kind of case is most of the time a forgotten sketch body than a real wanted body. This mesh could make the process extremely long or simply crash all the system.
 * The idea is to not cancel the mesh of the body in the case it will really expected but to avoid the generation of a huge mesh with unwanted hundreds of millions of triangles.
 * So if MaxSizeCriteria will generate a huge mesh, this criteria is abandoned.
 * The chosen limit value is 3000 elements by side
 */
constexpr int32 GetMaxElementCountPerSide()
{
	return 3000;
}

void FMinSizeCriterion::ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const
{
	double NumericPrecision = Edge.GetTolerance3D();
	if (Edge.Length() <= NumericPrecision)
	{
		return;
	}

	ApplyOnParameters(Coordinates, Points, Edge.GetDeltaUMaxs(), Edge.GetDeltaUMins(), [&](double NewMaxValue, double& OutDeltaUMax, double& OutDeltaUMin)
		{
			UpdateWithUMinValue(NewMaxValue, OutDeltaUMax, OutDeltaUMin);
		});
}


void FMaxSizeCriterion::ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const
{
	double NumericPrecision = Edge.GetTolerance3D();
	if (Edge.Length() <= NumericPrecision)
	{
		return;
	}

	const double ElementCount = Edge.Length() / Size;
	if (ElementCount > GetMaxElementCountPerSide())
	{
		return;
	}

	ApplyOnParameters(Coordinates, Points, Edge.GetDeltaUMaxs(), Edge.GetDeltaUMins(), [&](double NewMaxValue, double& OutDeltaUMax, double& OutDeltaUMin)
		{
			UpdateWithUMaxValue(NewMaxValue, OutDeltaUMax, OutDeltaUMin);
		});
}

void FSizeCriterion::ApplyOnParameters(const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points, TArray<double>& DeltaUMaxs, TArray<double>& DeltaUMins, TFunction<void(double, double&, double&)> UpdateDeltaU) const
{
	for (int32 Index = 1; Index < Coordinates.Num(); Index++)
	{
		const int32 PreviousIndex = Index - 1;
		const double DeltaCoordinate = Coordinates[Index] - Coordinates[PreviousIndex];
		const double ChordLength = FVector::Distance(Points[2 * PreviousIndex].Point, Points[2 * Index].Point);

		const double NewDeltaU = ComputeSizeCriterionValue(DeltaCoordinate, ChordLength);
		UpdateDeltaU(NewDeltaU, DeltaUMaxs[Index - 1], DeltaUMins[Index - 1]);
	}
}

void FMinSizeCriterion::UpdateDelta(double InDeltaCoordinate, double InUSag, double InDiagonalSag, double InVSag, double ChordLength, double DiagonalLength, double& OutSagDeltaUMax, double& OutSagDeltaUMin, FIsoCurvature& SurfaceCurvature) const
{
	if (ChordLength < DOUBLE_KINDA_SMALL_NUMBER)
	{
		return;
	}

	const double NewDeltaU = ComputeSizeCriterionValue(InDeltaCoordinate, ChordLength);
	UpdateWithUMinValue(NewDeltaU, OutSagDeltaUMax, OutSagDeltaUMin);
}

void FMaxSizeCriterion::UpdateDelta(double InDeltaCoordinate, double InUSag, double InDiagonalSag, double InVSag, double ChordLength, double DiagonalLength, double& OutSagDeltaUMax, double& OutSagDeltaUMin, FIsoCurvature& SurfaceCurvature) const
{
	if (ChordLength < DOUBLE_KINDA_SMALL_NUMBER)
	{
		return;
	}

	const double ElementCount = ChordLength / Size;
	if (ElementCount > GetMaxElementCountPerSide())
	{
		return;
	}

	const double NewDeltaU = ComputeSizeCriterionValue(InDeltaCoordinate, ChordLength);
	UpdateWithUMaxValue(NewDeltaU, OutSagDeltaUMax, OutSagDeltaUMin);
}

} // namespace UE::CADKernel
