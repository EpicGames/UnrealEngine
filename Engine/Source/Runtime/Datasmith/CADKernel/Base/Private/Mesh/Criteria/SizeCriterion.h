// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Mesh/Criteria/Criterion.h"

/**
 * Sag & Angle criterion.pdf
 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
*/

namespace UE::CADKernel
{
struct FCurvePoint;

class FSizeCriterion : public FCriterion
{
	friend class FEntity;

protected:
	double Size;

	FSizeCriterion(double InSize = 1.)
		: Size(InSize)
	{
	}

	double ComputeSizeCriterionValue(double InDeltaU, double ChordLength) const
	{
		return  (ChordLength > DOUBLE_KINDA_SMALL_NUMBER) ? InDeltaU * Size / ChordLength : DOUBLE_BIG_NUMBER;
	}

public:

	void Serialize(FCADKernelArchive& Ar)
	{
		FCriterion::Serialize(Ar);
		Ar << Size;
	}

	double Value() const override
	{
		return Size;
	}

	static double DefaultValue(ECriterion Type)
	{
		switch (Type)
		{
		case ECriterion::MinSize:
			return 0.1;
		case ECriterion::MaxSize:
			return 30;
		}
		return 0;
	}

	void ApplyOnParameters(const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points, TArray<double>& DeltaUMax, TArray<double>& DeltaUMins, TFunction<void(double, double&, double&)> UpdateDeltaU) const;
};

class FMinSizeCriterion : public FSizeCriterion
{
	friend class FEntity;
protected:
	FMinSizeCriterion(double InSize = 0.05)
		: FSizeCriterion(InSize)
	{
	}

public:

	virtual ECriterion GetCriterionType() const override
	{
		return ECriterion::MinSize;
	}

	virtual void ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const override;

	/**
	 * MinSizeCrtierion sets DeltaUMin.
	 * @see FCriterion::UpdateWithUMinValue
	 */
	virtual void UpdateDelta(double InDeltaU, double InUSag, double InDiagonalSag, double InVSag, double ChordLength, double DiagonalLength, double& OutSagDeltaUMax, double& OutSagDeltaUMin, FIsoCurvature& SurfaceCurvature) const override;
};

class FMaxSizeCriterion : public FSizeCriterion
{
	friend class FEntity;
protected:
	FMaxSizeCriterion(double InSize = 10000.)
		: FSizeCriterion(InSize)
	{
	}

public:
	virtual ECriterion GetCriterionType() const override
	{
		return ECriterion::MaxSize;
	}

	virtual void ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const override;

	/**
	 * MaxSizeCriterion sets DeltaUMax.
	 * @see FCriterion::UpdateWithUMaxValue
	 */
	virtual void UpdateDelta(double InDeltaU, double InUSag, double InDiagonalSag, double InVSag, double ChordLength, double DiagonalLength, double& OutSagDeltaUMax, double& OutSagDeltaUMin, FIsoCurvature& SurfaceCurvature) const override;
};

} // namespace UE::CADKernel

