// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/SlopeUtils.h"
#include "Mesh/Meshers/IsoTriangulator/IsoCell.h"

namespace UE::CADKernel
{

class FBowyerWatsonTriangulator
{
protected:
	struct FTriangle
	{
		int32 VertexIndices[3];
		double SquareRadius;
		FVector Center;

		FTriangle(const int32& Index0, const int32& Index1, const int32& Index2, const TArray<TPair<int32, FVector2d>>& InVertices)
		{
			Set(Index0, Index1, Index2, InVertices);
		}

		void Set(const int32& Index0, const int32& Index1, const int32& Index2, const TArray<TPair<int32, FVector2d>>& InVertices)
		{
			VertexIndices[0] = Index0;
			VertexIndices[1] = Index1;
			VertexIndices[2] = Index2;
			FTriangle2D Triangle(InVertices[Index0].Value, InVertices[Index1].Value, InVertices[Index2].Value);
			Center = FVector(Triangle.CircumCircleCenterWithSquareRadius(SquareRadius), 0.);
		}
	};


public:

	/**
	 * @param Vertices the 2d point cloud to mesh
	 * @param OutEdgeVertices, the edges of the mesh. An edge is defined by the indices of its vertices
	 * So the ith edge is defined by the vertices EdgeVertexIndices[2 * ith] and EdgeVertexIndices[2 * ith + 1]
	 */
	FBowyerWatsonTriangulator(TArray<TPair<int32, FVector2d>>& InVertices, TArray<int32>& OutEdgeVertices)
		: VerticesCount(InVertices.Num())
		, Vertices(InVertices)
		, EdgeVertexIndices(OutEdgeVertices)
	{
		Init();
	}

	void Triangulate()
	{
		Vertices.Sort([](const TPair<int32, FVector2d>& Vertex1, const TPair<int32, FVector2d>& Vertex2) {return (Vertex1.Value.X + Vertex1.Value.Y) > (Vertex2.Value.X + Vertex2.Value.Y); });

		// initialization of Bowyer & Watson algorithm with a bounding mesh of the vertex cloud 
		// i.e. 2 triangles defined by the corners of the offset vertices bounding box
		MakeBoundingMesh();

		EdgeVertexIndices.Reserve(TriangleSet.Num() * 6);

		// insert each point in the mesh
		// The points are sorted on the diagonal of the bbox and are inserted from one end to the other  
		int32 VertexIndex = 0;
		for (int32 VIndex = 0; VIndex < VerticesCount; ++VIndex)
		{
			if (VIndex % 2 == 0)
			{
				VertexIndex = VIndex / 2;
			}
			else
			{
				VertexIndex = VerticesCount - 1 - VIndex / 2;
			}

			SelectedTriangleIndices.Reset(VerticesCount);
			AdditionalTriangleIndices.Reset(VerticesCount);

			const FVector2d& NewVertex = Vertices[VertexIndex].Value;

			bool bHaveDoubt = false;
			
			// find all triangles whose circumcircles contain the new vertex
			{
				// The problem is for triangle with huge circumscribed circle radius (flat triangle)
				// in this case, if distance between the new vertex and the circumscribed circle center nearly equals its radius
				//  - So the idea is to check that the new vertex is not outside the triangle and could generate a flatten triangle 
				//    To check this, the slop between each side is evaluated (ComputeUnorientedSlope(NewVertex, Triangle.Vertex[a], Triangle.Vertex[b]))
				//    if the slop is nearly null this mean that the new vertex is outside the triangle and will generate a flat triangle

				constexpr double IncreaseFactor = 1.001; // to process the case of new vertex on the circumscribed circle
				constexpr double ReducingFactor = 0.999 / IncreaseFactor; // to remove the case of new vertex on the circumscribed circle 
				constexpr double LargeReducingFactor = 0.99; // to remove the case of new vertex on the circumscribed circle 
				// the case of 4 points nearly on the same circle is manage in a second time i.e. we check that the 

				for (int32 TriangleIndex = 0; TriangleIndex < TriangleSet.Num(); TriangleIndex++)
				{
					const FVector2d Center(TriangleSet[TriangleIndex].Center.X, TriangleSet[TriangleIndex].Center.Y);

					const double SquareDistanceToCenter = FVector2d::DistSquared(Center, NewVertex);

					const double SquareRadiusMax = TriangleSet[TriangleIndex].SquareRadius * IncreaseFactor;
					const double SquareRadiusMin = SquareRadiusMax * ReducingFactor;
					const double SquareRadiusMinMin = SquareRadiusMin * LargeReducingFactor;

					if (SquareDistanceToCenter < SquareRadiusMin)
					{
						SelectedTriangleIndices.Add(TriangleIndex);
						if (SquareDistanceToCenter > SquareRadiusMinMin)
						{
							bHaveDoubt = true;
						}
					}
					else
					{
						if (SquareDistanceToCenter < SquareRadiusMax)
						{
							if (DoesTriangleContainingNewVertex(NewVertex, TriangleIndex))
							{
								SelectedTriangleIndices.Add(TriangleIndex);
							}
							else
							{
								AdditionalTriangleIndices.Add(TriangleIndex);
							}
						}
					}
				}
			}

			if (!bHaveDoubt)
			{
				bHaveDoubt = !DoSelectedTrianglesFormSinglePartition();
			}

			if (SelectedTriangleIndices.Num() == 0)
			{
				for (int32& TriangleIndex : AdditionalTriangleIndices)
				{
					if (DoesTriangleContainingNewVertex(NewVertex, TriangleIndex))
					{
						SelectedTriangleIndices.Add(TriangleIndex);
						TriangleIndex = -1;
					}
				}
			}
			else if(bHaveDoubt)
			{
				TArray<int32> TmpTriangleIndices;
				TmpTriangleIndices = MoveTemp(SelectedTriangleIndices);
				SelectedTriangleIndices.Reset(TmpTriangleIndices.Num());

				AddTrianglesContainingNewVertexToSelectedTriangles(NewVertex, TmpTriangleIndices);

				AddConnectedTrianglesToSelection(TmpTriangleIndices);

				for (int32& TriangleIndex : TmpTriangleIndices)
				{
					if (TriangleIndex >= 0)
					{
 						AdditionalTriangleIndices.Add(TriangleIndex);
					}
				}
			}

			if (AdditionalTriangleIndices.Num() > 0)
			{
				// Additional triangles are triangles that the new vertex is nearly coincident to the circle.
				// in this case, to select the triangle, the analyze is made with the dual space of the Delaunay triangulation: the Voronoi diagram.
				// https://docs.google.com/presentation/d/1Hr5jvLH8tm4KqXgmT-Di-2kFZYOT7MB246hhPKUk6Cw/edit?usp=sharing

				// Boundary vertex of selected triangles
				TArray<int32> BoundaryVertex;
				BoundaryVertex.Reserve(SelectedTriangleIndices.Num() + AdditionalTriangleIndices.Num() + 3);
				for (int32 TriangleIndex : SelectedTriangleIndices)
				{
					for (int32 Index = 0; Index < 3; Index++)
					{
						const int32 StartVertex = TriangleSet[TriangleIndex].VertexIndices[Index];
						BoundaryVertex.AddUnique(StartVertex);
					}
				}

				TArray<TPair<int32, double>> BoundaryVertexToSlope;
				for (int32 Index : BoundaryVertex)
				{
					FVector2d Vertex = Vertices[Index].Value;
					double Slope = ComputeSlope(NewVertex, Vertex);
					BoundaryVertexToSlope.Emplace(Index, Slope);
				}

				// Sort the vertex to make the boundary polygon
				BoundaryVertexToSlope.Sort([](const TPair<int32, double>& Vertex1, const TPair<int32, double>& Vertex2) {return Vertex1.Value < Vertex2.Value; });

				bool bTriangleHasBeenAdded = true;
				while (bTriangleHasBeenAdded)
				{
					bTriangleHasBeenAdded = false;
					for (int32& TriangleIndex : AdditionalTriangleIndices)
					{
						if (TriangleIndex < 0)
						{
							continue;
						}

						FTriangle& CandidateTriangle = TriangleSet[TriangleIndex];

						// if the CandidateTriangle is connected by an edge to the boundary polygon
						int32 CandidateVertexIndex = -1;
						int32 ConnectedVertex = 0;
						for (int32 Index = 0; Index < 3; Index++)
						{
							int32 Candidate = CandidateTriangle.VertexIndices[Index];
							if (BoundaryVertex.Find(Candidate) == INDEX_NONE)
							{
								CandidateVertexIndex = Candidate;
							}
							else
							{
								ConnectedVertex++;
							}
						}
						if (ConnectedVertex != 2)
						{
							continue;
						}

						FVector2d CandidateVertex = Vertices[CandidateVertexIndex].Value;

						double NewVertexSlope = ComputeSlope(NewVertex, CandidateVertex);

						int32 StartIndex = -1;
						int32 EndIndex = 0;
						for (int32 ApexIndex = 0; ApexIndex < BoundaryVertexToSlope.Num(); ++ApexIndex)
						{
							if (NewVertexSlope < BoundaryVertexToSlope[ApexIndex].Value)
							{
								EndIndex = ApexIndex;
								break;
							}
						}

						StartIndex = EndIndex == 0 ? BoundaryVertexToSlope.Num() - 1 : EndIndex - 1;

						FVector2d StartPoint = Vertices[BoundaryVertexToSlope[StartIndex].Key].Value;
						double SlopeNewStartPlusPi = ComputeSlope(NewVertex, StartPoint) + Slope::RightSlope;
						double SlopeStartCandidate = ComputePositiveSlope(StartPoint, CandidateVertex, SlopeNewStartPlusPi);

						FVector2d EndPoint = Vertices[BoundaryVertexToSlope[EndIndex].Key].Value;
						double SlopeNewEndMinusPi = ComputeSlope(NewVertex, EndPoint) - Slope::RightSlope;
						double SlopeEndCandidate = ComputePositiveSlope(EndPoint, CandidateVertex, SlopeNewEndMinusPi);

						if (SlopeStartCandidate > 0 && SlopeStartCandidate < Slope::RightSlope && SlopeEndCandidate > Slope::ThreeRightSlope && SlopeEndCandidate < Slope::TwoPiSlope)
						{
							bTriangleHasBeenAdded = true;
							if (EndIndex == 0 && NewVertexSlope > BoundaryVertexToSlope.Last().Value)
							{
								BoundaryVertexToSlope.Emplace(CandidateVertexIndex, NewVertexSlope);
							}
							else
							{
								BoundaryVertexToSlope.EmplaceAt(EndIndex, CandidateVertexIndex, NewVertexSlope);
							}
							BoundaryVertex.Add(CandidateVertexIndex);
							SelectedTriangleIndices.Add(TriangleIndex);
						}
						TriangleIndex = -1;
					}
				}
			}

			EdgeVertexIndices.Reset(VerticesCount);
 			FindBoundaryEdgesOfSelectedTriangles(EdgeVertexIndices);

			// make the new triangles : Each new triangle is defined by an edge of the boundary and the new vertex 
			{
				// The deleted triangles are replaced by the new ones
				int32 EdgeIndex = 0;
				for (int32 TriangleIndex : SelectedTriangleIndices)
				{
					while (EdgeVertexIndices[EdgeIndex] < 0)
					{
						EdgeIndex += 2;
					}
					TriangleSet[TriangleIndex].Set(EdgeVertexIndices[EdgeIndex + 1], EdgeVertexIndices[EdgeIndex], VertexIndex, Vertices);

					EdgeIndex += 2;
				}
				// When all deleted triangles are replaced, the new triangles are added in the array
				for (; EdgeIndex < EdgeVertexIndices.Num(); EdgeIndex += 2)
				{
					if (EdgeVertexIndices[EdgeIndex] < 0)
					{
						continue;
					}
					TriangleSet.Emplace(EdgeVertexIndices[EdgeIndex + 1], EdgeVertexIndices[EdgeIndex], VertexIndex, Vertices);
				}
			}
		}

		// Find all Edges and their type (inner edge or boundary edge)
		EdgeVertexIndices.Reset(TriangleSet.Num() * 6);

		for (int32 TriangleIndex = 0; TriangleIndex < TriangleSet.Num(); TriangleIndex++)
		{
			int32 Index = 0;
			for (; Index < 3; Index++)
			{
				if (TriangleSet[TriangleIndex].VertexIndices[Index] >= VerticesCount)
				{
					// one of the point is a corner of the bounding mesh
					// At least, only one edge is added and this edge is necessarily an outer edge
					break;
				}
			}
			bool bIsOuter = Index < 3;

			int32 EndVertex = TriangleSet[TriangleIndex].VertexIndices[2];
			for (Index = 0; Index < 3; Index++)
			{
				int32 StartVertex = TriangleSet[TriangleIndex].VertexIndices[Index];
				if (StartVertex < VerticesCount && EndVertex < VerticesCount)
				{
					int32 Endex = 0;
					for (; Endex < EdgeVertexIndices.Num(); Endex += 2)
					{
						// Does the edge exist
						if (EdgeVertexIndices[Endex] == EndVertex && EdgeVertexIndices[Endex + 1] == StartVertex)
						{
							if (!bIsOuter)
							{
								EdgeInstanceCount[Endex / 2]++;
							}
							break;
						}
					}

					if (Endex == EdgeVertexIndices.Num())
					{
						// No
						EdgeVertexIndices.Add(StartVertex);
						EdgeVertexIndices.Add(EndVertex);
						EdgeInstanceCount.Add(bIsOuter ? 0 : 1);
					}
				}
				EndVertex = StartVertex;
			}
		}

		// the bounding mesh vertices are removed
		Vertices.SetNum(VerticesCount);

		for (int32& Indice : EdgeVertexIndices)
		{
			Indice = Vertices[Indice].Key;
		}
	}

	int32 OuterEdgeCount() const
	{
		int32 EdgeCount = 0;
		for (int32 Index = 0; Index < EdgeVertexIndices.Num() / 2; ++Index)
		{
			if (EdgeInstanceCount[Index] < 2)
			{
				EdgeCount++;
			}
		}
		return EdgeCount;
	}

	/**
	 * Return the edge connected to 0 or 1 triangle
	 */
	void GetOuterEdges(TArray<int32>& OuterEdgeIndices) const
	{
		int32 EdgeCount = OuterEdgeCount();
		OuterEdgeIndices.Reserve(EdgeCount);
		for (int32 Index = 0, EdgeIndex = 0; Index < EdgeVertexIndices.Num(); ++EdgeIndex)
		{
			if (EdgeInstanceCount[EdgeIndex] < 2)
			{
				OuterEdgeIndices.Add(EdgeVertexIndices[Index++]);
				OuterEdgeIndices.Add(EdgeVertexIndices[Index++]);
			}
			else
			{
				Index += 2;
			}
		}
	}

	void GetOuterVertices(TSet<int32>& OuterVertexIndices) const
	{
		int32 EdgeCount = OuterEdgeCount();
		OuterVertexIndices.Reserve(EdgeCount);
		for (int32 Index = 0, EdgeIndex = 0; Index < EdgeVertexIndices.Num(); ++EdgeIndex)
		{
			if (EdgeInstanceCount[EdgeIndex] < 2)
			{
				OuterVertexIndices.Add(EdgeVertexIndices[Index++]);
				OuterVertexIndices.Add(EdgeVertexIndices[Index++]);
			}
			else
			{
				Index += 2;
			}
		}
	}

	TArray<int32> GetOuterVertices() const
	{
		TSet<int32> OuterVertices;
		GetOuterVertices(OuterVertices);
		return OuterVertices.Array();
	}

	void GetMesh(TArray<int32>& Triangles)
	{
		Triangles.Reset(TriangleSet.Num() * 3);
		for (const FTriangle& Triangle : TriangleSet)
		{
			Triangles.Append(Triangle.VertexIndices, 3);
		}
	}

#ifdef DEBUG_BOWYERWATSON
	static bool bDisplay;
#endif

private:
	int32 VerticesCount;

	TArray<TPair<int32, FVector2d>>& Vertices;

	/**
	 * An edge is defined by the indices of its vertices
	 * So the ith edge is defined by the vertices EdgeVertexIndices[2 * ith] and EdgeVertexIndices[2 * ith + 1]
	 */
	TArray<int32>& EdgeVertexIndices;

	TArray<FTriangle> TriangleSet;

	// It's use to mark all triangles whose circumcircles contain the next vertex
	TArray<int32> SelectedTriangleIndices;
	TArray<int32> AdditionalTriangleIndices;

	/**
	 * Use to determine if the edge is a border edge of inner edge
	 * If EdgeInstanceCount[ith] = 2, the edge is a inner edge
	 */
	TArray<int32> EdgeInstanceCount;

	void MakeBoundingMesh()
	{
		FAABB2D VerticesBBox;
		for (const TPair<int32, FVector2d>& Vertex : Vertices)
		{
			VerticesBBox += Vertex.Value;
		}

		const double DiagonalLength = VerticesBBox.DiagonalLength();
		VerticesBBox.Offset(DiagonalLength);

		int32 VerticesId = Vertices.Num();
		Vertices.Emplace(VerticesId++, VerticesBBox.GetCorner(3));
		Vertices.Emplace(VerticesId++, VerticesBBox.GetCorner(2));
		Vertices.Emplace(VerticesId++, VerticesBBox.GetCorner(0));
		Vertices.Emplace(VerticesId++, VerticesBBox.GetCorner(1));

		TriangleSet.Emplace(VerticesCount, VerticesCount + 1, VerticesCount + 2, Vertices);
		TriangleSet.Emplace(VerticesCount + 2, VerticesCount + 3, VerticesCount, Vertices);
	}

	void Init()
	{
		VerticesCount = Vertices.Num();
		Vertices.Reserve(VerticesCount + 4);
		TriangleSet.Reserve(VerticesCount);
		SelectedTriangleIndices.Reserve(VerticesCount);
		AdditionalTriangleIndices.Reserve(VerticesCount);
		EdgeVertexIndices.Reserve(4 * VerticesCount);
		EdgeInstanceCount.Reserve(2 * VerticesCount);
	}

	// Find the boundary edges of the selected triangles:
	// For all selected triangles, 
	//    For each triangle edges
	//       if the edge is not in EdgeVertexIndices: Add the edge i.e. add its vertex indices
	//       else (the edge is in EdgeVertexIndices), remove the edge of EdgeVertexIndices
	// As the triangles are oriented, the edge AB of a triangle is the edge BA of the adjacent triangle
	void FindBoundaryEdgesOfSelectedTriangles(TArray<int32>& BoundaryEdgeVertexIndices)
	{
		BoundaryEdgeVertexIndices.Reset(SelectedTriangleIndices.Num() * 6);
		for (int32 TriangleIndex : SelectedTriangleIndices)
		{
			int32 EndVertex = TriangleSet[TriangleIndex].VertexIndices[2];
			for (int32 Index = 0; Index < 3; Index++)
			{
				int32 StartVertex = TriangleSet[TriangleIndex].VertexIndices[Index];
				int32 Endex = 0;
				// Does the edge exist
				for (; Endex < BoundaryEdgeVertexIndices.Num(); Endex += 2)
				{
					if (BoundaryEdgeVertexIndices[Endex] == EndVertex && BoundaryEdgeVertexIndices[Endex + 1] == StartVertex)
					{
						BoundaryEdgeVertexIndices[Endex] = -1;
						BoundaryEdgeVertexIndices[Endex + 1] = -1;
						break;
					}
				}
				if (Endex == BoundaryEdgeVertexIndices.Num())
				{   // No
					BoundaryEdgeVertexIndices.Add(StartVertex);
					BoundaryEdgeVertexIndices.Add(EndVertex);
				}
				EndVertex = StartVertex;
			}
		}
	}

	bool DoSelectedTrianglesFormSinglePartition()
	{
		FindBoundaryEdgesOfSelectedTriangles(EdgeVertexIndices);

		int32 StartIndex = -1;
		int32 LastIndex = -1;
		for (int32 EdgeIndex = 0; EdgeIndex < EdgeVertexIndices.Num(); EdgeIndex += 2)
		{
			if (EdgeVertexIndices[EdgeIndex] < 0)
			{
				continue;
			}
			StartIndex = EdgeVertexIndices[EdgeIndex];
			LastIndex = EdgeVertexIndices[EdgeIndex + 1];
			EdgeVertexIndices[EdgeIndex] = -1;
			EdgeVertexIndices[EdgeIndex + 1] = -1;
			break;
		}

		const int32 IndiceCount = EdgeVertexIndices.Num();
		while (LastIndex != StartIndex)
		{
			int32 EdgeIndex = 0;
			for (; EdgeIndex < IndiceCount; EdgeIndex += 2)
			{
				if (EdgeVertexIndices[EdgeIndex] == LastIndex)
				{
					LastIndex = EdgeVertexIndices[EdgeIndex + 1];
					EdgeVertexIndices[EdgeIndex] = -1;
					EdgeVertexIndices[EdgeIndex + 1] = -1;
				}
			}
			if (EdgeIndex == IndiceCount)
			{
				return false;
			}
		}

		// All edges are selected ?
		bool bAllEdgesAreSelected = true;
		for (int32 EdgeIndex = 0; EdgeIndex < EdgeVertexIndices.Num(); EdgeIndex += 2)
		{
			if (EdgeVertexIndices[EdgeIndex] > 0)
			{
				return false;
			}
		}

		return true;
	}

	bool DoesTriangleContainingNewVertex(const FVector2d& NewVertex, int32 CandiadateTriangle)
	{
		const FVector2d& Point0 = Vertices[TriangleSet[CandiadateTriangle].VertexIndices[0]].Value;
		const FVector2d& Point1 = Vertices[TriangleSet[CandiadateTriangle].VertexIndices[1]].Value;
		const FVector2d& Point2 = Vertices[TriangleSet[CandiadateTriangle].VertexIndices[2]].Value;
		double Slope0 = ComputePositiveSlope(Point0, Point1, NewVertex);
		double Slope1 = ComputePositiveSlope(Point1, Point2, NewVertex);
		double Slope2 = ComputePositiveSlope(Point2, Point0, NewVertex);
		if (Slope0 < Slope::PiSlope && Slope1 < Slope::PiSlope && Slope2 < Slope::PiSlope)
		{
			return true;
		}
		return false;
	}

	void AddTrianglesContainingNewVertexToSelectedTriangles(const FVector2d& NewVertex, TArray<int32>& CandiadateTriangles)
	{
		for (int32& TriangleIndex : CandiadateTriangles)
		{
			if(DoesTriangleContainingNewVertex(NewVertex, TriangleIndex))
			{
				SelectedTriangleIndices.Add(TriangleIndex);
				TriangleIndex = -1;
			}
		}
	}

	void AddConnectedTrianglesToSelection(TArray<int32>& CandiadateTriangles)
	{
		EdgeVertexIndices.Reset((SelectedTriangleIndices.Num() + CandiadateTriangles.Num()) * 6);
		FindBoundaryEdgesOfSelectedTriangles(EdgeVertexIndices);

		bool bTriangleHasBeenAdded = true;
		while (bTriangleHasBeenAdded)
		{
			bTriangleHasBeenAdded = false;
			for (int32& TriangleIndex : CandiadateTriangles)
			{
				if (TriangleIndex < 0)
				{
					continue;
				}

				int32 EndVertex = TriangleSet[TriangleIndex].VertexIndices[2];

				bool bEdgeIsFound = false;
				int32 Index = 0;
				for (; Index < 3; Index++)
				{
					int32 StartVertex = TriangleSet[TriangleIndex].VertexIndices[Index];

					for (int32 Endex = 0; Endex < EdgeVertexIndices.Num(); Endex += 2)
					{
						if (EdgeVertexIndices[Endex] == EndVertex && EdgeVertexIndices[Endex + 1] == StartVertex)
						{
							EdgeVertexIndices[Endex] = -1;
							EdgeVertexIndices[Endex + 1] = -1;
							bEdgeIsFound = true;
							break;
						}
					}
					if (bEdgeIsFound)
					{
						break;
					}
					EndVertex = StartVertex;
				}

				if (bEdgeIsFound)
				{
					SelectedTriangleIndices.Add(TriangleIndex);
					EndVertex = TriangleSet[TriangleIndex].VertexIndices[2];
					for (int32 Endex = 0; Endex < 3; Endex++)
					{
						int32 StartVertex = TriangleSet[TriangleIndex].VertexIndices[Endex];
						if (Endex != Index)
						{
							EdgeVertexIndices.Add(StartVertex);
							EdgeVertexIndices.Add(EndVertex);
						}
						EndVertex = StartVertex;
					}
					bTriangleHasBeenAdded = true;
					TriangleIndex = -1;
				}
			}
		}
	}
};

}