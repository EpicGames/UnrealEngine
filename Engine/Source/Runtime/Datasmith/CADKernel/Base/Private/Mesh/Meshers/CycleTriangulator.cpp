// Copyright Epic Games, Inc. All Rights Reserved.
#include "Mesh/Meshers/CycleTriangulator.h"

#include "Core/Factory.h"
#include "Math/SlopeUtils.h"
#include "Mesh/Meshers/IsoTriangulator.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionSegmentTool.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Topo/TopologicalFace.h"

namespace UE::CADKernel
{

FCycleTriangulator::FCycleTriangulator(FIsoTriangulator& IsoTriangulator, const TArray<FIsoSegment*>& InCycle, const TArray<bool>& InCycleOrientation)
	: Grid(IsoTriangulator.Grid)
	, Cycle(InCycle)
	, CycleOrientation(InCycleOrientation)
	, InnerToOuterIsoSegmentsIntersectionTool(IsoTriangulator.InnerToOuterIsoSegmentsIntersectionTool)
	, Mesh(IsoTriangulator.Mesh)
	, IsoSegmentFactory(IsoTriangulator.IsoSegmentFactory)
	, CycleIntersectionTool(Grid, IsoTriangulator.Tolerances.GeometricTolerance)
{
}

bool FCycleTriangulator::MeshCycle()
{
#ifdef ADD_TRIANGLE_2D
	static int32 CycleId = 0;
	F3DDebugSession _(Grid.bDisplay, FString::Printf(TEXT("Mesh cycle %d"), ++CycleId));
	if (Grid.bDisplay)
	{
		Grid.DisplayIsoSegments(TEXT("Cycle"), EGridSpace::UniformScaled, Cycle);
		Wait(CycleId == 0);
	}
#endif

	// Check if the cycle is in self intersecting and fix it. 
	if (!CanCycleBeMeshed())
	{
		return false;
	}

	const double SquareMinSize = FMath::Square(Grid.GetMinElementSize());

	InitializeArrays();

	// Get cycle's nodes and set segments as they have a triangle outside the cycle (to don't try to mesh outside the cycle)
	InitializeCycleForMeshing();

	FillSegmentStack();

	while (true)
	{
		for (int32 Index = 0; Index < SegmentStack.Num(); ++Index)
		{
			FIsoSegment* Segment = SegmentStack[Index];
			if (Segment->HasntTriangle())
			{
				continue;
			}

			if (!Segment->IsDegenerated())
			{
				while (!Segment->HasTriangleOnLeft() && BuildTheBestPolygon(Segment, true));
				while (!Segment->HasTriangleOnRight() && BuildTheBestPolygon(Segment, false));
			}

			// if the segment failed to be meshed
			if (!Segment->HasTriangleOnLeft() || !Segment->HasTriangleOnRight())
			{
				UnmeshedSegment.Emplace(Segment);
			}
		}

		if (bFirstRun && UnmeshedSegment.Num())
		{
			// In case of incomplete cycle meshing, remove degenerated flags and rerun the process
			SegmentStack = MoveTemp(UnmeshedSegment);
			for (FIsoSegment* Segment : SegmentStack)
			{
				Segment->ResetDegenerated();
			}
			bFirstRun = false;
			continue;
		}
		break;
	}

	// Reset the flags "has triangle" of cycle's segments to avoid to block the meshing of next cycles
	for (FIsoSegment* Segment : Cycle)
	{
		Segment->ResetHasTriangle();
	}

	// Reset the flags "has triangle" of cycle's segments to avoid to block the meshing of next cycles
	for (FIsoSegment* Segment : Cycle)
	{
		Segment->ResetHasTriangle();
	}
	return true;
}

bool FCycleTriangulator::FindTheCycleToMesh(FIsoSegment* Segment, bool bOrientation, int32& StartIndexForMinLength)
{
	//const SlopeMethod GetSlopeAtEndNode = ClockwiseSlope;

#ifdef DEBUG_FIND_THE_CYCLE_TO_MESH
	static int32 CycleToMeshIndex = 1;
	static int32 CycleToMeshIndexToTest = 0;
	F3DDebugSession A(Grid.bDisplay, FString::Printf(TEXT("Find The Cycle To Mesh %d"), ++CycleToMeshIndex));

	if (Grid.bDisplay)
	{
		F3DDebugSession W(FString::Printf(TEXT("Start Segment")));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment, 1, EVisuProperty::BlueCurve);
	}
#endif

	FIsoNode* PreviousNode = bOrientation ? &Segment->GetFirstNode() : &Segment->GetSecondNode();
	FIsoNode* EndNode = bOrientation ? &Segment->GetSecondNode() : &Segment->GetFirstNode();
	FIsoNode* Node = EndNode;

	const FVector2d* PreviousPoint2D = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* NodePoint2D = &Node->Get2DPoint(EGridSpace::UniformScaled, Grid);

	double PreviousSlop = ComputeSlope(*NodePoint2D, *PreviousPoint2D);

#ifdef DEBUG_FIND_THE_CYCLE_TO_MESH
	F3DDebugSession W(Grid.bDisplay, TEXT("Points"));
#endif

	double MinLength = HUGE_VAL;
	int32 Index = 0;
	double SquareLengthSum = 0;
	for (FIsoSegment* NextSegment = FindNextSegment(Segment, Node, GetSlopeAtEndNode);; NextSegment = FindNextSegment(NextSegment, Node, GetSlopeAtEndNode))
	{
		FIsoNode* NextNode = &NextSegment->GetOtherNode(Node);
		const FVector2d& NextPoint2D = NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

		double NextSlop = ComputeSlope(*NodePoint2D, NextPoint2D);
		double NodeSlop = TransformIntoPositiveSlope(PreviousSlop - NextSlop);

#ifdef DEBUG_FIND_THE_CYCLE_TO_MESH
		if (Grid.bDisplay)
		{
			F3DDebugSession W((false && CycleToMeshIndex == CycleToMeshIndexToTest), FString::Printf(TEXT("Point %f"), NodeSlop));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *NextSegment, Index, EVisuProperty::YellowCurve);
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Node, Index, EVisuProperty::RedPoint);
		}
#endif
		SubCycleNodes.Add(Node);
		VertexIndexToSlopes.Emplace(Index, NodeSlop);
		double Length = NextSegment->Get2DLengthSquare(EGridSpace::UniformScaled, Grid);
		SquareLengthSum += Length;
		if (Length < MinLength)
		{
			MinLength = Length;
			StartIndexForMinLength = Index;
		}

		Node = NextNode;
		if (Segment == NextSegment)
		{
			break;
		}

		NodePoint2D = &NextPoint2D;
		PreviousSlop = SwapSlopeOrientation(NextSlop);
		Index++;
	}

	MeanSquareLength = SquareLengthSum / Index;

#ifdef DEBUG_FIND_THE_CYCLE_TO_MESH
	if (Grid.bDisplay && CycleToMeshIndex == CycleToMeshIndexToTest)
	{
		Wait();
	}
#endif

	SubCycleNodeCount = SubCycleNodes.Num();
	if (SubCycleNodeCount < 3)
	{
		return false;
	}

	return true;
}

bool IsCycleInsideCandidate(const FVector2d& Candidate, const FVector2d& Start, const FVector2d& End, const FVector2d& Next, TFunction<double(double)> ConvertSlope)
{
	const double StartEndSlop = ComputeSlope(Start, End);
	const double StartCandidateSlope = ComputeSlope(Start, Candidate);
	const double StartNextSlope = ComputeSlope(Start, Next);

	const double NextSlop = ConvertSlope(TransformIntoPositiveSlope(StartCandidateSlope - StartNextSlope));
	const double NewSlop = ConvertSlope(TransformIntoPositiveSlope(StartCandidateSlope - StartEndSlop));

	return (NewSlop > NextSlop);
};

bool FCycleTriangulator::ConfirmIntersection(const FIsoNode* Start, const FIsoNode* End, const FIsoNode* Candidate, const FIsoSegment* IntersectedSegment) const
{
	const FIsoSegment* StartCandidateSegment = Start->GetSegmentConnectedTo(Candidate);
	const FIsoSegment* EndCandidateSegment = End->GetSegmentConnectedTo(Candidate);
	if (StartCandidateSegment == IntersectedSegment || EndCandidateSegment == IntersectedSegment)
	{
		const int32 IntersectionCount = CycleIntersectionTool.CountIntersections(Start, End);
		if (IntersectionCount > 1)
		{
			return true;
		}
	}
	else
	{
		return true;
	}
	return false;
}

bool FCycleTriangulator::FindTheBestAcuteTriangle()
{
	const double MaxEdgeSquareLength = MeanSquareLength * 4.;
	Algo::Sort(VertexIndexToSlopes, [](const TPair<int, double>& A, const TPair<int, double>& B) {return A.Value < B.Value; });

	int32 NIndex = 0;

	int32 Index = -1;
	FIsoNode* CandidateNode = nullptr;
	const FVector2d* CandidatePoint = nullptr;

	int32 StartId = -1;
	FIsoNode* NewSegmentStart = nullptr;
	const FVector2d* StartPoint = nullptr;

	int32 EndId = -1;
	FIsoNode* NewSegmentEnd = nullptr;
	const FVector2d* EndPoint = nullptr;

	const FVector2d* NextPoint;
	const FVector2d* PrevPoint;

	TFunction<void()> SetFinalCandidate = [&]()
	{
		FirstSideStartIndex = Index;
		FirstSideEndIndex = StartId;
		TriangleThirdIndex = EndId;
		FirstSideStartNode = CandidateNode;
		FirstSideEndNode = NewSegmentStart;
		TriangleThirdNode = NewSegmentEnd;
	};

	TFunction<void(const int32&)> GetCandidateTriangleNodes = [&](const int32& InIndex)
	{
		Index = InIndex;

		CandidateNode = SubCycleNodes[Index];
		CandidatePoint = &CandidateNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

		StartId = NextIndex(Index);
		NewSegmentStart = SubCycleNodes[StartId];
		StartPoint = &NewSegmentStart->Get2DPoint(EGridSpace::UniformScaled, Grid);

		EndId = PreviousIndex(Index);
		NewSegmentEnd = SubCycleNodes[EndId];
		EndPoint = &NewSegmentEnd->Get2DPoint(EGridSpace::UniformScaled, Grid);

		const int32 NextId = NextIndex(StartId);
		const FIsoNode* NextStart = SubCycleNodes[NextId];
		NextPoint = &NextStart->Get2DPoint(EGridSpace::UniformScaled, Grid);

		const int32 PrevId = PreviousIndex(EndId);
		const FIsoNode* PrevEnd = SubCycleNodes[PrevId];
		PrevPoint = &PrevEnd->Get2DPoint(EGridSpace::UniformScaled, Grid);

#ifdef DEBUG_FIND_THE_CYCLE_TO_MESH
		if (Grid.bDisplay)
		{
			static int32 CandidateTriangleIndex = 0;
			FIsoNode* CandidateNode = SubCycleNodes[Index];
			F3DDebugSession W(FString::Printf(TEXT("Candidate Triangle %d"), ++CandidateTriangleIndex));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *CandidateNode, *NewSegmentEnd, 0, EVisuProperty::YellowCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *NewSegmentStart, *CandidateNode, 0, EVisuProperty::YellowCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *NewSegmentStart, *NewSegmentEnd, 0, EVisuProperty::PurpleCurve);
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *NewSegmentStart, 0, EVisuProperty::BluePoint);
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *NewSegmentEnd, 0, EVisuProperty::RedPoint);

			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *NewSegmentStart, *NextStart, 0, EVisuProperty::RedCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *NewSegmentEnd, *PrevEnd, 0, EVisuProperty::BlueCurve);
			if (CandidateTriangleIndex == 0)
			{
				Wait();
			}
		}
#endif
	};

	TFunction<double(double)> TransformIntoClockwise = [](double Value) -> double
	{
		return TransformIntoClockwiseSlope(Value);
	};

	TFunction<double(double)> TransformIntoCounterClockwise = [](double Value) -> double
	{
		return Value;
	};

	// Check if the most Acute Triangle is candidate
	constexpr double MaxSlopeToBeSelectedWithAcuteCriteria = Slope::QuaterPiSlope;
	for (; NIndex < SubCycleNodeCount; ++NIndex)
	{
		const TPair<int, double>& Candidate = VertexIndexToSlopes[NIndex];

		if (Candidate.Value > MaxSlopeToBeSelectedWithAcuteCriteria)
		{
			break;
		}

		GetCandidateTriangleNodes(Candidate.Key);

		if (IsCycleInsideCandidate(*CandidatePoint, *StartPoint, *EndPoint, *NextPoint, TransformIntoCounterClockwise)
			|| IsCycleInsideCandidate(*CandidatePoint, *EndPoint, *StartPoint, *PrevPoint, TransformIntoClockwise))
		{
			continue;
		}

		if (const FIsoSegment* Intersection = CycleIntersectionTool.FindIntersectingSegment(NewSegmentStart, NewSegmentEnd))
		{
			if (ConfirmIntersection(NewSegmentStart, NewSegmentEnd, CandidateNode, Intersection))
			{
				continue;
			}
		}

		const int32 NewSegmentIntersectionCount = InnerToOuterIsoSegmentsIntersectionTool.CountIntersections(*NewSegmentStart, *NewSegmentEnd);
		if (NewSegmentIntersectionCount > IntersectionCountAllowed)
		{
			continue;
		}

		bAcuteTriangle = true;
		SetFinalCandidate();
		return true;
	}

	return false;
}

// Find candidate nodes i.e. Nodes that can be linked with the Segment to build a valid triangle without intersection with other segments
bool FCycleTriangulator::FindCandidateNodes(int32 StartIndex)
{
	FirstSideStartIndex = StartIndex;
	FirstSideEndIndex = NextIndex(StartIndex);

	FirstSideStartNode = SubCycleNodes[FirstSideStartIndex];
	FirstSideEndNode = SubCycleNodes[FirstSideEndIndex];

	const FVector2d& StartPoint2D = FirstSideStartNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& EndPoint2D = FirstSideEndNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

	// StartMaxSlope and EndMaxSlope are at most equal to 4, because if the slop with candidate node is biggest to 4, the new triangle will be inverted
	double StartReferenceSlope = ComputePositiveSlope(StartPoint2D, EndPoint2D, 0);
	double EndReferenceSlope = SwapSlopeOrientation(StartReferenceSlope);

	for (int32 Index = NextIndex(FirstSideEndIndex); ; Index = NextIndex(Index))
	{
		if (Index == FirstSideStartIndex)
		{
			break;
		}

		FIsoNode* Node = SubCycleNodes[Index];

		const FVector2d& NodePoint2D = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);
		double SlopeAtStartNode = GetSlopeAtStartNode(StartPoint2D, NodePoint2D, StartReferenceSlope);
		double SlopeAtEndNode = GetSlopeAtEndNode(EndPoint2D, NodePoint2D, EndReferenceSlope);

		CandidateNodes.Emplace(Node, SlopeAtStartNode, SlopeAtEndNode, Index);
	}

	if (CandidateNodes.IsEmpty())
	{
		return false;
	}

#ifdef DEBUG_FIND_CANDIDATE_NODES
	F3DDebugSession A(Grid.bDisplay, TEXT("Find Candidate Nodes"));
	if (Grid.bDisplay)
	{
		F3DDebugSession A(Grid.bDisplay, TEXT("StartSeg"));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *FirstSideEndNode, 0, EVisuProperty::YellowCurve);
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *FirstSideStartNode, 0, EVisuProperty::BluePoint);
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *FirstSideEndNode, 0, EVisuProperty::RedPoint);
	}
#endif

	constexpr double MaxSlope = Slope::PiSlope - Slope::OneDegree;
	const double StartMaxSlope = FMath::Min(CandidateNodes.Last().SlopeAtStartNode, MaxSlope);
	const double EndMaxSlope = FMath::Min(CandidateNodes[0].SlopeAtEndNode, MaxSlope);

	if (CandidateNodes.Last().SlopeAtStartNode - Slope::Epsilon < StartMaxSlope && CandidateNodes.Last().SlopeAtEndNode < EndMaxSlope)
	{
		CandidateNodes.Last().bIsValid = true;
		ExtremityCandidateNodes[0] = &CandidateNodes.Last();
	}
	if (CandidateNodes[0].SlopeAtEndNode - Slope::Epsilon < EndMaxSlope && CandidateNodes[0].SlopeAtStartNode < StartMaxSlope)
	{
		CandidateNodes[0].bIsValid = true;
		ExtremityCandidateNodes[1] = &CandidateNodes[0];
	}

#ifdef DEBUG_FIND_CANDIDATE_NODES
	if (Grid.bDisplay)
	{
		F3DDebugSession A(Grid.bDisplay, TEXT("Start End Points"));
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *CandidateNodes.Last().Node, 0, EVisuProperty::RedPoint);
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *CandidateNodes[0].Node, 0, EVisuProperty::BluePoint);
	}
#endif

	for (FCandidateNode& CNode : CandidateNodes)
	{
		if (CNode.bIsValid)
		{
			continue;
		}

		FIsoNode* Node = CNode.Node;
		const double SlopeAtStartNode = CNode.SlopeAtStartNode;
		const double SlopeAtEndNode = CNode.SlopeAtEndNode;

		if (SlopeAtStartNode > StartMaxSlope)
		{
			continue;
		}

		if (SlopeAtEndNode > EndMaxSlope)
		{
			continue;
		}

		CNode.bIsValid = true;
	}

#ifdef DEBUG_FIND_CANDIDATE_NODES
	if (Grid.bDisplay)
	{
		F3DDebugSession A(Grid.bDisplay, TEXT("Points"));
		for (FCandidateNode& CNode : CandidateNodes)
		{
			//F3DDebugSession W(TEXT("Next Segment"));
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *CNode.Node, 0, CNode.bIsValid ? EVisuProperty::BluePoint : EVisuProperty::PinkPoint);
		}
		Wait(false);
	}
#endif

	return true;
}

bool FCycleTriangulator::FindTheBestCandidateNode()
{
	constexpr double MinSlopeToNotBeAligned = Slope::Epsilon;

	double MinCriteria = HUGE_VALUE;
	double CandidateSlopeAtStartNode = Slope::PiSlope - MinSlopeToNotBeAligned;
	double CandidateSlopeAtEndNode = Slope::PiSlope - MinSlopeToNotBeAligned;
	FCandidateNode* CandidateNode = nullptr;
	double CandidateEndSquareDistance = HUGE_VALUE;
	double CandidateStartSquareDistance = HUGE_VALUE;
	bool bIsDegeneratedDueSlope = false;
	bool bIsDegeneratedDueToLength = false;
	int32 CandidateIntersectionCount = 0;

	const FVector2d& StartPoint2D = FirstSideStartNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& EndPoint2D = FirstSideEndNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	double NewSegmentSquareLength2500 = FVector2d::DistSquared(StartPoint2D, EndPoint2D) * 2500; // need to check if the length of the sides are not disproportionate

#ifdef DEBUG_FIND_THE_BEST_CANDIDATE_NODE
	static int32 BestCandidateNode = 0;
	static int32 BestCandidateNodeTarget = 100000;
	++BestCandidateNode;
	if (Grid.bDisplay && BestCandidateNode >= BestCandidateNodeTarget)
	{
		F3DDebugSession _(FString::Printf(TEXT("Start Segment %d"), BestCandidateNode));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *FirstSideEndNode, 1, EVisuProperty::RedCurve);
		Wait(false);
	}
#endif

	TFunction<bool(const double, const double)> IsDegeneratedDueToSlope = [](const double SlopeAtStartNode, const double SlopeAtEndNode) -> bool
	{
		// check if the angle of the candidate triangle are not degenerated
		double MinSlope;
		double MaxSlope;
		FMath::GetMinMax(SlopeAtStartNode, SlopeAtEndNode, MinSlope, MaxSlope);
		constexpr double PiMinus5 = Slope::PiSlope - Slope::FiveDegree;
		return MinSlope < Slope::FiveDegree&& MaxSlope > PiMinus5;
	};

	TFunction<bool(const double, const double)> IsDegeneratedDueToLength = [&NewSegmentSquareLength2500](const double CandidateEndSquareDistance, const double CandidateStartSquareDistance)
	{
		// check if the length of the sides are not disproportionate min side minus than 50 times max length 
		const double MinSquareLenth = FMath::Min(CandidateEndSquareDistance, CandidateStartSquareDistance);
		return NewSegmentSquareLength2500 < MinSquareLenth;
	};

	const bool bCheckIntersectionWithIso = true;
	IntersectionCountAllowed = InnerToOuterIsoSegmentsIntersectionTool.CountIntersections(*FirstSideStartNode, *FirstSideEndNode);

	for (FNodeIntersectionCount& NodeIntersection : NodeToIntersection)
	{
		if (NodeIntersection.Value > IntersectionCountAllowed)
		{
			if (CandidateNode)
			{
				// if the candidate triangle is degenerated, allow intersection with iso to find a better candidate
				bIsDegeneratedDueSlope = IsDegeneratedDueToSlope(CandidateSlopeAtStartNode, CandidateSlopeAtEndNode);
				bIsDegeneratedDueToLength = IsDegeneratedDueToLength(CandidateEndSquareDistance, CandidateStartSquareDistance);
				const bool bIsDegenerated = bIsDegeneratedDueSlope || bIsDegeneratedDueToLength;

				if (!bIsDegenerated)
				{
					break;
				}
			}
			IntersectionCountAllowed = NodeIntersection.Value;
		}

		FCandidateNode* CNode = NodeIntersection.Key;
		FIsoNode* Node = CNode->Node;

		const FVector2d& NodePoint2D = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);

		double PointCriteria = IsoTriangulatorImpl::IsoscelesCriteriaMax(StartPoint2D, EndPoint2D, NodePoint2D);

#ifdef DEBUG_FIND_THE_BEST_CANDIDATE_NODE
		if (Grid.bDisplay && BestCandidateNode >= BestCandidateNodeTarget)
		{
			F3DDebugSession _(TEXT("Candidate triangle"));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *Node, 1, EVisuProperty::RedCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideEndNode, *Node, 2, EVisuProperty::RedCurve);
			Wait();
		}
#endif

		double CEndSquareDistance = FVector2d::DistSquared(StartPoint2D, NodePoint2D);
		double CStartSquareDistance = FVector2d::DistSquared(EndPoint2D, NodePoint2D);

		// if the previous candidate and the current candidate is Degenerated Due To Length, the previous is preferred
		if (bIsDegeneratedDueToLength && IsDegeneratedDueToLength(CEndSquareDistance, CStartSquareDistance))
		{
			continue;
		}

		// if the previous candidate and the current candidate is degenerated Due to slope, the previous is preferred
		if (bIsDegeneratedDueSlope && IsDegeneratedDueToSlope(CNode->SlopeAtStartNode, CNode->SlopeAtEndNode))
		{
			continue;
		}

		// the candidate triangle is inside the current candidate triangle
		// Angle is not accurate enough to define if a triangle is inside or not an other triangle (especially with acute triangle).
		// The length of the side is a better criteria but must be combine with the angles
		const bool bIsInsideCandidate = (CNode->SlopeAtStartNode - CandidateSlopeAtStartNode) < Slope::OneDegree
			&& (CNode->SlopeAtEndNode - CandidateSlopeAtEndNode) < Slope::OneDegree
			&& CandidateEndSquareDistance > CEndSquareDistance
			&& CandidateStartSquareDistance > CStartSquareDistance;

		// the candidate triangle is better the current candidate triangle and doesn't contain the current candidate triangle
		const bool bIsBetterCandidate = (PointCriteria < MinCriteria) && ((CNode->SlopeAtStartNode > CandidateSlopeAtStartNode) ^ (CNode->SlopeAtEndNode > CandidateSlopeAtEndNode));

		if (bIsInsideCandidate || bIsBetterCandidate)
		{
			// Check if the candidate segment is not in intersection with existing segments
			// if the segment exist, it has already been tested
			{
				const FIsoSegment* StartSegment = FirstSideStartNode->GetSegmentConnectedTo(Node);
				if (!StartSegment)
				{
					if(const FIsoSegment* Intersection = CycleIntersectionTool.FindIntersectingSegment(FirstSideStartNode, Node))
					{
						if (ConfirmIntersection(FirstSideStartNode, Node, FirstSideEndNode, Intersection))
						{
							continue;
						}
					}
				}
			}

			{
				const FIsoSegment* EndSegment = FirstSideEndNode->GetSegmentConnectedTo(Node);
				if(!EndSegment)
				{
					if (const FIsoSegment* Intersection = CycleIntersectionTool.FindIntersectingSegment(FirstSideEndNode, Node))
					{
						if (ConfirmIntersection(FirstSideEndNode, Node, FirstSideStartNode, EndSegment))
						{
							continue;
						}
					}
				}
			}

			MinCriteria = PointCriteria;
			CandidateNode = CNode;
			CandidateSlopeAtStartNode = CNode->SlopeAtStartNode;
			CandidateSlopeAtEndNode = CNode->SlopeAtEndNode;
			CandidateEndSquareDistance = CEndSquareDistance;
			CandidateStartSquareDistance = CStartSquareDistance;
			CandidateIntersectionCount = NodeIntersection.Value;
		}
	}

	if (!CandidateNode)
	{
		return false;
	}

#ifdef DEBUG_FIND_THE_BEST_CANDIDATE_NODE
	static int32 TriangleIndex2 = 0;
	if (Grid.bDisplay && CandidateNode)
	{
		FIsoNode* Node = CandidateNode->Node;
		F3DDebugSession _(FString::Printf(TEXT("Selected triangle %d"), ++TriangleIndex2));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *Node, 1, EVisuProperty::BlueCurve);
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideEndNode, *Node, 2, EVisuProperty::BlueCurve);
		Wait(false);
	}
#endif

#ifdef DEBUG_OPIMIZE_FIND_BEST_CANDIDATE
	TFunction<void() > DisplayContext = [&]()
	{
		if (Grid.bDisplay)
		{
			F3DDebugSession A(TEXT("Opimizer"));
			if (Grid.bDisplay)
			{
				F3DDebugSession B(TEXT("StartSeg"));
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *FirstSideEndNode, 0, EVisuProperty::YellowCurve);
			}

			{
				F3DDebugSession C(TEXT("Points"));
				for (FCandidateNode& CNode : CandidateNodes)
				{
					Grid.DisplayIsoNode(EGridSpace::UniformScaled, *CNode.Node, 0, CNode.bIsValid ? EVisuProperty::BluePoint : EVisuProperty::PinkPoint);
				}
			}

			{
				F3DDebugSession _(FString::Printf(TEXT("Selected triangle %d"), ++TriangleIndex2));
				FIsoNode* Node = CandidateNode->Node;
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *Node, 1, EVisuProperty::BlueCurve);
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideEndNode, *Node, 2, EVisuProperty::BlueCurve);
			}
		}
	};
#endif

	TFunction<void(FCandidateNode*, const double, const FVector2d&, double, double) > CompareCandidateAtSegmentNode = [&CandidateNode, &Grid = Grid](FCandidateNode* NewCandidate, const double CloseLimit, const FVector2d& SegmentPoint2D, double CandidateToSegmentSquareDistance, double SlopeDiffAtSegmentExtremity)
	{
		const FVector2d& NodePoint2D = NewCandidate->Node->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const double CFirstSquareDistance = FVector2d::DistSquared(SegmentPoint2D, NodePoint2D) * 2.;

		if (SlopeDiffAtSegmentExtremity < CloseLimit && CFirstSquareDistance < CandidateToSegmentSquareDistance)
		{
			CandidateNode = NewCandidate;
		}
	};

	// Optimize selection step: if the selected triangle is close to one of the next segment, check if a triangle from it is not better 
	if (ExtremityCandidateNodes[0] != CandidateNode && ExtremityCandidateNodes[1] != CandidateNode)
	{
		const int32 NewCandidateIntersectionCountAllowed = FMath::Max(CandidateIntersectionCount - 1, 0);
		if (ExtremityCandidateNodes[0] && ExtremityCandidateNodes[1])
		{
			const double SlopeDiffAtStart = ExtremityCandidateNodes[0]->SlopeAtStartNode - CandidateNode->SlopeAtStartNode;
			const double SlopeDiffAtEnd = ExtremityCandidateNodes[1]->SlopeAtEndNode - CandidateNode->SlopeAtEndNode;

			if (SlopeDiffAtStart < SlopeDiffAtEnd)
			{
				const FNodeIntersectionCount* ExtremityIntersectionCount = NodeToIntersection.FindByPredicate([Extremity = ExtremityCandidateNodes[0]](const FNodeIntersectionCount& Node) { return Node.Key == Extremity; });
				if (NewCandidateIntersectionCountAllowed >= ExtremityIntersectionCount->Value)
				{
					CompareCandidateAtSegmentNode(ExtremityCandidateNodes[0], ExtremityCandidateNodes[0]->SlopeAtStartNode * AThird, StartPoint2D, CandidateStartSquareDistance, SlopeDiffAtStart);
				}
			}
			else
			{
				const FNodeIntersectionCount* ExtremityIntersectionCount = NodeToIntersection.FindByPredicate([Extremity = ExtremityCandidateNodes[1]](const FNodeIntersectionCount& Node) { return Node.Key == Extremity; });
				if (NewCandidateIntersectionCountAllowed >= ExtremityIntersectionCount->Value)
				{
					CompareCandidateAtSegmentNode(ExtremityCandidateNodes[1], ExtremityCandidateNodes[1]->SlopeAtEndNode * AThird, EndPoint2D, CandidateEndSquareDistance, SlopeDiffAtEnd);
				}
			}
		}
		else if (ExtremityCandidateNodes[0])
		{
			const FNodeIntersectionCount* ExtremityIntersectionCount = NodeToIntersection.FindByPredicate([Extremity = ExtremityCandidateNodes[0]](const FNodeIntersectionCount& Node) { return Node.Key == Extremity; });
			if (NewCandidateIntersectionCountAllowed >= ExtremityIntersectionCount->Value)
			{
				const double SlopeDiffAtStart = ExtremityCandidateNodes[0]->SlopeAtStartNode - CandidateNode->SlopeAtStartNode;
				if (SlopeDiffAtStart < Slope::FifteenDegree)
				{
					CompareCandidateAtSegmentNode(ExtremityCandidateNodes[0], ExtremityCandidateNodes[0]->SlopeAtStartNode * AThird, StartPoint2D, CandidateStartSquareDistance, SlopeDiffAtStart);
				}
			}
		}
		else if (ExtremityCandidateNodes[1])
		{
			//FLoopNode const* const* StartNode = NodesOfLoop.FindByPredicate([](const FLoopNode* Node) { return !Node->IsDelete(); });
			const FNodeIntersectionCount* ExtremityIntersectionCount = NodeToIntersection.FindByPredicate([Extremity = ExtremityCandidateNodes[1]](const FNodeIntersectionCount& Node) { return Node.Key == Extremity; });
			if (NewCandidateIntersectionCountAllowed >= ExtremityIntersectionCount->Value)
			{
				const double SlopeDiffAtEnd = ExtremityCandidateNodes[1]->SlopeAtEndNode - CandidateNode->SlopeAtEndNode;
				if (SlopeDiffAtEnd < Slope::FifteenDegree)
				{
					CompareCandidateAtSegmentNode(ExtremityCandidateNodes[1], ExtremityCandidateNodes[1]->SlopeAtEndNode * AThird, EndPoint2D, CandidateEndSquareDistance, SlopeDiffAtEnd);
				}
			}
		}
	}

#ifdef DEBUG_FIND_THE_BEST_CANDIDATE_NODE
	static int32 TriangleIndex = 0;
	if (Grid.bDisplay && CandidateNode)
	{
		FIsoNode* Node = CandidateNode->Node;
		F3DDebugSession _(FString::Printf(TEXT("Selected triangle %d"), ++TriangleIndex));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideStartNode, *Node, 1, EVisuProperty::YellowCurve);
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *FirstSideEndNode, *Node, 2, EVisuProperty::YellowCurve);
		if (TriangleIndex > 1000)
		{
			Wait();
		}
	}
#endif

	if (CandidateNode)
	{
		TriangleThirdIndex = CandidateNode->Index;
		TriangleThirdNode = CandidateNode->Node;
		return true;
	}
	return false;
}

void FCycleTriangulator::ValidateAddNodesAccordingSlopeWithSide(FAdditionalIso& Side)
{
	constexpr double TooSmallSlope = Slope::FiveDegree;

	const FVector2d& StartPoint = Side.StartNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& EndPoint = Side.EndNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const double SlopeStartEnd = ComputeSlope(StartPoint, EndPoint);

	for (int32 Index = 0; Index < 2; ++Index)
	{
		const FIsoNode* Candidate = Side.Nodes[Index];
		if (!Candidate)
		{
			continue;
		}
		const FVector2d& CandidatePoint = Candidate->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const double SlopeStartCandidate = ComputeSlope(StartPoint, CandidatePoint);
		const double SlopeEndCandidate = ComputeSlope(EndPoint, CandidatePoint);

		double SlopeAtStart = TransformIntoOrientedSlope(SlopeStartEnd - SlopeStartCandidate);
		double SlopeAtEnd = TransformIntoOrientedSlope(SlopeEndCandidate - SwapSlopeOrientation(SlopeStartEnd));

		if (SlopeAtStart < Slope::NullSlope)
		{
			SlopeAtStart = Slope::PiSlope;
			Side.bForceNodes = false;
		}

		if (SlopeAtEnd < Slope::NullSlope)
		{
			SlopeAtEnd = Slope::PiSlope;
			Side.bForceNodes = false;
		}

#ifdef DEBUG_COMPUTE_SLOTE_CRITERION
		if (Grid.bDisplay)
		{
			F3DDebugSession _(FString::Printf(TEXT("Slope %f"), SlopeAtStart));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.StartNode, *Candidate, 1, EVisuProperty::BlueCurve);
		}
		if (Grid.bDisplay)
		{
			F3DDebugSession _(FString::Printf(TEXT("Slope %f"), SlopeAtEnd));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.EndNode, *Candidate, 1, EVisuProperty::BlueCurve);
			//Wait();
		}
#endif

		const double MaxSlope = FMath::Max(SlopeAtEnd, SlopeAtStart);
		const double SlopeCriterion = MaxSlope > Slope::HalfPiSlope ? Slope::PiSlope : FMath::Min(SlopeAtEnd, SlopeAtStart);
		if (!Side.bForceNodes && SlopeCriterion > TooSmallSlope)
		{
			Side.NodeIndices[Index] = -1;
			Side.Nodes[Index] = nullptr;
		}
	}
};

void FCycleTriangulator::SelectFinalNodes(FAdditionalIso& Side1, FAdditionalIso& Side2)
{
	// Check if complementary node is candidate triangle vertex
	{
		Side1.RemoveCandidateIfPresent(Side2.EndNode);
		Side2.RemoveCandidateIfPresent(Side1.StartNode);
	}

	if (!Side1.bForceNodes && !Side2.bForceNodes)
	{
		const int32 Side1Count = Side1.CandidateNodeCount();
		const int32 Side2Count = Side2.CandidateNodeCount();
		int32 CandidateCount = Side1Count + Side2Count;
		if (CandidateCount > 2)
		{
			ComputeSideCandidateEquilateralCriteria(Side1);
			ComputeSideCandidateEquilateralCriteria(Side2);
			double Criteria[4] = { Slope::NullSlope, Slope::NullSlope, Slope::NullSlope, Slope::NullSlope };
			Criteria[0] = Side1.EquilateralCriteria[0];
			Criteria[1] = Side1.EquilateralCriteria[1];
			Criteria[2] = Side2.EquilateralCriteria[0];
			Criteria[3] = Side2.EquilateralCriteria[1];

			int32 Indices[4] = { 0, 1, 2, 3 };
			Algo::Sort(Indices, [Criteria](int32 A, int32 B) { return Criteria[A] < Criteria[B]; });

			for (int32 Index = 0; Index < 4 && CandidateCount != 2; ++Index)
			{
				if (Criteria[Indices[Index]] > Slope::PiSlope)
				{
					break;
				}

				if (Indices[Index] > 1)
				{
					if (!ValidComplementaryNodeOrDeleteIt(Side2, Indices[Index] - 2))
					{
						CandidateCount--;
					}
				}
				else
				{
					if (!ValidComplementaryNodeOrDeleteIt(Side1, Indices[Index]))
					{
						CandidateCount--;
					}
				}
			}

			if (CandidateCount > 2)
			{
				CandidateCount = 0;
				int32 Index = 0;
				for (; CandidateCount != 2; ++Index)
				{
					if (Criteria[Indices[Index]] > Slope::PiSlope)
					{
						continue;
					}
					CandidateCount++;
				}
				for (; Index < 4; ++Index)
				{
					if (Indices[Index] > 1)
					{
						Side2.RemoveCandidate(Indices[Index] - 2);
					}
					else
					{
						Side1.RemoveCandidate(Indices[Index]);
					}
				}
			}
		}
	}

	if (Side1.bForceNodes)
	{
		Side2.Reset();
	}
	else if (Side2.bForceNodes)
	{
		Side1.Reset();
	}

}

void FCycleTriangulator::FindComplementaryNodes(FAdditionalIso& Side)
{
	const int32 StartIndex = Side.StartIndex;
	const int32 EndIndex = Side.EndIndex;
	int32 PotentialNodeCount = EndIndex - StartIndex - 1;
	if (PotentialNodeCount < 0)
	{
		PotentialNodeCount += SubCycleNodeCount;
	}

	if (PotentialNodeCount == 0)
	{
		return;
	}

	Side.NodeIndices[0] = NextIndex(StartIndex);
	Side.Nodes[0] = SubCycleNodes[Side.NodeIndices[0]];

	if (PotentialNodeCount > 1)
	{
		Side.NodeIndices[1] = PreviousIndex(EndIndex);
		Side.Nodes[1] = SubCycleNodes[Side.NodeIndices[1]];
	}
	Side.bForceNodes = PotentialNodeCount <= 2;

	ValidateAddNodesAccordingSlopeWithSide(Side);
	ValidateComplementaryNodesWithInsideAndIntersectionsCriteria(Side);
}

bool FCycleTriangulator::BuildTheBestPolygonFromTheSelectedTriangle()
{
	FIsoNode* SelectedNode = TriangleThirdNode;
	FAdditionalIso Side1(FirstSideEndIndex, TriangleThirdIndex, FirstSideEndNode, SelectedNode); // From end segment to candidate
	FAdditionalIso Side2(TriangleThirdIndex, PreviousIndex(FirstSideEndIndex), SelectedNode, FirstSideStartNode); // From candidate to start segment

	// Check if there are nodes that could be meshed with
	// i.e. if the next or previous node of the candidate or of the segment is nearly aligned, it would better to add the neighbor node 
	// and mesh them as a quadrilateral or pentagon polygon
	if (MaxIntersectionCounted == 0)
	{
		FindComplementaryNodes(Side1);
		FindComplementaryNodes(Side2);

		SelectFinalNodes(Side1, Side2);
	}

	PolygonNodes.Add(FirstSideEndNode);
	Side1.AddTo(PolygonNodes);
	PolygonNodes.Add(SelectedNode);
	Side2.AddTo(PolygonNodes);
	PolygonNodes.Add(FirstSideStartNode);

	return BuildSmallPolygon(PolygonNodes, false);
}

int32 PreviousIndex2(int32)
{
	return 0;
}

bool FCycleTriangulator::ValidComplementaryNodeOrDeleteIt(FAdditionalIso& Side, int32 CandidateIndex)
{
	const FIsoNode* Candidate = Side.Nodes[CandidateIndex];
	if (!Candidate)
	{
		return false;
	}

	const int32 OppositeIndex = CandidateIndex ? Side.StartIndex : Side.EndIndex;
	const FIsoNode* OppositeNode = CandidateIndex ? Side.StartNode : Side.EndNode;

	TFunction<int32(int32)> NextFunc = [&CandidateIndex, this](int32 Index)
	{
		return CandidateIndex ? NextIndex(Index) : PreviousIndex(Index);
	};

	const FVector2d& CandidatePoint = Candidate->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& OppositePoint = OppositeNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

	const int32 TestPointIndex = NextFunc(OppositeIndex);
	const FIsoNode* TestNode = SubCycleNodes[TestPointIndex];
	const FVector2d& TestPoint = TestNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

#ifdef DEBUG_VALID_COMPLEMENTARY_NODE
	if (Grid.bDisplay)
	{
		{
			F3DDebugSession _(TEXT("Selected Node"));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *OppositeNode, *Candidate, 1, EVisuProperty::BlueCurve);
		}

		{
			F3DDebugSession _(TEXT("Slope"));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *OppositeNode, *Candidate, 1, EVisuProperty::YellowCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *OppositeNode, *TestNode, 1, EVisuProperty::YellowCurve);
		}
	}
#endif

	// Is the new segment fully inside the cycle 
	double Slope = ComputeOrientedSlope(OppositePoint, CandidatePoint, TestPoint);
	if ((Slope > 0) == (CandidateIndex == 0))
	{
		const FIsoSegment* Segment = Candidate->GetSegmentConnectedTo(OppositeNode);
		if (Segment || !CycleIntersectionTool.DoesIntersect(Candidate, OppositeNode))
		{
			// The new segment is fully inside the cycle 
			return true;
		}
	}
#ifdef DEBUG_VALID_COMPLEMENTARY_NODE
	else if (Grid.bDisplay)
	{
		F3DDebugSession _(TEXT("intersect cycle => candidate is deleted"));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Candidate, *OppositeNode, 1, EVisuProperty::YellowCurve);
		Wait();
	}
#endif

	Side.NodeIndices[CandidateIndex] = -1;
	Side.Nodes[CandidateIndex] = nullptr;
	return false;
};

bool FCycleTriangulator::IsInnerSideSegmentInsideCycle(FAdditionalIso& Side)
{
	TFunction<bool(const FIsoNode*, const FIsoNode*, const FIsoNode*, const FIsoNode*)> IsInside = [&](const FIsoNode* P, const FIsoNode* A, const FIsoNode* N, const FIsoNode* B) -> bool
	{
		const FVector2d& PPoint = P->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& NPoint = N->Get2DPoint(EGridSpace::UniformScaled, Grid);

		const FVector2d& APoint = A->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& BPoint = B->Get2DPoint(EGridSpace::UniformScaled, Grid);

		double SlopeAP = ComputeSlope(APoint, PPoint);
		double SlopeAN = ComputeSlope(APoint, NPoint);
		double SlopeAB = ComputeSlope(APoint, BPoint);

		double SlopeMax = TransformIntoPositiveSlope(SlopeAP - SlopeAN);
		double SlopeCandidate = TransformIntoPositiveSlope(SlopeAP - SlopeAB);

#ifdef DEBUG_VALID_IS_INSIDE_CYCLE
		if (Grid.bDisplay)
		{
			F3DDebugSession SAA(TEXT("Angle"));

			{
				Grid.DisplayIsoPolyline(TEXT("Poly"), EGridSpace::UniformScaled, (const TArray<const FIsoNode*>&) SubCycleNodes, EVisuProperty::YellowPoint);
			}

			{
				F3DDebugSession _(TEXT("Prev"));
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *P, *A, 1, EVisuProperty::BlueCurve);
			}
			{
				F3DDebugSession _(TEXT("Next"));
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *A, *N, 1, EVisuProperty::PurpleCurve);
			}
			{
				F3DDebugSession _(TEXT("Seg"));
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *A, *B, 1, EVisuProperty::RedCurve);
			}
			Wait();
		}
#endif
		return SlopeCandidate < SlopeMax;
	};

	const FIsoNode* StartSegment = Side.Nodes[0];
	const FIsoNode* EndSegment = Side.Nodes[1];

	{
		const int32 PrevStartIndex = PreviousIndex(Side.NodeIndices[0]);
		const int32 NextStartIndex = NextIndex(Side.NodeIndices[0]);
		const FIsoNode* PrevStart = SubCycleNodes[PrevStartIndex];
		const FIsoNode* NextStart = SubCycleNodes[NextStartIndex];

		if (!IsInside(PrevStart, StartSegment, NextStart, EndSegment))
		{
			return false;
		}
	}

	{
		const int32 PrevStartIndex = PreviousIndex(Side.NodeIndices[1]);
		const int32 NextStartIndex = NextIndex(Side.NodeIndices[1]);
		const FIsoNode* PrevStart = SubCycleNodes[PrevStartIndex];
		const FIsoNode* NextStart = SubCycleNodes[NextStartIndex];

		if (!IsInside(PrevStart, EndSegment, NextStart, StartSegment))
		{
			return false;
		}
	}
	return true;
}

void FCycleTriangulator::ValidateComplementaryNodesWithInsideAndIntersectionsCriteria(FAdditionalIso& Side)
{
	for (int32 Index = 0; Index < 2; ++Index)
	{
		if (Side.Nodes[Index])
		{
			if (InnerToOuterIsoSegmentsIntersectionTool.DoesIntersect(Index == 0 ? *Side.EndNode : *Side.StartNode, *Side.Nodes[Index]))
			{
				Side.RemoveCandidate(Index);
				Side.bForceNodes = false;
			}
		}
	}

	if (Side.bForceNodes)
	{
		return;
	}

	if (Side.Nodes[0] && Side.Nodes[1])
	{
#ifdef DEBUG_VALID_COMPLEMENTARY_NODE
		if (Grid.bDisplay)
		{
			F3DDebugSession _(TEXT("Valid Complementary Nodes"));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.Nodes[0], *Side.Nodes[1], 1, EVisuProperty::RedCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.Nodes[1], *Side.EndNode, 1, EVisuProperty::BlueCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.Nodes[0], *Side.StartNode, 1, EVisuProperty::BlueCurve);
			Wait(false);
		}
#endif
		// Check if [Side.Nodes[0] - Side.Nodes[1]] intersect cycle
		const FIsoSegment* Segment = Side.Nodes[0]->GetSegmentConnectedTo(Side.Nodes[1]);
		if (!IsInnerSideSegmentInsideCycle(Side) || (!Segment && CycleIntersectionTool.DoesIntersect(Side.Nodes[0], Side.Nodes[1])))
		{
			// intersect => choose between NextFinalNode and LastPotentialNode
#ifdef DEBUG_VALID_COMPLEMENTARY_NODE
			if (Grid.bDisplay)
			{
				F3DDebugSession _(TEXT("choose between NextFinalNode and LastPotentialNode"));
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.Nodes[0], *Side.EndNode, 1, EVisuProperty::BlueCurve);
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Side.Nodes[1], *Side.StartNode, 1, EVisuProperty::BlueCurve);
				Wait(false);
			}
#endif
			ValidComplementaryNodeOrDeleteIt(Side, 0);
			ValidComplementaryNodeOrDeleteIt(Side, 1);

			if (Side.CandidateNodeCount() == 2)
			{
				ComputeSideCandidateEquilateralCriteria(Side);
				if (Side.EquilateralCriteria[0] < Side.EquilateralCriteria[1])
				{
					Side.RemoveCandidate(0);
				}
				else
				{
					Side.RemoveCandidate(1);
				}
			}
		}
		return;
	}

	if (Side.Nodes[0])
	{
		ValidComplementaryNodeOrDeleteIt(Side, 0);
	}

	if (Side.Nodes[1])
	{
		ValidComplementaryNodeOrDeleteIt(Side, 1);
	}
}

void FCycleTriangulator::ComputeSideCandidateEquilateralCriteria(FAdditionalIso& Side)
{
	if (Side.CandidateNodeCount() == 1)
	{
		int32 Index = Side.Nodes[0] ? 0 : 1;
		const FVector2d& SideBStartPoint = Side.StartNode->Get2DPoint(EGridSpace::UniformScaled, Grid);;
		const FVector2d& SideBEndPoint = Side.EndNode->Get2DPoint(EGridSpace::UniformScaled, Grid);;
		const FIsoNode* SideBCandidate = Side.Nodes[Index];
		const FVector2d& SideBCandidatePoint = SideBCandidate->Get2DPoint(EGridSpace::UniformScaled, Grid);

		const double SideBCriteria = IsoTriangulatorImpl::EquilateralSlopeCriteria(SideBStartPoint, SideBCandidatePoint, SideBEndPoint);
		Side.EquilateralCriteria[Index] = SideBCriteria;
	}
	else
	{
		const FIsoNode* StartNode = Side.StartNode;
		const FIsoNode* EndNode = Side.EndNode;
		const FIsoNode* Candidate0 = Side.Nodes[0];
		const FIsoNode* Candidate1 = Side.Nodes[1];

		const FVector2d& StartPoint = StartNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& EndPoint = EndNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& Candidate0Point = Candidate0->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& Candidate1Point = Candidate1->Get2DPoint(EGridSpace::UniformScaled, Grid);

		Side.EquilateralCriteria[0] = IsoTriangulatorImpl::EquilateralSlopeCriteria(StartPoint, Candidate0Point, Candidate1Point);
		Side.EquilateralCriteria[1] = IsoTriangulatorImpl::EquilateralSlopeCriteria(Candidate0Point, Candidate1Point, EndPoint);
	}
}

bool FCycleTriangulator::BuildTheBestPolygon(FIsoSegment* StartSegment, bool bStartOrientation)
{
	//
	// From the Segment :
	// - (Step 1) Find all candidates nodes i.e.nodes in the cycle from the end of the current segment to the beginning of this segment
	// = > PotentialCandidateNodes + Segment is the cycle that need to be filled.
	// PotentialCandidateNodes = Array<Node*, SlopeAtStartNode, SlopeAtEndNode>
	// 
	// - If PotentialCandidateNodes.Num() < 4
	// 		if Can be meshed as a standard polygon(Triangle to Pentagon) i.e.polygon are not intersecting isoline
	// 			= > Mesh as a standard polygon
	// 
	// - (Step 2) In PotentialCandidateNodes, select CandidateNodes(*)
	// 
	// - (Step 3) Sort CandidateNodes according to the number of intersection with iso-lines = > NodeToIntersection
	// 
	// - (Step 4) In NodeToIntersection, select the best node i.e. for the less of intersection with iso-lines, the node that make the most equilateral triangle with the Segment
	// 
	// - (Step 5) If a node is found, check if there are potential node that could be meshed with i.e. if the next of previous node of the candidate or of the segment is nearly
	// aligned, it would better to select also the neighbor node and mesh them as a quadrilateral or pentagon polygon
	// 
	// 
	// (*) Candidate nodes :
	// For each extremity(A, B) of a segment, in the existing connected segments, the segment with the smallest relative slop is identified([A, X0] and [B, Xn]).
	// These segments define the sector in which the best triangle could be.
	// The triangle to build is the best triangle(according to the Isosceles Criteria) connecting the Segment to one of the allowed nodes(X) between X0and Xn.
	// Allowed nodes(X) are in the sector, Disallowed nodes(Z) are outside the sector
	// 
	//                    ------Z------Xn-------X------X-----X-------X0----Z-----Z---
	//                                  \                           /
	//                                   \    Allowed triangles    /   
	//             Not allowed triangles  \                       /   Not allowed triangles
	//                                     \                     /
	//                          ----Z-------A------Segment------B------Z---
	//
	//                                    Not allowed triangles
	// 

#ifdef DEBUG_FIND_BEST_POLYGON
	static int32 PolyIndex = 0;
	F3DDebugSession W(Grid.bDisplay, FString::Printf(TEXT("New Polygon %d"), ++PolyIndex));
#endif

	CleanContext();

	int32 StartIndexForMinLength = -1;
	if (!FindTheCycleToMesh(StartSegment, bStartOrientation, StartIndexForMinLength))
	{
		return false;
	}

	// case of triangle to pentagon
	const bool bCheckIntersectionWithIso = true;
	if (SubCycleNodeCount < 6 && BuildSmallPolygon(SubCycleNodes, bCheckIntersectionWithIso))
	{
		return true;
	}

	if (!FindTheBestAcuteTriangle())
	{
		if (!FindCandidateNodes(StartIndexForMinLength))
		{
			return false;
		}

		// ==================================================================================================================================================================
		// Step 4: 
		//    Sort CandidateNodes according to the number of intersection with isolines
		{
			MaxIntersectionCounted = CountIntersectionWithIso();
			if (MaxIntersectionCounted)
			{
				NodeToIntersection.Sort([](const FNodeIntersectionCount& N1, const FNodeIntersectionCount& N2) { return N1.Value < N2.Value; });
			}
		}

		// ==================================================================================================================================================================
		// Step 5:
		//    In NodeToIntersection, select the best node i.e. for the less of intersection with isolines, 
		//    the node that makes the most equilateral triangle with the StartSegment

		if (!FindTheBestCandidateNode())
		{
			return false;
		}
	}

	// ==================================================================================================================================================================
	// Step 6
	//    If a node is found: mesh
	return BuildTheBestPolygonFromTheSelectedTriangle();
}

bool FCycleTriangulator::BuildSmallPolygon(TArray<FIsoNode*>& CandidatNodes, bool bCheckIntersectionWithIso)
{
	switch (CandidatNodes.Num())
	{
	case 0:
	case 1:
	case 2:
		return false;
	case 3:
	{
		return BuildTriangle(CandidatNodes);
	}
	case 4:
	{
		if (bCheckIntersectionWithIso && IsIntersectingIso(CandidatNodes))
		{
			return false;
		}
		return BuildQuadrilateral(CandidatNodes);
	}
	case 5:
	{
		if (bCheckIntersectionWithIso && IsIntersectingIso(CandidatNodes))
		{
			return false;
		}
		return BuildPentagon(CandidatNodes);
	}
	default:
		return false;
	}
}

bool  FCycleTriangulator::BuildTriangle(TArray<FIsoNode*>& CandidatNodes)
{
	ensureCADKernel(CandidatNodes.Num() == 3);
	if (!BuildSegmentIfNeeded(CandidatNodes))
	{
		return false;
	}

	Mesh.AddTriangle(CandidatNodes[0]->GetGlobalIndex(), CandidatNodes[1]->GetGlobalIndex(), CandidatNodes[2]->GetGlobalIndex());

	SortCycleIntersectionToolIfNeeded();
	return true;
}

bool FCycleTriangulator::BuildPentagon(TArray<FIsoNode*>& CandidatNodes)
{
	ensureCADKernel(CandidatNodes.Num() == 5);
	if (!BuildSegmentIfNeeded(CandidatNodes))
	{
		return false;
	}

	Polygon::MeshPentagon(Grid, CandidatNodes.GetData(), Mesh);
	SortCycleIntersectionToolIfNeeded();
	return true;
}

bool  FCycleTriangulator::BuildQuadrilateral(TArray<FIsoNode*>& CandidatNodes)
{
	ensureCADKernel(CandidatNodes.Num() == 4);
	if (!BuildSegmentIfNeeded(CandidatNodes))
	{
		return false;
	}

	Polygon::MeshQuadrilateral(Grid, CandidatNodes.GetData(), Mesh);
	SortCycleIntersectionToolIfNeeded();
	return true;
}

namespace Polygon
{

void MeshTriangle(const FGrid& Grid, FIsoNode** Nodes, FFaceMesh& Mesh)
{
	Mesh.AddTriangle(Nodes[0]->GetGlobalIndex(), Nodes[1]->GetGlobalIndex(), Nodes[2]->GetGlobalIndex());
}

void MeshQuadrilateral(const FGrid& Grid, FIsoNode** Nodes, FFaceMesh& Mesh)
{
	const FVector2d* NodeCoordinates[4];
	for (int32 Index = 0; Index < 4; ++Index)
	{
		NodeCoordinates[Index] = &Nodes[Index]->Get2DPoint(EGridSpace::UniformScaled, Grid);
	}

	double SegmentSlopes[4];
	SegmentSlopes[0] = ComputeSlope(*NodeCoordinates[0], *NodeCoordinates[1]);
	SegmentSlopes[1] = ComputeSlope(*NodeCoordinates[1], *NodeCoordinates[2]);
	SegmentSlopes[2] = ComputeSlope(*NodeCoordinates[2], *NodeCoordinates[3]);
	SegmentSlopes[3] = ComputeSlope(*NodeCoordinates[3], *NodeCoordinates[0]);

	double RelativeSlopes[4];
	RelativeSlopes[0] = TransformIntoOrientedSlope(SegmentSlopes[1] - SegmentSlopes[0]);
	RelativeSlopes[1] = TransformIntoOrientedSlope(SegmentSlopes[2] - SegmentSlopes[1]);
	RelativeSlopes[2] = TransformIntoOrientedSlope(SegmentSlopes[3] - SegmentSlopes[2]);
	RelativeSlopes[3] = TransformIntoOrientedSlope(SegmentSlopes[0] - SegmentSlopes[3]);

	int32 FlattenNodeIndex = 0;
	for (int32 IndexAngle = 0; IndexAngle < 4; ++IndexAngle)
	{
		if (RelativeSlopes[IndexAngle] < RelativeSlopes[FlattenNodeIndex])
		{
			FlattenNodeIndex = IndexAngle;
		}
	}

	int32 NodeIndices[4];
	NodeIndices[0] = FlattenNodeIndex;
	for (int32 IndexN = 1; IndexN < 4; ++IndexN)
	{
		NodeIndices[IndexN] = NodeIndices[IndexN - 1] == 3 ? 0 : NodeIndices[IndexN - 1] + 1;
	}

	Mesh.AddTriangle(Nodes[NodeIndices[1]]->GetGlobalIndex(), Nodes[NodeIndices[3]]->GetGlobalIndex(), Nodes[NodeIndices[0]]->GetGlobalIndex());
	Mesh.AddTriangle(Nodes[NodeIndices[1]]->GetGlobalIndex(), Nodes[NodeIndices[2]]->GetGlobalIndex(), Nodes[NodeIndices[3]]->GetGlobalIndex());
}

void MeshPentagon(const FGrid& Grid, FIsoNode** Nodes, FFaceMesh& Mesh)
{
	double TriangleSlopes[10];
	const FVector2d* NodeCoordinates[5];
	for (int32 Index = 0; Index < 5; ++Index)
	{
		NodeCoordinates[Index] = &Nodes[Index]->Get2DPoint(EGridSpace::UniformScaled, Grid);
	}

#ifdef DEBUG_MESH_PENTAGON
	if (Grid.bDisplay)
	{
		F3DDebugSession G(TEXT("Mesh Pentagon"));
		for (int32 Index = 0; Index < 5; ++Index)
		{
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Nodes[Index], Index, EVisuProperty::YellowPoint);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Nodes[Index], *Nodes[Index == 4 ? 0 : Index + 1], Index, EVisuProperty::OrangeCurve);
		}
		Wait();
	}
#endif 

	/**
	 * Slope(AB, AC)
	 */
	TFunction<double(double, double)> DiffSlopeVsIsocele = [](double SlopeAB, double SlopeCA) -> double
	{
		const double SlopeAC = SwapSlopeOrientation(SlopeCA);
		double Slope_AB_AC = TransformIntoOrientedSlope(SlopeAC - SlopeAB);
		return FMath::Abs(Slope_AB_AC - Slope::ThirdPiSlope);
	};

	TFunction<double(double, double, double)> MaxDiffSlopeVsEquilateral = [DiffSlopeVsIsocele](double SlopeAB, double SlopeBC, double SlopeCA) -> double
	{
		double A = DiffSlopeVsIsocele(SlopeAB, SlopeCA);
		double B = DiffSlopeVsIsocele(SlopeBC, SlopeAB);
		double C = DiffSlopeVsIsocele(SlopeCA, SlopeBC);
		return FMath::Max3(A, B, C);
	};

	TFunction<int32(int32, int32)> MaxSlopeOf2 = [&TriangleSlopes](int32 A, int32 B) -> int32
	{
		return TriangleSlopes[A] > TriangleSlopes[B] ? A : B;
	};

	TFunction<int32(int32, int32, int32, int32)> MaxSlopeOf = [&TriangleSlopes, &MaxSlopeOf2](int32 A, int32 B, int32 C, int32 D) -> int32
	{
		return MaxSlopeOf2(A, MaxSlopeOf2(B, MaxSlopeOf2(C, D)));
	};

	const double Slope40 = ComputeSlope(*NodeCoordinates[4], *NodeCoordinates[0]);
	const double Slope01 = ComputeSlope(*NodeCoordinates[0], *NodeCoordinates[1]);
	const double Slope12 = ComputeSlope(*NodeCoordinates[1], *NodeCoordinates[2]);
	const double Slope23 = ComputeSlope(*NodeCoordinates[2], *NodeCoordinates[3]);
	const double Slope34 = ComputeSlope(*NodeCoordinates[3], *NodeCoordinates[4]);

	const double Slope02 = ComputeSlope(*NodeCoordinates[0], *NodeCoordinates[2]);
	const double Slope03 = ComputeSlope(*NodeCoordinates[0], *NodeCoordinates[3]);
	const double Slope13 = ComputeSlope(*NodeCoordinates[1], *NodeCoordinates[3]);
	const double Slope14 = ComputeSlope(*NodeCoordinates[1], *NodeCoordinates[4]);
	const double Slope24 = ComputeSlope(*NodeCoordinates[2], *NodeCoordinates[4]);

	const double Slope20 = SwapSlopeOrientation(Slope02);
	const double Slope30 = SwapSlopeOrientation(Slope03);
	const double Slope31 = SwapSlopeOrientation(Slope13);
	const double Slope41 = SwapSlopeOrientation(Slope14);
	const double Slope42 = SwapSlopeOrientation(Slope24);

	int32 TriangleIndices[10] = { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9 };

	// For each triangle, compute the max difference of slope vs an equilateral triangle
	TriangleSlopes[Triangle012] = MaxDiffSlopeVsEquilateral(Slope01, Slope12, Slope20);
	TriangleSlopes[Triangle013] = MaxDiffSlopeVsEquilateral(Slope01, Slope13, Slope30);
	TriangleSlopes[Triangle014] = MaxDiffSlopeVsEquilateral(Slope01, Slope14, Slope40);
	TriangleSlopes[Triangle023] = MaxDiffSlopeVsEquilateral(Slope02, Slope23, Slope30);
	TriangleSlopes[Triangle024] = MaxDiffSlopeVsEquilateral(Slope02, Slope24, Slope40);
	TriangleSlopes[Triangle034] = MaxDiffSlopeVsEquilateral(Slope03, Slope34, Slope40);
	TriangleSlopes[Triangle123] = MaxDiffSlopeVsEquilateral(Slope12, Slope23, Slope31);
	TriangleSlopes[Triangle124] = MaxDiffSlopeVsEquilateral(Slope12, Slope24, Slope41);
	TriangleSlopes[Triangle134] = MaxDiffSlopeVsEquilateral(Slope13, Slope34, Slope41);
	TriangleSlopes[Triangle234] = MaxDiffSlopeVsEquilateral(Slope23, Slope34, Slope42);

	// sort the triangles from the worth equilateral to the best
	Algo::Sort(TriangleIndices, [&TriangleSlopes](int32 A, int32 B) { return TriangleSlopes[A] > TriangleSlopes[B]; });

	// Remove the pentagons containing the worst triangle until ValidPentagons while ValidPentagons is not empty
	EPentagon ValidPentagons = EPentagon::All;
	for (int32 Index = 0; Index < 10; Index++)
	{
		const EPentagon Valid = TriangleToPentagon[TriangleIndices[Index]];
		EPentagon PreviousValid = ValidPentagons;
		ValidPentagons &= ~Valid;
		if (ValidPentagons == EPentagon::None)
		{
			ValidPentagons = PreviousValid;
			break;
		}
	}

	int32 SelectedTriangles[3];
	TFunction<void(int32, int32, int32, int32)> SelectInQuadrilateral = [&SelectedTriangles, &MaxSlopeOf](int32 TriangleABC, int32 TriangleABD, int32 TriangleACD, int32 TriangleBCD)
	{
		int32 NotSelected = MaxSlopeOf(TriangleABC, TriangleABD, TriangleACD, TriangleBCD);
		if (NotSelected == TriangleABC || NotSelected == TriangleACD)
		{
			SelectedTriangles[1] = TriangleABD;
			SelectedTriangles[2] = TriangleBCD;
		}
		else
		{
			SelectedTriangles[1] = TriangleABC;
			SelectedTriangles[2] = TriangleACD;
		}
	};

	// Get the selected triangles
	switch (ValidPentagons)
	{
	case EPentagon::P012_023_034:
		SelectedTriangles[0] = Triangle012;
		SelectedTriangles[1] = Triangle023;
		SelectedTriangles[2] = Triangle034;
		break;
	case EPentagon::P012_023_034_or_P012_024_234:
		SelectedTriangles[0] = Triangle012;
		SelectInQuadrilateral(Triangle023, Triangle024, Triangle034, Triangle234);
		break;
	case EPentagon::P012_023_034_or_P013_034_123:
		SelectedTriangles[0] = Triangle034;
		SelectInQuadrilateral(Triangle012, Triangle013, Triangle023, Triangle123);
		break;
	case EPentagon::P012_024_234:
		SelectedTriangles[0] = Triangle012;
		SelectedTriangles[1] = Triangle024;
		SelectedTriangles[2] = Triangle234;
		break;
	case EPentagon::P012_024_234_or_P014_124_234:
		SelectedTriangles[0] = Triangle234;
		SelectInQuadrilateral(Triangle012, Triangle014, Triangle024, Triangle124);
		break;
	case EPentagon::P013_034_123:
		SelectedTriangles[0] = Triangle013;
		SelectedTriangles[1] = Triangle034;
		SelectedTriangles[2] = Triangle123;
		break;
	case EPentagon::P013_034_123_or_P014_123_134:
		SelectedTriangles[0] = Triangle123;
		SelectInQuadrilateral(Triangle013, Triangle014, Triangle034, Triangle134);
		break;
	case EPentagon::P014_123_134:
		SelectedTriangles[0] = Triangle014;
		SelectedTriangles[1] = Triangle123;
		SelectedTriangles[2] = Triangle134;
		break;
	case EPentagon::P014_123_134_or_P014_124_234:
		SelectedTriangles[0] = Triangle014;
		SelectInQuadrilateral(Triangle123, Triangle124, Triangle134, Triangle234);
		break;
	case EPentagon::P014_124_234:
		SelectedTriangles[0] = Triangle014;
		SelectedTriangles[1] = Triangle124;
		SelectedTriangles[2] = Triangle234;
		break;
	default:
		break;
	}

	// Add the selected triangles to the mesh
	for (int32 Index = 0; Index < 3; ++Index)
	{
		Mesh.AddTriangle(Nodes[TrianglesOfPentagon[SelectedTriangles[Index]][0]]->GetGlobalIndex(), Nodes[TrianglesOfPentagon[SelectedTriangles[Index]][1]]->GetGlobalIndex(), Nodes[TrianglesOfPentagon[SelectedTriangles[Index]][2]]->GetGlobalIndex());
	}
}

}

bool FCycleTriangulator::CanCycleBeMeshed()
{
	bool bHasSelfIntersection = true;

	for (const FIsoSegment* Segment : Cycle)
	{
		if (CycleIntersectionTool.DoesIntersect(*Segment))
		{
			UE_LOGF(LogCADKernelBase, Warning, "A cycle of the surface %d is in self intersecting. The mesh of this sector is canceled.\n", Grid.GetFace().GetId());

#ifdef DEBUG_CAN_CYCLE_BE_FIXED_AND_MESHED
			if (bDisplay)
			{
				F3DDebugSession _(TEXT("New Segment"));
				F3DDebugSession A(TEXT("Segments"));
				Display(EGridSpace::UniformScaled, *Segment, 0, EVisuProperty::RedPoint);
				Display(EGridSpace::UniformScaled, *IntersectingSegment, 0, EVisuProperty::BluePoint);
				Wait();
			}
#endif
			return false;
		}
		CycleIntersectionTool.AddSegment(*Segment);
	}

	return true;
}

FIsoSegment* FCycleTriangulator::FindNextSegment(const FIsoSegment* StartSegment, const FIsoNode* StartNode, SlopeMethod GetSlope) const
{
	const FVector2d& StartPoint = StartNode->Get2DPoint(Space, Grid);
	const FVector2d& EndPoint = (StartSegment->IsFirstNode(StartNode)) ? StartSegment->GetSecondNode().Get2DPoint(Space, Grid) : StartSegment->GetFirstNode().Get2DPoint(Space, Grid);

#ifdef DEBUG_FIND_NEXTSEGMENT
	F3DDebugSession G(bDisplayStar, TEXT("FindNextSegment"));
	if (bDisplayStar)
	{
		F3DDebugSession _(TEXT("Start Segment"));
		Display(EGridSpace::Default2D, *StartNode);
		Display(EGridSpace::Default2D, *StartSegment);
		Display(Space, *StartNode);
		Display(Space, *StartSegment);
	}
#endif

	double ReferenceSlope = ComputePositiveSlope(StartPoint, EndPoint, 0);

	double MaxSlope = 8.1;
	FIsoSegment* NextSegment = nullptr;

	for (FIsoSegment* Segment : StartNode->GetConnectedSegments())
	{
		const FVector2d& OtherPoint = (Segment->IsFirstNode(StartNode)) ? Segment->GetSecondNode().Get2DPoint(Space, Grid) : Segment->GetFirstNode().Get2DPoint(Space, Grid);

		double Slope = GetSlope(StartPoint, OtherPoint, ReferenceSlope);
		if (Slope < SMALL_NUMBER_SQUARE) Slope = 8;

#ifdef DEBUG_FIND_NEXTSEGMENT
		if (bDisplayStar)
		{
			F3DDebugSession _(FString::Printf(TEXT("Segment slop %f"), Slope));
			Display(EGridSpace::Default2D, *Segment);
			Display(Space, *Segment);
		}
#endif

		if (Slope < MaxSlope || NextSegment == StartSegment)
		{
			NextSegment = Segment;
			MaxSlope = Slope;
		}
	}

	return NextSegment;
}

void FCycleTriangulator::InitializeArrays()
{
	// Get cycle's nodes and set segments as they have a triangle outside the cycle (to don't try to mesh outside the cycle)
	NodeCount = Cycle.Num();

	UnmeshedSegment.Reserve(NodeCount);
	NodeToIntersection.Reserve(NodeCount);

	const int32 EstimationOfSegmentCount = 5 * NodeCount;
	CycleIntersectionTool.Reserve(EstimationOfSegmentCount);
	SegmentStack.Reserve(EstimationOfSegmentCount);

	// Sub cycle array
	CandidateNodes.Reserve(NodeCount);
	SubCycleNodes.Reserve(NodeCount);
	PolygonNodes.Reserve(5);
	VertexIndexToSlopes.Reserve(NodeCount);
	MeanSquareLength = 0;

	CycleIntersectionTool.AddSegments(Cycle);
	SortCycleIntersectionToolIfNeeded();

}

void FCycleTriangulator::InitializeCycleForMeshing()
{
	using BoolIter = TArray<bool>::RangedForConstIteratorType;
	using SegmentIter = TArray<FIsoSegment*>::RangedForConstIteratorType;

	TFunction<bool(FIsoSegment*, EIsoSegmentStates)> HasTriangleOn = [](FIsoSegment* Segment, EIsoSegmentStates Side) -> bool
	{
		if (Segment->HasTriangleOn(Side))
		{
#ifdef CADKERNEL_DEBUG
			Wait();
			ensureCADKernel(false);
#endif
			return true;
		}
		Segment->SetHasTriangleOn(Side);
		return false;
	};

	BoolIter SegmentOrientation = CycleOrientation.begin();
	for (SegmentIter Segment = Cycle.begin(); Segment != Cycle.end(); ++Segment, ++SegmentOrientation)
	{
		if (*SegmentOrientation)
		{
			if (HasTriangleOn(*Segment, EIsoSegmentStates::RightTriangle))
			{
				return;
			}
		}
		else
		{
			if (HasTriangleOn(*Segment, EIsoSegmentStates::LeftTriangle))
			{
				return;
			}
		}
	}

	// If the Segment has 2 adjacent triangles, the segment is a inner cycle segment
	// It will have triangle in both side
	//
	//    X---------------X----------------X      X---------------X----------------X
	//    |                                |      |                                |  
	//    |         X--------------------X |      |                                |  
	//    |         |                    | |      |                                |  
	//    X---------X  <- inner segment  | |  or  X---------X  <- inner segment    |
	//    |         |                    | |      |                                |  
	//    |         X--------------------X |      |                                |  
	//    |                                |      |                                |
	//    X---------------X----------------X      X---------------X----------------X
	//
	for (FIsoSegment* Segment : Cycle)
	{
		if (Segment->HasTriangleOnRightAndLeft())
		{
			Segment->ResetHasTriangle();
		}

		if (Segment->GetFirstNode().GetConnectedSegments().Num() == 1 ||
			Segment->GetSecondNode().GetConnectedSegments().Num() == 1)
		{
			Segment->ResetHasTriangle();
		}
	}
}

void FCycleTriangulator::FillSegmentStack()
{
	const double SquareMinSize = FMath::Square(Grid.GetMinElementSize());

	TArray<double> SegmentLengths;
	TArray<int32> NodeIndex;
	NodeIndex.Reserve(NodeCount);
	SegmentLengths.Reserve(NodeCount);

	for (int32 Index = 0; Index < NodeCount; ++Index)
	{
		NodeIndex.Add(Index);
	}

	for (int32 Index = 0, NextIndex = 1; Index < NodeCount; ++Index, ++NextIndex)
	{
		if (NextIndex == NodeCount)
		{
			NextIndex = 0;
		}

		double Length = Cycle[Index]->Get3DLengthSquare(Grid);
		if (Length < SquareMinSize)
		{
			Cycle[Index]->SetAsDegenerated();
		}
		SegmentLengths.Add(Length);
	}

	// Sort the nodes to process segments from the longest to the shortest 
	NodeIndex.Sort([SegmentLengths](const int32& Index1, const int32& Index2) { return SegmentLengths[Index1] > SegmentLengths[Index2]; });

	for (int32 Index = 0; Index < NodeCount; ++Index)
	{
		SegmentStack.Add(Cycle[NodeIndex[Index]]);
	}
}

bool FCycleTriangulator::BuildSegmentIfNeeded(TArray<FIsoNode*>& CandidatNodes)
{
	FIsoNode* StartNode = CandidatNodes.Last();
	for (int32 Index = 0; Index < CandidatNodes.Num(); ++Index)
	{
		FIsoNode* EndNode = CandidatNodes[Index];
		if (StartNode == EndNode)
		{
			return false;
		}

		if (!BuildSegmentIfNeeded(StartNode, EndNode))
		{
			return false;
		}
		StartNode = EndNode;
	}
	return true;
}

bool FCycleTriangulator::BuildSegmentIfNeeded(FIsoNode* NodeA, FIsoNode* NodeB)
{
	FIsoSegment* ABSegment = NodeA->GetSegmentConnectedTo(NodeB);
	return BuildSegmentIfNeeded(NodeA, NodeB, ABSegment);
}

bool FCycleTriangulator::BuildSegmentIfNeeded(FIsoNode* NodeA, FIsoNode* NodeB, FIsoSegment* ABSegment)
{
	if (ABSegment)
	{
		if (ABSegment->IsFirstNode(NodeA))
		{
			if (ABSegment->HasTriangleOnLeft())
			{
				return false;
			}
			ABSegment->SetHasTriangleOnLeft();
		}
		else
		{
			if (ABSegment->HasTriangleOnRight())
			{
				return false;
			}
			ABSegment->SetHasTriangleOnRight();
		}
	}
	else
	{
		FIsoSegment& NewSegment = IsoSegmentFactory.New();
		NewSegment.Init(*NodeA, *NodeB, ESegmentType::Unknown);
		if (!NewSegment.ConnectToNode())
		{
			IsoSegmentFactory.DeleteEntity(&NewSegment);
			return false;
		}
		NewSegment.SetHasTriangleOnLeft();
		SegmentStack.Add(&NewSegment);

		bNeedIntersectionToolUpdate = true;
		CycleIntersectionTool.AddSegment(NewSegment);
	}

	return true;
};

void FCycleTriangulator::SortCycleIntersectionToolIfNeeded()
{
	if (!bNeedIntersectionToolUpdate)
	{
		return;
	}
	CycleIntersectionTool.Sort();
	bNeedIntersectionToolUpdate = false;
}

void FCycleTriangulator::CleanContext()
{
	bAcuteTriangle = false;

	SubCycleNodeCount = 0;
	SubCycleNodes.Reset();
	CandidateNodes.Reset();

	ExtremityCandidateNodes[0] = nullptr;
	ExtremityCandidateNodes[1] = nullptr;

	PolygonNodes.Reset();
	NodeToIntersection.Reset();

	VertexIndexToSlopes.Reset();

	FirstSideStartIndex = -1;
	FirstSideEndIndex = -1;
	TriangleThirdIndex = -1;
	FirstSideStartNode = nullptr;
	FirstSideEndNode = nullptr;
	TriangleThirdNode = nullptr;
}

}