// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Factory.h"

#include "Math/SlopeUtils.h"
#include "Mesh/MeshEnum.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionSegmentTool.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionIsoSegmentTool.h"

namespace UE::CADKernel
{
class FGrid;
class FFaceMesh;
class FIntersectionSegmentTool;
class FIsoSegment;
class FIsoTriangulator;

typedef TFunction<double(const FVector2d&, const FVector2d&, double)> SlopeMethod;
typedef TFunction<int32(int32)> NextIndexMethod;

struct FCandidateNode
{
	FIsoNode* Node;
	double SlopeAtStartNode;
	double SlopeAtEndNode;
	int32 Index;
	bool bIsValid;

	FCandidateNode() = default;
	FCandidateNode(FIsoNode* InNode, double InSlopeAtStartNode, double InSlopeAtEndNode, int32 InIndex)
		:Node(InNode)
		, SlopeAtStartNode(InSlopeAtStartNode)
		, SlopeAtEndNode(InSlopeAtEndNode)
		, Index(InIndex)
		, bIsValid(false)
	{}
};

struct FAdditionalIso
{
	int32 StartIndex = 0;
	FIsoNode* StartNode = nullptr;

	int32 EndIndex = 0;
	FIsoNode* EndNode = nullptr;

	int32 NodeIndices[2] = { -1, -1 };
	FIsoNode* Nodes[2] = { nullptr , nullptr };

	double EquilateralCriteria[2] = { Slope::TwoPiSlope, Slope::TwoPiSlope };

	bool bForceNodes = false;

	FAdditionalIso(int32 InStartIndex, int32 InEndIndex, FIsoNode* InStartNode, FIsoNode* InEndNode)
	: StartIndex(InStartIndex)
	, StartNode(InStartNode)
	, EndIndex(InEndIndex)
	, EndNode(InEndNode)
	{}

	int32 CandidateNodeCount()
	{
		return (Nodes[0] ? 1 : 0) + (Nodes[1] ? 1 : 0);
	}

	void AddTo(TArray<FIsoNode*>& PolygonNodes)
	{
		for(int32 Index = 0; Index < 2; Index++)
		{
			if (Nodes[Index])
			{
				PolygonNodes.Add(Nodes[Index]);
			}
		}
	}

	void RemoveCandidate(int32 Index)
	{
		NodeIndices[Index] =  -1;
		Nodes[Index] = nullptr;
		EquilateralCriteria[Index] = Slope::TwoPiSlope;
	}

	void Reset()
	{
		NodeIndices[0] = -1;
		NodeIndices[1] = -1;
		Nodes[0] = nullptr;
		Nodes[1] = nullptr;
		EquilateralCriteria[0] = Slope::TwoPiSlope;
		EquilateralCriteria[1] = Slope::TwoPiSlope;
		bForceNodes = false;
	}

	void RemoveCandidateIfPresent(FIsoNode* ForbiddenNode)
	{
		for (int32 Index = 0; Index < 2; Index++)
		{
			if (Nodes[Index] == ForbiddenNode)
			{
 				RemoveCandidate(Index);
				bForceNodes = false;
			}
		}
	}
};

using FNodeIntersectionCount = TPair<FCandidateNode*, int32>;

class FCycleTriangulator
{
private:
	const FGrid& Grid;

	int32 NodeCount = 0;
	const TArray<FIsoSegment*>& Cycle;
	const TArray<bool>& CycleOrientation;

	const EGridSpace Space = EGridSpace::UniformScaled;
	const FIntersectionIsoSegmentTool& InnerToOuterIsoSegmentsIntersectionTool;
	FFaceMesh& Mesh;
	TFactory<FIsoSegment>& IsoSegmentFactory;

	FIntersectionSegmentTool CycleIntersectionTool;

	TArray<FIsoSegment*> SegmentStack;

	// Array of segments that failed to mesh
	TArray<FIsoSegment*> UnmeshedSegment;

	bool bFirstRun = true;

	// Sub cycle data
	int32 SubCycleNodeCount = 0;
	TArray<FIsoNode*> SubCycleNodes;

	TArray<TPair<int32, double>> VertexIndexToSlopes;
	double MeanSquareLength = 0;
	bool bAcuteTriangle = false;

	int32 IntersectionCountAllowed = 0;
	int32 MaxIntersectionCounted = 0;

	int32 FirstSideStartIndex = -1;
	int32 FirstSideEndIndex = -1;
	int32 TriangleThirdIndex = -1;
	FIsoNode* FirstSideStartNode;
	FIsoNode* FirstSideEndNode;
	FIsoNode* TriangleThirdNode;

	// Candidates nodes for FindCandidateNodes
	TArray<FCandidateNode> CandidateNodes;
	FCandidateNode* ExtremityCandidateNodes[2] = { nullptr, nullptr };
	TArray<FNodeIntersectionCount> NodeToIntersection;

	// Final polygon nodes
	TArray<FIsoNode*> PolygonNodes;


	bool bNeedIntersectionToolUpdate = true;

	const SlopeMethod GetSlopeAtStartNode = CounterClockwiseSlope;
	const SlopeMethod GetSlopeAtEndNode = ClockwiseSlope;

public:
	FCycleTriangulator(FIsoTriangulator& IsoTriangulator, const TArray<FIsoSegment*>& InCycle, const TArray<bool>& InCycleOrientation);

	bool MeshCycle();

private:
	bool CanCycleBeMeshed();
	FIsoSegment* FindNextSegment(const FIsoSegment* StartSegment, const FIsoNode* StartNode, SlopeMethod GetSlope) const;

	void InitializeArrays();
	void CleanContext();
	void InitializeCycleForMeshing();
	void FillSegmentStack();

	bool BuildTheBestPolygon(FIsoSegment* SegmentToMesh, bool bOrientation);

	int32 NextIndex(int32 Index)
	{
		return (Index + 1) == SubCycleNodeCount ? 0 : Index + 1;
	};

	int32 PreviousIndex(int32 Index)
	{
		return Index == 0 ? SubCycleNodeCount - 1 : Index - 1;
	};

	/**
	 * Find all potential candidate nodes
	 * Find the node with the smallest slope between its adjacent segments     
	 */
	bool FindTheCycleToMesh(FIsoSegment* Segment, bool bOrientation, int32& StartIndexForMinLength);

	bool FindTheBestAcuteTriangle();

	/**
	 * Find all potential candidate nodes i.e. Nodes that can be linked with the Segment to build a valid triangle without intersection with other segments
	 * The candidates are saved in PotentialCandidateNodes
	 */
	bool FindCandidateNodes(int32 StartIndex);

	bool FindTheBestCandidateNode();

	bool BuildTheBestPolygonFromTheSelectedTriangle();

	void FindComplementaryNodes(FAdditionalIso& Side);
	void ValidateAddNodesAccordingSlopeWithSide(FAdditionalIso& Side);
	/**
	 * Valid additional nodes i.e. check if they can be added in The Side Of Selected Triangle
	 */
	void ValidateComplementaryNodesWithInsideAndIntersectionsCriteria(FAdditionalIso& Side);
	bool ValidComplementaryNodeOrDeleteIt(FAdditionalIso& Side, int32 Index);
	bool IsInnerSideSegmentInsideCycle(FAdditionalIso& Side);

	void SelectFinalNodes(FAdditionalIso& Side1, FAdditionalIso& Side2);
	void ComputeSideCandidateEquilateralCriteria(FAdditionalIso& Side);

	bool BuildTriangle(TArray<FIsoNode*>& CandidatNodes);

 	bool BuildQuadrilateral(TArray<FIsoNode*>& CandidatNodes);
	bool BuildPentagon(TArray<FIsoNode*>& CandidatNodes);

	bool BuildSmallPolygon(TArray<FIsoNode*>& CandidatNodes, bool bCheckIntersectionWithIso);


	bool BuildSegmentIfNeeded(TArray<FIsoNode*>& CandidatNodes);
	bool BuildSegmentIfNeeded(FIsoNode* NodeA, FIsoNode* NodeB);
	bool BuildSegmentIfNeeded(FIsoNode* NodeA, FIsoNode* NodeB, FIsoSegment* ABSegment);

	void SortCycleIntersectionToolIfNeeded();

	int32 IsIntersectingIso(const TArray<FIsoNode*>& Nodes)
	{
		for (int32 IndexA = 0; IndexA < Nodes.Num() - 1; ++IndexA)
		{
			for (int32 IndexB = IndexA + 1; IndexB < Nodes.Num() - 1; ++IndexB)
			{
				if (InnerToOuterIsoSegmentsIntersectionTool.DoesIntersect(*Nodes[IndexA], *Nodes[IndexB]))
				{
					return true;
				}
			}
		}
		return false;
	}

	int32 CountIntersectionWithIso()
	{
		FirstSideStartNode = SubCycleNodes[FirstSideStartIndex];
		FirstSideEndNode = SubCycleNodes[FirstSideEndIndex];

		NodeToIntersection.Reserve(CandidateNodes.Num());
		int32 Max = 0;
		for (int32 Index = 0; Index < CandidateNodes.Num(); ++Index)
		{
			FCandidateNode& CNode = CandidateNodes[Index];
			if (!CNode.bIsValid)
			{
				continue;
			}

			const FIsoNode* Node = CNode.Node;

			int32 IntersectionCount = InnerToOuterIsoSegmentsIntersectionTool.CountIntersections(*FirstSideStartNode, *Node);
			IntersectionCount = FMath::Max(IntersectionCount, InnerToOuterIsoSegmentsIntersectionTool.CountIntersections(*FirstSideEndNode, *Node));

			NodeToIntersection.Emplace(&CNode, IntersectionCount);
			if (Max < IntersectionCount)
			{
				Max = IntersectionCount;
			}
		}
		return Max;
	}

	/**
	 * In case of intersection with the loop, confirm that it was a real intersection and not a nearly parallel segment
	 */
	bool ConfirmIntersection(const FIsoNode* Start, const FIsoNode* End, const FIsoNode* Candidate, const FIsoSegment* IntersectedSegment) const;
};

namespace Polygon
{

enum ETriangleOfPentagon : uint8
{
	Triangle012 = 0,
	Triangle013,
	Triangle014,
	Triangle023,
	Triangle024,
	Triangle034,
	Triangle123,
	Triangle124,
	Triangle134,
	Triangle234,
	NoTriangle,
};

// Triangle to vertex indices (cf ETriangle)
constexpr uint8 TrianglesOfPentagon[10][3] = { { 0, 1, 2 }, { 0, 1, 3 }, { 0, 1, 4 }, { 0, 2, 3 }, { 0, 2, 4 }, { 0, 3, 4 }, { 1, 2, 3 }, { 1, 2, 4 }, { 1, 3, 4 }, { 2, 3, 4 } };

// Triangles of pentagon or pair of pentagon Triangles
enum class EPentagon : uint8
{
	None = 0x00u,
	P012_023_034 = 0x01u,
	P012_024_234 = 0x02u,
	P012_023_034_or_P012_024_234 = 0x03u,
	P013_034_123 = 0x04u,
	P012_023_034_or_P013_034_123 = 0x05u,
	P014_123_134 = 0x08u,
	P013_034_123_or_P014_123_134 = 0x0Cu,
	P014_124_234 = 0x10u,
	P012_024_234_or_P014_124_234 = 0x12u,
	P014_123_134_or_P014_124_234 = 0x18u,
	All = 0x1Fu,
};
ENUM_CLASS_FLAGS(EPentagon);

// pentagon (or pair) containing the triangle (cf ETriangle)
constexpr EPentagon TriangleToPentagon[10] = {
	EPentagon::P012_023_034_or_P012_024_234, // Triangle012
	EPentagon::P013_034_123,                 // Triangle013
	EPentagon::P014_123_134_or_P014_124_234, // Triangle014
	EPentagon::P012_023_034,                 // Triangle023
	EPentagon::P012_024_234,                 // Triangle024
	EPentagon::P012_023_034_or_P013_034_123, // Triangle034
	EPentagon::P013_034_123_or_P014_123_134, // Triangle123
	EPentagon::P014_124_234,                 // Triangle124
	EPentagon::P014_123_134,                 // Triangle134
	EPentagon::P012_024_234_or_P014_124_234  // Triangle234
};

void MeshTriangle(const FGrid& Grid, FIsoNode** Nodes, FFaceMesh& Mesh);
void MeshQuadrilateral(const FGrid& Grid, FIsoNode** Nodes, FFaceMesh& Mesh);
void MeshPentagon(const FGrid& Grid, FIsoNode** Nodes, FFaceMesh& Mesh);
}

}