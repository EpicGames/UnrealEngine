// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/IsoTriangulator.h"

#include "Core/Chrono.h"
#include "Math/MathConst.h"
#include "Math/Point.h"
#include "Mesh/Meshers/BowyerWatsonTriangulator.h"
#include "Mesh/Meshers/CycleTriangulator.h"
#include "Mesh/Meshers/IsoTriangulator/IsoCell.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Structure/Grid.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/LoopCleaner.h"
#include "Mesh/Structure/ThinZone2D.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Utils/ArrayUtils.h"

namespace UE::CADKernel
{

#ifdef DEBUG_BOWYERWATSON
bool FBowyerWatsonTriangulator::bDisplay = false;
#endif

namespace IsoTriangulatorImpl
{
const double MaxSlopeToBeIso = 0.125;

const double LimitValueMin(double Slope)
{
	return Slope - MaxSlopeToBeIso;
}

const double LimitValueMax(double Slope)
{
	return Slope + MaxSlopeToBeIso;
}

struct FCandidateSegment
{
	FLoopNode& StartNode;
	FLoopNode& EndNode;
	double Length;

	FCandidateSegment(const FGrid& Grid, FLoopNode& Node1, FLoopNode& Node2)
		: StartNode(Node1)
		, EndNode(Node2)
	{
		Length = FVector2d::Distance(Node1.Get2DPoint(EGridSpace::UniformScaled, Grid), Node2.Get2DPoint(EGridSpace::UniformScaled, Grid));
	}
};

}

FIsoTriangulator::FIsoTriangulator(FGrid& InGrid, FFaceMesh& OutMesh, const FMeshingTolerances& InTolerances)
	: Grid(InGrid)
	, Mesh(OutMesh)
	, LoopSegmentsIntersectionTool(InGrid, InTolerances.GeometricTolerance)
	, InnerSegmentsIntersectionTool(InGrid, InTolerances.GeometricTolerance)
	, InnerToOuterIsoSegmentsIntersectionTool(InGrid, InTolerances.GeometricTolerance)
	, ThinZoneIntersectionTool(InGrid, InTolerances.GeometricTolerance)
	, Tolerances(InTolerances)
{
	FinalInnerSegments.Reserve(3 * Grid.InnerNodesCount());
	IndexOfLowerLeftInnerNodeSurroundingALoop.Reserve(Grid.GetLoopCount());

#ifdef DEBUG_ISOTRIANGULATOR

#ifdef DEBUG_ONLY_SURFACE_TO_DEBUG
	if (Grid.GetFace().GetId() == FaceToDebug)
#endif
	{
		bDisplay = true;
	}

#endif
}

bool FIsoTriangulator::Triangulate()
{
	EGridSpace DisplaySpace = EGridSpace::UniformScaled;

	// =============================================================================================================
	// Build the first elements (IsoNodes (i.e. Inner nodes), Loops nodes, and knows segments 
	// =============================================================================================================

	BuildNodes();

	FillMeshNodes();
	BuildLoopSegments();

	FLoopCleaner LoopCleaner(*this);
	if (!LoopCleaner.Run())
	{
		UE_LOGF(LogCADKernelBase, Warning, "The meshing of the surface %d failed due to a degenerated loop\n", Grid.GetFace().GetId());
		return false;
	}

	//Fill Intersection tool
	LoopSegmentsIntersectionTool.Empty(LoopSegments.Num());
	LoopSegmentsIntersectionTool.AddSegments(LoopSegments);
	LoopSegmentsIntersectionTool.Sort();

	GetThinZonesMesh();

	LoopSegmentsIntersectionTool.AddSegments(ThinZoneSegments);
	LoopSegmentsIntersectionTool.Sort();

	FinalToLoops.Append(ThinZoneSegments);


	BuildInnerSegments();

	// =============================================================================================================
	// =============================================================================================================

	BuildInnerSegmentsIntersectionTool();

	// =============================================================================================================
	// 	   For each cell
	// 	      - Connect loops together and to cell vertices
	// 	           - Find subset of node of each loop
	// 	           - build Delaunay connection
	// 	           - find the shortest segment to connect each connected loop by Delaunay
	// =============================================================================================================

	ConnectCellLoops();

	SelectSegmentsToLinkInnerToLoop();

	// =============================================================================================================
	// Make the final tessellation 
	// =============================================================================================================

	// Triangulate between inner grid boundary and loops
	TriangulateOverCycle(EGridSpace::Scaled);

	// Finalize the mesh by the tessellation of the inner grid
	TriangulateInnerNodes();

	return true;
}

void FIsoTriangulator::BuildNodes()
{
	LoopNodeCount = 0;
	for (const TArray<FVector2d>& LoopPoints : Grid.GetLoops2D(EGridSpace::Default2D))
	{
		LoopNodeCount += (int32)LoopPoints.Num();
	}
	LoopStartIndex.Reserve(Grid.GetLoops2D(EGridSpace::Default2D).Num());
	LoopNodes.Reserve((int32)(LoopNodeCount * 1.2 + 5)); // reserve more in case it need to create complementary nodes

	// Loop nodes
	int32 NodeIndex = 0;
	int32 LoopIndex = 0;
	for (const TArray<FVector2d>& LoopPoints : Grid.GetLoops2D(EGridSpace::Default2D))
	{
		LoopStartIndex.Add(LoopNodeCount);
		const TArray<int32>& LoopIds = Grid.GetNodeIdsOfFaceLoops()[LoopIndex];
		FLoopNode* NextNode = nullptr;
		FLoopNode* FirstNode = &LoopNodes.Emplace_GetRef(LoopIndex, 0, NodeIndex++, LoopIds[0]);
		FLoopNode* PreviousNode = FirstNode;
		for (int32 Index = 1; Index < LoopPoints.Num(); ++Index)
		{
			NextNode = &LoopNodes.Emplace_GetRef(LoopIndex, Index, NodeIndex++, LoopIds[Index]);
			PreviousNode->SetNextConnectedNode(NextNode);
			NextNode->SetPreviousConnectedNode(PreviousNode);
			PreviousNode = NextNode;
		}
		PreviousNode->SetNextConnectedNode(FirstNode);
		FirstNode->SetPreviousConnectedNode(PreviousNode);
		LoopIndex++;
	}

	// Inner node
	InnerNodes.Reserve(Grid.InnerNodesCount());
	GlobalIndexToIsoInnerNodes.Init(nullptr, Grid.GetTotalCuttingCount());

	InnerNodeCount = 0;
	for (int32 Index = 0; Index < (int32)Grid.GetTotalCuttingCount(); ++Index)
	{
		if (Grid.IsNodeInsideAndMeshable(Index))
		{
			FIsoInnerNode& Node = InnerNodes.Emplace_GetRef(Index, NodeIndex++, InnerNodeCount++);
			GlobalIndexToIsoInnerNodes[Index] = &Node;
		}
	}
}

void FIsoTriangulator::FillMeshNodes()
{
	int32 TriangleNum = 50 + (int32)((2 * InnerNodeCount + LoopNodeCount) * 1.1);
	Mesh.Init(TriangleNum, InnerNodeCount + LoopNodeCount);

	TArray<FVector>& InnerNodeCoordinates = Mesh.GetNodeCoordinates();
	InnerNodeCoordinates.Reserve(InnerNodeCount);
	for (int32 Index = 0; Index < (int32)Grid.GetInner3DPoints().Num(); ++Index)
	{
		if (Grid.IsNodeInsideAndMeshable(Index))
		{
			InnerNodeCoordinates.Emplace(Grid.GetInner3DPoints()[Index]);
		}
	}

	int32 StartId = Mesh.RegisterCoordinates();
	for (FIsoInnerNode& Node : InnerNodes)
	{
		Node.OffsetId(StartId);
	}

	Mesh.VerticesGlobalIndex.SetNum(InnerNodeCount + LoopNodeCount);
	int32 Index = 0;
	for (FLoopNode& Node : LoopNodes)
	{
		Mesh.VerticesGlobalIndex[Index++] = Node.GetNodeId();
	}

	for (FIsoInnerNode& Node : InnerNodes)
	{
		Mesh.VerticesGlobalIndex[Index++] = Node.GetNodeId();
	}

	for (FLoopNode& Node : LoopNodes)
	{
		Mesh.Normals.Emplace(Node.GetNormal(Grid));
	}

	for (FIsoInnerNode& Node : InnerNodes)
	{
		Mesh.Normals.Emplace(Node.GetNormal(Grid));
	}

	for (FLoopNode& Node : LoopNodes)
	{
		const FVector2d& UVCoordinate = Node.Get2DPoint(EGridSpace::Scaled, Grid);
		Mesh.UVMap.Emplace(UVCoordinate.X, UVCoordinate.Y);
	}

	for (FIsoInnerNode& Node : InnerNodes)
	{
		const FVector2d& UVCoordinate = Node.Get2DPoint(EGridSpace::Scaled, Grid);
		Mesh.UVMap.Emplace(UVCoordinate.X, UVCoordinate.Y);
	}
}

void FIsoTriangulator::BuildLoopSegments()
{
	LoopSegments.Reserve(LoopNodeCount);

	int32 LoopIndex = 0;
	for (FLoopNode& Node : LoopNodes)
	{
		if (Node.IsDelete())
		{
			continue;
		}

		FIsoSegment& Segment = IsoSegmentFactory.New();
		Segment.Init(Node, Node.GetNextNode(), ESegmentType::Loop);
		if (Segment.ConnectToNode())
		{
			LoopSegments.Add(&Segment);
		}
		else
		{
			IsoSegmentFactory.DeleteEntity(&Segment);
		}
	}
}

void FIsoTriangulator::GetThinZonesMesh()
{
	TMap<int32, FLoopNode*> IndexToNode;
	for (FLoopNode& Node : LoopNodes)
	{
		if (Node.IsDelete())
		{
			continue;
		}

		IndexToNode.Add(Node.GetNodeId(), &Node);
	}

	{
		for (const FThinZone2D& ThinZone : Grid.GetFace().GetThinZones())
		{
			GetThinZoneMesh(IndexToNode, ThinZone);
		}
	}

	ThinZoneIntersectionTool.Empty(0);
}

void FIsoTriangulator::GetThinZoneMesh(const TMap<int32, FLoopNode*>& IndexToNode, const FThinZone2D& ThinZone)
{
	using namespace IsoTriangulatorImpl;

	TArray<TPair<int32, FPairOfIndex>> CrossZoneElements;
	TArray<FCandidateSegment> MeshOfThinZones;

	FAddMeshNodeFunc AddElement = [&CrossZoneElements](const int32 NodeIndice, const FVector2d& MeshNode2D, double MeshingTolerance3D, const FEdgeSegment& EdgeSegment, const FPairOfIndex& OppositeNodeIndices)
	{
		if (CrossZoneElements.Num() && CrossZoneElements.Last().Key == NodeIndice)
		{
			CrossZoneElements.Last().Value.Add(OppositeNodeIndices);
		}
		else
		{
			CrossZoneElements.Emplace(NodeIndice, OppositeNodeIndices);
		}
	};

	FReserveContainerFunc Reserve = [&CrossZoneElements](int32 MeshVertexCount)
	{
		CrossZoneElements.Reserve(CrossZoneElements.Num() + MeshVertexCount);
	};

	ThinZone.GetFirstSide().GetExistingMeshNodes(Grid.GetFace(), Mesh.GetMeshModel(), Reserve, AddElement, /*bWithTolerance*/ false);
	ThinZone.GetSecondSide().GetExistingMeshNodes(Grid.GetFace(), Mesh.GetMeshModel(), Reserve, AddElement, /*bWithTolerance*/ false);

	MeshOfThinZones.Reserve(CrossZoneElements.Num() * 2);

	TFunction<void(FLoopNode*, FLoopNode*)> AddSegmentFromNode = [&MeshOfThinZones, this](FLoopNode* NodeA, FLoopNode* NodeB)
	{
		if (!NodeA)
		{
			return;
		}

		if (!NodeB)
		{
			return;
		}

		if (&NodeA->GetPreviousNode() == NodeB || &NodeB->GetNextNode() == NodeA)
		{
			return;
		}

		if (NodeA->GetSegmentConnectedTo(NodeB))
		{
			return;
		}

		const FVector2d& CoordinateA = NodeA->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& CoordinateB = NodeB->Get2DPoint(EGridSpace::UniformScaled, Grid);

		// Is Outside and not too flat at Node1
		const double FlatAngle = 0.1;
		if (NodeA->IsSegmentBeInsideFace(CoordinateB, Grid, FlatAngle))
		{
			return;
		}

		// Is Outside and not too flat at Node2
		if (NodeB->IsSegmentBeInsideFace(CoordinateA, Grid, FlatAngle))
		{
			return;
		}

		MeshOfThinZones.Emplace(Grid, *NodeA, *NodeB);
	};

	TFunction<void(int32, int32)> AddSegment = [&IndexToNode, AddSegmentFromNode](int32 IndexNodeA, int32 IndexNodeB)
	{
		if (IndexNodeA < 0 || IndexNodeB < 0)
		{
			return;
		}

		if (IndexNodeA == IndexNodeB)
		{
			return;
		}

		FLoopNode* const* NodeA = IndexToNode.Find(IndexNodeA);
		FLoopNode* const* NodeB = IndexToNode.Find(IndexNodeB);
		if (NodeA && NodeB)
		{
			AddSegmentFromNode(*NodeA, *NodeB);
		}
	};

	for (const TPair<int32, FPairOfIndex>& CrossZoneElement : CrossZoneElements)
	{
		AddSegment(CrossZoneElement.Key, CrossZoneElement.Value[0]);
		AddSegment(CrossZoneElement.Key, CrossZoneElement.Value[1]);
	}

	Algo::Sort(MeshOfThinZones, [](const FCandidateSegment& SegmentA, const FCandidateSegment& SegmentB) { return SegmentA.Length < SegmentB.Length; });

	ThinZoneIntersectionTool.Reserve(ThinZoneIntersectionTool.Count() + MeshOfThinZones.Num());

	for (FCandidateSegment& CandidateSegment : MeshOfThinZones)
	{
		if (FIsoSegment::IsItAlreadyDefined(&CandidateSegment.StartNode, &CandidateSegment.EndNode))
		{
			continue;
		}

		if (LoopSegmentsIntersectionTool.DoesIntersect(CandidateSegment.StartNode, CandidateSegment.EndNode))
		{
			continue;
		}

		if (ThinZoneIntersectionTool.DoesIntersect(CandidateSegment.StartNode, CandidateSegment.EndNode))
		{
			continue;
		}

		FIsoSegment& Segment = IsoSegmentFactory.New();
		Segment.Init(CandidateSegment.StartNode, CandidateSegment.EndNode, ESegmentType::ThinZone);

		if(Segment.ConnectToNode())
		{
			CandidateSegment.StartNode.SetThinZoneNodeMarker();
			CandidateSegment.EndNode.SetThinZoneNodeMarker();
			Segment.SetFinalMarker();
			ThinZoneSegments.Add(&Segment);
			ThinZoneIntersectionTool.AddSegment(Segment);
		}
		else
		{
			IsoSegmentFactory.DeleteEntity(&Segment);
		}
	}
}

void FIsoTriangulator::BuildInnerSegments()
{
	// Build segments according to the Grid following u then following v
	// Build segment must not be in intersection with the loop
	int32 NumU = Grid.GetCuttingCount(EIso::IsoU);
	int32 NumV = Grid.GetCuttingCount(EIso::IsoV);

	LoopSegmentsIntersectionTool.Reserve(InnerSegmentsIntersectionTool.Count());


	// Check if the loop tangents the grid between Node1 and Node 2
	//                            
	//                       \   /  Loop		                       \     /  Loop
	//                        \./ 				                        \   / 
	//        Node1 *------------------* Node2 	        Node1 *----------\./-------* Node2 
	//                                                                       
	//
	TFunction<bool(const FVector2d&, const FVector2d&, const ESegmentType, const double)> AlmostHitsLoop = [&](const FVector2d& Node1, const FVector2d& Node2, const ESegmentType InType, const double Tolerance) -> bool
	{
		if (InType == ESegmentType::IsoV)
		{
			for (const TArray<FVector2d>& Loop : Grid.GetLoops2D(EGridSpace::UniformScaled))
			{
				for (const FVector2d& LoopPoint : Loop)
				{
					if (FMath::IsNearlyEqual(LoopPoint.Y, Node1.Y, Tolerance))
					{
						if (Node1.X - DOUBLE_SMALL_NUMBER < LoopPoint.X && LoopPoint.X < Node2.X + DOUBLE_SMALL_NUMBER)
						{
							return true;
						}
					}
				}
			}
		}
		else
		{
			for (const TArray<FVector2d>& Loop : Grid.GetLoops2D(EGridSpace::UniformScaled))
			{
				for (const FVector2d& LoopPoint : Loop)
				{
					if (FMath::IsNearlyEqual(LoopPoint.X, Node1.X, Tolerance))
					{
						if (Node1.Y - DOUBLE_SMALL_NUMBER < LoopPoint.Y && LoopPoint.Y < Node2.Y + DOUBLE_SMALL_NUMBER)
						{
							return true;
						}
					}
				}
			}
		}
		return false;
	};

	TFunction<void(const int32, const int32, const ESegmentType)> AddToInnerToOuterSegmentsIntersectionTool = [&](const int32 IndexNode1, const int32 IndexNode2, const ESegmentType InType)
	{
		const FVector2d& Point1 = Grid.GetInner2DPoint(EGridSpace::UniformScaled, IndexNode1);
		const FVector2d& Point2 = Grid.GetInner2DPoint(EGridSpace::UniformScaled, IndexNode2);

		InnerToOuterIsoSegmentsIntersectionTool.AddIsoSegment(Point1, Point2, InType);
	};

	TFunction<void(const int32, const int32, const ESegmentType)> AddToInnerSegments = [&](const int32 IndexNode1, const int32 IndexNode2, const ESegmentType InType)
	{
		FIsoInnerNode& Node1 = *GlobalIndexToIsoInnerNodes[IndexNode1];
		FIsoInnerNode& Node2 = *GlobalIndexToIsoInnerNodes[IndexNode2];
		FIsoSegment& Segment = IsoSegmentFactory.New();
		Segment.Init(Node1, Node2, InType);
		if (Segment.ConnectToNode())
		{
			FinalInnerSegments.Add(&Segment);
		}
		else
		{
			IsoSegmentFactory.DeleteEntity(&Segment);
		}
	};

	TFunction<void(const int32, const int32, const ESegmentType, const double)> BuildSegmentIfValid = [&](const int32 IndexNode1, const int32 IndexNode2, const ESegmentType InType, const double Tolerance)
	{
		if (Grid.IsNodeOusideFaceButClose(IndexNode1) && Grid.IsNodeOusideFaceButClose(IndexNode2))
		{
			AddToInnerToOuterSegmentsIntersectionTool(IndexNode1, IndexNode2, InType);
			return;
		}

		if (Grid.IsNodeOutsideFace(IndexNode1) && Grid.IsNodeOutsideFace(IndexNode2))
		{
			return;
		}

		if (Grid.IsNodeInsideAndCloseToLoop(IndexNode1) && Grid.IsNodeInsideAndCloseToLoop(IndexNode2))
		{
			if (LoopSegmentsIntersectionTool.DoesIntersect(Grid.GetInner2DPoint(EGridSpace::UniformScaled, IndexNode1), Grid.GetInner2DPoint(EGridSpace::UniformScaled, IndexNode2))
				|| AlmostHitsLoop(Grid.GetInner2DPoint(EGridSpace::UniformScaled, IndexNode1), Grid.GetInner2DPoint(EGridSpace::UniformScaled, IndexNode2), InType, Tolerance))
			{
				AddToInnerToOuterSegmentsIntersectionTool(IndexNode1, IndexNode2, InType);
			}
			else
			{
				AddToInnerSegments(IndexNode1, IndexNode2, InType);
			}

			return;
		}

		if (Grid.IsNodeInsideAndMeshable(IndexNode1) && Grid.IsNodeInsideAndMeshable(IndexNode2))
		{
			AddToInnerSegments(IndexNode1, IndexNode2, InType);
			return;
		}

		if (Grid.IsNodeInsideButTooCloseToLoop(IndexNode1) && Grid.IsNodeInsideButTooCloseToLoop(IndexNode2))
		{
			return;
		}

		AddToInnerToOuterSegmentsIntersectionTool(IndexNode1, IndexNode2, InType);
	};

	TFunction<const TArray<double>(const TArray<double>&)> ComputeLocalTolerance = [](const TArray<double>& UniformCutting) -> const TArray<double>
	{
		const int32 Num = UniformCutting.Num();
		TArray<double> TolerancesAlongU;
		TArray<double> Temp;
		TolerancesAlongU.Reserve(Num);
		Temp.Reserve(Num);
		for (int32 Index = 1; Index < Num; Index++)
		{
			Temp.Add((UniformCutting[Index] - UniformCutting[Index - 1]));
		}
		TolerancesAlongU.Add(Temp[0] * 0.1);
		for (int32 Index = 1; Index < Num - 1; Index++)
		{
			TolerancesAlongU.Add((Temp[Index] + Temp[Index - 1]) * 0.05);
		}
		TolerancesAlongU.Add(Temp.Last() * 0.1);
		return MoveTemp(TolerancesAlongU);
	};

	// Process along V
	{
		TArray<double> TolerancesAlongU = ComputeLocalTolerance(Grid.GetUniformCuttingCoordinatesAlongIso(EIso::IsoU));
		for (int32 UIndex = 0; UIndex < NumU; UIndex++)
		{
			for (int32 VIndex = 0; VIndex < NumV - 1; VIndex++)
			{
				BuildSegmentIfValid(Grid.GobalIndex(UIndex, VIndex), Grid.GobalIndex(UIndex, VIndex + 1), ESegmentType::IsoU, TolerancesAlongU[UIndex]);
			}
		}
	}

	// Process along U
	{
		TArray<double> TolerancesAlongV = ComputeLocalTolerance(Grid.GetUniformCuttingCoordinatesAlongIso(EIso::IsoV));
		for (int32 VIndex = 0; VIndex < NumV; VIndex++)
		{
			for (int32 UIndex = 0; UIndex < NumU - 1; UIndex++)
			{
				BuildSegmentIfValid(Grid.GobalIndex(UIndex, VIndex), Grid.GobalIndex(UIndex + 1, VIndex), ESegmentType::IsoV, TolerancesAlongV[VIndex]);
			}
		}
	}

	InnerToOuterIsoSegmentsIntersectionTool.Sort();
}

void FIsoTriangulator::BuildInnerSegmentsIntersectionTool()
{
	// Find Boundary Segments Of Inner Triangulation
	// 
	// A pixel grid is build. 
	// A pixel is the quadrangle of the inner grid
	// The grid pixel are initialized to False
	//
	// A pixel is True if one of its boundary segment does not exist 
	// The inner of the grid is all pixel False
	// The boundary of the inner triangulation is defined by all segments adjacent to different cells 
	// 
	//    T      T	     T                                                  
	//       0 ----- 0 												     0 ----- 0 
	//    T  |   F   |   T       T       T      T         			     |       |    
	//       0 ----- 0               0 ----- 0 						     0       0               0 ----- 0 
	//    T  |   F   |   T       T   |   F   |  T					     |       |               |       |  
	//       0 ----- 0 ----- 0 ----- 0 ----- 0 						     0       0 ----- 0 ----- 0       0 	
	//    T  |   F   |   F   |   F   |   F   |	T					     |                               |	
	//       0 ----- 0 ----- 0 ----- 0 ----- 0 						     0                               0 	
	//    T  |   F   |   F   |   F   |   F   |	T					     |                               |	
	//       0 ----- 0 ----- 0 ----- 0 ----- 0 						     0 ----- 0 ----- 0 ----- 0 ----- 0 	
	//    T      T 		 T		 T		 T		T					                
	// 
	// https://docs.google.com/presentation/d/1qUVOH-2kU_QXBVKyRUcdDy1Y6WGkcaJCiaS8wGjSZ6M/edit?usp=sharing
	// Slide "Boundary Segments Of Inner Triangulation"

	int32 NumU = Grid.GetCuttingCount(EIso::IsoU);
	int32 NumV = Grid.GetCuttingCount(EIso::IsoV);

	TArray<uint8> Pixel;
	Pixel.Init(0, Grid.GetTotalCuttingCount());

	// A pixel is True if one of its boundary segment does not exist 
	for (int32 IndexV = 0, Index = 0; IndexV < NumV; ++IndexV)
	{
		for (int32 IndexU = 0; IndexU < NumU; ++IndexU, ++Index)
		{
			if (!Grid.IsNodeInsideAndMeshable(Index))
			{
				continue;
			}

			if (!GlobalIndexToIsoInnerNodes[Index]->IsLinkedToNextU())
			{
				Pixel[Index] = true;
				Pixel[Index - NumU] = true;
			}

			if (!GlobalIndexToIsoInnerNodes[Index]->IsLinkedToPreviousU())
			{
				Pixel[Index - 1] = true;
				Pixel[Index - 1 - NumU] = true;
			}

			if (!GlobalIndexToIsoInnerNodes[Index]->IsLinkedToNextV())
			{
				Pixel[Index] = true;
				Pixel[Index - 1] = true;
			}

			if (!GlobalIndexToIsoInnerNodes[Index]->IsLinkedToPreviousV())
			{
				Pixel[Index - NumU] = true;
				Pixel[Index - NumU - 1] = true;
			}
		}
	}

	// The boundary of the inner triangulation is defined by all segments adjacent to a "True" cell 
	// These segments are added to InnerSegmentsIntersectionTool
	InnerSegmentsIntersectionTool.Reserve((int32)FinalInnerSegments.Num());

	for (FIsoSegment* Segment : FinalInnerSegments)
	{
		int32 IndexFirstNode = Segment->GetFirstNode().GetIndex();
		int32 IndexSecondNode = 0;
		switch (Segment->GetType())
		{
		case ESegmentType::IsoU:
			IndexSecondNode = IndexFirstNode - NumU;
			break;
		case ESegmentType::IsoV:
			IndexSecondNode = IndexFirstNode - 1;
			break;
		default:
			ensureCADKernel(false);
		}
		if (Pixel[IndexFirstNode] || Pixel[IndexSecondNode])
		{
			InnerSegmentsIntersectionTool.AddSegment(*Segment);
		}
	}

	FindInnerGridCellSurroundingSmallLoop();

	// initialize the intersection tool
	InnerSegmentsIntersectionTool.Sort();
}

// =============================================================================================================
// 	   For each cell
// 	      - Connect loops together and to cell vertices
// 	           - Find subset of node of each loop
// 	           - build Delaunay connection
// 	           - find the shortest segment to connect each connected loop by Delaunay
// =============================================================================================================
void FIsoTriangulator::ConnectCellLoops()
{
	TArray<FCell> Cells;
	FindCellContainingBoundaryNodes(Cells);

	for (FCell& Cell : Cells)
	{
		InitCellCorners(Cell);
		Cell.InitLoopConnexions();
	}

	InnerToLoopCandidateSegments.Reserve(Cells.Num() * 2);

	FinalToLoops.Reserve(LoopNodeCount + InnerNodeCount);
	for (FCell& Cell : Cells)
	{
		if (Cell.CellLoops.Num())
		{
			Cell.FindCandidateToConnectLoopsByNeighborhood();

			FindCandidateToConnectCellCornerToLoops(Cell);

			Cell.SelectSegmentToConnectLoops(IsoSegmentFactory);
			Cell.SelectSegmentToConnectLoopToCorner(IsoSegmentFactory);
			Cell.CheckAllLoopsConnectedTogetherAndConnect();
		}

		FinalToLoops.Append(Cell.FinalSegments);
	}
}

void FIsoTriangulator::FindCellContainingBoundaryNodes(TArray<FCell>& Cells)
{
	TArray<int32> NodeToCellIndices;
	TArray<int32> SortedIndex;

	const int32 CountU = Grid.GetCuttingCount(EIso::IsoU);
	const int32 CountV = Grid.GetCuttingCount(EIso::IsoV);
	const int32 MaxUV = Grid.GetTotalCuttingCount();

	const TArray<double>& IsoUCoordinates = Grid.GetUniformCuttingCoordinatesAlongIso(EIso::IsoU);
	const TArray<double>& IsoVCoordinates = Grid.GetUniformCuttingCoordinatesAlongIso(EIso::IsoV);

	NodeToCellIndices.Reserve(LoopNodeCount);
	{
		int32 IndexU = 0;
		int32 IndexV = 0;
		int32 Index = 0;
		int32 DeletedNodeCount = 0;

		for (const FLoopNode& LoopPoint : LoopNodes)
		{
			if (!LoopPoint.IsDelete())
			{
				const FVector2d& Coordinate = LoopPoint.Get2DPoint(EGridSpace::UniformScaled, Grid);
				ArrayUtils::FindCoordinateIndex(IsoUCoordinates, Coordinate.X, IndexU);
				ArrayUtils::FindCoordinateIndex(IsoVCoordinates, Coordinate.Y, IndexV);

				NodeToCellIndices.Emplace(IndexV * CountU + IndexU);
			}
			else
			{
				DeletedNodeCount++;
				NodeToCellIndices.Emplace(MaxUV);
			}
			SortedIndex.Emplace(Index++);
		}

		Algo::Sort(SortedIndex, [&](const int32& Index1, const int32& Index2)
			{
				return (NodeToCellIndices[Index1] < NodeToCellIndices[Index2]);
			});

		SortedIndex.SetNum(SortedIndex.Num() - DeletedNodeCount);
	}

	int32 CountOfCellsFilled = 1;
	{
		int32 CellIndex = NodeToCellIndices[0];
		for (int32 Index : SortedIndex)
		{
			if (CellIndex != NodeToCellIndices[Index])
			{
				CellIndex = NodeToCellIndices[Index];
				CountOfCellsFilled++;
			}
		}
	}

	// build Cells
	{
		Cells.Reserve(CountOfCellsFilled);
		int32 CellIndex = NodeToCellIndices[SortedIndex[0]];
		TArray<FLoopNode*> CellNodes;
		CellNodes.Reserve(LoopNodeCount);

		for (int32 Index : SortedIndex)
		{
			if (CellIndex != NodeToCellIndices[Index])
			{
				Cells.Emplace(CellIndex, CellNodes, *this);

				CellIndex = NodeToCellIndices[Index];
				CellNodes.Reset(LoopNodeCount);
			}

			FLoopNode& LoopNode = LoopNodes[Index];
			if (!LoopNode.IsDelete())
			{
				CellNodes.Add(&LoopNode);
			}
		}
		Cells.Emplace(CellIndex, CellNodes, *this);
	}
}

bool FIsoTriangulator::CanCycleBeMeshed(const TArray<FIsoSegment*>& Cycle, FIntersectionSegmentTool& CycleIntersectionTool)
{
	bool bHasSelfIntersection = true;

	for (const FIsoSegment* Segment : Cycle)
	{
		if (CycleIntersectionTool.DoesIntersect(*Segment))
		{
			UE_LOGF(LogCADKernelBase, Warning, "A cycle of the surface %d is in self intersecting. The mesh of this sector is canceled.\n", Grid.GetFace().GetId());

			return false;
		}
	}

	return true;
}

void FIsoTriangulator::MeshCycle(const TArray<FIsoSegment*>& Cycle, const TArray<bool>& CycleOrientation)
{
	switch (Cycle.Num())
	{
	case 2:
		return;
	case 3:
		return MeshCycleOf<3>(Cycle, CycleOrientation, Polygon::MeshTriangle);
	case 4:
		return MeshCycleOf<4>(Cycle, CycleOrientation, Polygon::MeshQuadrilateral);
	case 5:
		return MeshCycleOf<5>(Cycle, CycleOrientation, Polygon::MeshPentagon);
	default:
		MeshLargeCycle(Cycle, CycleOrientation);
	}
}

void FIsoTriangulator::MeshLargeCycle(const TArray<FIsoSegment*>& Cycle, const TArray<bool>& CycleOrientation)
{
	FCycleTriangulator CycleTriangulator(*this, Cycle, CycleOrientation);
	CycleTriangulator.MeshCycle();
}

/**
 * The purpose is to add surrounding segments to the small loop to intersection tool to prevent traversing inner segments
 * A loop is inside inner segments
 *									|			 |
 *								   -----------------
 *									|	 XXX	 |
 *									|	XXXXX	 |
 *									|	 XXX	 |
 *								   -----------------
 *									|			 |
 *
 */
void FIsoTriangulator::FindInnerGridCellSurroundingSmallLoop()
{
	if (GlobalIndexToIsoInnerNodes.Num() == 0)
	{
		// No inner node
		return;
	}

	// when an internal loop is inside inner UV cell
	int32 NumU = Grid.GetCuttingCount(EIso::IsoU);
	int32 NumV = Grid.GetCuttingCount(EIso::IsoV);
	const TArray<double>& UCoordinates = Grid.GetCuttingCoordinatesAlongIso(EIso::IsoU);
	const TArray<double>& VCoordinates = Grid.GetCuttingCoordinatesAlongIso(EIso::IsoV);

	const TArray<TArray<FVector2d>>& Loops = Grid.GetLoops2D(EGridSpace::Default2D);
	for (int32 LoopIndex = 1; LoopIndex < Loops.Num(); ++LoopIndex)
	{
		FVector2d FirstPoint = Loops[LoopIndex][0];

		int32 IndexU = 0;
		for (; IndexU < NumU - 1; ++IndexU)
		{
			if ((FirstPoint.X > UCoordinates[IndexU]) && (FirstPoint.X < UCoordinates[IndexU + 1] + DOUBLE_SMALL_NUMBER))
			{
				break;
			}
		}

		int32 IndexV = 0;
		for (; IndexV < NumV - 1; ++IndexV)
		{
			if ((FirstPoint.Y > VCoordinates[IndexV]) && (FirstPoint.Y < VCoordinates[IndexV + 1] + DOUBLE_SMALL_NUMBER))
			{
				break;
			}
		}

		double UMin = UCoordinates[IndexU];
		double UMax = UCoordinates[IndexU + 1] + DOUBLE_SMALL_NUMBER;
		double VMin = VCoordinates[IndexV];
		double VMax = VCoordinates[IndexV + 1] + DOUBLE_SMALL_NUMBER;

		bool bBoudardyIsSurrounded = true;
		for (const FVector2d& LoopPoint : Loops[LoopIndex])
		{
			if (LoopPoint.X < UMin || LoopPoint.X > UMax || LoopPoint.Y < VMin || LoopPoint.Y > VMax)
			{
				bBoudardyIsSurrounded = false;
				break;
			}
		}

		if (bBoudardyIsSurrounded)
		{
			int32 Index = IndexV * NumU + IndexU;
			IndexOfLowerLeftInnerNodeSurroundingALoop.Add((int32)Index);

			FIsoInnerNode* Node = GlobalIndexToIsoInnerNodes[Index];
			if (Node == nullptr)
			{
				Node = GlobalIndexToIsoInnerNodes[Index + 1];
			}
			if (Node != nullptr)
			{
				for (FIsoSegment* Segment : Node->GetConnectedSegments())
				{
					if (Segment->GetType() == ESegmentType::IsoU)
					{
						if (Segment->GetSecondNode().GetIndex() == Index + 1)
						{
							InnerSegmentsIntersectionTool.AddSegment(*Segment);
						}
					}
					else if (Segment->GetSecondNode().GetIndex() == Index + NumU)
					{
						InnerSegmentsIntersectionTool.AddSegment(*Segment);
					}
				}
			}

			Index = (IndexV + 1) * NumU + IndexU + 1;
			Node = GlobalIndexToIsoInnerNodes[Index];
			if (Node == nullptr)
			{
				Node = GlobalIndexToIsoInnerNodes[Index - 1];
			}
			if (Node != nullptr)
			{
				for (FIsoSegment* Segment : Node->GetConnectedSegments())
				{
					if (Segment->GetType() == ESegmentType::IsoU)
					{
						if (Segment->GetFirstNode().GetIndex() == Index - 1)
						{
							InnerSegmentsIntersectionTool.AddSegment(*Segment);
						}
					}
					else if (Segment->GetFirstNode().GetIndex() == Index - NumU)
					{
						InnerSegmentsIntersectionTool.AddSegment(*Segment);
					}
				}
			}
		}
	}
}

void FIsoTriangulator::TriangulateOverCycle(const EGridSpace Space)
{
	TArray<FIsoSegment*> Cycle;
	Cycle.Reserve(100);
	TArray<bool> CycleOrientation;
	CycleOrientation.Reserve(100);

	// first the external segments (loop segments) are processed 
	for (FIsoSegment* Segment : LoopSegments)
	{
		if (!Segment->HasCycleOnLeft())
		{
			Cycle.Empty();
			CycleOrientation.Empty();
			bool bLeftSide = true;
			if (!FindCycle(Segment, bLeftSide, Cycle, CycleOrientation))
			{
				continue;
			}

			MeshCycle(Cycle, CycleOrientation);
		}
	}

	// then all segments are processed 
	for (FIsoSegment* Segment : FinalToLoops)
	{
		if (!Segment->HasCycleOnLeft())
		{
			Cycle.Empty();
			CycleOrientation.Empty();
			bool bLeftSide = true;
			if (!FindCycle(Segment, bLeftSide, Cycle, CycleOrientation))
			{
				continue;
			}

			MeshCycle(Cycle, CycleOrientation);
		}

		if (!Segment->HasCycleOnRight())
		{
			Cycle.Empty();
			CycleOrientation.Empty();
			bool bLeftSide = false;
			if (!FindCycle(Segment, bLeftSide, Cycle, CycleOrientation))
			{
				continue;
			}

			MeshCycle(Cycle, CycleOrientation);
		}
	}
}

bool FIsoTriangulator::FindCycle(FIsoSegment* StartSegment, bool LeftSide, TArray<FIsoSegment*>& Cycle, TArray<bool>& CycleOrientation)
{
	Cycle.Empty();
	CycleOrientation.Empty();

	FIsoSegment* Segment = StartSegment;
	FIsoNode* Node;

	if (LeftSide)
	{
		Segment->SetHaveCycleOnLeft();
		Node = &StartSegment->GetSecondNode();
	}
	else
	{
		Segment->SetHaveCycleOnRight();
		Node = &StartSegment->GetFirstNode();
	}

	Cycle.Add(StartSegment);
	CycleOrientation.Add(LeftSide);
	Segment = StartSegment;

	for (;;)
	{
		Segment = FindNextSegment(EGridSpace::UniformScaled, Segment, Node, ClockwiseSlope);
		if (Segment == nullptr)
		{
			Cycle.Empty();
			break;
		}

		if (Segment == StartSegment)
		{
			break;
		}

		Cycle.Add(Segment);

		if (&Segment->GetFirstNode() == Node)
		{
			if (Segment->HasCycleOnLeft())
			{
				return false;
			}
			Segment->SetHaveCycleOnLeft();
			Node = &Segment->GetSecondNode();
			CycleOrientation.Add(true);
		}
		else
		{
			if (Segment->HasCycleOnRight())
			{
				return false;
			}
			Segment->SetHaveCycleOnRight();
			Node = &Segment->GetFirstNode();
			CycleOrientation.Add(false);
		}
	}
	return true;
}

FIsoSegment* FIsoTriangulator::FindNextSegment(EGridSpace Space, const FIsoSegment* StartSegment, const FIsoNode* StartNode, SlopeMethod GetSlope) const
{
	const FVector2d& StartPoint = StartNode->Get2DPoint(Space, Grid);
	const FVector2d& EndPoint = (StartNode == &StartSegment->GetFirstNode()) ? StartSegment->GetSecondNode().Get2DPoint(Space, Grid) : StartSegment->GetFirstNode().Get2DPoint(Space, Grid);

	double ReferenceSlope = ComputePositiveSlope(StartPoint, EndPoint, 0);

	double MaxSlope = 8.1;
	FIsoSegment* NextSegment = nullptr;

	for (FIsoSegment* Segment : StartNode->GetConnectedSegments())
	{
		const FVector2d& OtherPoint = (StartNode == &Segment->GetFirstNode()) ? Segment->GetSecondNode().Get2DPoint(Space, Grid) : Segment->GetFirstNode().Get2DPoint(Space, Grid);

		double Slope = GetSlope(StartPoint, OtherPoint, ReferenceSlope);
		if (Slope < SMALL_NUMBER_SQUARE)
		{
			Slope = 8;
		}

		if (Slope < MaxSlope || NextSegment == StartSegment)
		{
			NextSegment = Segment;
			MaxSlope = Slope;
		}
	}

	return NextSegment;
}

void FIsoTriangulator::TriangulateInnerNodes()
{
	int32 NumU = Grid.GetCuttingCount(EIso::IsoU);
	int32 NumV = Grid.GetCuttingCount(EIso::IsoV);

	for (int32 vIndex = 0, Index = 0; vIndex < NumV - 1; vIndex++)
	{
		for (int32 uIndex = 0; uIndex < NumU - 1; uIndex++, Index++)
		{
			// Do the lower nodes of the cell exist
			if (!GlobalIndexToIsoInnerNodes[Index] || !GlobalIndexToIsoInnerNodes[Index + 1])
			{
				continue;
			}

			// Is the lower left node connected
			if (!GlobalIndexToIsoInnerNodes[Index]->IsLinkedToNextU() || !GlobalIndexToIsoInnerNodes[Index]->IsLinkedToNextV())
			{
				continue;
			}

			// Do the upper nodes of the cell exist
			int32 OppositIndex = Index + NumU + 1;
			if (!GlobalIndexToIsoInnerNodes[OppositIndex] || !GlobalIndexToIsoInnerNodes[OppositIndex - 1])
			{
				continue;
			}

			// Is the top right node connected
			if (!GlobalIndexToIsoInnerNodes[OppositIndex]->IsLinkedToPreviousU() || !GlobalIndexToIsoInnerNodes[OppositIndex]->IsLinkedToPreviousV())
			{
				continue;
			}

			bool bIsSurroundingALoop = false;
			for (int32 BorderIndex : IndexOfLowerLeftInnerNodeSurroundingALoop)
			{
				if (Index == BorderIndex)
				{
					bIsSurroundingALoop = true;
					break;
				}
			}
			if (bIsSurroundingALoop)
			{
				continue;
			}

			Mesh.AddTriangle(GlobalIndexToIsoInnerNodes[Index]->GetGlobalIndex(), GlobalIndexToIsoInnerNodes[Index + 1]->GetGlobalIndex(), GlobalIndexToIsoInnerNodes[OppositIndex]->GetGlobalIndex());
			Mesh.AddTriangle(GlobalIndexToIsoInnerNodes[OppositIndex]->GetGlobalIndex(), GlobalIndexToIsoInnerNodes[OppositIndex - 1]->GetGlobalIndex(), GlobalIndexToIsoInnerNodes[Index]->GetGlobalIndex());
		}
		Index++;
	}
}

void FCell::FindCandidateToConnectLoopsByNeighborhood()
{
	for (FCellConnexion& LoopConnexion : LoopConnexions)
	{
		TryToConnectTwoSubLoopsWithShortestSegment(LoopConnexion);
	}
}

void FCell::SelectSegmentToConnectLoops(TFactory<FIsoSegment>& SegmentFactory)
{
	TArray<FCellConnexion*> LoopConnexionPtrs;
	LoopConnexionPtrs.Reserve(LoopConnexions.Num());
	for (FCellConnexion& Connexion : LoopConnexions)
	{
		if (Connexion.Loop2.IsCellCorner())
		{
			continue;
		}
		LoopConnexionPtrs.Add(&Connexion);
	}

	Algo::Sort(LoopConnexionPtrs, [&](const FCellConnexion* LoopConnexion1, const FCellConnexion* LoopConnexion2)
		{
			return LoopConnexion1->MinDistance < LoopConnexion2->MinDistance;
		});

	{
		const int32 LoopCount = CellLoops.Num();
		for (FCellConnexion* LoopConnexion : LoopConnexionPtrs)
		{
			if (!LoopConnexion->bIsConnexionWithOuter && LoopConnexion->IsShortestPath(LoopCount))
			{
				TryToCreateSegment(*LoopConnexion);
			}
		}

		for (FCellConnexion* LoopConnexion : LoopConnexionPtrs)
		{
			if (LoopConnexion->bIsConnexionWithOuter && LoopConnexion->IsShortestPathToOuterLoop(LoopCount))
			{
				TryToCreateSegment(*LoopConnexion);
			}
		}
	}
}

void FCell::SelectSegmentToConnectLoopToCorner(TFactory<FIsoSegment>& SegmentFactory)
{
	TArray<FCellConnexion*> LoopConnexionPtrs;
	LoopConnexionPtrs.Reserve(LoopConnexions.Num());
	for (FCellConnexion& Connexion : LoopConnexions)
	{
		if (Connexion.Loop2.IsCellCorner() && !Connexion.Segment)
		{
			LoopConnexionPtrs.Add(&Connexion);
		}
	}

	Algo::Sort(LoopConnexionPtrs, [&](const FCellConnexion* LoopConnexion1, const FCellConnexion* LoopConnexion2)
		{
			return LoopConnexion1->MinDistance < LoopConnexion2->MinDistance;
		});

	{
		const int32 LoopCount = CellLoops.Num();
		for (FCellConnexion* LoopConnexion : LoopConnexionPtrs)
		{
			if (!LoopConnexion->bIsConnexionWithOuter && LoopConnexion->IsShortestPathToCorner(LoopCount))
			{
				TryToCreateSegment(*LoopConnexion);
			}
		}

		for (FCellConnexion* LoopConnexion : LoopConnexionPtrs)
		{
			if (LoopConnexion->bIsConnexionWithOuter)
			{
				// if the outerLoop is already connected to an innerLoop, the segment is not create.
				// The aim is to avoid to create long segment that generate degenerated triangle
				bool bCreateSegment = true;
				for (const FCellConnexion* Connexion : LoopConnexion->Loop1.Connexions)
				{
					if (Connexion->Segment && Connexion->Segment->IsAFinalSegment())
					{
						const FCellLoop& InnerLoop = Connexion->Loop1.bIsOuterLoop ? Connexion->Loop2 : Connexion->Loop1;
						if (!InnerLoop.IsCellCorner())
						{
							bCreateSegment = false;
						}
					}
				}
				if (bCreateSegment)
				{
					TryToCreateSegment(*LoopConnexion);
				}
			}
		}
	}
}

void FCellLoop::PropagateAsConnected()
{
	bIsConnected = true;
	for (FCellConnexion* Connexion : Connexions)
	{
		if (!Connexion->Segment)
		{
			continue;
		}

		FCellLoop* OtherCell = Connexion->GetOtherLoop(this);
		if (!OtherCell->bIsConnected)
		{
			OtherCell->PropagateAsConnected();
		}
	}
}

void FCell::CheckAllLoopsConnectedTogetherAndConnect()
{
	for (FCellCorner& CellCorner : CellCorners)
	{
		CellCorner.PropagateAsConnected();
	}

	for (FCellLoop& CellLoop : CellLoops)
	{
		if (CellLoop.bIsOuterLoop)
		{
			CellLoop.PropagateAsConnected();
		}
	}
}

void FCell::TryToCreateSegment(FCellConnexion& LoopConnexion)
{
	const FVector2d& ACoordinates = LoopConnexion.NodeA->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& BCoordinates = LoopConnexion.NodeB->Get2DPoint(EGridSpace::UniformScaled, Grid);

	LoopConnexion.Segment = Triangulator.GetOrTryToCreateSegment(*this, LoopConnexion.NodeA, ACoordinates, LoopConnexion.NodeB, BCoordinates, Slope::OneDegree);
	if (LoopConnexion.Segment && !LoopConnexion.Segment->IsAFinalSegment())
	{
		LoopConnexion.Segment->SetFinalMarker();
		if(LoopConnexion.Segment->ConnectToNode())
		{
			IntersectionTool.AddSegment(*LoopConnexion.Segment);
			FinalSegments.Add(LoopConnexion.Segment);
		}
	}
}


void FCell::InitLoopConnexions()
{
	const int32 MaxInnerToCornerConnexions = CellCorners.Num() * (InnerLoopCount + OuterLoopCount);

	switch (InnerLoopCount)
	{
	case 1:
	{
		LoopConnexions.Reserve(OuterLoopCount + MaxInnerToCornerConnexions);
		LoopCellBorderIndices = { OuterLoopCount };
		break;
	}
	case 2:
	{
		LoopConnexions.Reserve(1 + 2 * OuterLoopCount + MaxInnerToCornerConnexions);
		LoopConnexions.Emplace(CellLoops[OuterLoopCount], CellLoops[OuterLoopCount + 1]);

		LoopCellBorderIndices = { OuterLoopCount , OuterLoopCount + 1 };
		break;
	}
	case 3:
	{
		LoopConnexions.Reserve(3 + 3 * OuterLoopCount + MaxInnerToCornerConnexions);
		LoopConnexions.Emplace(CellLoops[OuterLoopCount], CellLoops[OuterLoopCount + 1]);
		LoopConnexions.Emplace(CellLoops[OuterLoopCount], CellLoops[OuterLoopCount + 2]);
		LoopConnexions.Emplace(CellLoops[OuterLoopCount + 1], CellLoops[OuterLoopCount + 2]);

		LoopCellBorderIndices = { OuterLoopCount , OuterLoopCount + 1 , OuterLoopCount + 2 };
		break;
	}
	default:
	{
		TArray<TPair<int32, FVector2d>> LoopBarycenters = GetLoopBarycenters();

		TArray<int32> EdgeVertexIndices;
		FBowyerWatsonTriangulator BWTriangulator(LoopBarycenters, EdgeVertexIndices);
		BWTriangulator.Triangulate();
		LoopCellBorderIndices = BWTriangulator.GetOuterVertices();
		LoopConnexions.Reserve((EdgeVertexIndices.Num() >> 1) + LoopCellBorderIndices.Num() * OuterLoopCount + MaxInnerToCornerConnexions);

		for (int32 Index = 0; Index < EdgeVertexIndices.Num();)
		{
			const int32 StartIndex = Index++;
			const int32 EndIndex = Index++;
			ensureCADKernel(LoopConnexions.Max() > LoopConnexions.Num());
			LoopConnexions.Emplace(CellLoops[EdgeVertexIndices[StartIndex]], CellLoops[EdgeVertexIndices[EndIndex]]);
		}
	}
	}

	for (int32 Index = 0; Index < OuterLoopCount; ++Index)
	{
		for (int32 BIndex : LoopCellBorderIndices)
		{
			ensureCADKernel(LoopConnexions.Max() > LoopConnexions.Num());
			LoopConnexions.Emplace(CellLoops[Index], CellLoops[BIndex]);
		}
	}

	const int32 ConnexionCount = LoopConnexions.Num();
	for (FCellLoop& Loop : CellLoops)
	{
		Loop.Connexions.Reserve(ConnexionCount);
	}
}

void FCell::TryToConnectTwoSubLoopsWithShortestSegment(FCellConnexion& LoopConnexion)
{
	const TArray<FLoopNode*>& LoopA = LoopConnexion.Loop1.Nodes;
	const TArray<FLoopNode*>& LoopB = LoopConnexion.Loop2.Nodes;

	double MinDistanceSquare = HUGE_VALUE_SQUARE;

	for (FLoopNode* NodeA : LoopA)
	{
		const FVector2d& ACoordinates = NodeA->Get2DPoint(EGridSpace::UniformScaled, Grid);

		for (FLoopNode* NodeB : LoopB)
		{
			const FVector2d& BCoordinates = NodeB->Get2DPoint(EGridSpace::UniformScaled, Grid);

			double SquareDistance = FVector2d::DistSquared(ACoordinates, BCoordinates);
			if (SquareDistance < MinDistanceSquare)
			{
				MinDistanceSquare = SquareDistance;
				LoopConnexion.NodeA = NodeA;
				LoopConnexion.NodeB = NodeB;
			}
		}
	}

	LoopConnexion.MinDistance = FMath::Sqrt(MinDistanceSquare);

#ifdef DEBUG_CONNECT_CELL_SUB_LOOPS_BY_NEIGHBORHOOD
	if (LoopConnexion.NodeA && LoopConnexion.NodeB)
	{
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *LoopConnexion.NodeA, *LoopConnexion.NodeB, 0, EVisuProperty::RedCurve);
	}
#endif
}

void FIsoTriangulator::TryToConnectTwoLoopsWithIsocelesTriangle(FCell& Cell, const TArray<FLoopNode*>& LoopA, const TArray<FLoopNode*>& LoopB)
{

	TFunction<FIsoNode* (FIsoSegment*)> FindBestTriangle = [&](FIsoSegment* Segment) -> FIsoNode*
	{
		SlopeMethod GetSlopeAtStartNode = ClockwiseSlope;
		SlopeMethod GetSlopeAtEndNode = CounterClockwiseSlope;

		// StartNode = A
		FIsoNode& StartNode = Segment->GetSecondNode();
		// EndNode = B
		FIsoNode& EndNode = Segment->GetFirstNode();

		//
		// For each segment of the LoopA, find in loopB a vertex that made the best triangle i.e. the triangle the most isosceles.
		// According to the knowledge of the orientation, only inside triangles are tested
		//
		// These computations are done in the UniformScaled space to avoid numerical error due to length distortion between U or V space and U or V Length.
		// i.e. if:
		// (UMax - UMin) / (VMax - VMin) is big 
		// and 
		// "medium length along U" / "medium length along V" is small 
		// 
		// To avoid flat triangle, a candidate point must defined a minimal slop with [A, X0] or [B, Xn] to not be aligned with one of them. 
		//

		FIsoNode* CandidatNode = nullptr;
		FIsoSegment* StartToCandiatSegment = nullptr;
		FIsoSegment* EndToCandiatSegment = nullptr;

		const FVector2d& StartPoint2D = StartNode.Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& EndPoint2D = EndNode.Get2DPoint(EGridSpace::UniformScaled, Grid);

		double StartReferenceSlope = ComputePositiveSlope(StartPoint2D, EndPoint2D, 0);
		double EndReferenceSlope = StartReferenceSlope < 4 ? StartReferenceSlope + 4 : StartReferenceSlope - 4;

		double MinCriteria = HUGE_VALUE;
		const double MinSlopeToNotBeAligned = 0.0001;
		double CandidateSlopeAtStartNode = 8.;
		double CandidateSlopeAtEndNode = 8.;

		for (FLoopNode* Node : LoopB)
		{
			if (Node->IsDeleteOrThinNode())
			{
				continue;
			}

			// Check if the node is inside the sector (X) or outside (Z)
			const FVector2d& NodePoint2D = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);

			double PointCriteria = IsoTriangulatorImpl::IsoscelesCriteriaMax(StartPoint2D, EndPoint2D, NodePoint2D);

			// Triangle that are too open (more than rectangle triangle) are not tested 
			if (PointCriteria > Slope::RightSlope)
			{
				continue;
			}

			double SlopeAtStartNode = GetSlopeAtStartNode(StartPoint2D, NodePoint2D, StartReferenceSlope);
			double SlopeAtEndNode = GetSlopeAtEndNode(EndPoint2D, NodePoint2D, EndReferenceSlope);

			// check the side of the candidate point according to the segment
			if (SlopeAtStartNode <= MinSlopeToNotBeAligned)
			{
				continue;
			}

			if (
				// the candidate triangle is inside the current candidate triangle
				((SlopeAtStartNode < (CandidateSlopeAtStartNode + MinSlopeToNotBeAligned)) && (SlopeAtEndNode < (CandidateSlopeAtEndNode + MinSlopeToNotBeAligned)))
				||
				// the candidate triangle is better than the current candidate triangle and doesn't contain the current candidate triangle
				((PointCriteria < MinCriteria) && ((SlopeAtStartNode > CandidateSlopeAtStartNode) ^ (SlopeAtEndNode > CandidateSlopeAtEndNode))))
			{
				// check if the candidate segment is not in intersection with existing segments
				// if the segment exist, it has already been tested
				FIsoSegment* StartSegment = StartNode.GetSegmentConnectedTo(Node);
				FIsoSegment* EndSegment = EndNode.GetSegmentConnectedTo(Node);

				if (!StartSegment && LoopSegmentsIntersectionTool.DoesIntersect(StartNode, *Node))
				{
					continue;
				}

				if (!EndSegment && LoopSegmentsIntersectionTool.DoesIntersect(EndNode, *Node))
				{
					continue;
				}

				MinCriteria = PointCriteria;
				CandidatNode = Node;
				StartToCandiatSegment = StartSegment;
				EndToCandiatSegment = EndSegment;
				CandidateSlopeAtStartNode = SlopeAtStartNode;
				CandidateSlopeAtEndNode = SlopeAtEndNode;
			}
		}

		return CandidatNode;
	};

	// for each segment of LoopA
	for (int32 IndexA = 0; IndexA < LoopA.Num() - 1; ++IndexA)
	{
		FLoopNode* NodeA1 = LoopA[IndexA];
		FLoopNode* NodeA2 = LoopA[IndexA + 1];

		if (NodeA1->IsDeleteOrThinNode() || NodeA2->IsDeleteOrThinNode())
		{
			continue;
		}

		const FVector2d& A1Coordinates = NodeA1->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& A2Coordinates = NodeA2->Get2DPoint(EGridSpace::UniformScaled, Grid);

		FIsoSegment* Segment = NodeA1->GetSegmentConnectedTo(NodeA2);

		FIsoNode* Node = FindBestTriangle(Segment);
		if (Node && !Node->IsDeleteOrThinNode())
		{
			const FVector2d& NodeCoordinates = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);
			if (!NodeA1->IsDeleteOrThinNode())
			{
				GetOrTryToCreateSegment(Cell, NodeA1, A1Coordinates, Node, NodeCoordinates, 0.1);
			}
			if (!NodeA2->IsDeleteOrThinNode())
			{
				GetOrTryToCreateSegment(Cell, NodeA2, A2Coordinates, Node, NodeCoordinates, 0.1);
			}
		}
	}

};

void FIsoTriangulator::TryToConnectVertexSubLoopWithTheMostIsoSegment(FCell& Cell, const TArray<FLoopNode*>& Loop)
{
	const double FlatSlope = 0.10; // ~5 deg: The segment must make an angle less than 10 deg with the Iso
	double MinSlope = FlatSlope + DOUBLE_SMALL_NUMBER; //.25; // ~10 deg: The segment must make an angle less than 10 deg with the Iso


	if (Loop.Num() <= 2)
	{
		return;
	}

	int32 LoopCount = Loop.Num();
	for (int32 IndexA = 0; IndexA < LoopCount - 2; ++IndexA)
	{
		FLoopNode* CandidateB = nullptr;

		FLoopNode* CandidateA = Loop[IndexA];
		if (CandidateA->IsThinZoneNode())
		{
			continue;
		}

		const FVector2d& ACoordinates = CandidateA->Get2DPoint(EGridSpace::UniformScaled, Grid);

		FLoopNode* NextA = Loop[IndexA + 1];
		const FVector2d& NextACoordinates = NextA->Get2DPoint(EGridSpace::UniformScaled, Grid);

		double ReferenceSlope = 0;
		{
			// slope SegmentA (NodeA, NextA)
			double Slope = ComputeUnorientedSlope(ACoordinates, NextACoordinates, 0);
			if (Slope > 1.5 && Slope < 2.5)
			{
				ReferenceSlope = 0;
			}
			else if ((Slope < 0.5) || (Slope > 3.5))
			{
				ReferenceSlope = 2;
			}
			else
			{
				// SegmentA is neither close to IsoV nor IsoU
				continue;
			}
		}

		for (int32 IndexB = IndexA + 2; IndexB < LoopCount; ++IndexB)
		{
			FLoopNode* NodeB = Loop[IndexB];
			if (NodeB->IsThinZoneNode())
			{
				continue;
			}

			const FVector2d& BCoordinates = NodeB->Get2DPoint(EGridSpace::UniformScaled, Grid);

			double Slope = ComputeSlopeRelativeToReferenceAxis(ACoordinates, BCoordinates, ReferenceSlope);
			if (Slope < MinSlope)
			{
				MinSlope = Slope;
				CandidateB = NodeB;
			}
		}

		if (MinSlope < FlatSlope)
		{
			const FVector2d& BCoordinates = CandidateB->Get2DPoint(EGridSpace::UniformScaled, Grid);

			GetOrTryToCreateSegment(Cell, CandidateA, ACoordinates, CandidateB, BCoordinates, 0.1);
			MinSlope = FlatSlope + DOUBLE_SMALL_NUMBER;
		}
	}

};

void FIsoTriangulator::TryToConnectTwoSubLoopsWithTheMostIsoSegment(FCell& Cell, const TArray<FLoopNode*>& LoopA, const TArray<FLoopNode*>& LoopB)
{
	const double FlatSlope = 0.10; // ~5 deg: The segment must make an angle less than 10 deg with the Iso

	for (FLoopNode* CandidateA : LoopA)
	{
		if (CandidateA->IsThinZoneNode())
		{
			continue;
		}

		FLoopNode* CandidateB = nullptr;
		const FVector2d& ACoordinates = CandidateA->Get2DPoint(EGridSpace::UniformScaled, Grid);

		double MinSlope = FlatSlope + DOUBLE_SMALL_NUMBER;// 0.25; // ~15 deg: The segment must make an angle less than 10 deg with the Iso
		double MinLengthSquare = HUGE_VALUE;

		for (FLoopNode* NodeB : LoopB)
		{
			if (NodeB->IsThinZoneNode())
			{
				continue;
			}

			const FVector2d& BCoordinates = NodeB->Get2DPoint(EGridSpace::UniformScaled, Grid);

			double Slope = ComputeSlopeRelativeToNearestAxis(ACoordinates, BCoordinates);
			if (Slope < MinSlope)
			{
				MinSlope = Slope;
				// If the slope of the candidate segments is nearly zero, then select the shortest
				if (MinSlope < DOUBLE_KINDA_SMALL_NUMBER)
				{
					double DistanceSquare = FVector2d::DistSquared(BCoordinates, ACoordinates);
					if (DistanceSquare > MinLengthSquare)
					{
						continue;
					}
					MinLengthSquare = DistanceSquare;
				}
				CandidateB = NodeB;
			}
		}

		if (MinSlope < FlatSlope)
		{
			const FVector2d& BCoordinates = CandidateB->Get2DPoint(EGridSpace::UniformScaled, Grid);

			GetOrTryToCreateSegment(Cell, CandidateA, ACoordinates, CandidateB, BCoordinates, 0.1);
			MinSlope = FlatSlope + DOUBLE_SMALL_NUMBER;
		}
	}
}

FIsoSegment* FIsoTriangulator::GetOrTryToCreateSegment(FCell& Cell, FLoopNode* NodeA, const FVector2d& ACoordinates, FIsoNode* NodeB, const FVector2d& BCoordinates, const double FlatAngle)
{
	if (FIsoSegment* Segment = NodeA->GetSegmentConnectedTo(NodeB))
	{
		return Segment;
	}

	if (InnerSegmentsIntersectionTool.DoesIntersect(*NodeA, *NodeB))
	{
		return nullptr;
	}

	if (ThinZoneIntersectionTool.DoesIntersect(*NodeA, *NodeB))
	{
		return nullptr;
	}

	if (Cell.IntersectionTool.DoesIntersect(*NodeA, *NodeB))
	{
		return nullptr;
	}

	if (LoopSegmentsIntersectionTool.DoesIntersect(*NodeA, *NodeB))
	{
		return nullptr;
	}

	// Is Outside and not too flat at NodeA
	if (NodeA->IsSegmentBeInsideFace(BCoordinates, Grid, FlatAngle))
	{
		return nullptr;
	}

	// Is Outside and not too flat at NodeB
	if (NodeB->IsALoopNode())
	{
		if (((FLoopNode*)NodeB)->IsSegmentBeInsideFace(ACoordinates, Grid, FlatAngle))
		{
			return nullptr;
		}
	}

	FIsoSegment& Segment = IsoSegmentFactory.New();
	Segment.Init(*NodeA, *NodeB, ESegmentType::LoopToLoop);
	Segment.SetCandidate();
	Cell.CandidateSegments.Add(&Segment);

	return &Segment;
};

void FIsoTriangulator::InitCellCorners(FCell& Cell)
{
	FIsoInnerNode* CellNodes[4];
	int32 Index = Cell.Id;
	CellNodes[0] = GlobalIndexToIsoInnerNodes[Index++];
	CellNodes[1] = GlobalIndexToIsoInnerNodes[Index];
	Index += Grid.GetCuttingCount(EIso::IsoU);;
	CellNodes[2] = GlobalIndexToIsoInnerNodes[Index--];
	CellNodes[3] = GlobalIndexToIsoInnerNodes[Index];

	for (int32 ICell = 0; ICell < 4; ++ICell)
	{
		if (CellNodes[ICell])
		{
			Cell.CellCorners.Emplace(ICell, *CellNodes[ICell], Grid);
		}
	}
}

void FIsoTriangulator::FindCandidateToConnectCellCornerToLoops(FCell& Cell)
{
	if (Cell.CellCorners.IsEmpty())
	{
		return;
	}

	TFunction<void(FCellLoop&, FCellCorner&)> FindAndTryCreateCandidateSegmentToLinkLoopToCorner = [&](FCellLoop& LoopCell, FCellCorner& CellCorner)
	{
		FIsoInnerNode& CornerNode = CellCorner.CornerNode;
		const FVector2d& CornerPoint = CellCorner.Barycenter;

		const TArray<FLoopNode*>& LoopA = LoopCell.Nodes;

		double MinDistanceSquare = HUGE_VALUE_SQUARE;
		int32 MinIndexA = -1;
		for (int32 IndexA = 0; IndexA < LoopA.Num(); ++IndexA)
		{
			FLoopNode* NodeA = LoopA[IndexA];

			const FVector2d& ACoordinates = NodeA->Get2DPoint(EGridSpace::UniformScaled, Grid);

			double SquareDistance = FVector2d::DistSquared(ACoordinates, CornerPoint);
			if (SquareDistance < MinDistanceSquare)
			{
				MinDistanceSquare = SquareDistance;
				MinIndexA = IndexA;
			}
		}

		if (MinIndexA >= 0)
		{
			FLoopNode* NodeA = LoopA[MinIndexA];
			const FVector2d& ACoordinates = NodeA->Get2DPoint(EGridSpace::UniformScaled, Grid);

			double SlopeVsCell = ComputeSlopeRelativeToNearestAxis(ACoordinates, CornerPoint);
			if (SlopeVsCell < Slope::OneDegree)
			{
				// if the candidate segment is to close to the cell border, we cannot check if in the other side of the cell there are not also a parallel candidate segment
				// Add it in InnerToLoopCandidateSegments to be processed at the end

				FIsoSegment& Segment = IsoSegmentFactory.New();
				Segment.Init(*NodeA, CornerNode, ESegmentType::InnerToLoopU);
				Segment.SetCandidate();

				// The connection is nevertheless created to avoid failed in CheckAllLoopsConnectedTogetherAndConnect while the connection while be create in a second time based on InnerToLoopCandidateSegments
				ensureCADKernel(Cell.LoopConnexions.Max() > Cell.LoopConnexions.Num());
				FCellConnexion& Connexion = Cell.LoopConnexions.Emplace_GetRef(LoopCell, CellCorner);
				Connexion.NodeA = NodeA;
				Connexion.NodeB = &CornerNode;
				Connexion.Segment = &Segment;

				InnerToLoopCandidateSegments.Add(&Segment);
			}
			else
			{
				ensureCADKernel(Cell.LoopConnexions.Max() > Cell.LoopConnexions.Num());
				FCellConnexion& Connexion = Cell.LoopConnexions.Emplace_GetRef(LoopCell, CellCorner);
				Connexion.NodeA = NodeA;
				Connexion.NodeB = &CornerNode;
			}
		}
	};

	int32 IntersectionToolCount = Cell.IntersectionTool.Count();
	int32 NewSegmentCount = Cell.CandidateSegments.Num() - IntersectionToolCount;
	Cell.IntersectionTool.AddSegments(Cell.CandidateSegments.GetData() + IntersectionToolCount, NewSegmentCount);
	Cell.IntersectionTool.Sort();

	for (FCellCorner& CellCorner : Cell.CellCorners)
	{
		for (FCellLoop& LoopCell : Cell.CellLoops)
		{
			FindAndTryCreateCandidateSegmentToLinkLoopToCorner(LoopCell, CellCorner);
		}
	}
}


void FIsoTriangulator::SelectSegmentsToLinkInnerToLoop()
{
	LoopSegmentsIntersectionTool.AddSegments(FinalToLoops);

	TArray<TPair<double, FIsoSegment*>> LengthOfCandidateSegments;
	LengthOfCandidateSegments.Reserve(InnerToLoopCandidateSegments.Num());
	for (FIsoSegment* Segment : InnerToLoopCandidateSegments)
	{
		LengthOfCandidateSegments.Emplace(Segment->Get2DLengthSquare(EGridSpace::UniformScaled, Grid), Segment);
	}

	FIntersectionSegmentTool IntersectionTool(Grid, Tolerances.GeometricTolerance);
	IntersectionTool.Reserve(InnerToLoopCandidateSegments.Num());

	Algo::Sort(LengthOfCandidateSegments, [&](const TPair<double, FIsoSegment*>& P1, const TPair < double, FIsoSegment*>& P2) { return P1.Key < P2.Key; });

	// Validate the first candidate segments
	for (const TPair<double, FIsoSegment*>& Candidate : LengthOfCandidateSegments)
	{
		FIsoSegment* Segment = Candidate.Value;

		if (IntersectionTool.DoesIntersect(*Segment))
		{
			IsoSegmentFactory.DeleteEntity(Segment);
			continue;
		}

		if (LoopSegmentsIntersectionTool.DoesIntersect(*Segment))
		{
			IsoSegmentFactory.DeleteEntity(Segment);
			continue;
		}

		if (FIsoSegment::IsItAlreadyDefined(&Segment->GetFirstNode(), &Segment->GetSecondNode()))
		{
			IsoSegmentFactory.DeleteEntity(Segment);
			continue;
		}

		FinalToLoops.Add(Segment);
		IntersectionTool.AddSegment(*Segment);
		Segment->SetSelected();
		Segment->SetFinalMarker();
		if (!Segment->ConnectToNode())
		{
			IsoSegmentFactory.DeleteEntity(Segment);
		}
	}
	CandidateSegments.Empty();
}

} //namespace UE::CADKernel