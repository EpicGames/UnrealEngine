// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Factory.h"
#include "Math/Geometry.h"
#include "Math/Point.h"
#include "Math/SlopeUtils.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionIsoSegmentTool.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionSegmentTool.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Meshers/ParametricMesherConstantes.h"

namespace UE::CADKernel
{

class FGrid;
class FIntersectionSegmentTool;
class FFaceMesh;
struct FCell;

using FMeshPolygonFunc = TFunction<void(const FGrid&, FIsoNode*[], FFaceMesh&)>;

struct FCellConnexion;

class FIsoTriangulator
{
	friend class FCycleTriangulator;
	friend class FLoopCleaner;
	friend class FParametricMesher;
	friend struct FCell;

protected:

	FGrid& Grid;
	FFaceMesh& Mesh;

	TArray<int32> LoopStartIndex;
	TArray<FLoopNode> LoopNodes;
	int32 LoopNodeCount = 0;
	TArray<FLoopNode*> SortedLoopNodes;

	/**
	 * GlobalIndexToIsoInnerNodes contains only inner nodes of the grid, if GlobalIndexToIsoInnerNodes[Index] == null, the point is outside the domain
	 */
	TArray<FIsoInnerNode*> GlobalIndexToIsoInnerNodes;

	/**
	 * Static array of InnerNodes. Only used for allocation needs
	 */
	TArray<FIsoInnerNode> InnerNodes;
	int32 InnerNodeCount = 0;

	TFactory<FIsoSegment> IsoSegmentFactory;

	TArray<FIsoSegment*> LoopSegments;
	TArray<FIsoSegment*> ThinZoneSegments;
	TArray<FIsoSegment*> FinalInnerSegments;
	TArray<FIsoSegment*> InnerToOuterSegments;

	/** 
	 * Waiting list for SelectSegmentsToLinkInnerToLoop
	 * Segments are identified in each cell but not directly processed 
	 */
	TArray<FIsoSegment*> InnerToLoopCandidateSegments;

	/**
	 * Tool to check if a segment intersect or not existing segments.
	 * To be optimal, depending on the segment, only a subset of segment is used.
	 * Checks intersection with loop.
	 */
	FIntersectionSegmentTool LoopSegmentsIntersectionTool;

	/** Boundary of the inner grid */
	FIntersectionSegmentTool InnerSegmentsIntersectionTool;
	
	/** To check if the candidate segment intersect a iso line */
	FIntersectionIsoSegmentTool InnerToOuterIsoSegmentsIntersectionTool;
	
	/** To check if a candidate segment intersect a thin zone that is already meshed */
	FIntersectionSegmentTool ThinZoneIntersectionTool;

	/**
	 * Define all the lower left index of grid node that the upper cell is surrounding a loop
	 * This is set in FindSegmentIsoUVSurroundingSmallLoop
	 * and use in TriangulateInnerNodes to don't generate the both cell triangles
	 */
	TArray<int32> IndexOfLowerLeftInnerNodeSurroundingALoop;

	/**
	 * Segments to link inner to boundary and boundary to boundary
	 * From FindIsoSegmentToLinkInnerToBoundary
	 * SelectSegmentInCandidateSegments
	 */
	TArray<FIsoSegment*> FinalToLoops;

	/**
	 *
	 */
	TArray<FIsoSegment*> CandidateSegments;

	bool bDisplay = false;

	bool bNeedCheckOrientation = false;

public:
	const FMeshingTolerances& Tolerances;

public:

	FIsoTriangulator(FGrid& InGrid, FFaceMesh& OutMesh, const FMeshingTolerances& InTolerance);

	/**
	 * Main method
	 * @return false if the tessellation failed
	 */
	bool Triangulate();

	void BuildNodes();

	/**
	 * Build the segments of the loops and check if each loop is self intersecting.
	 * @return false if the loop is self intersecting
	 */
	void BuildLoopSegments();

	void BuildInnerSegments();

	/**
	 * Add temporary loops defining thin zones to avoid the tessellation of these zone.
	 * These zones are tessellated in a specific process
	 */
	void GetThinZonesMesh();
	void GetThinZoneMesh(const TMap<int32, FLoopNode*>& IndexToNode, const FThinZone2D& ThinZone);

	/**
	 * Fill mesh node data (Position, normal, UV, Index) of the FFaceMesh object
	 */
	void FillMeshNodes();

	/**
	 * Build the Inner Segments Intersection Tool used to check if a candidate segment crosses the inner mesh
	 *
	 * The minimal set of segments of the intersection tool is the boundaries of the inner triangulation.
	 *
	 * https://docs.google.com/presentation/d/1qUVOH-2kU_QXBVKyRUcdDy1Y6WGkcaJCiaS8wGjSZ6M/edit?usp=sharing
	 * Slide "Boundary Segments Of Inner Triangulation"
	 */
	void BuildInnerSegmentsIntersectionTool();

	/**
	 * The purpose of the method is to add surrounding segments (boundary of an unitary inner grid cell) to the small loop to intersection tool to prevent traversing inner segments
	 * A loop is inside inner segments
	 *									|			 |
	 *								   -----------------
	 *									|	 XXX	 |
	 *									|	XXXXX	 |
	 *									|	 XXX	 |
	 *								   -----------------
	 *									|			 |
	 * https://docs.google.com/presentation/d/1qUVOH-2kU_QXBVKyRUcdDy1Y6WGkcaJCiaS8wGjSZ6M/edit?usp=sharing
	 * Slide "Find Inner Grid Cell Surrounding Small Loop"
	 * This method finalizes BuildInnerSegmentsIntersectionTool
	 */
	void FindInnerGridCellSurroundingSmallLoop();

	void ConnectCellLoops();
	void FindCellContainingBoundaryNodes(TArray<FCell>& Cells);

	void FindCandidateToConnectCellCornerToLoops(FCell& Cell);

	void SelectSegmentsToLinkInnerToLoop();

	void InitCellCorners(FCell& Cell);

	/**
	 * Finalize the tessellation between inner grid boundary and loops.
	 * The final set of segments define a network
	 * Each minimal cycle is tessellated independently
	 */
	void TriangulateOverCycle(const EGridSpace Space);

	/**
	 * Find in the network a minimal cycle stating from a segment
	 * @return false if the new cycle crosses a segment already used
	 */
	bool FindCycle(FIsoSegment* StartSegment, bool bLeftSide, TArray<FIsoSegment*>& Cycle, TArray<bool>& CycleOrientation);

	/**
	 * Generate the "Delaunay" tessellation of the cycle.
	 * The algorithm is based on frontal process
	 */
	void MeshCycle(const TArray<FIsoSegment*>& Cycle, const TArray<bool>& CycleOrientation);
	void MeshLargeCycle(const TArray<FIsoSegment*>& Cycle, const TArray<bool>& CycleOrientation);

	template <uint32 Dim>
	void MeshCycleOf(const TArray<FIsoSegment*>& Cycle, const TArray<bool>& CycleOrientation, FMeshPolygonFunc MeshPolygonFunc)
	{
		FIsoNode* Nodes[Dim];
		for (int32 Index = 0; Index < Dim; ++Index)
		{
			Nodes[Index] = CycleOrientation[Index] ? &Cycle[Index]->GetFirstNode() : &Cycle[Index]->GetSecondNode();
		}

		MeshPolygonFunc(Grid, Nodes, Mesh);
	}


	bool CanCycleBeMeshed(const TArray<FIsoSegment*>& Cycle, FIntersectionSegmentTool& CycleIntersectionTool);

	/**
	 * Finalization of the mesh by the tessellation of the inner grid
	 */
	void TriangulateInnerNodes();

	/**
	 * Sorted loop node array used to have efficient loop proximity node research
	 */
	void SortLoopNodes()
	{
		SortedLoopNodes.Reserve(LoopNodes.Num());
		for (FLoopNode& LoopNode : LoopNodes)
		{
			SortedLoopNodes.Add(&LoopNode);
		}

		Algo::Sort(SortedLoopNodes, [this](const FLoopNode* Node1, const FLoopNode* Node2)
			{
				const FVector2d& Node1Coordinates = Node1->Get2DPoint(EGridSpace::Default2D, Grid);
				const FVector2d& Node2Coordinates = Node2->Get2DPoint(EGridSpace::Default2D, Grid);
				return (Node1Coordinates.X + Node1Coordinates.Y) < (Node2Coordinates.X + Node2Coordinates.Y);
			});
	}

	const FGrid& GetGrid() const
	{
		return Grid;
	}

private:

	FIsoSegment* FindNextSegment(EGridSpace Space, const FIsoSegment* StartSegment, const FIsoNode* StartNode, SlopeMethod GetSlop) const;

	// ==========================================================================================
	// 	   Create segments
	// ==========================================================================================

	void TryToConnectTwoLoopsWithIsocelesTriangle(FCell& Cell, const TArray<FLoopNode*>& SubLoopA, const TArray<FLoopNode*>& SubLoopB);

	/**
	 *    --X------X-----X--  <-- SubLoopA
	 *             I
	 *             I    <- The most isoSegment
	 *             I
	 *    ----X----X--------  <-- SubLoopB
	 *
	 */
	void TryToConnectTwoSubLoopsWithTheMostIsoSegment(FCell& Cell, const TArray<FLoopNode*>& SubLoopA, const TArray<FLoopNode*>& SubLoopB);

	/**
	 *    X--X------X-----X--  <-- SubLoop
	 *    |         I
	 *    |         I   <- The most isoSegment
	 *    |         I
	 *    X----X----X--------
	 *
	 */
	void TryToConnectVertexSubLoopWithTheMostIsoSegment(FCell& Cell, const TArray<FLoopNode*>& SubLoop);
	FIsoSegment* GetOrTryToCreateSegment(FCell& Cell, FLoopNode* NodeA, const FVector2d& ACoordinates, FIsoNode* NodeB, const FVector2d& BCoordinates, const double FlatAngle);

public:
#ifdef CADKERNEL_DEV
	FIsoTriangulatorChronos Chronos;
#endif

#ifdef CADKERNEL_DEBUG
	void DisplayPixels(TArray<uint8>& Pixel) const;

	void DisplayPixel(const int32 Index) const;
	void DisplayPixel(const int32 IndexU, const int32 IndexV) const;

	void DisplayIsoNodes(EGridSpace Space) const;

	void DisplayLoops(const TCHAR* Message, bool bOneNode = true, bool bSplitBySegment = false) const;
	void DisplayLoopsByNextAndPrevious(const TCHAR* Message) const;

	void DisplayCells(const TArray<FCell>& Cells) const;
	void DisplayCell(const FCell& Cell) const;
	void DrawCellBoundary(int32 Index, EVisuProperty Property) const;

	void DisplayCellConnexion(const FCellConnexion& LoopConnexion, EVisuProperty Property) const;
	void DisplayCellConnexions(const FString& Message, const TArray<FCellConnexion>& LoopConnexions, EVisuProperty Property) const;
#endif

};

namespace IsoTriangulatorImpl
{

/**
 * Criteria to find the optimal "Delaunay" triangle starting from the segment AB to a set of point P
 * A "Delaunay" triangle is an equilateral triangle
 * The optimal value is the smallest value.
 */
inline double CotangentCriteria(const FVector& APoint, const FVector& BPoint, const FVector& PPoint, FVector& OutNormal)
{
	const double BigValue = HUGE_VALUE;

	FVector PA = APoint - PPoint;
	FVector PB = BPoint - PPoint;

	// the ratio between the scalar product PA.PB (=|PA| |PB| cos (A,P,B) )
	// with the norm of the cross product |PA^PB| (=|PA| |PB| |sin(A,P,B)|)
	// is compute. 
	double ScalareProduct = PA | PB;
	OutNormal = PA ^ PB;
	double NormOFScalarProduct = sqrt(OutNormal | OutNormal);

	// PPoint is aligned with (A,B)
	if (NormOFScalarProduct < DOUBLE_SMALL_NUMBER)
	{
		return BigValue;
	}

	// return Cotangent value 
	return ScalareProduct / NormOFScalarProduct;
}

inline double CotangentCriteria(const FVector2d& APoint, const FVector2d& BPoint, const FVector2d& PPoint)
{
	const double BigValue = HUGE_VALUE;

	FVector2d PA = APoint - PPoint;
	FVector2d PB = BPoint - PPoint;

	// the ratio between the scalar product PA.PB (=|PA| |PB| cos (A,P,B) )
	// with the norm of the cross product |PA^PB| (=|PA| |PB| |sin(A,P,B)|)
	// is compute. 
	double ScalareProduct = PA | PB;
	double OutNormal = PA ^ PB;
	double NormOFPointProduct = FMath::Abs(OutNormal);

	if (NormOFPointProduct < DOUBLE_SMALL_NUMBER)
	{
		// PPoint is aligned with (A,B)
		return BigValue;
	}

	// return Cotangent value 
	return ScalareProduct / NormOFPointProduct;
}

struct FPairOfDouble
{
	double Value1;
	double Value2;
};

inline FPairOfDouble IsoscelesCriteria(const FVector2d& APoint, const FVector2d& BPoint, const FVector2d& CPoint)
{
	const double SlopeAB = ComputeSlope(APoint, BPoint);
	const double SlopeAC = ComputeSlope(APoint, CPoint);
	const double SlopeBC = ComputeSlope(BPoint, CPoint);
	const double SlopeBA = SwapSlopeOrientation(SlopeAB);

	return { TransformIntoOrientedSlope(SlopeAC - SlopeAB), TransformIntoOrientedSlope(SlopeBA - SlopeBC) };
}

inline double IsoscelesCriteriaMax(const FVector2d& APoint, const FVector2d& BPoint, const FVector2d& CPoint)
{
	const FPairOfDouble Criteria = IsoscelesCriteria(APoint, BPoint, CPoint);
	return FMath::Max(Criteria.Value1, Criteria.Value2);
}

inline double IsoscelesCriteriaMin(const FVector2d& APoint, const FVector2d& BPoint, const FVector2d& CPoint)
{
	const FPairOfDouble Criteria = IsoscelesCriteria(APoint, BPoint, CPoint);
	return FMath::Min(Criteria.Value1, Criteria.Value2);
}

inline double EquilateralSlopeCriteria(const FVector2d& APoint, const FVector2d& BPoint, const FVector2d& CPoint)
{
	const double SlopeAB = ComputeSlope(APoint, BPoint);
	const double SlopeAC = ComputeSlope(APoint, CPoint);
	const double SlopeBC = ComputeSlope(BPoint, CPoint);
	const double SlopeBA = SwapSlopeOrientation(SlopeAB);
	const double SlopeCA = SwapSlopeOrientation(SlopeAC);
	const double SlopeCB = SwapSlopeOrientation(SlopeBC);

	const double A = TransformIntoOrientedSlope(SlopeAC - SlopeAB);
	const double B = TransformIntoOrientedSlope(SlopeBA - SlopeBC);
	const double C = TransformIntoOrientedSlope(SlopeCB - SlopeCA);

	return FMath::Max3(A, B, C);
}

template<class PointType>
inline double EquilateralCriteria(const PointType& SegmentA, const PointType& SegmentB, const PointType& Point)
{
	double Criteria1 = FMath::Abs(CoordinateOfProjectedPointOnSegment<PointType>(SegmentA, SegmentB, Point, false) - 0.5);
	double Criteria2 = FMath::Abs(CoordinateOfProjectedPointOnSegment<PointType>(Point, SegmentA, SegmentB, false) - 0.5);
	double Criteria3 = FMath::Abs(CoordinateOfProjectedPointOnSegment<PointType>(SegmentB, Point, SegmentA, false) - 0.5);
	return Criteria1 + Criteria2 + Criteria3;
}


} // namespace FIsoTriangulatorImpl

} // namespace UE::CADKernel

