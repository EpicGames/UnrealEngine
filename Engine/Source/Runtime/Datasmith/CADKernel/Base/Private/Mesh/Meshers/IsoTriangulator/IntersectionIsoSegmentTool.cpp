// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/IsoTriangulator/IntersectionIsoSegmentTool.h"
#include "Geo/GeoEnum.h"

namespace UE::CADKernel
{

FIntersectionIsoSegmentTool::FIntersectionIsoSegmentTool(const FGrid& InGrid, const double Tolerance)
	: Grid(InGrid)
{
	TFunction<void(TArray<TPair<double, int32>>&, const TArray<double>&)> Reserve = [](TArray<TPair<double, int32>>& InCoordToIndex, const TArray<double>& InCuttingCoordinates)
	{
		InCoordToIndex.Reserve(InCuttingCoordinates.Num());
		for (const double Coordinate : InCuttingCoordinates)
		{
			InCoordToIndex.Emplace(Coordinate, 0);
		}
	};

	Reserve(CoordToIndex[EIso::IsoU], Grid.GetUniformCuttingCoordinatesAlongIso(EIso::IsoU));
	Reserve(CoordToIndex[EIso::IsoV], Grid.GetUniformCuttingCoordinatesAlongIso(EIso::IsoV));

	const int32 MaxSegmentCount = FMath::Min(Grid.InnerNodesCount(), Grid.GetCuttingCount(EIso::IsoU) * Grid.GetCuttingCount(EIso::IsoV) - Grid.InnerNodesCount());
	USegments.Reserve(MaxSegmentCount);
	VSegments.Reserve(MaxSegmentCount);
}

void FIntersectionIsoSegmentTool::AddIsoSegment(const FVector2d& StartPoint, const FVector2d& EndPoint, const ESegmentType InType)
{
	switch(InType)
	{ 
	case ESegmentType::IsoU:
		USegments.Emplace(StartPoint, EndPoint);
		break;

	case ESegmentType::IsoV:
		VSegments.Emplace(StartPoint, EndPoint);
		break;

	default:
		break;
	}
}

template <class TIsoSegment>
int32 TCountIntersections(const TArray<TIsoSegment>& Segments, EIso Iso, int32 Index, const FSegment2D& InSegment, const FSurfacicBoundary& InSegmentBoundary)
{
	using namespace IntersectionIsoSegmentTool;

	const double Max = InSegmentBoundary.Get(Iso).GetMax();
	
	int32 IntersectionCount = 0;
	EIso OtherIso = Other(Iso);
	for (; Index < Segments.Num(); ++Index)
	{
		const FIntersectionIsoSegment& IsoSegment = Segments[Index];
		if (Max < IsoSegment.GetIsoCoordinate())
		{
			break;
		}
		if (InSegmentBoundary.Get(OtherIso).GetMin() > IsoSegment.GetMaxCoordinate())
		{
			continue;
		}
		if (InSegmentBoundary.Get(OtherIso).GetMax() < IsoSegment.GetMinCoordinate())
		{
			break;
		}

		if (InSegmentBoundary.Get(OtherIso).GetMin() >= IsoSegment.GetMinCoordinate() && InSegmentBoundary.Get(OtherIso).GetMax() <= IsoSegment.GetMaxCoordinate())
		{
			// Allow Overlapping: is InSegmentBoundary coincident with IsoSegment ?
			if (InSegmentBoundary.Get(Iso).GetMin() + DOUBLE_KINDA_SMALL_NUMBER > IsoSegment.GetIsoCoordinate()
				&& InSegmentBoundary.Get(Iso).GetMax() - DOUBLE_KINDA_SMALL_NUMBER < IsoSegment.GetIsoCoordinate())
			{
				continue;
			}

			IntersectionCount++;
			continue;
		}

		const FSegment2D IsoSegment2D = IsoSegment.GetSegment2D();
		if (DoIntersect(InSegment, IsoSegment2D))
		{
			IntersectionCount++;
		}
	}
	return IntersectionCount;
}

template <class TIsoSegment>
bool TDoesIntersect(const TArray<TIsoSegment>& Segments, EIso Iso, int32 Index, const FSegment2D& InSegment, const FSurfacicBoundary& InSegmentBoundary)
{
	using namespace IntersectionIsoSegmentTool;

	const double Max = InSegmentBoundary.Get(Iso).GetMax();

	int32 IntersectionCount = 0;

	EIso OtherIso = Other(Iso);
	for (; Index < Segments.Num(); ++Index)
	{
		const FIntersectionIsoSegment& IsoSegment = Segments[Index];
		if (Max < IsoSegment.GetIsoCoordinate())
		{
			break;
		}
		if (InSegmentBoundary.Get(OtherIso).GetMin() > IsoSegment.GetMaxCoordinate())
		{
			continue;
		}
		if (InSegmentBoundary.Get(OtherIso).GetMax() < IsoSegment.GetMinCoordinate())
		{
			break;
		}

		if (InSegmentBoundary.Get(OtherIso).GetMin() >= IsoSegment.GetMinCoordinate() && InSegmentBoundary.Get(OtherIso).GetMax() <= IsoSegment.GetMaxCoordinate())
		{
			// Allow Overlapping: is InSegmentBoundary coincident with IsoSegment ?
			if (InSegmentBoundary.Get(Iso).GetMin() + DOUBLE_KINDA_SMALL_NUMBER > IsoSegment.GetIsoCoordinate()
				&& InSegmentBoundary.Get(Iso).GetMax() - DOUBLE_KINDA_SMALL_NUMBER < IsoSegment.GetIsoCoordinate())
			{
				continue;
			}

			return true;
		}

		const FSegment2D IsoSegment2D = IsoSegment.GetSegment2D();
		if (DoIntersect(InSegment, IsoSegment2D))
		{
			return true;
		}
	}
	return false;
}

int32 FIntersectionIsoSegmentTool::CountIntersections(const FIsoNode& StartPoint, const FIsoNode& EndPoint) const
{
	const FSegment2D Segment(StartPoint.Get2DPoint(EGridSpace::UniformScaled, Grid), EndPoint.Get2DPoint(EGridSpace::UniformScaled, Grid));
	const FSurfacicBoundary SegmentBoundary(Segment.Point0, Segment.Point1);

	int32 IndexU = GetStartIndex(EIso::IsoU, SegmentBoundary);
	int32 IndexV = GetStartIndex(EIso::IsoV, SegmentBoundary);

	return TCountIntersections(USegments, EIso::IsoU, IndexU, Segment, SegmentBoundary) + TCountIntersections(VSegments, EIso::IsoV, IndexV, Segment, SegmentBoundary);
}

bool FIntersectionIsoSegmentTool::DoesIntersect(const FIsoNode& StartPoint, const FIsoNode& EndPoint) const
{
	const FSegment2D Segment(StartPoint.Get2DPoint(EGridSpace::UniformScaled, Grid), EndPoint.Get2DPoint(EGridSpace::UniformScaled, Grid));
	const FSurfacicBoundary SegmentBoundary(Segment.Point0, Segment.Point1);

	int32 IndexU = GetStartIndex(EIso::IsoU, SegmentBoundary);

	if (TDoesIntersect(USegments, EIso::IsoU, IndexU, Segment, SegmentBoundary))
	{
		return true;
	}

	int32 IndexV = GetStartIndex(EIso::IsoV, SegmentBoundary);
	return TDoesIntersect(VSegments, EIso::IsoV, IndexV, Segment, SegmentBoundary);
}

int32 FIntersectionIsoSegmentTool::GetStartIndex(EIso Iso, const FSurfacicBoundary& SegmentBoundary) const
{
	double Min = SegmentBoundary.Get(Iso).GetMin() - SMALL_NUMBER;
	for (const TPair<double, int32>& CoordIndex : CoordToIndex[Iso])
	{
		if (Min < CoordIndex.Key)
		{
			return CoordIndex.Value;
		}
	}
	return CoordToIndex[Iso].Last().Value;
}

int32 FIntersectionIsoSegmentTool::GetStartIndex(EIso Iso, double Min) const
{
	Min -= SMALL_NUMBER;
	for (const TPair<double, int32>& CoordIndex : CoordToIndex[Iso])
	{
		if (Min < CoordIndex.Key)
		{
			return CoordIndex.Value;
		}
	}
	return CoordToIndex[Iso].Last().Value;
}

template <class TIsoSegment>
void SortCoordToIndex(TArray<TPair<double, int32>>& CoordToIndex, const TArray<TIsoSegment>& Segments)
{
	int32 IndexOfCuttingCoordinate = 0;
	double Coordinate = CoordToIndex[IndexOfCuttingCoordinate].Key + SMALL_NUMBER;
	int32 SegmentCount = 0;
	for (const TIsoSegment& Segment : Segments)
	{
		const double IsoCoordinate = Segment.GetIsoCoordinate();
		while (Coordinate < IsoCoordinate)
		{
			++IndexOfCuttingCoordinate;
			CoordToIndex[IndexOfCuttingCoordinate].Value = SegmentCount;
			Coordinate = CoordToIndex[IndexOfCuttingCoordinate].Key + SMALL_NUMBER;
		}
		++SegmentCount;
	}
	for (++IndexOfCuttingCoordinate; IndexOfCuttingCoordinate < CoordToIndex.Num(); ++IndexOfCuttingCoordinate)
	{
		CoordToIndex[IndexOfCuttingCoordinate].Value = SegmentCount;
	}
};

void FIntersectionIsoSegmentTool::Sort()
{
	USegments.Sort();
	VSegments.Sort();

	SortCoordToIndex(CoordToIndex[EIso::IsoU], USegments);
	SortCoordToIndex(CoordToIndex[EIso::IsoV], VSegments);

	bIsSorted = true;
}



#ifdef CADKERNEL_DEBUG
void FIntersectionIsoSegmentTool::Display(bool bDisplay, const TCHAR* Message, EVisuProperty Property) const
{
	if (!bDisplay)
	{
		return;
	}

	int32 Index = 0;
	Open3DDebugSession(Message);
	for (const IntersectionIsoSegmentTool::FIntersectionIsoSegment& Segment : USegments)
	{
		const FSegment2D Segment2D = Segment.GetSegment2D();
		DisplaySegment(Segment2D[0] * DisplayScale, Segment2D[1] * DisplayScale, Index++, Property);
	}
	for (const IntersectionIsoSegmentTool::FIntersectionIsoSegment& Segment : VSegments)
	{
		const FSegment2D Segment2D = Segment.GetSegment2D();
		DisplaySegment(Segment2D[0] * DisplayScale, Segment2D[1] * DisplayScale, Index++, Property);
	}
	Close3DDebugSession();
	Wait();
}
#endif

} // namespace UE::CADKernel

