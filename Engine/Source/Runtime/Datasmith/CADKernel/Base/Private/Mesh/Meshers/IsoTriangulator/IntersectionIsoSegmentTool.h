// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Math/Boundary.h"

namespace UE::CADKernel
{
class FGrid;
class FIsoNode;

namespace IntersectionIsoSegmentTool
{

class FIntersectionIsoSegment
{
private:
	const FVector2d& Point0;
	const FVector2d& Point1;
	double IsoCoordinate;
	double MinCoordinate;
	double MaxCoordinate;

public:
	
	FIntersectionIsoSegment(const FVector2d& StartPoint, const FVector2d& EndPoint, const double InIsoCoordinate, const double InStartCoordinate, const double InEndCoordinate)
		: Point0(StartPoint)
		, Point1(EndPoint)
		, IsoCoordinate(InIsoCoordinate)
		, MinCoordinate(FMath::Min(InStartCoordinate, InEndCoordinate))
		, MaxCoordinate(FMath::Max(InStartCoordinate, InEndCoordinate))
	{}

	virtual ~FIntersectionIsoSegment()
	{}

	virtual FVector2d GetMinPoint() const = 0;
	virtual FVector2d GetMaxPoint() const = 0;

	double GetIsoCoordinate() const 
	{
		return IsoCoordinate;
	}

	double GetMinCoordinate() const
	{
		return MinCoordinate;
	}

	double GetMaxCoordinate() const
	{
		return MaxCoordinate;
	}

	FSegment2D GetSegment2D() const
	{
		return FSegment2D(Point0, Point1);
	}

	friend bool operator<(const FIntersectionIsoSegment& A, const FIntersectionIsoSegment& B)
	{
		if (FMath::IsNearlyEqual(A.IsoCoordinate, B.IsoCoordinate))
		{
			return A.MinCoordinate < B.MinCoordinate;
		}
		else
		{
			return A.IsoCoordinate < B.IsoCoordinate;
		}
	}
};

class FIsoUSegment : public FIntersectionIsoSegment
{
public:
	FIsoUSegment(const FVector2d& StartPoint, const FVector2d& EndPoint)
		: FIntersectionIsoSegment(StartPoint, EndPoint, StartPoint.X, StartPoint.Y, EndPoint.Y)
	{}

	virtual FVector2d GetMinPoint() const override
	{
		return FVector2d(GetIsoCoordinate(), GetMinCoordinate());
	}
	virtual FVector2d GetMaxPoint() const override
	{
		return FVector2d(GetIsoCoordinate(), GetMaxCoordinate());
	}
};

class FIsoVSegment : public FIntersectionIsoSegment
{
public:
	FIsoVSegment(const FVector2d& StartPoint, const FVector2d& EndPoint)
		: FIntersectionIsoSegment(StartPoint, EndPoint, StartPoint.Y, StartPoint.X, EndPoint.X)
	{}

	virtual FVector2d GetMinPoint() const override
	{
		return FVector2d(GetMinCoordinate(), GetIsoCoordinate());
	}

	virtual FVector2d GetMaxPoint() const override
	{
		return FVector2d(GetMaxCoordinate(), GetIsoCoordinate());
	}

};

}

class FIntersectionIsoSegmentTool
{
private:
	const FGrid& Grid;

	TArray<TPair<double, int32>> CoordToIndex[2];
	
	TArray<IntersectionIsoSegmentTool::FIsoUSegment> USegments;
	TArray<IntersectionIsoSegmentTool::FIsoVSegment> VSegments;

	bool bIsSorted = false;

public:
	FIntersectionIsoSegmentTool(const FGrid& InGrid, const double Tolerance);

	void AddIsoSegment(const FVector2d& StartPoint, const FVector2d& EndPoint, const ESegmentType InType);

	bool DoesIntersect(const FIsoNode& StartNode, const FIsoNode& EndNode) const;

	int32 CountIntersections(const FIsoNode& StartNode, const FIsoNode& EndNode) const;

	void Sort();

private:
	int32 GetStartIndex(EIso Iso, double Min) const;
	int32 GetStartIndex(EIso Iso, const FSurfacicBoundary& Boundary) const;

public:
#ifdef CADKERNEL_DEBUG
	void Display(bool bDisplay, const TCHAR* Message, EVisuProperty Property = EVisuProperty::BlueCurve) const;
#endif
};

} // namespace UE::CADKernel

