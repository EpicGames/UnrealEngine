// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/IsoTriangulator/IntersectionSegmentTool.h"

#include "Math/Geometry.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Structure/Grid.h"
#include "Utils/Util.h"

namespace UE::CADKernel
{ 

const FIsoSegment* FIntersectionSegmentTool::FindIntersectingSegment(const FIsoSegment& Segment) const
{
	using namespace IntersectionSegmentTool;
	const IntersectionSegmentTool::FSegment* IntersectingSegment = TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(&Segment.GetFirstNode(), &Segment.GetSecondNode());
	return IntersectingSegment ? IntersectingSegment->GetIsoSegment() : nullptr;
}

FIsoSegment* FIntersectionSegmentTool::FindIntersectingSegment(const FIsoSegment& Segment)
{
	return const_cast<FIsoSegment*> (static_cast<const FIntersectionSegmentTool*>(this)->FindIntersectingSegment(Segment));
}

IntersectionSegmentTool::FSegment::FSegment(const FGrid& Grid, const double Tolerance, const FIsoNode& StartNode, const FIsoNode& EndNode)
	: IntersectionToolBase::FSegment(Tolerance, StartNode.Get2DPoint(EGridSpace::UniformScaled, Grid), EndNode.Get2DPoint(EGridSpace::UniformScaled, Grid))
	, IsoSegment(nullptr)
{
}

IntersectionSegmentTool::FSegment::FSegment(const FGrid& Grid, const double Tolerance, const FIsoNode& StartNode, const FVector2d& EndPoint)
	: IntersectionToolBase::FSegment(Tolerance, StartNode.Get2DPoint(EGridSpace::UniformScaled, Grid), EndPoint)
	, IsoSegment(nullptr)
{
}

IntersectionSegmentTool::FSegment::FSegment(const FGrid& Grid, const double Tolerance, const FIsoSegment& InSegment)
	: IntersectionToolBase::FSegment(Tolerance, InSegment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), InSegment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid))
	, IsoSegment(&InSegment)
{
}

bool IntersectionSegmentTool::FSegment::IsValid() const
{
	return IsoSegment != nullptr && !IsoSegment->IsDelete();
}


const FIsoNode* IntersectionSegmentTool::FSegment::GetFirstNode() const
{
	return &IsoSegment->GetFirstNode();
}

const FIsoNode* IntersectionSegmentTool::FSegment::GetSecondNode() const
{
	return &IsoSegment->GetSecondNode();
}

} // namespace UE::CADKernel
