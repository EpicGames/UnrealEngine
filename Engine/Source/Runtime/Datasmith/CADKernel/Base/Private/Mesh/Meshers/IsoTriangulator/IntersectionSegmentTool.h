// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Boundary.h"
#include "Math/Geometry.h"

namespace UE::CADKernel
{
class FGrid;
class FIsoNode;
class FIsoInnerNode;
class FIsoSegment;

enum class EConnectionType : uint8
{
	DoesntStartFrom = 0,
	StartFrom,
	SuperimposedByOrOn,
	SameSegment,
};

namespace IntersectionToolBase
{
struct FSegment
{
	const FSegment2D Segment2D;

	/**
	 * Segment's axis aligned bounding box
	 */
	const FSurfacicBoundary Boundary;

	/**
	 * Uses as criterion to sort segments to optimize
	 * AxisMin = Boundary[EIso::IsoU].Min + Boundary[EIso::IsoV].Min
	 */
	double AxisMin;
	double AxisMax;

	/**
	 * WARNING StartPoint, EndPoint must be defined in EGridSpace::UniformScaled
	 */
	FSegment(const double Tolerance, const FVector2d& StartPoint, const FVector2d& EndPoint)
		: Segment2D(StartPoint, EndPoint)
		, Boundary(StartPoint, EndPoint, Tolerance)
	{
		AxisMin = Boundary[EIso::IsoU].Min + Boundary[EIso::IsoV].Min;
		AxisMax = Boundary[EIso::IsoU].Max + Boundary[EIso::IsoV].Max;
	}

	virtual ~FSegment() = default;

	virtual bool IsValid() const = 0;

	virtual const FIsoNode* GetFirstNode() const = 0;
	virtual const FIsoNode* GetSecondNode() const = 0;

	virtual const FIsoSegment* GetIsoSegment() const
	{
		return nullptr;
	}

	static EConnectionType IsSuperimposed(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD, bool bSameOrientation)
	{
		const FVector2d AB = SegmentAB.GetVector().GetSafeNormal();
		const FVector2d CD = SegmentCD.GetVector().GetSafeNormal();
		const double ParallelCoef = AB ^ CD;
		if (FMath::IsNearlyZero(ParallelCoef, DOUBLE_KINDA_SMALL_NUMBER))
		{
			const double OrientationCoef = AB | CD;
			if ((OrientationCoef >= 0) == bSameOrientation)
			{
				return EConnectionType::SuperimposedByOrOn;
			}
		}
		return EConnectionType::StartFrom;
	};

	EConnectionType DoesItStartFromAndSuperimposed(const FIsoNode* StartNode, const FVector2d* EndPoint, const FSegment2D& InSegment) const
	{
		if (GetFirstNode() == StartNode)
		{
			constexpr bool bSameOrientation = true;
			return IsSuperimposed(Segment2D, InSegment, bSameOrientation);
		}
		if (GetSecondNode() == StartNode)
		{
			constexpr bool bNotSameOrientation = false;
			return IsSuperimposed(Segment2D, InSegment, bNotSameOrientation);
		}

		return EConnectionType::DoesntStartFrom;
	}

	EConnectionType DoesItStartFromAndSuperimposed(const FVector2d* StartPoint, const FVector2d* EndPoint, const FSegment2D& InSegment) const
	{
		return EConnectionType::DoesntStartFrom;
	}

	EConnectionType DoesItStartFromAndSuperimposed(const FIsoNode* StartNode, const FIsoNode* EndNode, const FSegment2D& InSegment) const
	{
		if (GetFirstNode() == StartNode)
		{
			if (GetSecondNode() == EndNode)
			{
				return EConnectionType::SameSegment;
			}
			return IsSuperimposed(Segment2D, InSegment, true);
		}

		if (GetSecondNode() == EndNode)
		{
			return IsSuperimposed(Segment2D, InSegment, true);
		}

		if (GetFirstNode() == EndNode)
		{
			if (GetSecondNode() == StartNode)
			{
				return EConnectionType::SameSegment;
			}
			return IsSuperimposed(Segment2D, InSegment, false);
		}

		if (GetSecondNode() == StartNode)
		{
			return IsSuperimposed(Segment2D, InSegment, false);
		}

		return EConnectionType::DoesntStartFrom;
	}

	bool IsFullyBefore(const FSegment& Segment) const
	{
		return AxisMax < Segment.AxisMin;
	}

	bool IsFullyAfter(const FSegment& Segment) const
	{
		return Segment.AxisMax < AxisMin;
	}

	bool CouldItIntersect(const FSurfacicBoundary& SegmentBoundary) const
	{
		if ((Boundary[EIso::IsoU].Min > SegmentBoundary[EIso::IsoU].Max) || (Boundary[EIso::IsoV].Min > SegmentBoundary[EIso::IsoV].Max))
		{
			return false;
		}

		if ((Boundary[EIso::IsoU].Max < SegmentBoundary[EIso::IsoU].Min) || (Boundary[EIso::IsoV].Max < SegmentBoundary[EIso::IsoV].Min))
		{
			return false;
		}
		return true;
	}

	virtual bool DoesItIntersect(const FSegment& Segment) const
	{
		if (!CouldItIntersect(Segment.Boundary))
		{
			return false;
		}

		return DoIntersect(Segment2D, Segment.Segment2D);
	}

	bool IsParallelWith(const FSegment& Segment) const
	{
		return AreParallel(Segment2D, Segment.Segment2D);
	}
};
}

namespace IntersectionSegmentTool
{

struct FSegment : public IntersectionToolBase::FSegment
{
	const FIsoSegment* IsoSegment;

	FSegment(const FGrid& Grid, const double Tolerance, const FIsoSegment& InSegment);
	FSegment(const FGrid& Grid, const double Tolerance, const FIsoNode& StartNode, const FIsoNode& EndNode);
	FSegment(const FGrid& Grid, const double Tolerance, const FIsoNode& StartNode, const FVector2d& EndPoint);
	FSegment(const FGrid& Grid, const double Tolerance, const FVector2d& StartPoint, const FVector2d& EndPoint)
		: IntersectionToolBase::FSegment(Tolerance, StartPoint, EndPoint)
		, IsoSegment(nullptr)
	{
	}

	virtual bool IsValid() const override;

	virtual const FIsoNode* GetFirstNode() const override;
	virtual const FIsoNode* GetSecondNode() const override;

	virtual const FIsoSegment* GetIsoSegment() const override
	{
		return IsoSegment;
	}

};

}

template<typename SegmentType>
class TIntersectionSegmentTool
{
protected:
	const FGrid& Grid;

	TArray<SegmentType> Segments;
	bool bSegmentsAreSorted;

	const double Tolerance;

public:
	TIntersectionSegmentTool(const FGrid& InGrid, const double InTolerance)
		: Grid(InGrid)
		, bSegmentsAreSorted(false)
		, Tolerance(InTolerance)
	{
	}

	virtual ~TIntersectionSegmentTool() = default;

	void Empty(int32 InMaxNum)
	{
		Segments.Reset(InMaxNum);
		bSegmentsAreSorted = false;
	}

	void Reserve(int32 InMaxNum)
	{
		Segments.Reserve(InMaxNum);
	}

	void RemoveLast()
	{
		ensureCADKernel(!bSegmentsAreSorted);
		Segments.RemoveAt(Segments.Num() - 1);
		bSegmentsAreSorted = false;
	}

	void SetCount(int32 NewCount)
	{
		ensureCADKernel(NewCount < Segments.Num() && !bSegmentsAreSorted);
		Segments.SetNum(NewCount);
	}

	int32 Count()
	{
		return Segments.Num();
	}

	/**
	 * segments are sorted by DMin increasing
	 */
	void Sort()
	{
		Algo::Sort(Segments, [](const SegmentType& Segment1, const SegmentType& Segment2) { return Segment1.AxisMin < Segment2.AxisMin; });
		bSegmentsAreSorted = true;
	}

	template<typename ExtremityType1, typename ExtremityType2>
	const SegmentType* FindIntersectingSegment(const ExtremityType1* StartExtremity, const ExtremityType2* EndExtremity) const
	{
		using namespace IntersectionSegmentTool;
		const SegmentType InSegment(Grid, Tolerance, *StartExtremity, *EndExtremity);

		for (const SegmentType& Segment : Segments)
		{
			if (!Segment.IsValid())
			{
				continue;
			}

			if (bSegmentsAreSorted)
			{
				if (Segment.IsFullyBefore(InSegment))
				{
					continue;
				}

				if (Segment.IsFullyAfter(InSegment))
				{
					break;
				}
			}

			switch (Segment.DoesItStartFromAndSuperimposed(StartExtremity, EndExtremity, InSegment.Segment2D))
			{
			case EConnectionType::SameSegment:
			case EConnectionType::StartFrom:
				continue;
			case EConnectionType::SuperimposedByOrOn:
				return &Segment;
			case EConnectionType::DoesntStartFrom:
			default:
				break;
			}

			if (Segment.DoesItIntersect(InSegment))
			{
				return &Segment;
			}
		}

		return nullptr;
	}

	template<typename ExtremityType1, typename ExtremityType2>
	int32 FindIntersectingSegments(const ExtremityType1* StartExtremity, const ExtremityType2* EndExtremity, TArray<const FIsoSegment*>* OutIntersectedSegments) const
	{
		SegmentType InSegment(Grid, Tolerance, *StartExtremity, *EndExtremity);
		if (OutIntersectedSegments)
		{
			OutIntersectedSegments->Reset(10);
		}

		int32 IntersectionCount = 0;
		for (const SegmentType& Segment : Segments)
		{
			if (!Segment.IsValid())
			{
				continue;
			}

			if (bSegmentsAreSorted)
			{
				if (Segment.IsFullyBefore(InSegment))
				{
					continue;
				}

				if (Segment.IsFullyAfter(InSegment))
				{
					break;
				}
			}

			switch (Segment.DoesItStartFromAndSuperimposed(StartExtremity, EndExtremity, InSegment.Segment2D))
			{
			case EConnectionType::StartFrom:
			case EConnectionType::SameSegment:
				continue;

			case EConnectionType::SuperimposedByOrOn:
				++IntersectionCount;
				if (OutIntersectedSegments)
				{
					OutIntersectedSegments->Add(Segment.GetIsoSegment());
				}
				continue;

			case EConnectionType::DoesntStartFrom:
			default:
				break;
			}

			if (Segment.DoesItIntersect(InSegment))
			{
				++IntersectionCount;
				if (OutIntersectedSegments)
				{
					OutIntersectedSegments->Add(Segment.GetIsoSegment());
				}
			}
		}

		return IntersectionCount;
	}

#ifdef CADKERNEL_DEBUG
	virtual void Display(bool bDisplay, const TCHAR* Message, EVisuProperty Property = EVisuProperty::BlueCurve) const
	{
		if (!bDisplay)
		{
			return;
		}

		int32 Index = 0;
		Open3DDebugSession(Message);
		for (const SegmentType& Segment : Segments)
		{
			DisplaySegment(Segment.Segment2D[0] * DisplayScale, Segment.Segment2D[1] * DisplayScale, Index++, Property);
		}
		Close3DDebugSession();
	}
#endif

};

class FIntersectionSegmentTool : public TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>
{
public:
	FIntersectionSegmentTool(const FGrid& InGrid, const double Tolerance)
		: TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>(InGrid, Tolerance)
	{
	}

	bool Update(const FIsoSegment* Segment)
	{
		int32 SegmentIndex = Segments.IndexOfByPredicate([Segment](const IntersectionSegmentTool::FSegment& SegmentIter)
			{
				return SegmentIter.IsoSegment == Segment;
			});

		if (SegmentIndex != INDEX_NONE)
		{
			Segments.RemoveAt(SegmentIndex);
			Segments.EmplaceAt(SegmentIndex, Grid, Tolerance, *Segment);
			bSegmentsAreSorted = false;
			return true;
		}
		return false;
	}

	void AddSegments(FIsoSegment** InNewSegments, int32 Count)
	{
		bSegmentsAreSorted = false;
		Segments.Reserve(Count + Segments.Num());
		for (int32 Index = 0; Index < Count; ++Index)
		{
			FIsoSegment* NewSegment = InNewSegments[Index];
			AddSegment(*NewSegment);
		}
	}

	void AddSegments(const TArray<FIsoSegment*>& InNewSegments)
	{
		bSegmentsAreSorted = false;
		Segments.Reserve(InNewSegments.Num() + Segments.Num());
		for (FIsoSegment* Segment : InNewSegments)
		{
			AddSegment(*Segment);
		}
	}

	void AddSegment(const FIsoSegment& Segment)
	{
		bSegmentsAreSorted = false;
		Segments.Emplace(Grid, Tolerance, Segment);
	}

	void AddSegment(const FVector2d& StartPoint, const FVector2d& EndPoint)
	{
		bSegmentsAreSorted = false;
		Segments.Emplace(Grid, Tolerance, StartPoint, EndPoint);
	}

	/**
	 * @return true if the segment is found and removed
	 */
	bool Remove(const FIsoSegment* Segment)
	{
		int32 SegmentIndex = Segments.IndexOfByPredicate([Segment](const IntersectionSegmentTool::FSegment& SegmentIter)
			{
				return SegmentIter.IsoSegment == Segment;
			});

		if (SegmentIndex != INDEX_NONE)
		{
			Segments.RemoveAt(SegmentIndex);
			return true;
		}
		return false;
	}

	const FIsoSegment* FindIntersectingSegment(const FIsoSegment& Segment) const;
	FIsoSegment* FindIntersectingSegment(const FIsoSegment& Segment);

	bool DoesIntersect(const FIsoSegment& Segment) const
	{
		return FindIntersectingSegment(Segment) != nullptr;
	}

	/**
	 * WARNING StartPoint, EndPoint must be defined in EGridSpace::UniformScaled
	 */
	bool DoesIntersect(const FVector2d& StartPoint, const FVector2d& EndPoint) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(&StartPoint, &EndPoint) != nullptr;
	}

	/**
	 * Allow StartNode to be connected to one segment
	 * WARNING EndPoint must be defined in EGridSpace::UniformScaled
	 */
	bool DoesIntersect(const FIsoNode& StartNode, const FVector2d& EndPoint) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(&StartNode, &EndPoint) != nullptr;
	}

	bool DoesIntersect(const FIsoNode* StartNode, const FIsoNode* EndNode) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(StartNode, EndNode) != nullptr;
	}

	bool DoesIntersect(const FIsoNode& StartNode, const FIsoNode& EndNode) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(&StartNode, &EndNode) != nullptr;
	}

	/**
	 * Allow StartNode and EndNode to be connected to one segment
	 */
	const FIsoSegment* FindIntersectingSegment(const FIsoNode& StartNode, const FIsoNode& EndNode) const
	{
		const IntersectionSegmentTool::FSegment* IntersectingSegment = TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(&StartNode, &EndNode);
		if (IntersectingSegment)
		{
			return IntersectingSegment->IsoSegment;
		}
		return nullptr;
	}

	const FIsoSegment* FindIntersectingSegment(const FIsoNode* StartNode, const FIsoNode* EndNode) const
	{
		const IntersectionSegmentTool::FSegment* IntersectingSegment = TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegment(StartNode, EndNode);
		if (IntersectingSegment)
		{
			return IntersectingSegment->IsoSegment;
		}
		return nullptr;
	}

	bool FindIntersectingSegments(const FIsoNode* StartNode, const FIsoNode* EndNode, TArray<const FIsoSegment*>& OutIntersections) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegments(StartNode, EndNode, &OutIntersections) > 0;
	}

	bool FindIntersectingSegments(const FIsoNode& StartNode, const FIsoNode& EndNode, TArray<const FIsoSegment*>& OutIntersections) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegments(&StartNode, &EndNode, &OutIntersections) > 0;
	}

	int32 CountIntersections(const FIsoNode* StartNode, const FIsoNode* EndNode) const
	{
		return TIntersectionSegmentTool<IntersectionSegmentTool::FSegment>::FindIntersectingSegments(StartNode, EndNode, nullptr);
	}
};

} // namespace UE::CADKernel

