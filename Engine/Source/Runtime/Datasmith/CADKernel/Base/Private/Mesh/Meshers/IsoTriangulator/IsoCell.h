// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Factory.h"
#include "Core/Types.h"
#include "Mesh/MeshEnum.h"
#include "Mesh/Meshers/IsoTriangulator.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionSegmentTool.h"
#include "Mesh/Structure/Grid.h"
#include "Topo/TopologicalFace.h"

namespace UE::CADKernel
{
class FGrid;
class FIsoTriangulator;
struct FLoopConnexion;

struct FCellLoop
{
	int32 Id;
	FVector2d Barycenter;
	TArray<FLoopNode*> Nodes;
	TArray<FCellConnexion*> Connexions;
	bool bIsOuterLoop = false;
	bool bIsConnected = false;

	FCellLoop(const int32 InIndex, TArray<FLoopNode*>& InNodes, const FGrid& Grid)
		: Id(InIndex)
		, Barycenter(FVector2d::ZeroVector)
		, Nodes(InNodes)
		, bIsOuterLoop(Nodes[0]->GetLoopIndex() == 0)
	{
		for (const FLoopNode* Node : Nodes)
		{
			Barycenter += Node->Get2DPoint(EGridSpace::UniformScaled, Grid);
		}
		Barycenter /= (double)Nodes.Num();
	}

	FCellLoop(const int32 InIndex)
		: Id(InIndex)
		, Barycenter(FVector::ZeroVector)
	{
	}

	virtual ~FCellLoop() = default;

	virtual bool IsCellCorner() const 
	{
		return false;
	}

	virtual bool IsCellCornerOrOuterLoop() const
	{
		return bIsOuterLoop;
	}

	void PropagateAsConnected();
};

struct FCellCorner : public FCellLoop
{
	FIsoInnerNode& CornerNode;

	FCellCorner(const int32 InIndex, FIsoInnerNode& InNode, const FGrid& Grid)
		: FCellLoop(InIndex)
		, CornerNode(InNode)
	{
		Barycenter = CornerNode.Get2DPoint(EGridSpace::UniformScaled, Grid);
	}

	virtual bool IsCellCorner() const override
	{
		return true;
	}

	virtual bool IsCellCornerOrOuterLoop() const
	{
		return true;
	}

};

struct FCellConnexion
{
	bool bIsConnexionWithOuter = false;
	FCellLoop& Loop1;
	FCellLoop& Loop2;

	double MinDistance = HUGE_VALUE_SQUARE;

	FLoopNode* NodeA = nullptr;
	FIsoNode* NodeB = nullptr;

	FIsoSegment* Segment = nullptr;

	FCellConnexion(FCellLoop& InLoop1, FCellLoop& InLoop2)
		: Loop1(InLoop1)
		, Loop2(InLoop2)
	{
		bIsConnexionWithOuter = Loop1.bIsOuterLoop || (!Loop2.IsCellCorner() && Loop2.bIsOuterLoop);
		Loop1.Connexions.Add(this);
		Loop2.Connexions.Add(this);
	}

	const FCellLoop* GetOtherLoop(const FCellLoop* Loop) const
	{
		return (Loop == &Loop1) ? &Loop2 : &Loop1;
	}

	FCellLoop* GetOtherLoop(const FCellLoop* Loop)
	{
		return (Loop == &Loop1) ? &Loop2 : &Loop1;
	}

	bool IsShortestPath(const int32 MaxLoopCount)
	{
		TMap<const FCellLoop*, double> DistanceToLoops;
		DistanceToLoops.Reserve(MaxLoopCount);
		DistanceToLoops.Add(&Loop2, 0.);
		double MinPathDistance = HUGE_VALUE;
		while (true)
		{
			double DistanceToCurrent = HUGE_VALUE;
			TPair<const FCellLoop*, double>* CurrentLoop = nullptr;
			for (TPair<const FCellLoop*, double>& DistanceToLoop : DistanceToLoops)
			{
				if (DistanceToLoop.Value >= 0. && DistanceToLoop.Value < DistanceToCurrent)
				{
					DistanceToCurrent = DistanceToLoop.Value;
					CurrentLoop = &DistanceToLoop;
				}
			}

			if (!CurrentLoop || CurrentLoop->Value > MinDistance)
			{
				return true;
			}

			if (CurrentLoop->Key == &Loop1)
			{
				return (CurrentLoop->Value > MinDistance);
			}

			for (FCellConnexion* Connexion : CurrentLoop->Key->Connexions)
			{
				if (Connexion == this)
				{
					continue;
				}

				const FCellLoop* NextLoop = Connexion->GetOtherLoop(CurrentLoop->Key);
				if (NextLoop->IsCellCorner())
				{
					continue;
				}

				if (NextLoop->bIsOuterLoop)
				{
					continue;
				}

				const double DistanceToNextByCurrent = CurrentLoop->Value + Connexion->MinDistance;

				double* DistanceToNextLoop = DistanceToLoops.Find(NextLoop);
				if (DistanceToNextLoop)
				{
					if (*DistanceToNextLoop > DistanceToNextByCurrent)
					{
						*DistanceToNextLoop = DistanceToNextByCurrent;
					}
				}
				else
				{
					DistanceToLoops.Add(NextLoop, DistanceToNextByCurrent);
				}
			}

			CurrentLoop->Value = -HUGE_VALUE;
		}

		return true;
	}

	bool IsShortestPathToOuterLoop(const int32 MaxLoopCount)
	{
		TMap<const FCellLoop*, double> DistanceToLoops;
		DistanceToLoops.Reserve(MaxLoopCount);
		DistanceToLoops.Add(&Loop2, 0.);
		double MinPathDistance = HUGE_VALUE;

		while (true)
		{
			double DistanceToCurrent = HUGE_VALUE;
			TPair<const FCellLoop*, double>* CurrentLoopPair = nullptr;
			for (TPair<const FCellLoop*, double>& DistanceToLoop : DistanceToLoops)
			{
				if (DistanceToLoop.Value >= 0. && DistanceToLoop.Value < DistanceToCurrent)
				{
					DistanceToCurrent = DistanceToLoop.Value;
					CurrentLoopPair = &DistanceToLoop;
				}
			}

			if (!CurrentLoopPair || CurrentLoopPair->Value > MinDistance)
			{
				return true;
			}

			const FCellLoop* CurrentLoop = CurrentLoopPair->Key;
			double& CurrentLoopDistance = CurrentLoopPair->Value;

			for (FCellConnexion* Connexion : CurrentLoop->Connexions)
			{
				if (Connexion == this)
				{
					continue;
				}

				const FCellLoop* NextLoop = Connexion->GetOtherLoop(CurrentLoop);
				if (NextLoop->IsCellCorner())
				{
					continue;
				}

				const double DistanceToNextByCurrent = CurrentLoopDistance + Connexion->MinDistance;
				if (NextLoop->bIsOuterLoop)
				{
					if (DistanceToNextByCurrent < MinDistance)
					{
						return false;
					}
					continue;
				}

				double* DistanceToNextLoop = DistanceToLoops.Find(NextLoop);
				if (DistanceToNextLoop)
				{
					if (*DistanceToNextLoop > DistanceToNextByCurrent)
					{
						*DistanceToNextLoop = DistanceToNextByCurrent;
					}
				}
				else
				{
					DistanceToLoops.Add(NextLoop, DistanceToNextByCurrent);
				}
			}

			CurrentLoopDistance = -HUGE_VALUE;
		}

		return true;
	}

	bool IsShortestPathToCorner(const int32 MaxLoopCount)
	{
		TMap<const FCellLoop*, double> DistanceToLoops;
		DistanceToLoops.Reserve(MaxLoopCount);
		DistanceToLoops.Add(&Loop2, 0.);
		double MinPathDistance = HUGE_VALUE;

		while (true)
		{
			double DistanceToCurrent = HUGE_VALUE;
			TPair<const FCellLoop*, double>* CurrentLoopPair = nullptr;
			for (TPair<const FCellLoop*, double>& DistanceToLoop : DistanceToLoops)
			{
				if (DistanceToLoop.Value >= 0. && DistanceToLoop.Value < DistanceToCurrent)
				{
					DistanceToCurrent = DistanceToLoop.Value;
					CurrentLoopPair = &DistanceToLoop;
				}
			}

			if (!CurrentLoopPair || CurrentLoopPair->Value > MinDistance)
			{
				return true;
			}

			const FCellLoop* CurrentLoop = CurrentLoopPair->Key;
			double& CurrentLoopDistance = CurrentLoopPair->Value;

			for (FCellConnexion* Connexion : CurrentLoop->Connexions)
			{
				if (Connexion == this)
				{
					continue;
				}

				const FCellLoop* NextLoop = Connexion->GetOtherLoop(CurrentLoop);
				if (NextLoop->bIsOuterLoop)
				{
					continue;
				}

				const double DistanceToNextByCurrent = CurrentLoopDistance + Connexion->MinDistance;
				if (NextLoop->IsCellCorner())
				{
					if ((NextLoop == &Loop1) && (DistanceToNextByCurrent < MinDistance))
					{
						return false;
					}
					continue;
				}

				double* DistanceToNextLoop = DistanceToLoops.Find(NextLoop);
				if (DistanceToNextLoop)
				{
					if (*DistanceToNextLoop > DistanceToNextByCurrent)
					{
						*DistanceToNextLoop = DistanceToNextByCurrent;
					}
				}
				else
				{
					DistanceToLoops.Add(NextLoop, DistanceToNextByCurrent);
				}
			}

			CurrentLoopDistance = -HUGE_VALUE;
		}

		return true;
	}
};

struct FCellPath
{
	double Length = 0;
	FCellLoop* CurrentLoop;
	TArray<FCellLoop> Path;
};

struct FCell
{
	FIsoTriangulator& Triangulator;
	const FGrid& Grid;
	int32 Id;

	int32 InnerLoopCount = 0;
	int32 OuterLoopCount = 0;

	TArray<FIsoSegment*> CandidateSegments;
	TArray<FIsoSegment*> FinalSegments;

	FIntersectionSegmentTool IntersectionTool;

	TArray<FCellLoop> CellLoops;
	TArray<FCellCorner> CellCorners;
	TArray<FCellConnexion> LoopConnexions;
	TArray<int32> LoopCellBorderIndices;

	FCell(const int32 InLoopIndex, TArray<FLoopNode*>& InNodes, FIsoTriangulator& InTriangulator)
		: Triangulator(InTriangulator)
		, Grid(Triangulator.GetGrid())
		, Id(InLoopIndex)
		, IntersectionTool(Triangulator.GetGrid(), Triangulator.Tolerances.GeometricTolerance)
	{
		const int32 NodeCount = InNodes.Num();
		ensureCADKernel(NodeCount > 0);

		// Subdivide InNodes in SubLoop
		Algo::Sort(InNodes, [&](const FLoopNode* LoopNode1, const FLoopNode* LoopNode2)
			{
				return LoopNode1->GetGlobalIndex() < LoopNode2->GetGlobalIndex();
			});

		int32 LoopCount = 0;
		FLoopNode* PreviousNode = nullptr;
		for (FLoopNode* Node : InNodes)
		{
			if (&Node->GetPreviousNode() != PreviousNode)
			{
				LoopCount++;
			}
			PreviousNode = Node;
		}

		CellLoops.Reserve(LoopCount);

		LoopCount = 0;
		TArray<FLoopNode*> LoopNodes;
		LoopNodes.Reserve(NodeCount);

		int32 LoopIndex = -1;
		FCellLoop* FirstLoopCell = nullptr;

		TFunction<void(TArray<FLoopNode*>&)> MakeLoopCell = [&FirstLoopCell, &CellLoops = CellLoops, &LoopIndex, &Grid = Grid](TArray<FLoopNode*>& LoopNodes)
		{
			if (LoopNodes.Num())
			{
				if ((LoopIndex == LoopNodes[0]->GetLoopIndex()) && (&LoopNodes.Last()->GetNextNode() == FirstLoopCell->Nodes[0]))
				{
					LoopNodes.Append(FirstLoopCell->Nodes);
					FirstLoopCell->Nodes = LoopNodes;
				}
				else
				{
					CellLoops.Emplace(CellLoops.Num(), LoopNodes, Grid);
				}

				if (LoopIndex != LoopNodes[0]->GetLoopIndex())
				{
					LoopIndex = LoopNodes[0]->GetLoopIndex();
					FirstLoopCell = &CellLoops.Last();
				}
			}
		};

		PreviousNode = nullptr;
		for (FLoopNode* Node : InNodes)
		{
			Node->SetMarker1();
		}

		for (FLoopNode* Node : InNodes)
		{
			if (Node->IsDeleteOrHasMarker2() )
			{
				continue;
			}

			FLoopNode* StartNode = &Node->GetPreviousNode();
			while (StartNode->HasMarker1() && (StartNode != Node))
			{
				StartNode = &StartNode->GetPreviousNode();
			}
			if (!StartNode->HasMarker1())
			{
				StartNode = &StartNode->GetNextNode();
			}

			LoopNodes.Reset(NodeCount);
			FLoopNode* NextNode = StartNode;
			while (NextNode->HasMarker1NotMarker2())
			{
				LoopNodes.Add(NextNode);
				NextNode->SetMarker2();
				NextNode = &NextNode->GetNextNode();
			}

			MakeLoopCell(LoopNodes);
		}

		for (FLoopNode* Node : InNodes)
		{
			Node->ResetMarkers();
		}

		for(const FCellLoop& LoopCell : CellLoops)
		{
			if (LoopCell.bIsOuterLoop)
			{
				OuterLoopCount++;
			}
			else
			{
				InnerLoopCount++;
			}
		}
	}

	void InitLoopConnexions();

	TArray<TPair<int32, FVector2d>> GetLoopBarycenters()
	{
		TArray<TPair<int32, FVector2d>> LoopBarycenters;
		LoopBarycenters.Reserve(CellLoops.Num());
		for (const FCellLoop& LoopCell : CellLoops)
		{
			if(!LoopCell.bIsOuterLoop)
			{
				LoopBarycenters.Emplace(LoopCell.Id, LoopCell.Barycenter);
			}
		}
		return MoveTemp(LoopBarycenters);
	}

	void FindCandidateToConnectLoopsByNeighborhood();
	void FindCandidateToConnectCellCornerToLoops();

	void SelectSegmentToConnectLoops(TFactory<FIsoSegment>& SegmentFactory);
	void SelectSegmentToConnectLoopToCorner(TFactory<FIsoSegment>& SegmentFactory);
	void CheckAllLoopsConnectedTogetherAndConnect();

	//void ConnectLoopsByNeighborhood2();

	/**
	 *  SubLoopA                  SubLoopB
	 *      --X---X             X-----X--
	 *             \           /
	 *              \         /
	 *               X=======X
	 *              /         \
	 *             /           \
	 *      --X---X             X-----X--
	 *
	 *     ======= ShortestSegment
	 */
	void TryToConnectTwoSubLoopsWithShortestSegment(FCellConnexion& LoopConnexion);

	void TryToCreateSegment(FCellConnexion& LoopConnexion);
};

} // namespace UE::CADKernel

