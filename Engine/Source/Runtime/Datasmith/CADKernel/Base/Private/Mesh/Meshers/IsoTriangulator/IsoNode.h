// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/SlopeUtils.h"
#include "Mesh/MeshEnum.h"
#include "Mesh/Structure/Grid.h"

namespace UE::CADKernel
{
class FGrid;
class FIsoSegment;

enum class EIsoLink : uint8
{
	IsoUNext = 0,
	IsoVNext = 1,
	IsoUPrevious = 2,
	IsoVPrevious = 3,
};

enum class EIsoNodeStates : uint16
{
	None = 0x0000u,	// No flags.

	LinkedToPreviousU = 0x0001u,
	LinkedToNextU = 0x0002u,
	LinkedToPreviousV = 0x0004u,
	LinkedToNextV = 0x0008u,

	NodeComplete = 0x000Fu,

	LinkedToLoop = 0x0010u,
	InnerMeshLoop = 0x0020u,
	ThinZoneNode = 0x0040u,

	Delete = 0x0080u,

	FirstQuarter = 0x0100u,
	SecondQuarter = 0x0200u,
	ThirdQuarter = 0x0400u,
	FourthQuarter = 0x0800u,

	NearlyIsoU = 0x1000u,  // Loop nodes
	NearlyIsoV = 0x2000u,  // Loop nodes

	HasMarker1 = 0x4000u,
	HasMarker2 = 0x8000u,

	All = 0xFFFFu
};

ENUM_CLASS_FLAGS(EIsoNodeStates);

class FIsoInnerNode;
class FLoopNode;

/**
 * Inner Node of the TopologicalFace, these nodes are IsoNodes as they are build according to the TopologicalFace iso cutting coordinates UV
 */
class FIsoNode
{
protected:
	TArray<FIsoSegment*> ConnectedSegments;
	EIsoNodeStates States;

	int32 LocalIndex; // Index of the node either in loop nodes either in inner nodes
	int32 GlobalIndex; // Index of the node in the face
	int32 NodeId; // Id of the node in the mesh

public:
	FIsoNode(int32 InNodeIndex, int32 InFaceIndex, int32 InNodeId)
		: ConnectedSegments()
		, States(EIsoNodeStates::None)
		, LocalIndex(InNodeIndex)
		, GlobalIndex(InFaceIndex)
		, NodeId(InNodeId)
	{
		ConnectedSegments.Reserve(5);
	};

	virtual ~FIsoNode() = default;

	virtual void Delete()
	{
		ConnectedSegments.Empty();
		GlobalIndex = -1;

		States = EIsoNodeStates::Delete;
	}

	bool IsDelete() const
	{
		return (States & EIsoNodeStates::Delete) == EIsoNodeStates::Delete;
	}

	const int32 GetIndex() const
	{
		return LocalIndex;
	}

	const int32 GetGlobalIndex() const
	{
		return GlobalIndex;
	}

	const int32 GetNodeId() const
	{
		return NodeId;
	}

	const TArray<FIsoSegment*>& GetConnectedSegments() const
	{
		return ConnectedSegments;
	}

	FIsoSegment* GetSegmentConnectedTo(const FIsoNode* Node) const;

	void ConnectSegment(FIsoSegment& Segment)
	{
		ConnectedSegments.Add(&Segment);
	}

	void DisconnectSegment(FIsoSegment& Segment)
	{
		ConnectedSegments.RemoveSingle(&Segment);
	}

	const virtual bool IsALoopNode() const = 0;

	void SetThinZoneNodeMarker()
	{
		States |= EIsoNodeStates::ThinZoneNode;
	}

	bool IsThinZoneNode() const
	{
		return (States & EIsoNodeStates::ThinZoneNode) == EIsoNodeStates::ThinZoneNode;
	}

	bool IsDeleteOrThinNode() const
	{
		constexpr EIsoNodeStates DeleteOrThinNode = EIsoNodeStates::ThinZoneNode | EIsoNodeStates::Delete;
		return (States & DeleteOrThinNode) != EIsoNodeStates::None;
	}

	void SetLinkedToLoopMarker()
	{
		States |= EIsoNodeStates::LinkedToLoop;
	}

	void SetLinkedToLoopInQuarter(int32 Quarter)
	{
		switch (Quarter)
		{
		case 0:
			States |= EIsoNodeStates::FirstQuarter;
		case 1:
			States |= EIsoNodeStates::SecondQuarter;
		case 2:
			States |= EIsoNodeStates::ThirdQuarter;
		default:
			States |= EIsoNodeStates::FourthQuarter;
		}
	}

	bool IsLinkedToLoopInNearlyIso(int32 Iso) const
	{
		if (Iso > 3)
		{
			Iso -= 4;
		}
		switch ((EIsoLink)Iso)
		{
		case EIsoLink::IsoUNext:
			return (States & EIsoNodeStates::LinkedToNextU) == EIsoNodeStates::LinkedToNextU;
		case EIsoLink::IsoVNext:
			return (States & EIsoNodeStates::LinkedToNextV) == EIsoNodeStates::LinkedToNextV;
		case EIsoLink::IsoUPrevious:
			return (States & EIsoNodeStates::LinkedToPreviousU) == EIsoNodeStates::LinkedToPreviousU;
		default:
			return (States & EIsoNodeStates::LinkedToPreviousV) == EIsoNodeStates::LinkedToPreviousV;
		}
	}

	void SetLinkedToIso(EIsoLink Iso) // [3Pi/2 2Pi]
	{
		switch (Iso)
		{
		case EIsoLink::IsoUNext:
			States |= EIsoNodeStates::LinkedToNextU;
			break;
		case EIsoLink::IsoVNext:
			States |= EIsoNodeStates::LinkedToNextV;
			break;
		case EIsoLink::IsoUPrevious:
			States |= EIsoNodeStates::LinkedToPreviousU;
			break;
		default:
			States |= EIsoNodeStates::LinkedToPreviousV;
		}
	}

	void SetLinkedToIso(int32 Iso) // [3Pi/2 2Pi]
	{
		if (Iso > 3)
		{
			Iso -= 4;
		}
		switch ((EIsoLink)Iso)
		{
		case EIsoLink::IsoUNext:
			States |= EIsoNodeStates::LinkedToNextU;
			break;
		case EIsoLink::IsoVNext:
			States |= EIsoNodeStates::LinkedToNextV;
			break;
		case EIsoLink::IsoUPrevious:
			States |= EIsoNodeStates::LinkedToPreviousU;
			break;
		default:
			States |= EIsoNodeStates::LinkedToPreviousV;
		}
	}

	bool HasMarker1() const
	{
		return ((States & EIsoNodeStates::HasMarker1) == EIsoNodeStates::HasMarker1);
	}

	bool HasMarker2() const
	{
		return ((States & EIsoNodeStates::HasMarker2) == EIsoNodeStates::HasMarker2);
	}

	bool IsDeleteOrHasMarker2() const
	{
		constexpr EIsoNodeStates HasMarkerDeleteOrMarker2 = EIsoNodeStates::Delete | EIsoNodeStates::HasMarker2;
		return ((States & HasMarkerDeleteOrMarker2) != EIsoNodeStates::None);
	}

	bool HasMarker1NotMarker2() const
	{
		constexpr EIsoNodeStates HasMarker12 = EIsoNodeStates::HasMarker1 | EIsoNodeStates::HasMarker2;
		return ((States & HasMarker12) == EIsoNodeStates::HasMarker1);
	}

	void SetMarker1()
	{
		States |= EIsoNodeStates::HasMarker1;
	}

	void SetMarker2()
	{
		States |= EIsoNodeStates::HasMarker2;
	}

	void ResetMarkers()
	{
		constexpr EIsoNodeStates ResetAllMarkers = ~(EIsoNodeStates::HasMarker1 | EIsoNodeStates::HasMarker2);
		States &= ResetAllMarkers;
	}

	/**
	 * Return the 2d coordinate of the node according to the space
	 */
	virtual const FVector2d& Get2DPoint(EGridSpace Space, const FGrid& Grid) const = 0;
	virtual void Set2DPoint(EGridSpace Space, FGrid& Grid, const FVector2d& NewCoordinate) = 0;

	virtual const FVector& Get3DPoint(const FGrid& Grid) const = 0;
	virtual const FVector3f& GetNormal(const FGrid& Grid) const = 0;

	virtual bool operator==(const FIsoNode& OtherNode) const = 0;
	virtual bool IsEqualTo(const FLoopNode& OtherNode) const = 0;
	virtual bool IsEqualTo(const FIsoInnerNode& OtherNode) const = 0;

	virtual uint32 GetTypeHash() const = 0;

};

/**
 * Node of the loop of the TopologicalFace
 */
class FLoopNode : public FIsoNode
{
protected:
	int32 LoopIndex;
	FLoopNode* ConnectedLoopNodes[2];

public:
	FLoopNode(int32 InLoopIndex, int32 InNodeLoopIndex, int32 InNodeFaceIndex, int32 NodeId)
		: FIsoNode(InNodeLoopIndex, InNodeFaceIndex, NodeId)
		, LoopIndex(InLoopIndex)
		, ConnectedLoopNodes{ nullptr, nullptr }
	{
	};

	virtual void Delete() override
	{
		FIsoNode::Delete();

		//LoopIndex = -1;
		ConnectedLoopNodes[0] = nullptr;
		ConnectedLoopNodes[1] = nullptr;
	}

	virtual bool IsEqualTo(const FLoopNode& OtherNode) const
	{
		return ((GetIndex() == OtherNode.GetIndex()) && (GetLoopIndex() == OtherNode.GetLoopIndex()));
	}

	virtual bool IsEqualTo(const FIsoInnerNode& OtherNode) const
	{
		return false;
	}

	bool operator==(const FIsoNode& OtherNode) const
	{
		return OtherNode.IsEqualTo(*this);
	}

	virtual const bool IsALoopNode() const override
	{
		return true;
	}

	const int32 GetLoopIndex() const
	{
		return LoopIndex;
	}

	void SetNextConnectedNode(FLoopNode* NextNode)
	{
		ConnectedLoopNodes[1] = NextNode;
	}
	void SetPreviousConnectedNode(FLoopNode* PreviousNode)
	{
		ConnectedLoopNodes[0] = PreviousNode;
	}

	FLoopNode& GetPreviousNode() const
	{
		return *ConnectedLoopNodes[0];
	}

	FLoopNode& GetNextNode() const
	{
		return *ConnectedLoopNodes[1];
	}

	FIsoSegment& GetPreviousSegment() const
	{
		return *GetSegmentConnectedTo(ConnectedLoopNodes[0]);
	}

	FIsoSegment& GetNextSegment() const
	{
		return *GetSegmentConnectedTo(ConnectedLoopNodes[1]);
	}

	/**
	 * Return the 2d coordinate of the node according to the space
	 */
	virtual const FVector2d& Get2DPoint(EGridSpace Space, const FGrid& Grid) const override
	{
		return Grid.GetLoop2DPoint(Space, LoopIndex, LocalIndex);
	}

	virtual void Set2DPoint(EGridSpace Space, FGrid& Grid, const FVector2d& NewCoordinate) override
	{
		Grid.SetLoop2DPoint(Space, LoopIndex, LocalIndex, NewCoordinate);
	}

	virtual const FVector& Get3DPoint(const FGrid& Grid) const override
	{
		return Grid.GetLoops3D()[LoopIndex][LocalIndex];
	}

	virtual const FVector3f& GetNormal(const FGrid& Grid) const override
	{
		return Grid.GetLoopNormals()[LoopIndex][LocalIndex];
	}

	/**
	 * Only for display purpose as it return a copy of the point
	 */
	//virtual const FVector GetPoint(EGridSpace Space, const FGrid& Grid) const override
	//{
	//	switch (Space)
	//	{
	//	case EGridSpace::Default2D:
	//	case EGridSpace::Scaled:
	//	case EGridSpace::UniformScaled:
	//		return Grid.GetLoop2DPoint(Space, LoopIndex, LocalIndex);
	//	}
	//	return FVector::ZeroVector;
	//}

	/**
	 * Check if the segment starting from this loop node (S) to EndSegmentCoordinate (E) is inside the face
	 * Warning, inner loops (loop index > 0) are not in the same orientation than outer loop
	 *
	 * Inside e.g.:
	 *    X---------------X----------------X        X------------------X-----------X
	 *    |                                |        |                              |
	 *    |         X-----------------X    |        |            X-------------X   |
	 *    |         |                 |    |        |            |             |   |
	 *    X   E-----S                 |    |    or  S---------E  |             |   |
	 *    |         |                 |    |        |            |             |   |
	 *    |         X-----------------X    |        |            X-------------X   |
	 *    |                                |        |                              |
	 *    X---------------X----------------X        X------------------X-----------X
	 *
	 * Outside e.g.:
	 *    X---------------X----------------X        X------------------X-----------X
	 *    |                                |        |                              |
	 *    |         X-----------------X    |        |            X-------------X   |
	 *    |         |                 |    |        |            |             |   |
	 *    |         S------E          |    |    or  |            |             |   S------E
	 *    |         |                 |    |        |            |             |   |
	 *    |         X-----------------X    |        |            X-------------X   |
	 *    |                                |        |                              |
	 *    X---------------X----------------X        X------------------X-----------X
	 */
	bool IsSegmentBeInsideFace(const FVector2d& EndSegmentCoordinate, const FGrid& Grid, const double FlatAngle) const
	{
		return (!IsPointPInsideSectorABC(GetPreviousNode().Get2DPoint(EGridSpace::UniformScaled, Grid), Get2DPoint(EGridSpace::UniformScaled, Grid), GetNextNode().Get2DPoint(EGridSpace::UniformScaled, Grid), EndSegmentCoordinate, FlatAngle));
	}

	virtual uint32 GetTypeHash() const override
	{
		return HashCombine(GetIndex(), GetLoopIndex() + 1);
	}
};

inline uint32 GetTypeHash(const FIsoNode& Node)
{
	return Node.GetTypeHash();
}

class FIsoInnerNode : public FIsoNode
{
public:
	FIsoInnerNode(int32 NodeIndex, int32 FaceIndex, int32 NodeId)
		: FIsoNode(NodeIndex, FaceIndex, NodeId)
	{
	};

	virtual bool IsEqualTo(const FLoopNode& OtherNode) const
	{
		return false;
	}

	virtual bool IsEqualTo(const FIsoInnerNode& OtherNode) const
	{
		return (GetIndex() == OtherNode.GetIndex());
	}

	bool operator==(const FIsoNode& OtherNode) const
	{
		return OtherNode.IsEqualTo(*this);
	}

	const bool IsIsolated() const
	{
		return (ConnectedSegments.Num() == 0);
	}

	virtual const bool IsALoopNode() const override
	{
		return false;
	}

	bool IsLinkedToPreviousU() const
	{
		return (States & EIsoNodeStates::LinkedToPreviousU) == EIsoNodeStates::LinkedToPreviousU;
	}

	bool IsLinkedToNextU() const
	{
		return (States & EIsoNodeStates::LinkedToNextU) == EIsoNodeStates::LinkedToNextU;
	}

	bool IsLinkedToPreviousV() const
	{
		return (States & EIsoNodeStates::LinkedToPreviousV) == EIsoNodeStates::LinkedToPreviousV;
	}

	bool IsLinkedToNextV() const
	{
		return (States & EIsoNodeStates::LinkedToNextV) == EIsoNodeStates::LinkedToNextV;;
	}

	/** the node is connected in its 4 directions*/
	bool IsComplete() const
	{
		return (States & EIsoNodeStates::NodeComplete) == EIsoNodeStates::NodeComplete;
	}

	bool IsLinkedToBoundary() const
	{
		return (States & EIsoNodeStates::LinkedToLoop) == EIsoNodeStates::LinkedToLoop;
	}

	virtual const FVector2d& Get2DPoint(EGridSpace Space, const FGrid& Grid) const override
	{
		return Grid.GetInner2DPoint(Space, LocalIndex);
	}

	virtual void Set2DPoint(EGridSpace Space, FGrid& Grid, const FVector2d& NewCoordinate) override
	{
		return Grid.SetInner2DPoint(Space, LocalIndex, NewCoordinate);
	}

	virtual const FVector& Get3DPoint(const FGrid& Grid) const override
	{
		return Grid.GetInner3DPoint(LocalIndex);
	}

	virtual const FVector3f& GetNormal(const FGrid& Grid) const override
	{
		return Grid.GetPointNormal(LocalIndex);
	}

	/**
	 * Only for display purpose as it return a copy of the point
	 */
	//virtual const FVector GetPoint(EGridSpace Space, const FGrid& Grid) const override
	//{
	//	switch (Space)
	//	{
	//	case EGridSpace::Default2D:
	//	case EGridSpace::Scaled:
	//	case EGridSpace::UniformScaled:
	//		return Grid.GetInner2DPoint(Space, LocalIndex);
	//	}
	//	return FVector::ZeroVector;
	//}

	void OffsetId(int32 StartId)
	{
		NodeId += StartId;
	}

	virtual uint32 GetTypeHash() const override
	{
		return ::GetTypeHash(GetIndex());
	}
};


} // namespace UE::CADKernel

