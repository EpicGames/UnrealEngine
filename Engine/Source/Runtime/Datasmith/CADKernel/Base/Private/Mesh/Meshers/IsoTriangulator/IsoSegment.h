// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"

namespace UE::CADKernel
{

enum class ESegmentType : uint8
{
	Unknown = 0,
	Loop = 1,
	ThinZone = 2,
	IsoU,
	IsoV,
	LoopToLoop, // IsBoundaryToInner use that xxxToBoundary are bigger than BoundaryToBoundary
	IsoLoopToLoop,
	InnerToLoopU,
	InnerToLoopV,
	InnerToLoop,
};

enum class EIsoSegmentStates : uint8
{
	None = 0x00u,	// No flags.

	Candidate = 0x01u,

	LeftCycle = 0x02u,
	RightCycle = 0x04u,

	Degenerate = 0x08u,

	LeftTriangle = 0x10u,
	RightTriangle = 0x20u,
	SegmentComplete = 0x30u,

	Final = 0x40u,
	Delete = 0x80u,

	All = 0xFFu
};

ENUM_CLASS_FLAGS(EIsoSegmentStates);


class FGrid;

class FIsoSegment
{
protected:
	FIsoNode* FirstNode;
	FIsoNode* SecondNode;
	ESegmentType Type;
	EIsoSegmentStates States;

public:

	FIsoSegment()
		: FirstNode(nullptr)
		, SecondNode(nullptr)
		, Type(ESegmentType::Unknown)
		, States(EIsoSegmentStates::None)
	{
	}

	void Init(FIsoNode& InFirstNode, FIsoNode& InSecondNode, const ESegmentType InType)
	{
		States = EIsoSegmentStates::None;
		FirstNode = &InFirstNode;
		SecondNode = &InSecondNode;

		Type = InType;
		switch (Type)
		{
		case ESegmentType::IsoU:
			InFirstNode.SetLinkedToIso(EIsoLink::IsoUNext);
			InSecondNode.SetLinkedToIso(EIsoLink::IsoUPrevious);
			break;
		case ESegmentType::IsoV:
			InFirstNode.SetLinkedToIso(EIsoLink::IsoVNext);
			InSecondNode.SetLinkedToIso(EIsoLink::IsoVPrevious);
			break;
		}
	}

	bool ConnectToNode()
	{
		if (IsItAlreadyDefined(FirstNode, SecondNode))
		{
			return false;
		}

		FirstNode->ConnectSegment(*this);
		SecondNode->ConnectSegment(*this);
		return true;
	}

	static bool	IsItAlreadyDefined(const FIsoNode* StartNode, const FIsoNode* EndNode)
	{
		for (const FIsoSegment* Segment : StartNode->GetConnectedSegments())
		{
			const FIsoNode& OtherNode = (StartNode == &Segment->GetFirstNode()) ? Segment->GetSecondNode() : Segment->GetFirstNode();
			if (&OtherNode == EndNode)
			{
				return true;
			}
		}
		return false;
	}

	void Clean()
	{
		if (FirstNode)
		{
			FirstNode->DisconnectSegment(*this);
		}
		if (SecondNode)
		{
			SecondNode->DisconnectSegment(*this);
		}
		FirstNode = nullptr;
		SecondNode = nullptr;
		States = EIsoSegmentStates::Delete;
	}

	bool IsDelete() const
	{
		return States == EIsoSegmentStates::Delete;
	}

	void SetSelected()
	{
		States &= ~EIsoSegmentStates::Candidate;
	}

	void SetCandidate()
	{
		States |= EIsoSegmentStates::Candidate;
	}

	bool IsACandidate() const 
	{
		return (States & EIsoSegmentStates::Candidate) == EIsoSegmentStates::Candidate;
	}

	void SetFinalMarker()
	{
		States |= EIsoSegmentStates::Final;
	}

	bool IsAFinalSegment() const
	{
		return (States & EIsoSegmentStates::Final) == EIsoSegmentStates::Final;
	}

	bool HasCycleOnLeft()
	{
		return (States & EIsoSegmentStates::LeftCycle) == EIsoSegmentStates::LeftCycle;
	}

	void SetHaveCycleOnLeft()
	{
		States |= EIsoSegmentStates::LeftCycle;
	}

	bool HasCycleOnRight()
	{
		return (States & EIsoSegmentStates::RightCycle) == EIsoSegmentStates::RightCycle;
	}

	void SetHaveCycleOnRight()
	{
		States |= EIsoSegmentStates::RightCycle;
	}

	bool HasTriangleOn(EIsoSegmentStates Side)
	{
		return (States & Side) == Side;
	}

	bool HasTriangleOnLeft()
	{
		return (States & EIsoSegmentStates::LeftTriangle) == EIsoSegmentStates::LeftTriangle;
	}

	bool HasTriangleOnRight()
	{
		return (States & EIsoSegmentStates::RightTriangle) == EIsoSegmentStates::RightTriangle;
	}

	bool HasntTriangle()
	{
		constexpr EIsoSegmentStates BoothSides = EIsoSegmentStates::RightTriangle | EIsoSegmentStates::LeftTriangle;
		return (States & BoothSides) == EIsoSegmentStates::None;
	}

	bool HasTriangleOnRightAndLeft()
	{
		constexpr EIsoSegmentStates BoothSides = EIsoSegmentStates::RightTriangle | EIsoSegmentStates::LeftTriangle;
		return (States & BoothSides) == BoothSides;
	}

	void SetHasTriangleOn(EIsoSegmentStates Side)
	{
		States |= Side;
	}

	void SetHasTriangleOnLeft()
	{
		States |= EIsoSegmentStates::LeftTriangle;
	}

	void SetHasTriangleOnRight()
	{
		States |= EIsoSegmentStates::RightTriangle;
	}

	void SetHasInnerTriangle(bool bOrientation)
	{
		if (bOrientation)
		{
			SetHasTriangleOnLeft();
		}
		else
		{
			SetHasTriangleOnRight();
		}
	}

	void ResetHasTriangle()
	{
		States &= ~EIsoSegmentStates::SegmentComplete;
	}

	void SetAsDegenerated()
	{
		States |= EIsoSegmentStates::Degenerate;
	}

	void ResetDegenerated()
	{
		States &= ~EIsoSegmentStates::Degenerate;
	}

	bool IsDegenerated() const
	{
		return (States & EIsoSegmentStates::Degenerate) == EIsoSegmentStates::Degenerate;
	}

	bool IsFirstNode(const FIsoNode* Node) const
	{
		return FirstNode == Node;
	}

	const FIsoNode& GetFirstNode() const
	{
		return *FirstNode;
	}

	FIsoNode& GetFirstNode()
	{
		return *FirstNode;
	}

	const FIsoNode& GetSecondNode() const
	{
		return *SecondNode;
	}

	FIsoNode& GetSecondNode()
	{
		return *SecondNode;
	}

	void SetFirstNode(FIsoNode& NewNode)
	{
		FirstNode = &NewNode;

		((FLoopNode*)SecondNode)->SetPreviousConnectedNode((FLoopNode*)FirstNode);
		((FLoopNode*)FirstNode)->SetNextConnectedNode((FLoopNode*)SecondNode);
	}

	void SetSecondNode(FIsoNode& NewNode)
	{
		SecondNode = &NewNode;

		((FLoopNode*)SecondNode)->SetPreviousConnectedNode((FLoopNode*)FirstNode);
		((FLoopNode*)FirstNode)->SetNextConnectedNode((FLoopNode*)SecondNode);
	}

	void ReplaceNode(FIsoNode& OldNode, FIsoNode& NewNode)
	{
		if (FirstNode == &OldNode)
		{
			SetFirstNode(NewNode);
		}
		else 
		{
			ensureCADKernel(SecondNode == &OldNode);
			SetSecondNode(NewNode);
		}
	}

	void SwapOrientation()
	{
		FIsoNode* TempNode = SecondNode;
		SecondNode = FirstNode;
		FirstNode = TempNode;

		((FLoopNode*)FirstNode)->SetNextConnectedNode((FLoopNode*)SecondNode);
		((FLoopNode*)SecondNode)->SetPreviousConnectedNode((FLoopNode*)FirstNode);
	}

	const FIsoNode& GetOtherNode(const FIsoNode* Node) const
	{
		return (FirstNode == Node) ? *SecondNode : *FirstNode;
	}

	FIsoNode& GetOtherNode(const FIsoNode* Node)
	{
		return (FirstNode == Node) ? *SecondNode : *FirstNode;
	}

	const ESegmentType GetType() const
	{
		return Type;
	}

	double Get2DLengthSquare(EGridSpace Space, const FGrid& Grid) const
	{
		const FVector2d& Point1 = FirstNode->Get2DPoint(Space, Grid);
		const FVector2d& Point2 = SecondNode->Get2DPoint(Space, Grid);
		return FVector2d::DistSquared(Point1, Point2);
	}

	double Get3DLengthSquare(const FGrid& Grid) const
	{
		const FVector& Point1 = FirstNode->Get3DPoint(Grid);
		const FVector& Point2 = SecondNode->Get3DPoint(Grid);
		return FVector::DistSquared(Point1, Point2);
	}
};

inline uint32 GetTypeHash(const FIsoSegment& Segment)
{
	uint32 StartHash = GetTypeHash(Segment.GetFirstNode());
	uint32 EndHash = GetTypeHash(Segment.GetSecondNode());
	if (StartHash < EndHash)
	{
		return HashCombine(StartHash, EndHash);
	}
	else
	{
		return HashCombine(EndHash, StartHash);
	}
};

inline uint32 GetTypeHash(const FIsoSegment& Segment0, const FIsoSegment& Segment1)
{
	uint32 StartHash = GetTypeHash(Segment0);
	uint32 EndHash = GetTypeHash(Segment1);
	if (StartHash < EndHash)
	{
		return HashCombine(StartHash, EndHash);
	}
	else
	{
		return HashCombine(EndHash, StartHash);
	}
};


inline FIsoSegment* FIsoNode::GetSegmentConnectedTo(const FIsoNode* Node) const
{
	if (this == Node)
	{
		return nullptr;
	}

	for (FIsoSegment* Segment : ConnectedSegments)
	{
		if (&Segment->GetFirstNode() == Node)
		{
			return Segment;
		}
		if (&Segment->GetSecondNode() == Node)
		{
			return Segment;
		}
	}
	return nullptr;
}

} // namespace UE::CADKernel

