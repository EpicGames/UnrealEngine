// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/IsoTriangulator.h"

#include "Math/Geometry.h"
#include "Math/Point.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Topo/TopologicalFace.h"
#include "Algo/AllOf.h"

namespace CADKernel
{

//#define DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT


#ifdef UNUSED

bool FIsoTriangulator::RemovePickToOutside(const TArray<FLoopNode*>& NodesOfLoop, const TPair<double, double>& Intersection, const TPair<double, double>& NextIntersection, bool bForward)
{
	const TPair<double, double> OutSideLoop(Intersection.Value, NextIntersection.Value);
	if (IsSubLoopBiggerThanMainLoop(NodesOfLoop, OutSideLoop, bForward))
	{
		return false;
	}

	using namespace IsoTriangulatorImpl;
	GetNextNodeMethod GetNext = bForward ? GetNextNodeImpl : GetPreviousNodeImpl;
	GetNextNodeMethod GetPrevious = bForward ? GetPreviousNodeImpl : GetNextNodeImpl;

	const int32 NodeCount = NodesOfLoop.Num();

	TFunction<FVector2d(double, FLoopNode*, FLoopNode*)> IntersectingPoint = [&](double Coordinate, FLoopNode* Start, FLoopNode* End) -> FVector2d
	{
		const FVector2d& StartPoint = Start->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& EndPoint = End->Get2DPoint(EGridSpace::UniformScaled, Grid);

		int32 StartIndex = (int32)Coordinate;
		Coordinate -= StartIndex;
		FVector2d Intersection = PointOnSegment<FVector2d>(StartPoint, EndPoint, Coordinate);

		//{
		//	F3DDebugSession _(TEXT("Intersected Point"));
		//	DisplayPoint(StartPoint, EVisuProperty::RedPoint);
		//	DisplayPoint(EndPoint, EVisuProperty::RedPoint);
		//	DisplayPoint(Intersection, EVisuProperty::BluePoint);
		//}

		return Intersection;
	};

	FLoopNode* TmpNode = GetNodeAt(NodesOfLoop, NextIndex(NodeCount, (int32)Intersection.Value));
	FLoopNode* EndNode = GetNodeAt(NodesOfLoop, (int32)NextIntersection.Value);
	if (TmpNode == nullptr || TmpNode->IsDelete() || EndNode == nullptr || EndNode->IsDelete())
	{
		return false;
	}

	FLoopNode* StartNode = GetPrevious(TmpNode);
	FVector2d FirstIntersection = IntersectingPoint(Intersection.Value, StartNode, TmpNode);

	TmpNode = GetNext(EndNode);
	FVector2d SecondIntersection = IntersectingPoint(NextIntersection.Value, EndNode, TmpNode);

	FVector2d MiddlePoint = (FirstIntersection + SecondIntersection) * 0.5;

	TmpNode = GetNext(StartNode);
	while (TmpNode && (TmpNode != EndNode) && !TmpNode->IsDelete())
	{
		if (!RemoveNodeOfLoop(*TmpNode))
		{
			return false;
		}
		TmpNode = GetNext(StartNode);
	}

	FVector2d MoveDirection = SecondIntersection - FirstIntersection;
	double Length = MoveDirection.Length();
	if (FMath::IsNearlyZero(Length))
	{
		FLoopNode* StartSegment = GetNodeAt(NodesOfLoop, (int32)Intersection.Key);
		if (StartSegment == nullptr)
		{
			return false;
		}
		MoveDirection = GetNext(StartSegment)->Get2DPoint(EGridSpace::UniformScaled, Grid) - StartSegment->Get2DPoint(EGridSpace::UniformScaled, Grid);
		Length = MoveDirection.Length();
	}

	MoveDirection /= Length;
	MoveDirection = FVector2d(-MoveDirection.Y, MoveDirection.X);
	MoveDirection *= GeometricTolerance;
	MiddlePoint += MoveDirection;

	if (!EndNode->IsDelete())
	{
		EndNode->Set2DPoint(EGridSpace::UniformScaled, Grid, MiddlePoint);
	}

	return true;
}



bool FIsoTriangulator::IsSubLoopBiggerThanMainLoop(const TArray<FLoopNode*>& NodesOfLoop, const TPair<double, double>& Intersection, bool bForward)
{
	using namespace IsoTriangulatorImpl;
	GetNextConstNodeMethod GetNext = bForward ? GetNextConstNodeImpl : GetPreviousConstNodeImpl;
	GetNextConstNodeMethod GetPrevious = bForward ? GetPreviousConstNodeImpl : GetNextConstNodeImpl;

	const int32 NodeCount = NodesOfLoop.Num();

	TFunction<double(const FLoopNode*, const FLoopNode*)> ComputeLength = [&](const FLoopNode* Start, const FLoopNode* End) -> double
	{
		double Length = 0;
		const FLoopNode* NextNode = nullptr;
		for (const FLoopNode* Node = Start; Node != End; Node = NextNode)
		{
			NextNode = GetNext(Node);
			Length += Node->Get3DPoint(Grid).Distance(NextNode->Get3DPoint(Grid));
		}
		return Length;
	};

	// check the sampling of the sub-loop vs the other part of the loop (deleted nodes are counted)
	int32 SubLoopSegmentCount = (int32)Intersection.Value - (int32)Intersection.Key;
	int32 OtherSegmentCount = (int32)Intersection.Key + NodeCount - (int32)Intersection.Value;
	if (SubLoopSegmentCount * 4 > OtherSegmentCount)
	{
		// The sub-loop is quite sampled to be bigger than the main part of the loop
		// it's better to comput the length of each parts

		// the sub-loop is not yet process so there are not deleted points
		const FLoopNode* SubLoopStartNode = GetNodeAt(NodesOfLoop, NextIndex(NodeCount, (int32)Intersection.Key));
		const FLoopNode* SubLoopEndNode = GetNodeAt(NodesOfLoop, (int32)Intersection.Value);
		if (SubLoopStartNode->IsDelete() || SubLoopEndNode->IsDelete())
		{
			return false;
		}

		const FLoopNode* MainLoopStartNode = GetNext(SubLoopEndNode);
		const FLoopNode* MainLoopEndNode = GetPrevious(SubLoopStartNode);

		// compute each sub-loop length
		double SubLoopLength = ComputeLength(SubLoopStartNode, SubLoopEndNode);
		double MainLoopLength = ComputeLength(MainLoopStartNode, MainLoopEndNode);

		if (SubLoopLength > MainLoopLength)
		{
			return true;
		}
	}

	return false;
}


bool FIsoTriangulator::SpreadCoincidentNodes(const TArray<FLoopNode*>& NodesOfLoop, TPair<double, double> Intersection)
{
	// TODO
	// 	   Use case:
	//	      FilesToProcess.append([1, 0, 0, 1, r"D:/Data/Cad Files/SolidWorks/p014 - Unreal Sport Bike/Headset front left brake.SLDPRT"])
	//        selection = [1967]

	//using namespace IsoTriangulatorImpl;
	//GetNextNodeMethod GetNext = bForward ? GetNextNodeImpl : GetPreviousNodeImpl;

	//FLoopNode* Node0 = GetNodeAt(NodesOfLoop, (int32)Intersection.Key);
	//FLoopNode* PreviousNode0 = Node0->
	//FLoopNode* Node1 = GetNext(Node0);
	//FLoopNode* NextNode = GetNext(Node1);

	//const FVector2d* PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	//const FVector2d* Point0 = &Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	//const FVector2d* Point1 = &Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);
	//const FVector2d* NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	return true;
}

bool FIsoTriangulator::MovePickBehind(const TArray<FLoopNode*>& NodesOfLoop, TPair<double, double> Intersection, bool bKeyIsExtremity)
{
	// TODO
	return true;
}










void FIsoTriangulator::FillIntersectionToolWithOuterLoop()
{
	for (FLoopNode& Node : LoopNodes)
	{
		if (Node.GetLoopIndex() != 0)
		{
			break;
		}

		FIsoSegment* Segment = Node.GetSegmentConnectedTo(&Node.GetNextNode());
		if(Segment != nullptr)
		{
			LoopSegmentsIntersectionTool.AddSegment(*Segment);
		}
	}
}

bool FIsoTriangulator::FindLoopIntersectionAndFixIt()
{
	TArray<FLoopNode*> BestStartNodeOfLoop;
	FindBestLoopExtremity(BestStartNodeOfLoop);

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
	if (bDisplay)
	{
		Display(EGridSpace::UniformScaled, TEXT("Loops Orientation"), LoopSegments, false, true, EVisuProperty::BlueCurve);
		DisplayLoops(TEXT("FindLoopIntersectionAndFixIt Before"), false, true);
		F3DDebugSession _(TEXT("BestStartNodeOfLoop"));
		for (const FLoopNode* Node : BestStartNodeOfLoop)
		{
			Display(EGridSpace::UniformScaled, *Node, 0, EVisuProperty::BluePoint);
		}
		Wait();
	}
#endif

	TArray<FLoopNode*> LoopNodesFromStartNode;
	TArray<TPair<double, double>> Intersections;

	// for each loop, start by the best node, find all intersections
	bool bIsAnOuterLoop = true;
	for (FLoopNode* StartNode : BestStartNodeOfLoop)
	{
		bNeedCheckOrientation = false;
		LoopNodesFromStartNode.Empty(LoopNodes.Num());
		Intersections.Empty(5);

		// LoopNodesFromStartNode is the set of node of the loop oriented as if the loop is an external loop
		// The use of FLoopNode::GetNextNode() is not recommended
		// Prefer to use GetSegmentToNodeMethode GetNext() set with GetFirstNode of GetSecondNode according to bOuterLoop

		IsoTriangulatorImpl::GetLoopNodeStartingFrom(StartNode, bIsAnOuterLoop, LoopNodesFromStartNode);

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
		DisplayLoop(EGridSpace::UniformScaled, TEXT("LoopIntersections: start"), LoopNodesFromStartNode, true, EVisuProperty::YellowPoint);
		Wait(bDisplay);
#endif

		RemoveLoopPicks(LoopNodesFromStartNode, Intersections);

		if (LoopNodesFromStartNode.Num() == 0)
		{
			continue;
		}

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
		DisplayLoop(EGridSpace::UniformScaled, TEXT("LoopIntersections: remove pick"), LoopNodesFromStartNode, true, EVisuProperty::YellowPoint);
		Wait(bDisplay);
#endif
		// At this step, LoopNodesFromStartNode cannot have deleted node
		FindLoopIntersections(LoopNodesFromStartNode, bIsAnOuterLoop, Intersections);

		// WARNING From this step, RemoveLoopIntersections can delete nodes. 
		if (!RemoveLoopIntersections(LoopNodesFromStartNode, Intersections, bIsAnOuterLoop))
		{
			UE_LOGF(LogCADKernelBase, Warning, "Loop intersections of the surface %d cannot be fixed. The mesh of this surface is canceled.\n", Grid.GetFace()->GetId());
			return false;
		}

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
		DisplayLoop(EGridSpace::UniformScaled, TEXT("LoopIntersections: remove self intersection"), LoopNodesFromStartNode, true, EVisuProperty::YellowPoint);
		Wait(bDisplay);
#endif

		RemoveLoopPicks(LoopNodesFromStartNode, Intersections);

		if (LoopNodesFromStartNode.Num() == 0)
		{
			continue;
		}

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
		DisplayLoop(EGridSpace::UniformScaled, TEXT("LoopIntersections: remove pick"), LoopNodesFromStartNode, true, EVisuProperty::YellowPoint);
		Wait(bDisplay);
#endif

		FixLoopOrientation(LoopNodesFromStartNode);

		bIsAnOuterLoop = false;
	}

	if (!CheckMainLoopConsistency())
	{
		return false;
	}


	if (Grid.GetLoopCount() > 1)
	{
		// Remove intersection between loops
		FixIntersectionBetweenLoops();

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
		if (bDisplay)
		{
			DisplayLoops(TEXT("FindLoopIntersectionAndFixIt Step2"), false, true);
			Wait();
		}
#endif
	}
	else
	{
		LoopSegmentsIntersectionTool.Empty(LoopSegments.Num());
		for (FIsoSegment* Segment : LoopSegments)
		{
			LoopSegmentsIntersectionTool.AddSegment(*Segment);
		}
	}

	if (!CheckMainLoopConsistency())
	{
		return false;
	}

#ifdef DEBUG_LOOP_INTERSECTION_AND_FIX_IT		
	if (bDisplay)
	{
		DisplayLoops(TEXT("FindLoopIntersectionAndFixIt 3"), false, true);
		Wait();
	}
#endif

	return true;
}

//#define DEBUG_CLOSED_OUSIDE_POINT




//#define DEBUG_TWO_CONSECUTIVE_INTERSECTING
bool FIsoTriangulator::TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(const FIsoSegment& IntersectingSegment, FIsoSegment& Segment)
{
	FLoopNode* Node = nullptr;
	FLoopNode* PreviousNode = nullptr;
	FLoopNode* NextNode = nullptr;

	TSegment<FVector2d> IntersectingSegment2D(IntersectingSegment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), IntersectingSegment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid));
	TSegment<FVector2d> Segment2D(Segment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), Segment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid));

	double IntersectingSegmentSlop = ComputeOrientedSlope(IntersectingSegment2D.Point0, IntersectingSegment2D.Point1, 0);
	double SegmentSlop = ComputeUnorientedSlope(Segment2D.Point1, Segment2D.Point0, IntersectingSegmentSlop);
	if (SegmentSlop > 2)
	{
		SegmentSlop = 4 - SegmentSlop;
	}

	// if the segment and IntersectingSegment are parallel, segment are moved inside
	if (SegmentSlop < 0.01)
	{
		double StartPointSquareDistance = SquareDistanceOfPointToSegment<FVector2d>(Segment2D.Point0, IntersectingSegment2D.Point0, IntersectingSegment2D.Point1);
		double EndPointSquareDistance = SquareDistanceOfPointToSegment<FVector2d>(Segment2D.Point1, IntersectingSegment2D.Point0, IntersectingSegment2D.Point1);
		if (StartPointSquareDistance < SquareGeometricTolerance && EndPointSquareDistance < SquareGeometricTolerance)
		{
			OffsetSegment(Segment, Segment2D, IntersectingSegment2D);
			return true;
		}
	}

	// check if the intersection is not at the extremity
	{
		double Coordinate = 0;
		FindIntersectionOfSegments2D(Segment2D, IntersectingSegment2D, Coordinate);
		if (FMath::IsNearlyZero(Coordinate))
		{
			// can add a test to offset the outside node and not the node a 
			OffsetNode((FLoopNode&)Segment.GetFirstNode(), IntersectingSegment2D);
			return true;
		}
		else if (FMath::IsNearlyEqual(Coordinate, 1))
		{
			OffsetNode((FLoopNode&)Segment.GetSecondNode(), IntersectingSegment2D);
			return true;
		}
	}

	double OrientedSlop = ComputeOrientedSlope(IntersectingSegment2D.Point0, Segment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), IntersectingSegmentSlop);
	if (OrientedSlop >= 0)
	{
		Node = (FLoopNode*)&Segment.GetSecondNode();
		PreviousNode = (FLoopNode*)&Segment.GetFirstNode();
		NextNode = (FLoopNode*)&Node->GetNextNode();
	}
	else
	{
		Node = (FLoopNode*)&Segment.GetFirstNode();
		PreviousNode = (FLoopNode*)&Segment.GetSecondNode();
		NextNode = (FLoopNode*)&Node->GetPreviousNode();
	}

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("Intersecting Segments"));
		Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::BlueCurve);
		Display(EGridSpace::UniformScaled, *Node, *NextNode, 0, EVisuProperty::BlueCurve);
		Display(EGridSpace::UniformScaled, IntersectingSegment, 0, EVisuProperty::RedCurve);
		Display(EGridSpace::UniformScaled, *Node, 0, EVisuProperty::RedPoint);
		Wait(false);
	}
#endif

	TSegment<FVector2d> NextSegment2D(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid));
	if (!FastIntersectSegments2D(NextSegment2D, IntersectingSegment2D))
	{
		return false;
	}

	TSegment<FVector2d>  PreviousSegment2D(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid));

	FVector2d Intersection1 = FindIntersectionOfSegments2D(PreviousSegment2D, IntersectingSegment2D);
	FVector2d Intersection2 = FindIntersectionOfSegments2D(NextSegment2D, IntersectingSegment2D);

	double Coordinate;
	FVector2d ProjectedPoint = ProjectPointOnSegment<FVector2d>(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), Intersection1, Intersection2, Coordinate);

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("ProjectedPoint"));
		DisplayPoint(ProjectedPoint, EVisuProperty::RedPoint);
	}
#endif

	FVector2d SegmentTangent = IntersectingSegment2D.Point1 - IntersectingSegment2D.Point0;
	SegmentTangent.Normalize();
	FVector2d MoveDirection = FVector2d(-SegmentTangent.Y, SegmentTangent.X);

	MoveDirection *= GeometricTolerance;

	FVector2d NewCoordinate = ProjectedPoint + MoveDirection;

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("NewCoordinate"));
		DisplayPoint(NewCoordinate, EVisuProperty::BluePoint);
	}
#endif

	if ((PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid).SquareDistance(NewCoordinate) < SquareGeometricTolerance2) ||
		(NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid).SquareDistance(NewCoordinate) < SquareGeometricTolerance2))
	{
		RemoveNodeOfLoop(*Node);
	}
	else
	{
		Node->Set2DPoint(EGridSpace::UniformScaled, Grid, NewCoordinate);
		FIsoSegment* NextSegment = Node->GetSegmentConnectedTo(NextNode);
		LoopSegmentsIntersectionTool.Update(&Segment);
		LoopSegmentsIntersectionTool.Update(NextSegment);

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
		if (bDisplay)
		{
			F3DDebugSession _(TEXT("New position"));
			DisplayPoint(Intersection1, EVisuProperty::RedPoint);
			DisplayPoint(Intersection2, EVisuProperty::RedPoint);
			Display(EGridSpace::UniformScaled, *Node, EVisuProperty::RedPoint);
			Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::GreenCurve/*, true*/);
			Display(EGridSpace::UniformScaled, *Node, *NextNode, 0, EVisuProperty::GreenCurve/*, true*/);
			Wait(false);
		}
#endif
	}

	return true;
}

//#define DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT
void FIsoTriangulator::FixIntersectionBetweenLoops()
{
	const double MaxGap = Grid.GetMinElementSize();

	ensureCADKernel(2 < LoopSegments.Num());

#ifdef DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT
	int32 Iteration = 0;
	F3DDebugSession _(bDisplay, TEXT("FixIntersectionBetweenLoops"));
#endif

	TSet<uint32> IntersectionAlreadyProceed;

	LoopSegmentsIntersectionTool.Empty(LoopSegments.Num());
	int32 Index = 0;
	for (; Index < LoopSegments.Num(); ++Index)
	{
		if (((FLoopNode&)LoopSegments[Index]->GetFirstNode()).GetLoopIndex() != 0)
		{
			break;
		}
		LoopSegmentsIntersectionTool.AddSegment(*LoopSegments[Index]);
	}

	for (; Index < LoopSegments.Num(); )
	{
		ensureCADKernel(LoopSegments[Index]);

		FIsoSegment& Segment = *LoopSegments[Index];
		ensureCADKernel(!Segment.IsDelete());

#ifdef DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT
		if (bDisplay)
		{
			LoopSegmentsIntersectionTool.Display(TEXT("IntersectionTool"));
			F3DDebugSession _(*FString::Printf(TEXT("Segment to proceed %d %d"), Index, Iteration++));
			Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::BlueCurve/*, true*/);
		}
#endif

		if (const FIsoSegment* IntersectingSegment = LoopSegmentsIntersectionTool.DoesIntersect(Segment))
		{
#ifdef DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT
			if (bDisplay)
			{
				LoopSegmentsIntersectionTool.Display(TEXT("IntersectionTool"));
				{
					F3DDebugSession _(*FString::Printf(TEXT("Segment to proceed %d %d"), Index, Iteration++));
					Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::BlueCurve/*, true*/);
				}
				{
					F3DDebugSession _(TEXT("Intersecting Segments"));
					Display(EGridSpace::UniformScaled, *IntersectingSegment, 0, EVisuProperty::RedCurve/*, true*/);
				}
				Wait(true);
			}
#endif

			uint32 IntersectionHash = GetTypeHash(*IntersectingSegment, Segment);
			bool bNotProceed = !IntersectionAlreadyProceed.Find(IntersectionHash);
			IntersectionAlreadyProceed.Add(IntersectionHash);

			bool bIsFixed = true;
			bool bIsSameLoop = ((FLoopNode&)Segment.GetFirstNode()).GetLoopIndex() == ((FLoopNode&)IntersectingSegment->GetFirstNode()).GetLoopIndex();

			// Swapping segments is possible only with outer loop
			if (bNotProceed)
			{
				if (!TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(*IntersectingSegment, Segment))
				{
					if (!TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(Segment, const_cast<FIsoSegment&>(*IntersectingSegment)))
					{
						if (bIsSameLoop)
						{
							// check if it's realy used 
							bIsFixed = TryToRemoveSelfIntersectionByMovingTheClosedOusidePoint(*IntersectingSegment, Segment);
						}
						else
						{
							RemoveIntersectionByMovingOutsideSegmentNodeInside(*IntersectingSegment, Segment);
						}
					}
				}

				if (bIsFixed)
				{
					// segment is moved, the previous segment is retested. Thanks to that, only one loop is enough.
					if (Index > 1)
					{
						LoopSegmentsIntersectionTool.RemoveLast();
					}
				}
			}
			else if (bIsSameLoop)
			{
				bIsFixed = TryToRemoveIntersectionBySwappingSegments(const_cast<FIsoSegment&>(*IntersectingSegment), Segment);
				if (!bIsFixed)
				{
					ensureCADKernel(false);
				}
			}
			else
			{
				// if no more append remove it
				ensureCADKernel(false);
				LoopSegmentsIntersectionTool.AddSegment(Segment);
			}

#ifdef DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT
			if (false)
			{
				DisplayLoops(TEXT("After fix"));
				Wait(false);
			}
#endif
		}
		else
		{
			LoopSegmentsIntersectionTool.AddSegment(Segment);
		}

		if (!Segment.IsDelete())
		{
			RemovePickOfLoop(Segment);
		}

		Index = LoopSegmentsIntersectionTool.Count();
	}

#ifdef DEBUG_FIND_LOOP_INTERSECTION_AND_FIX_IT
	if (bDisplay)
	{
		DisplayLoops(TEXT("After fix"));
		Wait(false);
	}
#endif

	LoopSegmentsIntersectionTool.Sort();
}

//#define DEBUG_SELF_CLOSED_OUSIDE_POINT

//#define DEBUG_REMOVE_PICK_OF_LOOP
void FIsoTriangulator::RemovePickOfLoop(FIsoSegment& Segment)
{
	if (Segment.GetType() != ESegmentType::Loop)
	{
		return;
	}

	FLoopNode* Node0 = &(FLoopNode&)Segment.GetFirstNode();
	FLoopNode* Node1 = &(FLoopNode&)Segment.GetSecondNode();
	FLoopNode* PreviousNode = &Node0->GetPreviousNode();
	FLoopNode* NextNode = &Node1->GetNextNode();

	const FVector2d* PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* Point0 = &Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* Point1 = &Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

	bool bNodeRemoved = true;
	bool bPickRemoved = false;
	while (LoopSegments.Num() >= 3)
	{
		if (CheckAndRemovePick(*PreviousPoint, *Point0, *Point1, *Node0))
		{
			if (PreviousNode->IsDelete())
			{
				Wait();
			}

			Point0 = PreviousPoint;
			Node0 = PreviousNode;
			PreviousNode = &Node0->GetPreviousNode();
			PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
			bPickRemoved = true;
			continue;
		}

		if (CheckAndRemovePick(*Point0, *Point1, *NextPoint, *Node1))
		{
			if (NextNode->IsDelete())
			{
				Wait();
			}

			Point1 = NextPoint;
			Node1 = NextNode;
			NextNode = &Node1->GetNextNode();
			NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
			bPickRemoved = true;
			continue;
		}

		break;
	}

#ifdef DEBUG_REMOVE_PICK_OF_LOOP
	if (bDisplay && bPickRemoved)
	{
		DisplayLoops(TEXT("After pick removed"));
		Wait(false);
	}
#endif
}


#endif


} //namespace CADKernel