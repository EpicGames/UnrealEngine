// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/MesherTools.h"

#include "Math/Boundary.h"
#include "Topo/TopologicalEdge.h"

namespace UE::CADKernel
{

void FMesherTools::ComputeFinalCuttingPointsWithPreferredCuttingPoints(const TArray<double>& CrossingUs, TArray<double> DeltaUs, const TArray<FCuttingPoint>& PreferredCuttingPoints, const FLinearBoundary& Boundary, TArray<double>& OutCuttingPoints)
{
	// Evaluate the number of Cutting points
	{
		double DeltaUMin = HUGE_VALUE;
		for (const double& Delta : DeltaUs)
		{
			if (Delta < DeltaUMin)
			{
				DeltaUMin = Delta;
			}
		}
		int32 NodeNum = 5 + (int32)(Boundary.Length() / DeltaUMin);
		OutCuttingPoints.Reserve(NodeNum);
	}

	int32 StartCrossingIndex = 0;
	while (CrossingUs[StartCrossingIndex + 1] <= Boundary.Min)
	{
		++StartCrossingIndex;
	};

	// Step 1: Compute the number of mesh nodes i.e.compute the time of the travel between Boundary.Min and Boundary.Max with the initial speeds DeltaUs
	int32 CrossingIndex = StartCrossingIndex;
	double LastStep = Boundary.Min;
	double TravelTime = 0;
	do
	{
		double NextStep = FMath::Min(CrossingUs[CrossingIndex + 1], Boundary.Max);
		TravelTime += (NextStep - LastStep) / DeltaUs[CrossingIndex];
		CrossingIndex++;
		LastStep = NextStep;
	} while (LastStep + DOUBLE_SMALL_NUMBER < Boundary.Max);


	// Step 2: Define the number of step(each step lasts one) as it must be an integer bigger than 1
	double CorrectedFinalTravelTime = FMath::Max((double)(FMath::CeilToInt32(TravelTime)), 1.);

	// Step 3: Adjust the speeds than the travel respect the count of step
	{
		double CorrectiveFactor = TravelTime / CorrectedFinalTravelTime;
		for (double& DeltaU : DeltaUs)
		{
			DeltaU *= CorrectiveFactor;
		}
	}

	TArray<double> TravelTimeOfNeighborsU;
	TravelTimeOfNeighborsU.Reserve(PreferredCuttingPoints.Num() + 1);
	TravelTimeOfNeighborsU.SetNum(PreferredCuttingPoints.Num());

	// Defined the travel time to each Preferred Cutting Points
	{
		LastStep = Boundary.Min;
		TravelTime = 0;
		int32 UIndex = StartCrossingIndex;
		for (int32 NeighbourIndex = 0; NeighbourIndex < PreferredCuttingPoints.Num(); ++NeighbourIndex)
		{
			for (; UIndex < DeltaUs.Num(); ++UIndex)
			{
				const double DeltaU = DeltaUs[UIndex];
				const double PreferredCuttingPoint = PreferredCuttingPoints[NeighbourIndex].Coordinate;

				if (PreferredCuttingPoint - DOUBLE_SMALL_NUMBER > CrossingUs[UIndex + 1])
				{
					TravelTime += (CrossingUs[UIndex + 1] - LastStep) / DeltaU;
					LastStep = CrossingUs[UIndex + 1];
					continue;
				}

				if (CrossingUs[UIndex] - DOUBLE_SMALL_NUMBER < PreferredCuttingPoint && PreferredCuttingPoint < CrossingUs[UIndex + 1] + DOUBLE_SMALL_NUMBER)
				{
					const double DeltaTravelTime = (PreferredCuttingPoint - LastStep) / DeltaU;
					TravelTimeOfNeighborsU[NeighbourIndex] = TravelTime + DeltaTravelTime;
				}
				break;
			}
		}

		// if some NeighborsU are near the end (less than 0.45x the expected element size), they are ignored 
		int32 NeighbourIndex = 0;
		for (; NeighbourIndex < TravelTimeOfNeighborsU.Num(); ++NeighbourIndex)
		{
			if (TravelTimeOfNeighborsU[NeighbourIndex] > CorrectedFinalTravelTime - 0.45)
			{
				break;
			}
		}

		if (NeighbourIndex == TravelTimeOfNeighborsU.Num())
		{
			TravelTimeOfNeighborsU.Add(CorrectedFinalTravelTime);
		}
		else
		{
			TravelTimeOfNeighborsU[NeighbourIndex] = CorrectedFinalTravelTime;
			TravelTimeOfNeighborsU.SetNum(NeighbourIndex + 1);
		}
	}


	int32 NeighbourIndex = 0;

	double TargetTime = 0.;
	double TimeDelta = CorrectedFinalTravelTime;
	// Define the next Target time step i.e. a Preferred Cutting Points or a round step   
	TFunction<void(double&, double&)> IncreaseTargetTimeAccordingToNeighbors = [&](double& TargetTime, double& TimeDelta)
	{
		if (NeighbourIndex < TravelTimeOfNeighborsU.Num() && TargetTime + TimeDelta + DOUBLE_SMALL_NUMBER > TravelTimeOfNeighborsU[NeighbourIndex])
		{
			TimeDelta = CorrectedFinalTravelTime - TargetTime;
			for (; NeighbourIndex < TravelTimeOfNeighborsU.Num(); ++NeighbourIndex)
			{
				double BestGap = TravelTimeOfNeighborsU[NeighbourIndex] - TargetTime;
				if (BestGap < 0.45)
				{
					continue;
				}

				int32 BestIndex = NeighbourIndex;
				if (BestGap < 1)
				{
					BestGap = FMath::Abs(1 - BestGap);
					for (int32 Index = NeighbourIndex + 1; Index < TravelTimeOfNeighborsU.Num(); ++Index)
					{
						double Gap = FMath::Abs(1 - TravelTimeOfNeighborsU[Index] + TargetTime);
						if (Gap < BestGap)
						{
							BestGap = Gap;
							BestIndex = Index;
						}
						else
						{
							break;
						}
					}
				}
				NeighbourIndex = BestIndex;
				TimeDelta = TravelTimeOfNeighborsU[NeighbourIndex] - TargetTime;
				int32 StepNum = FMath::Max((int32)(TimeDelta + 0.5), 1);
				TimeDelta /= StepNum;
				break;
			}
		}

		TargetTime += TimeDelta;
	};

	// Step 4: Define the steps (if a preferred step is near the next step, the preferred step is used)

	LastStep = Boundary.Min;
	TravelTime = 0;
	OutCuttingPoints.Add(Boundary.Min);

	IncreaseTargetTimeAccordingToNeighbors(TargetTime, TimeDelta);
	for (CrossingIndex = StartCrossingIndex; CrossingIndex < DeltaUs.Num();)
	{
		double& DeltaU = DeltaUs[CrossingIndex];
		double DeltaTravelTime = (CrossingUs[CrossingIndex + 1] - LastStep) / DeltaU;
		double MaxTravelTime = TravelTime + DeltaTravelTime;
		while (TimeDelta > DOUBLE_SMALL_NUMBER && MaxTravelTime > TargetTime - DOUBLE_SMALL_NUMBER)
		{
			DeltaTravelTime = TargetTime - TravelTime;
			LastStep += DeltaU * DeltaTravelTime;
			OutCuttingPoints.Add(LastStep);
			TravelTime = TargetTime;

			IncreaseTargetTimeAccordingToNeighbors(TargetTime, TimeDelta);
		}

		if (OutCuttingPoints.Last() + DOUBLE_SMALL_NUMBER > Boundary.GetMax())
		{
			break;
		}

		CrossingIndex++;
		LastStep = CrossingUs[CrossingIndex];
		TravelTime = MaxTravelTime;
	}

	// Set the last to insure the good value
	OutCuttingPoints.Last() = Boundary.GetMax();
};

void FMesherTools::ComputeFinalCuttingPointsWithImposedCuttingPoints(const TArray<double>& CrossingUs, const TArray<double>& DeltaUs, const TArray<FCuttingPoint>& ImposedCuttingPoints, TArray<double>& OutCuttingPointUs)
{
	// Evaluate the number of Cutting points
	TArray<double> TravelTimeByStep;
	{
		double DeltaUMin = HUGE_VALUE;
		for (const double& Delta : DeltaUs)
		{
			if (Delta < DeltaUMin)
			{
				DeltaUMin = Delta;
			}
		}
		int32 NodeNum = 5 + (int32)((ImposedCuttingPoints.Last().Coordinate - ImposedCuttingPoints[0].Coordinate) / DeltaUMin);
		OutCuttingPointUs.Reserve(NodeNum);
		TravelTimeByStep.Reserve(NodeNum);
	}

	int32 LastCrossingIndex = 0;
	int32 CrossingIndex = 0;

	// Laps time at crossing points between UMin et UMax
	TFunction<void(const double, const double, double&, double&)> LapTimesUntilNextPoint = [&](const double StartU, const double EndU, double& StepCount, double& CorrectiveFactor)
	{
		CrossingIndex = LastCrossingIndex;
		double LastStep = StartU;
		double TravelTime = 0;

		TravelTimeByStep.Empty();
		while(CrossingIndex < CrossingUs.Num() - 1)
		{
			double Step = CrossingUs[CrossingIndex + 1];
			Step = FMath::Min(Step, EndU);
			double StepTime = (Step - LastStep) / DeltaUs[CrossingIndex];
			TravelTimeByStep.Add(StepTime);
			TravelTime += StepTime;
			LastStep = Step;
			if (LastStep + DOUBLE_SMALL_NUMBER > EndU)
			{
				break;
			}
			CrossingIndex++;
		}

		StepCount = FMath::Max(1., ((double)((int32)(TravelTime + 0.5))));
		CorrectiveFactor = TravelTime / StepCount;
	};

	TFunction<void(const double, const double, const double)> ComputeCuttingPointUntilNextImposedPoint = [&](const double EndU, const double NbStep, const double CorrectiveFactor)
	{
		int32 CrossingIndex = LastCrossingIndex;
		double TravelTime = 0;
		double TravelLength = OutCuttingPointUs.Last();
		double Step = 1;
		for (double StepTime : TravelTimeByStep)
		{
			double CorrectedStepTime = StepTime / CorrectiveFactor;
			double CorrectedDeltaU = DeltaUs[CrossingIndex] * CorrectiveFactor;
			while (TravelTime + CorrectedStepTime + DOUBLE_SMALL_NUMBER > Step)
			{
				double Time = Step - TravelTime;
				OutCuttingPointUs.Add(TravelLength + CorrectedDeltaU * Time);
				Step++;
				if (Step + DOUBLE_SMALL_NUMBER > NbStep)
				{
					OutCuttingPointUs.Add(EndU);
					return;
				}
			}
			TravelLength += CorrectedDeltaU * CorrectedStepTime;
			TravelTime += CorrectedStepTime;
			CrossingIndex++;
		}
	};

	OutCuttingPointUs.Add(ImposedCuttingPoints[0].Coordinate);

	while (CrossingUs[LastCrossingIndex + 1] <= ImposedCuttingPoints[0].Coordinate)
	{
		++LastCrossingIndex;
	};

	// Between each imposed cutting points
	double NextStep = ImposedCuttingPoints[0].Coordinate;
	for (int32 Index = 1; Index < ImposedCuttingPoints.Num(); ++Index)
	{
		double StartStep = NextStep;
		double StepCount = 0;
		double CorrectiveFactor = 0;
		NextStep = ImposedCuttingPoints[Index].Coordinate;

		// Step 1: Define the number of step (each step lasts one) as it must be an integer bigger than 1
		// Step 2: Adjust the speeds than the travel respect the count of step
		LapTimesUntilNextPoint(StartStep, NextStep, StepCount, CorrectiveFactor);

		if (StepCount == 1)
		{
			OutCuttingPointUs.Add(NextStep);
		}
		else
		{
			ComputeCuttingPointUntilNextImposedPoint(NextStep, StepCount, CorrectiveFactor);
		}
		LastCrossingIndex = CrossingIndex;
	}
};

void FMesherTools::FillImposedIsoCuttingPoints(TArray<double>& UEdgeSetOfIntersectionWithIso, ECoordinateType CoordinateType, double EdgeToleranceGeo, const FTopologicalEdge& Edge, TArray<FCuttingPoint>& OutImposedIsoVertexSet)
{
	FLinearBoundary EdgeBoundary = Edge.GetBoundary();

	int32 StartIndex = OutImposedIsoVertexSet.Num();
	Algo::Sort(UEdgeSetOfIntersectionWithIso);
	double PreviousU = -HUGE_VALUE;
	for (double InterU : UEdgeSetOfIntersectionWithIso)
	{
		// Remove coordinate nearly equal to boundary
		if ((InterU - EdgeToleranceGeo) < EdgeBoundary.GetMin() || (InterU + EdgeToleranceGeo) > EdgeBoundary.GetMax())
		{
			continue;
		}

		// Remove coordinate inside thin zone
		for (FLinearBoundary ThinZone : Edge.GetThinZoneBounds())
		{
			if (ThinZone.Contains(InterU))
			{
				continue;
			}
		}

		// Remove nearly duplicate 
		if (InterU - PreviousU < EdgeToleranceGeo)
		{
			continue;
		}

		OutImposedIsoVertexSet.Emplace(InterU, CoordinateType);
		PreviousU = InterU;
	}


	const int32 NewCoordinateCount = OutImposedIsoVertexSet.Num() - StartIndex;
	if (NewCoordinateCount > 0)
	{
		if (NewCoordinateCount == 1)
		{
			int32 CuttingPointIndex = 0;
			while (CuttingPointIndex < Edge.GetCrossingPointUs().Num() && Edge.GetCrossingPointUs()[CuttingPointIndex] + DOUBLE_SMALL_NUMBER <= OutImposedIsoVertexSet[StartIndex].Coordinate)
			{
				++CuttingPointIndex;
			};
			if (CuttingPointIndex > 0)
			{
				--CuttingPointIndex;
			}
			// #cadkernel_check: To investigate further - This assert should not happen anymore.
			if (Edge.GetDeltaUMaxs().IsValidIndex(CuttingPointIndex))
			{
				OutImposedIsoVertexSet[StartIndex].IsoDeltaU = Edge.GetDeltaUMaxs()[CuttingPointIndex] * AQuarter;
			}
			else
			{
				OutImposedIsoVertexSet.SetNum(StartIndex, EAllowShrinking::No);
			}
		}
		else
		{
			int32 Index = StartIndex + 1;
			OutImposedIsoVertexSet[StartIndex].IsoDeltaU = (OutImposedIsoVertexSet[StartIndex + 1].Coordinate - OutImposedIsoVertexSet[StartIndex].Coordinate) * AQuarter;
			for (; Index < OutImposedIsoVertexSet.Num() - 1; ++Index)
			{
				OutImposedIsoVertexSet[Index].IsoDeltaU = (OutImposedIsoVertexSet[Index + 1].Coordinate - OutImposedIsoVertexSet[Index - 1].Coordinate) * AEighth;
			}
			OutImposedIsoVertexSet[Index].IsoDeltaU = (OutImposedIsoVertexSet[Index].Coordinate - OutImposedIsoVertexSet[Index - 1].Coordinate) * AQuarter;
		}
	}
}


}
