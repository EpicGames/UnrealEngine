// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Boundary.h"
#include "Mesh/MeshEnum.h"

namespace UE::CADKernel
{
class FTopologicalEdge;

struct FCuttingPoint;
struct FLinearBoundary;

class FMesherTools
{
public:

	/**
	 * The purpose of this method is to compute the cutting of a face or an edge by preferring as soon as possible the suggested cuttings.
	 * In input a set of coordinate (CrossingUs) and the DeltaU between each coordinate (DeltaUs) which makes it possible to respect the mesh criteria.
	 *
	 * By analogy with a travel:
	 * - the target time between each cutting points is one
	 * - the speed of the travel at a point p(u) is DeltaUs(u)
	 *
	 * Step 1: Compute the number of mesh nodes i.e. compute the time of the travel between Boundary.Min and Boundary.Max with the initial speeds DeltaUs
	 * Step 2: Define the number of step (each step lasts one) as it must be an integer bigger than 1
	 * Step 3: Adjust the speeds than the travel respect the count of step
	 * Step 4: Define the steps (if a preferred step is near the next step, the preferred step is used)
	 *
	 * Mandatory: PreferredCuttingPoints.Last() == UMax,
	 */
	static void ComputeFinalCuttingPointsWithPreferredCuttingPoints(const TArray<double>& CrossingUs, TArray<double> DeltaUs, const TArray<FCuttingPoint>& PreferredCuttingPoints, const FLinearBoundary& Boundary, TArray<double>& OutCuttingPoints);

	/**
	 * The purpose of this method is to compute the cutting of a face or an edge by using the imposed cuttings.
	 * In input a set of coordinate (CrossingUs) and the DeltaU between each coordinate (DeltaUs) which makes it possible to respect the mesh criteria.
	 *
	 * By analogy with a travel:
	 * - the target time between each cutting points is one
	 * - the speed of the travel at a point p(u) is DeltaUs(u)
	 *
	 * Between each imposed cutting points
	 * Step 1: Define the number of step (each step lasts one) as it must be an integer bigger than 1
	 * Step 2: Adjust the speeds than the travel respect the count of step
	 * Step 3: Compute the steps until the next imposed point
	 *
	 * Mandatory: ImposedCuttingPoints[0] == UMin && ImposedCuttingPoints.Last() == UMax,
	 */
	static void ComputeFinalCuttingPointsWithImposedCuttingPoints(const TArray<double>& CrossingUs, const TArray<double>& DeltaUs, const TArray<FCuttingPoint>& ImposedCuttingPoints, TArray<double>& OutCuttingPoints);

	/**
	 * Fill an array of CuttingPointCoordinates with "PointCount" coordinates evenly divided between the extremities of the boundary
	 */
	static void FillCuttingPointCoordinates(const FLinearBoundary& Boundary, int32 PointCount, TArray<double>& OutCuttingPointCoordinates)
	{
		PointCount--;
		ensureCADKernel(PointCount > 0);
		double Delta = Boundary.Length() / PointCount;
		OutCuttingPointCoordinates.Empty(PointCount);
		OutCuttingPointCoordinates.Add(Boundary.GetMin());
		for (int32 Index = 0; Index < PointCount; ++Index)
		{
			OutCuttingPointCoordinates.Add(OutCuttingPointCoordinates.Last() + Delta);
		}
	}

	static void FillImposedIsoCuttingPoints(TArray<double>& UEdgeSetOfIntersectionWithIso, ECoordinateType CoordinateType, double EdgeToleranceGeo, const FTopologicalEdge& Edge, TArray<FCuttingPoint>& OutImposedIsoVertexSet);
};

} // namespace UE::CADKernel

