// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/ParametricFaceMesher.h"

#include "Mesh/Criteria/Criterion.h"
#include "Mesh/Meshers/IsoTriangulator.h"
#include "Mesh/Meshers/MesherTools.h"
#include "Mesh/Meshers/ParametricMesherConstantes.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Mesh/Structure/ThinZone2DFinder.h"
#include "Mesh/Structure/ThinZone2D.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"

#include "Geo/Curves/CurveUtilities.h"
#include "Geo/Surfaces/SurfaceUtilities.h"
#include "Topo/TopologicalFaceUtilities.h"
#include "CompGeom/Delaunay2.h"
#include "HAL/IConsoleManager.h"

static bool GDetectPlanarFace = false;
FAutoConsoleVariableRef PlanarFaceCVar(
	TEXT("CADKernel.FaceMesher.DetectPlanarFace"),
	GDetectPlanarFace,
	TEXT(""),
	ECVF_Default);

namespace UE::CADKernel
{
FParametricFaceMesher::FParametricFaceMesher(FTopologicalFace& InFace, FModelMesh& InMeshModel, const FMeshingTolerances& InTolerances, bool bActivateThinZoneMeshing)
	: Face(InFace)
	, MeshModel(InMeshModel)
	, Tolerances(InTolerances)
#if CADKERNEL_THINZONE
	, bThinZoneMeshing(bActivateThinZoneMeshing)
#endif
	, Grid(Face, MeshModel)
{
}

void FParametricFaceMesher::Mesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FParametricFaceMesher::GenerateCloud)
	
	Face.GetOrCreateMesh(MeshModel);

	if (Face.IsNotMeshable())
	{
		return;
	}

	MeshVerticesOfFace(Face);

	if (!GenerateCloud() || Grid.IsDegenerated())
	{
		UE_LOGF(LogCADKernelBase, Warning, "The meshing of the surface %d failed due to a degenerated grid\n", Face.GetId());

		Face.IsDegenerated();
		Face.SetMeshedMarker();
		return;
	}

	if (!MeshPlanarFace())
	{
		FFaceMesh& SurfaceMesh = Face.GetOrCreateMesh(MeshModel);
		FIsoTriangulator IsoTrianguler(Grid, SurfaceMesh, Tolerances);

		if (IsoTrianguler.Triangulate())
		{
			if (Face.IsBackOriented())
			{
				SurfaceMesh.InverseOrientation();
			}
			MeshModel.AddMesh(SurfaceMesh);
		}
	}

	Face.SetMeshedMarker();
}

bool FParametricFaceMesher::GenerateCloud()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FParametricFaceMesher::GenerateCloud)
	
	Grid.DefineCuttingParameters();
	if (!Grid.GeneratePointCloud())
	{
		return false;
	}

	if (bThinZoneMeshing)
	{
		if (Grid.GetFace().HasThinZone())
		{
			MeshThinZones();
		}
	}

	MeshFaceLoops();

	Grid.ProcessPointCloud();

	return true;
}

void FParametricFaceMesher::MeshThinZones(TArray<FTopologicalEdge*>& EdgesToMesh, const bool bFinalMeshing)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FParametricFaceMesher::MeshThinZones)

	{
		for (FTopologicalEdge* Edge : EdgesToMesh)
		{
			if (Edge->IsPreMeshed())
			{
				continue;
			}


			{
				for (FThinZoneSide* ZoneSide : Edge->GetThinZoneSides())
				{
					if (ZoneSide->HasMarker1())
					{
						continue;
					}
					ZoneSide->SetMarker1();

					DefineImposedCuttingPointsBasedOnOtherSideMesh(*ZoneSide);
				}
			}
		}

		for (FTopologicalEdge* Edge : EdgesToMesh)
		{
			for (FThinZoneSide* ZoneSide : Edge->GetThinZoneSides())
			{
				if (ZoneSide->HasMarker1())
				{
					ZoneSide->ResetMarker1();
				}
			}
		}
	}

	{
		for (FTopologicalEdge* Edge : EdgesToMesh)
		{
			if (Edge->IsMeshed())
			{
				continue;
			}

			Mesh(*Edge, bFinalMeshing);
		}
	}
}

namespace ThinZoneMesherTools
{
void ResetMarkers(TArray<FTopologicalEdge*>& EdgesWithThinZones, TArray<FThinZone2D>& ThinZones)
{
	for (FTopologicalEdge* Edge : EdgesWithThinZones)
	{
		Edge->ResetMarkers();
	}

	for (FThinZone2D& Zone : ThinZones)
	{
		Zone.ResetMarkers();
		Zone.GetFirstSide().ResetMarkers();
		Zone.GetSecondSide().ResetMarkers();
	}
};
};

void FParametricFaceMesher::SortThinZoneSides(TArray<FThinZone2D*>& ThinZones)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::DefineCuttingParameters)

	Face.ResetMarkersRecursively();
	int32 EdgeCount = Face.EdgeCount();
	ZoneAEdges.Reserve(EdgeCount);
	ZoneBEdges.Reserve(EdgeCount);

	WaitingThinZones.Reserve(ThinZones.Num());

	TFunction<void(FThinZone2D*)> AddToWaitingList = [&WaitingList = WaitingThinZones](FThinZone2D* Zone)
	{
		Zone->SetWaitingMarker();
		WaitingList.Add(Zone);
	};

	TFunction<void(FThinZone2D*)> SetAndGet = [&WaitingList = WaitingThinZones, &ZoneA = ZoneAEdges, &ZoneB = ZoneBEdges](FThinZone2D* Zone)
	{
		Zone->SetEdgesZoneSide();
		Zone->GetEdges(ZoneA, ZoneB);
	};


	int32 Index = 1;
	for (FThinZone2D* Zone : ThinZones)
	{
		Zone->CheckEdgesZoneSide();


		if (Zone->GetFirstSide().HasMarker1And2())
		{
			AddToWaitingList(Zone);
			continue;
		}

		if (Zone->GetSecondSide().HasMarker1And2())
		{
			AddToWaitingList(Zone);
			continue;
		}

		if (!Zone->GetFirstSide().HasMarker1Or2() && !Zone->GetSecondSide().HasMarker1Or2())
		{
			SetAndGet(Zone);
			continue;
		}

		if (Zone->GetFirstSide().HasMarker1() && !Zone->GetSecondSide().HasMarker1())
		{
			SetAndGet(Zone);
			continue;
		}

		if (Zone->GetFirstSide().HasMarker1() && Zone->GetSecondSide().HasMarker1())
		{
			AddToWaitingList(Zone);
			continue;
		}

		if (!Zone->GetFirstSide().HasMarker2() && !Zone->GetSecondSide().HasMarker1())
		{
			SetAndGet(Zone);
			continue;
		}

		if (Zone->GetFirstSide().HasMarker2() && Zone->GetSecondSide().HasMarker2()) // E
		{
			AddToWaitingList(Zone);
			continue;
		}

		if (Zone->GetFirstSide().HasMarker2() && !Zone->GetSecondSide().HasMarker2()) // F
		{
			Zone->Swap();
			SetAndGet(Zone);
			continue;
		}

		if (!Zone->GetFirstSide().HasMarker1() && Zone->GetSecondSide().HasMarker1())
		{
			Zone->Swap();
			SetAndGet(Zone);
			continue;
		}

		ensureCADKernel(false);
	}

	Face.ResetMarkersRecursively();
}

void FParametricFaceMesher::MeshThinZones()
{
	TArray<FThinZone2D>& FaceThinZones = Face.GetThinZones();

	if (FaceThinZones.IsEmpty())
	{
		return;
	}

	TArray<FThinZone2D*> ThinZones;
	ThinZones.Reserve(FaceThinZones.Num());
	for (FThinZone2D& FaceThinZone : FaceThinZones)
	{
		ThinZones.Add(&FaceThinZone);
	}

	while (ThinZones.Num())
	{
		int32 WaitingThinZoneCount = ThinZones.Num();

		MeshThinZones(ThinZones);

		if (WaitingThinZoneCount == WaitingThinZones.Num())
		{
			break;
		}
		ThinZones = MoveTemp(WaitingThinZones);
	}
}


void FParametricFaceMesher::MeshThinZones(TArray<FThinZone2D*>& ThinZones)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FParametricFaceMesher::MeshThinZones)
	
	SortThinZoneSides(ThinZones);

	TFunction<void(TArray<FTopologicalEdge*>)> TransfereCuttingPointsFromMeshedEdges = [](TArray<FTopologicalEdge*> Edges)
	{
		for (FTopologicalEdge* Edge : Edges)
		{
			FAddCuttingPointFunc AddCuttingPoint = [&Edge](const double Coordinate, const ECoordinateType Type, const FPairOfIndex OppositNodeIndices, const double DeltaU)
			{
				Edge->AddTwinsCuttingPoint(Coordinate, DeltaU);
			};

			constexpr bool bOnlyOppositNode = false;
			Edge->TransferCuttingPointFromMeshedEdge(bOnlyOppositNode, AddCuttingPoint);
		}
	};

	TransfereCuttingPointsFromMeshedEdges(ZoneAEdges);
	TransfereCuttingPointsFromMeshedEdges(ZoneBEdges);

	bool bFinalMeshing = false;
	{
		MeshThinZones(ZoneAEdges, bFinalMeshing);
	}

	{
		bFinalMeshing = true;
		MeshThinZones(ZoneBEdges, bFinalMeshing);
	}

	for (FTopologicalEdge* Edge : ZoneAEdges)
	{
		Edge->RemovePreMesh();
	}

	{
		MeshThinZones(ZoneAEdges, bFinalMeshing);
	}
}

void FParametricFaceMesher::DefineImposedCuttingPointsBasedOnOtherSideMesh(FThinZoneSide& SideToConstrain)
{
	using namespace ParametricMesherTool;

	FThinZoneSide& FrontSide = SideToConstrain.GetFrontThinZoneSide();

	TMap<int32, FVector2d> ExistingMeshNodes;
	TArray<FCrossZoneElement> CrossZoneElements;

	FAddMeshNodeFunc AddToCrossZoneElements = [&CrossZoneElements](const int32 NodeIndice, const FVector2d& MeshNode2D, double MeshingTolerance3D, const FEdgeSegment& EdgeSegment, const FPairOfIndex& OppositeNodeIndices)
	{
		if (CrossZoneElements.Num() && CrossZoneElements.Last().VertexId >= 0 && CrossZoneElements.Last().VertexId == NodeIndice)
		{
			CrossZoneElements.Last().Add(OppositeNodeIndices);
		}
		else
		{
			CrossZoneElements.Emplace(NodeIndice, MeshNode2D, MeshingTolerance3D, &EdgeSegment, OppositeNodeIndices);
		}
	};

	FAddMeshNodeFunc AddToExistingMeshNodes = [&ExistingMeshNodes](const int32 NodeIndice, const FVector2d& MeshNode2D, double MeshingTolerance3D, const FEdgeSegment& EdgeSegment, const FPairOfIndex& OppositeNodeIndices)
	{
		ExistingMeshNodes.Emplace(NodeIndice, MeshNode2D);
	};

	FReserveContainerFunc ReserveCrossZoneElements = [&CrossZoneElements](int32 MeshVertexCount)
	{
		CrossZoneElements.Reserve(MeshVertexCount);
	};

	FReserveContainerFunc ReserveExistingMeshNodes = [&ExistingMeshNodes](int32 MeshVertexCount)
	{
		ExistingMeshNodes.Reserve(MeshVertexCount);
	};

	SideToConstrain.GetExistingMeshNodes(Face, MeshModel, ReserveExistingMeshNodes, AddToExistingMeshNodes, /*bWithTolerance*/ false);
	FrontSide.GetExistingMeshNodes(Face, MeshModel, ReserveCrossZoneElements, AddToCrossZoneElements, /*bWithTolerance*/ true);

	const double MaxSquareThickness = FrontSide.GetMaxThickness() > SideToConstrain.GetMaxThickness() ? FMath::Square(3. * FrontSide.GetMaxThickness()) : FMath::Square(3. * SideToConstrain.GetMaxThickness());

	// Find the best projection of existing mesh vertices (CrossZone Vertex)
	for (FCrossZoneElement& CrossZoneElement : CrossZoneElements)
	{
		if (CrossZoneElement.OppositeVertexIndices[0] >= 0)
		{
			FVector2d* OppositeVertex = ExistingMeshNodes.Find(CrossZoneElement.OppositeVertexIndices[0]);
			if (OppositeVertex)
			{
				CrossZoneElement.OppositePoint2D = *OppositeVertex;
				CrossZoneElement.SquareThickness = 0;
			}
			continue;
		}

		double MinSquareThickness = MaxSquareThickness;
		FVector2d ClosePoint = FVector2d::ZeroVector;
		FEdgeSegment* CloseSegment = nullptr;
		double ClosePointCoordinate = -1;

		const FVector2d MeshNodeCoordinate = CrossZoneElement.VertexPoint2D;

		for (FEdgeSegment& Segment : SideToConstrain.GetSegments())
		{
			// check the angle between segment and Middle-SegementStart, Middle-SegementEnd.
			const double SlopeS = Segment.ComputeOrientedSlopeOf(MeshNodeCoordinate, Segment.GetExtemity(ELimit::Start));
			const double SlopeE = Segment.ComputeOrientedSlopeOf(MeshNodeCoordinate, Segment.GetExtemity(ELimit::End));
			if (SlopeE < 1. || SlopeS > 3.)
			{
				continue;
			}

			double CoordSegmentU;
			FVector2d Projection = Segment.ProjectPoint(MeshNodeCoordinate, CoordSegmentU);

			const double SquareDistance = FVector2d::DistSquared(MeshNodeCoordinate, Projection);
			if (MinSquareThickness > SquareDistance)
			{
				FEdgeSegment* SegmentPtr = &Segment;
				// Forbid the common extremity as candidate
				{
					constexpr double NearlyZero = 0. + DOUBLE_SMALL_NUMBER;
					constexpr double NearlyOne = 1. - DOUBLE_SMALL_NUMBER;

					const FEdgeSegment* CrossZoneSegment = CrossZoneElement.Segment;
					if (SegmentPtr == CrossZoneSegment->GetPrevious() && CoordSegmentU > NearlyOne)
					{
						continue;
					}
					if (SegmentPtr == CrossZoneSegment->GetNext() && CoordSegmentU < NearlyZero)
					{
						continue;
					}
				}

				MinSquareThickness = SquareDistance;
				ClosePoint = Projection;
				ClosePointCoordinate = CoordSegmentU;
				CloseSegment = SegmentPtr;
			}
		}

		if (CloseSegment)
		{
			CrossZoneElement.OppositePoint2D = ClosePoint;
			CrossZoneElement.OppositeSegment = CloseSegment;
			CrossZoneElement.OppositePointCoordinate = ClosePointCoordinate;
			CrossZoneElement.SquareThickness = MinSquareThickness;
		}
	}

	Algo::Sort(CrossZoneElements, [](FCrossZoneElement& ElementA, FCrossZoneElement& ElementB) { return ElementA.SquareThickness < ElementB.SquareThickness; });

	// Find candidates i.e. CrossZoneElement that are not in intersection with the sides and with selected CrossZoneElement
	FIntersectionTool IntersectionTool(FrontSide.GetSegments(), SideToConstrain.GetSegments(), CrossZoneElements.Num());
	for (FCrossZoneElement& CrossZoneElement : CrossZoneElements)
	{
		if (!CrossZoneElement.OppositeSegment)
		{
			continue;
		}

		if (!IntersectionTool.IsIntersectSides(CrossZoneElement))
		{
			IntersectionTool.AddCrossZoneElement(CrossZoneElement);
			CrossZoneElement.bIsSelected = true;
		}
	}

	// Add ImposedCuttingPointU
	for (FCrossZoneElement& CrossZoneElement : CrossZoneElements)
	{
		if (CrossZoneElement.bIsSelected && (CrossZoneElement.OppositeVertexIndices[0] < 0))
		{
			FTopologicalEdge* OppositeEdge = CrossZoneElement.OppositeSegment->GetEdge();
			if (OppositeEdge == nullptr)
			{
				continue;
			}

			const double OppositeCuttingPointU = CrossZoneElement.OppositeSegment->ComputeEdgeCoordinate(CrossZoneElement.OppositePointCoordinate);
			const double DeltaU = CrossZoneElement.OppositeSegment->ComputeDeltaU(CrossZoneElement.Tolerance3D);

			OppositeEdge->AddImposedCuttingPointU(OppositeCuttingPointU, CrossZoneElement.VertexId, DeltaU);
		}
	}
}

void FParametricFaceMesher::MeshFaceLoops()
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			Mesh(*Edge.Entity);
		}
	}
}

void FParametricFaceMesher::Mesh(FTopologicalVertex& InVertex)
{
	InVertex.GetOrCreateMesh(MeshModel);
}

void FParametricFaceMesher::MeshVerticesOfFace(FTopologicalFace& FaceToProcess)
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : FaceToProcess.GetLoops())
	{
		for (FOrientedEdge& Edge : Loop->GetEdges())
		{
			Mesh(*Edge.Entity->GetStartVertex());
			Mesh(*Edge.Entity->GetEndVertex());
		}
	}
}

void FParametricFaceMesher::Mesh(FTopologicalEdge& InEdge, bool bFinalMeshing)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FParametricFaceMesher::Mesh)

	FTopologicalEdge& ActiveEdge = *InEdge.GetLinkActiveEntity();
	if (ActiveEdge.IsMeshed())
	{
		if (ActiveEdge.GetMesh()->GetNodeCount() > 0)
		{
			return;
		}

		// In some case the 2d curve is a smooth curve and the 3d curve is a line and vice versa
		// In the particular case where the both case are opposed, we can have the 2d line sampled with 4 points, and the 2d curve sampled with 2 points (because in 3d, the 2d curve is a 3d line)
		// In this case, the loop is flat i.e. in 2d the meshes of the 2d line and 2d curve are coincident
		// So the grid is degenerated and the surface is not meshed
		// to avoid this case, the Edge is virtually meshed i.e. the nodes inside the edge have the id of the mesh of the vertices.
		InEdge.SetVirtuallyMeshedMarker();
	}

	if (ActiveEdge.IsThinPeak())
	{
		TArray<FCuttingPoint>& FinalEdgeCuttingPointCoordinates = ActiveEdge.GetCuttingPoints();
		FinalEdgeCuttingPointCoordinates.Emplace(ActiveEdge.GetStartCurvilinearCoordinates(), ECoordinateType::VertexCoordinate);
		FinalEdgeCuttingPointCoordinates.Emplace(ActiveEdge.GetEndCurvilinearCoordinates(), ECoordinateType::VertexCoordinate);
		ActiveEdge.GenerateMeshElements(MeshModel);
		return;
	}

	const FSurfacicTolerance& ToleranceIso = Face.GetIsoTolerances();

	// Get Edge intersection with inner surface mesh grid
	TArray<double> EdgeIntersectionWithIsoU_Coordinates;
	TArray<double> EdgeIntersectionWithIsoV_Coordinates;

	const TArray<double>& SurfaceTabU = Face.GetCuttingCoordinatesAlongIso(EIso::IsoU);
	const TArray<double>& SurfaceTabV = Face.GetCuttingCoordinatesAlongIso(EIso::IsoV);

	ApplyEdgeCriteria(InEdge);

	InEdge.ComputeIntersectionsWithIsos(SurfaceTabU, EIso::IsoU, ToleranceIso, EdgeIntersectionWithIsoU_Coordinates);
	InEdge.ComputeIntersectionsWithIsos(SurfaceTabV, EIso::IsoV, ToleranceIso, EdgeIntersectionWithIsoV_Coordinates);

	FLinearBoundary EdgeBounds = InEdge.GetBoundary();

	TArray<double>& DeltaUs = InEdge.GetDeltaUMaxs();

	FAddCuttingPointFunc AddCuttingPoint = [&InEdge](const double Coordinate, const ECoordinateType Type, const FPairOfIndex OppositNodeIndices, const double DeltaU)
	{
		for (int32 Index = 0; Index < 2; ++Index)
		{
			if (OppositNodeIndices[Index] >= 0)
			{
				InEdge.AddImposedCuttingPointU(Coordinate, OppositNodeIndices[Index], DeltaU);
			}
		}
	};

	// Case of self connected surface (e.g. cylinder) an edge 
	// The first edge is premeshed at step 1, but the activeEdge is not yet meshed
	// the twin edge is meshed at step 2
	if (ActiveEdge.IsPreMeshed())
	{
		constexpr bool bOnlyOppositNode = true;
		InEdge.TransferCuttingPointFromMeshedEdge(bOnlyOppositNode, AddCuttingPoint);

		FTopologicalEdge* PreMeshEdge = InEdge.GetPreMeshedTwin();
		if(PreMeshEdge)
		{
			PreMeshEdge->RemovePreMesh();
		}
	}

	InEdge.SortImposedCuttingPoints();
	const TArray<FImposedCuttingPoint>& EdgeImposedCuttingPoints = InEdge.GetImposedCuttingPoints();

	// build a edge mesh compiling inner surface cutting (based on criteria applied on the surface) and edge cutting (based on criteria applied on the curve)
	TArray<FCuttingPoint> ImposedIsoCuttingPoints;

	TFunction<void(int32&, int32&)> UpdateDeltaU = [&ImposedIsoCuttingPoints](int32& NewIndex, int32& Index)
	{
		if (ImposedIsoCuttingPoints[NewIndex].IsoDeltaU > ImposedIsoCuttingPoints[Index].IsoDeltaU)
		{
			ImposedIsoCuttingPoints[NewIndex].IsoDeltaU = ImposedIsoCuttingPoints[Index].IsoDeltaU;
		}
	};

	TFunction<void(int32&, int32&)> UpdateOppositNodeIndices = [&ImposedIsoCuttingPoints](int32& NewIndex, int32& Index)
	{
		if (ImposedIsoCuttingPoints[NewIndex].OppositNodeIndices[0] == -1)
		{
			ImposedIsoCuttingPoints[NewIndex].OppositNodeIndices[0] = ImposedIsoCuttingPoints[Index].OppositNodeIndices[0];
		}
		else if (ImposedIsoCuttingPoints[NewIndex].OppositNodeIndices[0] != ImposedIsoCuttingPoints[Index].OppositNodeIndices[0])
		{
			ImposedIsoCuttingPoints[NewIndex].OppositNodeIndices[1] = ImposedIsoCuttingPoints[Index].OppositNodeIndices[0];
		}
	};

	TFunction<void(int32&, int32&, ECoordinateType)> MergeImposedCuttingPoints = [&ImposedIsoCuttingPoints, UpdateOppositNodeIndices, UpdateDeltaU](int32& Index, int32& NewIndex, ECoordinateType NewType)
	{
		double DeltaU = FMath::Max(ImposedIsoCuttingPoints[NewIndex].IsoDeltaU, ImposedIsoCuttingPoints[Index].IsoDeltaU);

		if (ImposedIsoCuttingPoints[NewIndex].Coordinate + DeltaU > ImposedIsoCuttingPoints[Index].Coordinate)
		{
			if (ImposedIsoCuttingPoints[Index].Type == VertexCoordinate)
			{
				ImposedIsoCuttingPoints[NewIndex].Coordinate = ImposedIsoCuttingPoints[Index].Coordinate;
				ImposedIsoCuttingPoints[NewIndex].IsoDeltaU = ImposedIsoCuttingPoints[Index].IsoDeltaU;
				ImposedIsoCuttingPoints[NewIndex].Type = ImposedIsoCuttingPoints[Index].Type;
				UpdateOppositNodeIndices(NewIndex, Index);
				UpdateDeltaU(NewIndex, Index);
			}
			else if (ImposedIsoCuttingPoints[NewIndex].Type == VertexCoordinate)
			{
				if (ImposedIsoCuttingPoints[Index].Type == ImposedCoordinate)
				{
					UpdateOppositNodeIndices(NewIndex, Index);
					UpdateDeltaU(NewIndex, Index);
				}
			}
			else if (ImposedIsoCuttingPoints[NewIndex].Type == ImposedCoordinate)
			{
				if (ImposedIsoCuttingPoints[Index].Type == ImposedCoordinate)
				{
					ImposedIsoCuttingPoints[NewIndex].Coordinate = (ImposedIsoCuttingPoints[NewIndex].Coordinate + ImposedIsoCuttingPoints[Index].Coordinate) * 0.5;
					UpdateOppositNodeIndices(NewIndex, Index);
					UpdateDeltaU(NewIndex, Index);
				}
			}
			else if (ImposedIsoCuttingPoints[Index].Type == ImposedCoordinate)
			{
				ImposedIsoCuttingPoints[NewIndex].Coordinate = ImposedIsoCuttingPoints[Index].Coordinate;
				ImposedIsoCuttingPoints[NewIndex].Type = ImposedCoordinate;
				UpdateOppositNodeIndices(NewIndex, Index);
				UpdateDeltaU(NewIndex, Index);
			}
			else if (ImposedIsoCuttingPoints[NewIndex].Type != ImposedIsoCuttingPoints[Index].Type)
			{
				ImposedIsoCuttingPoints[NewIndex].Coordinate = (ImposedIsoCuttingPoints[NewIndex].Coordinate + ImposedIsoCuttingPoints[Index].Coordinate) * 0.5;
				ImposedIsoCuttingPoints[NewIndex].Type = IsoUVCoordinate;
				UpdateDeltaU(NewIndex, Index);
			}
		}
		else
		{
			++NewIndex;
			ImposedIsoCuttingPoints[NewIndex] = ImposedIsoCuttingPoints[Index];
		}
	};

	{
		int32 NbImposedCuttingPoints = EdgeImposedCuttingPoints.Num() + EdgeIntersectionWithIsoU_Coordinates.Num() + EdgeIntersectionWithIsoV_Coordinates.Num() + 2;
		ImposedIsoCuttingPoints.Reserve(NbImposedCuttingPoints);
	}


	const double EdgeBoundsLength = EdgeBounds.Length();
	const double EdgeDeltaUAtMin = FMath::Min(DeltaUs[0] * AQuarter, EdgeBoundsLength * AEighth);
	const double EdgeDeltaUAtMax = FMath::Min(DeltaUs.Last() * AQuarter, EdgeBoundsLength * AEighth);

	ImposedIsoCuttingPoints.Emplace(EdgeBounds.GetMin(), ECoordinateType::VertexCoordinate, FPairOfIndex::Undefined, EdgeDeltaUAtMin);
	ImposedIsoCuttingPoints.Emplace(EdgeBounds.GetMax(), ECoordinateType::VertexCoordinate, FPairOfIndex::Undefined, EdgeDeltaUAtMax);

	int32 Index = 0;
	for (const FImposedCuttingPoint& CuttingPoint : EdgeImposedCuttingPoints)
	{
		const double CuttingPointDeltaU = CuttingPoint.DeltaU;
		ImposedIsoCuttingPoints.Emplace(CuttingPoint.Coordinate, ECoordinateType::ImposedCoordinate, CuttingPoint.OppositNodeIndex, CuttingPointDeltaU * AThird);
	}

	// Add Edge intersection with inner surface grid Iso
	FVector2d ExtremityTolerances = InEdge.GetCurve()->GetExtremityTolerances(EdgeBounds);
	double EdgeTolerance = FMath::Min(ExtremityTolerances[0], ExtremityTolerances[1]);
	if (!EdgeIntersectionWithIsoU_Coordinates.IsEmpty())
	{
		FMesherTools::FillImposedIsoCuttingPoints(EdgeIntersectionWithIsoU_Coordinates, IsoUCoordinate, EdgeTolerance, InEdge, ImposedIsoCuttingPoints);
	}

	if (!EdgeIntersectionWithIsoV_Coordinates.IsEmpty())
	{
		FMesherTools::FillImposedIsoCuttingPoints(EdgeIntersectionWithIsoV_Coordinates, IsoVCoordinate, EdgeTolerance, InEdge, ImposedIsoCuttingPoints);
	}

	ImposedIsoCuttingPoints.Sort([](const FCuttingPoint& Point1, const FCuttingPoint& Point2) { return Point1.Coordinate < Point2.Coordinate; });

	// If a pair of point isoU/isoV is too close, get the middle of the points
	if (ImposedIsoCuttingPoints.Num() > 1)
	{
		int32 NewIndex = 0;
		for (int32 Andex = 1; Andex < ImposedIsoCuttingPoints.Num(); ++Andex)
		{
			if (ImposedIsoCuttingPoints[Andex].Type > ECoordinateType::ImposedCoordinate)
			{
				bool bIsDelete = false;
				for (const FLinearBoundary& ThinZone : InEdge.GetThinZoneBounds())
				{
					if (ThinZone.Contains(ImposedIsoCuttingPoints[Andex].Coordinate))
					{
						bIsDelete = true;
						break; // or continue
					}
				}
				if (bIsDelete)
				{
					continue;
				}
			}

			if (ImposedIsoCuttingPoints[NewIndex].Type == ECoordinateType::ImposedCoordinate || ImposedIsoCuttingPoints[Andex].Type == ECoordinateType::ImposedCoordinate)
			{
				MergeImposedCuttingPoints(Andex, NewIndex, ECoordinateType::ImposedCoordinate);
			}
			else if (ImposedIsoCuttingPoints[NewIndex].Type != ImposedIsoCuttingPoints[Andex].Type)
			{
				MergeImposedCuttingPoints(Andex, NewIndex, ECoordinateType::IsoUVCoordinate);
			}
			else
			{
				++NewIndex;
				ImposedIsoCuttingPoints[NewIndex] = ImposedIsoCuttingPoints[Andex];
			}
		}
		ImposedIsoCuttingPoints.SetNum(NewIndex + 1);
	}

	if (ImposedIsoCuttingPoints.Num() > 1 && (EdgeBounds.GetMax() - ImposedIsoCuttingPoints.Last().Coordinate) < FMath::Min(ImposedIsoCuttingPoints.Last().IsoDeltaU, InEdge.GetDeltaUMaxs().Last()))
	{
		ImposedIsoCuttingPoints.Last().Coordinate = EdgeBounds.GetMax();
		ImposedIsoCuttingPoints.Last().Type = VertexCoordinate;
	}
	else
	{
		ImposedIsoCuttingPoints.Emplace(EdgeBounds.GetMax(), ECoordinateType::VertexCoordinate, -1, InEdge.GetDeltaUMaxs().Last() * AQuarter);
	}

	// Final array of the edge mesh vertex 
	TArray<FCuttingPoint>& FinalEdgeCuttingPointCoordinates = InEdge.GetCuttingPoints();
	{
		// max count of vertex
		double MinDeltaU = HUGE_VALUE;
		for (const double& DeltaU : DeltaUs)
		{
			if (DeltaU < MinDeltaU)
			{
				MinDeltaU = DeltaU;
			}
		}

		int32 MaxNumberOfVertex = FMath::IsNearlyZero(MinDeltaU) ? 5 : (int32)((EdgeBounds.GetMax() - EdgeBounds.GetMin()) / MinDeltaU) + 5;
		FinalEdgeCuttingPointCoordinates.Empty(ImposedIsoCuttingPoints.Num() + MaxNumberOfVertex);
	}

	if (InEdge.IsDegenerated() || InEdge.IsVirtuallyMeshed())
	{
		if (ImposedIsoCuttingPoints.Num() == 2)
		{
			ImposedIsoCuttingPoints.EmplaceAt(1, (ImposedIsoCuttingPoints[0].Coordinate + ImposedIsoCuttingPoints[1].Coordinate) * 0.5, ECoordinateType::OtherCoordinate);
		}

		for (FCuttingPoint CuttingPoint : ImposedIsoCuttingPoints)
		{
			FinalEdgeCuttingPointCoordinates.Emplace(CuttingPoint.Coordinate, ECoordinateType::OtherCoordinate);
		}
		InEdge.GetLinkActiveEdge()->SetMeshedMarker();
		return;
	}

	TArray<double> CuttingPoints;
	FMesherTools::ComputeFinalCuttingPointsWithImposedCuttingPoints(InEdge.GetCrossingPointUs(), InEdge.GetDeltaUMaxs(), ImposedIsoCuttingPoints, CuttingPoints);
	int32 ImposedIndex = 0;
	int32 ImposedIsoCuttingPointsCount = ImposedIsoCuttingPoints.Num();
	for (const double& Coordinate : CuttingPoints)
	{
		if (FMath::IsNearlyEqual(ImposedIsoCuttingPoints[ImposedIndex].Coordinate, Coordinate))
		{
			FinalEdgeCuttingPointCoordinates.Emplace(ImposedIsoCuttingPoints[ImposedIndex]);
			++ImposedIndex;
		}
		else
		{
			while (ImposedIndex < ImposedIsoCuttingPointsCount && ImposedIsoCuttingPoints[ImposedIndex].Coordinate < Coordinate)
			{
				++ImposedIndex;
			}
			FinalEdgeCuttingPointCoordinates.Emplace(Coordinate, ECoordinateType::OtherCoordinate);
		}
	}

	if (bFinalMeshing)
	{
		InEdge.GenerateMeshElements(MeshModel);
	}
	else
	{
		ActiveEdge.SetPreMeshedMarker();
		InEdge.SetPreMeshedMarker();
	}
}

void FParametricFaceMesher::ApplyEdgeCriteria(FTopologicalEdge& Edge)
{
	FTopologicalEdge& ActiveEdge = *Edge.GetLinkActiveEdge();

	if (Edge.Length() < 2. * Tolerances.GeometricTolerance)
	{
		for (FTopologicalEdge* TwinEdge : Edge.GetTwinEntities())
		{
			TwinEdge->SetAsDegenerated();
		}
	}

	Edge.ComputeCrossingPointCoordinates();
	Edge.InitDeltaUs();
	const TArray<double>& CrossingPointUs = Edge.GetCrossingPointUs();

	TArray<double> Coordinates;
	Coordinates.SetNum(CrossingPointUs.Num() * 2 - 1);
	Coordinates[0] = CrossingPointUs[0];
	for (int32 ICuttingPoint = 1; ICuttingPoint < Edge.GetCrossingPointUs().Num(); ICuttingPoint++)
	{
		Coordinates[2 * ICuttingPoint - 1] = (CrossingPointUs[ICuttingPoint - 1] + CrossingPointUs[ICuttingPoint]) * 0.5;
		Coordinates[2 * ICuttingPoint] = CrossingPointUs[ICuttingPoint];
	}

	TArray<FCurvePoint> Points3D;
	Edge.EvaluatePoints(Coordinates, 0, Points3D);

	const TArray<TSharedPtr<FCriterion>>& Criteria = MeshModel.GetCriteria();
	for (const TSharedPtr<FCriterion>& Criterion : Criteria)
	{
		Criterion->ApplyOnEdgeParameters(Edge, CrossingPointUs, Points3D);
	}

	Edge.SetApplyCriteriaMarker();
	ActiveEdge.SetApplyCriteriaMarker();
}

void FParametricFaceMesher::MeshThinZoneSide(FThinZoneSide& Side, bool bFinalMeshing)
{
	if (!Side.HasMarker2())
	{
		return;
	}

	if (Side.IsProcessed())
	{
		return;
	}
	Side.ResetProcessedMarker();

	for (FTopologicalEdge* Edge : Side.GetEdges())
	{
		if (Edge->IsMeshed())
		{
			continue;
		}
		Mesh(*Edge, bFinalMeshing);
	}
}

bool FParametricFaceMesher::MeshPlanarFace()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FParametricFaceMesher::MeshPlanarFace)
		
	using namespace UE::Geometry;

	if (!GDetectPlanarFace || !SurfaceUtilities::IsPlanar(*Face.GetCarrierSurface()))
	{
		return false;
	}

	TArray<FGeneralPolygon2d> PolygonsOut;
	PolygonsOut.Reserve(Face.GetLoops().Num());

	const TArray<TArray<FVector2d>>& Loops2D = Grid.GetLoops2D(EGridSpace::Default2D);
	const TArray<TSharedPtr<FTopologicalLoop>>& Loops = Face.GetLoops();
	ensureCADKernel(Loops.Num() == Loops2D.Num());

	FGeneralPolygon2d* GPolygon = nullptr;
	bool bOuterIsCW = false;
	for (int32 Index = 0; Index < Loops.Num(); ++Index)
	{
		const TArray<FVector2d>& PointList = Loops2D[Index];
		TArray<FVector2d> VertexList;
		VertexList.Reserve(PointList.Num());
		for (const FVector2d& Point : PointList)
		{
			VertexList.Emplace(Point.X, Point.Y);
		}

		FPolygon2d Polygon2d(VertexList);

		if (Loops[Index]->IsExternal())
		{
			bOuterIsCW = Polygon2d.IsClockwise();
			GPolygon = &PolygonsOut.Emplace_GetRef(MoveTemp(Polygon2d));
		}
		else if (ensureCADKernel(GPolygon))
		{
			if (bOuterIsCW == Polygon2d.IsClockwise())
			{
				// The hole must be in the reverse direction
				Polygon2d.Reverse();
			}
			if (!ensureCADKernel(GPolygon->AddHole(Polygon2d, /*bCheckContainment =*/ true, /*bCheckOrientation =*/ true)))
			{
				// #cadkernel_check: Why this is happening?
				return false;
			}
		}
	}

	// Abort if there is no polygon to triangulate or the outer is only a segment.
	if (!ensureCADKernel(PolygonsOut.Num() > 0 && PolygonsOut[0].GetOuter().GetVertices().Num() > 2))
	{
		// #cadkernel_check: Why this is happening?
		return false;
	}

	FFaceMesh& FaceMesh = Face.GetOrCreateMesh(MeshModel);

	TArray<FVector>& NodeCoordinates = FaceMesh.GetNodeCoordinates();
	TArray<int32>& TrianglesVerticesIndex = FaceMesh.TrianglesVerticesIndex;
	TArray<int32>& VerticesGlobalIndex = FaceMesh.VerticesGlobalIndex;
	TArray<FVector3f>& Normals = FaceMesh.Normals;
	TArray<FVector2f>& UVMap = FaceMesh.UVMap;
	
	// Abort if tessellation is called twice on a Face
	// #cadkernel_check: Why this is happening?
	if (TrianglesVerticesIndex.Num() > 0)
	{
		ensureCADKernel(false);
		return false;
	}

	const int32 GlobalVertexCount = MeshModel.GetVertexCount();

	const FSurface& CarrierSurface = *Face.GetCarrierSurface();
	FSurfacicPoint SurfacicPoint;

	for (FGeneralPolygon2d& Polygon : PolygonsOut)
	{
		FDelaunay2 Delaunay;
		Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = false;

		TArray<FIndex3i> Triangles;
		TArray<FVector2d> Vertices;

		// FDelaunay2::Triangulate can return false although the triangles are valid
		// In that case, false is returned because some input points were not connected as expected
		// bFallbackToGeneralizedWinding is set to true to force the generation of triangles
		// #cadkernel_check: Why those points are not connected?
		Delaunay.Triangulate(Polygon, &Triangles, &Vertices, true /*bFallbackToGeneralizedWinding*/);
		if (!ensureCADKernel(Triangles.Num() > 0))
		{
			// #cadkernel_check: Why this is happening?
			return false;
		}

		const int32 StartIndex = VerticesGlobalIndex.Num();
		VerticesGlobalIndex.Reserve(StartIndex + Vertices.Num());

		Normals.Reserve(Normals.Num() + Vertices.Num());
		UVMap.Reserve(UVMap.Num() + Vertices.Num());
		NodeCoordinates.Reserve(NodeCoordinates.Num() + Vertices.Num());

		int32 Index = StartIndex;
		for (const FVector2d& Vertex : Vertices)
		{
			FVector Point;
			FVector3f Normal;
			CarrierSurface.EvaluatePointAndNormal(FVector2d(Vertex.X, Vertex.Y), Point, Normal);
			
			NodeCoordinates.Emplace(Point);
			Normals.Emplace(Normal);
			UVMap.Emplace((FVector2f)Vertex);

			VerticesGlobalIndex.Emplace(Index++ + GlobalVertexCount);
		}

		TrianglesVerticesIndex.SetNum(TrianglesVerticesIndex.Num() + Triangles.Num() * 3);
		int32* VertexIndices = TrianglesVerticesIndex.GetData() + StartIndex;

		for (const FIndex3i& TriIndices : Triangles)
		{
			VertexIndices[0] = TriIndices.A + StartIndex;
			VertexIndices[1] = TriIndices.B + StartIndex;
			VertexIndices[2] = TriIndices.C + StartIndex;

			VertexIndices += 3;
		}
	}

	if (Face.IsBackOriented())
	{
		FaceMesh.InverseOrientation();
	}

	FaceMesh.RegisterCoordinates();
	MeshModel.AddMesh(FaceMesh);

	return true;
}
} // namespace
