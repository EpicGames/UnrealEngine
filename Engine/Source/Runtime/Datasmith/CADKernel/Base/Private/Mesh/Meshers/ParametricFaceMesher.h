// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/Types.h"
#include "Mesh/Structure/Grid.h"

namespace UE::CADKernel
{
class FCriterion;
class FGrid;
class FMeshingTolerances;
class FThinZoneSide;
class FThinZone2D;
class FTopologicalEntity;
class FTopologicalLoop;

namespace ParametricMesherTool
{

struct FCrossZoneElement
{
	const int32 VertexId;
	FVector2d VertexPoint2D;
	const double Tolerance3D;
	const FEdgeSegment* Segment;
	FPairOfIndex OppositeVertexIndices;

	FVector2d OppositePoint2D;
	FEdgeSegment* OppositeSegment;
	double OppositePointCoordinate = -1.;

	double SquareThickness;

	bool bIsSelected = false;

	FCrossZoneElement(const int32 InVertexId, const FVector2d InPoint2D, const double InTolerance3D, const FEdgeSegment* InSegment, const FPairOfIndex& InOppositeVertexIndices)
		: VertexId(InVertexId)
		, VertexPoint2D(InPoint2D)
		, Tolerance3D(InTolerance3D)
		, Segment(InSegment)
		, OppositeVertexIndices(InOppositeVertexIndices)
		, OppositePoint2D(FVector2d::ZeroVector)
		, OppositeSegment(nullptr)
		, OppositePointCoordinate(-1.)
		, SquareThickness(-1.)
	{
	}

	FCrossZoneElement()
		: VertexId(-1)
		, VertexPoint2D(FVector2d::ZeroVector)
		, Tolerance3D(-1.)
		, Segment(nullptr)
		, OppositeVertexIndices(FPairOfIndex::Undefined)
		, OppositePoint2D(FVector2d::ZeroVector)
		, OppositeSegment(nullptr)
		, OppositePointCoordinate(-1.)
		, SquareThickness(-1.)
	{
	}

	void Add(const FPairOfIndex& VertexIndices)
	{
		OppositeVertexIndices.Add(VertexIndices);
	}
};


class FIntersectionTool
{
private:
	const TArray<FEdgeSegment>& Side1Segments;
	const TArray<FEdgeSegment>& Side2Segments;
	TArray<FCrossZoneElement*> SelectedCrossZoneElement;

public:

	FIntersectionTool(const TArray<FEdgeSegment>& InSide1Segments, const TArray<FEdgeSegment>& InSide2Segments, int32 MaxCrossZoneElementCount)
		: Side1Segments(InSide1Segments)
		, Side2Segments(InSide2Segments)
	{
		SelectedCrossZoneElement.Reserve(MaxCrossZoneElementCount);
	}

	bool IsIntersectSides(const FCrossZoneElement& CrossZoneElement)
	{
		FSegment2D CrossZoneSegment(CrossZoneElement.VertexPoint2D, CrossZoneElement.OppositePoint2D);
		if (IsIntersectSide(Side1Segments, CrossZoneSegment) || IsIntersectSide(Side2Segments, CrossZoneSegment) || IsIntersectCrossZoneElement(CrossZoneSegment))
		{
			return true;
		}
		return false;
	}

	void AddCrossZoneElement(FCrossZoneElement& Element)
	{
		SelectedCrossZoneElement.Add(&Element);
	}

private:

	bool IsIntersectSide(const TArray<FEdgeSegment>& Segments, const FSegment2D& CrossZoneSegment)
	{
		for (const FEdgeSegment& Segment : Segments)
		{
			FSegment2D SideSegment(Segment.GetExtemity(Start), Segment.GetExtemity(End));

			if (DoIntersectInside(CrossZoneSegment, SideSegment))
			{
				return true;
			}
		}
		return false;
	}

	bool IsIntersectCrossZoneElement(const FSegment2D& CrossZoneSegment)
	{
		for (const FCrossZoneElement* Segment : SelectedCrossZoneElement)
		{
			FSegment2D SideSegment(Segment->VertexPoint2D, Segment->OppositePoint2D);

			if (DoIntersectInside(CrossZoneSegment, SideSegment))
			{
				return true;
			}
		}
		return false;
	}
};

}

class FParametricFaceMesher
{
protected:
	FTopologicalFace& Face;
	FModelMesh& MeshModel;

	const FMeshingTolerances& Tolerances;
	bool bThinZoneMeshing = false;

	FGrid Grid;

public:

	FParametricFaceMesher(FTopologicalFace& Face, FModelMesh& InMeshModel, const FMeshingTolerances& InTolerances, bool bActivateThinZoneMeshing);

	void Mesh();

private:

	void Mesh(FTopologicalEdge& InEdge, bool bFinalMeshing = true);
	void Mesh(FTopologicalVertex& Vertex);
	void MeshVerticesOfFace(FTopologicalFace& Face);
	bool MeshPlanarFace();

	void MeshFaceLoops();


	void ApplyEdgeCriteria(FTopologicalEdge& Edge);

	/**
	 * @return false if the process fails i.e. the grid is degenerated or else.
	 */
	bool GenerateCloud();

	// Thin zone meshing data / context  =================================================================
	TArray<FThinZone2D*> WaitingThinZones;

	TArray<FTopologicalEdge*> ZoneAEdges;
	TArray<FTopologicalEdge*> ZoneBEdges;

	// Meshing thin zone methodes =================================================================
	/**
	 * Main methode
	 */
	void MeshThinZones();
	void MeshThinZones(TArray<FThinZone2D*>& ThinZones);

	/**
	 * Check that in one edge with further thin zones, the edge is for all zone in the same side otherwise, fix it
	 * Zone with conflicts will be saved in a waiting list
	 */
	void SortThinZoneSides(TArray<FThinZone2D*>& ThinZones);

	void MeshThinZones(TArray<FTopologicalEdge*>& EdgesToMesh, const bool bFinalMeshing);

	void MeshThinZoneSide(FThinZoneSide& Side, bool bFinalMeshing);
	void DefineImposedCuttingPointsBasedOnOtherSideMesh(FThinZoneSide& SideToConstrain);

#ifdef CADKERNEL_DEBUG
	void DisplayMeshOfFaceLoop();
	void DisplayThinZoneEdges(const TCHAR* Text, TArray<FTopologicalEdge*>& Edges, EVisuProperty Color, EVisuProperty Color2);
#endif

};

} // namespace UE::CADKernel

