// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Meshers/ParametricMesher.h"

#include "Mesh/Criteria/CriteriaGrid.h"
#include "Mesh/Criteria/Criterion.h"
#include "Mesh/Meshers/Mesher.h"
#include "Mesh/Meshers/ParametricFaceMesher.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Mesh/Structure/ThinZone2DFinder.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalShapeEntity.h"

#include "Async/ParallelFor.h"

#ifdef CADKERNEL_DEV
#include "Mesh/Meshers/MesherReport.h"
#endif

namespace UE::CADKernel
{

#ifdef CADKERNEL_DEV
TUniquePtr<FMesherReport> FMesherReport::MesherReport;
#endif

FMesher::FMesher(FModelMesh& InMeshModel, double InGeometricTolerance, bool bActivateThinZoneMeshing)
	: GeometricTolerance(InGeometricTolerance)
#if CADKERNEL_THINZONE
	, bThinZoneMeshing(bActivateThinZoneMeshing)
#endif
	, MeshModel(InMeshModel)
{
}

void FMesher::MeshEntities(TArray<FTopologicalShapeEntity*>& InEntities)
{
	FParametricMesher Mesher(MeshModel, GeometricTolerance, bThinZoneMeshing);
	Mesher.MeshEntities(InEntities);
}


FParametricMesher::FParametricMesher(FModelMesh& InMeshModel, double GeometricTolerance, bool bActivateThinZoneMeshing)
	: Tolerances(GeometricTolerance)
#if CADKERNEL_THINZONE
	, bThinZoneMeshing(bActivateThinZoneMeshing)
#endif
	, MeshModel(InMeshModel)
{
}

void FParametricMesher::MeshEntities(TArray<FTopologicalShapeEntity*>& InEntities)
{
	int32 FaceCount = 0;

	for (FTopologicalFace* Face : Faces)
	{
		if (Face == nullptr)
		{
			continue;
		}
		Face->SetMarker1();
	}

	// count faces
	for (FTopologicalShapeEntity* TopologicalEntity : InEntities)
	{
		if (TopologicalEntity == nullptr)
		{
			continue;
		}
		FaceCount += TopologicalEntity->FaceCount();
		TopologicalEntity->ResetMarkersRecursively();
	}
	Faces.Reserve(Faces.Num() + FaceCount);

	for (FTopologicalFace* Face : Faces)
	{
		if (Face == nullptr)
		{
			continue;
		}
		Face->ResetMarkers();
	}

	// Get independent Faces and propagate body's shells orientation
	for (FTopologicalShapeEntity* TopologicalEntity : InEntities)
	{
		if (TopologicalEntity == nullptr)
		{
			continue;
		}

		TopologicalEntity->PropagateBodyOrientation();
		TopologicalEntity->GetFaces(Faces);
	}

	for (FTopologicalFace* Face : Faces)
	{
		if (Face == nullptr)
		{
			continue;
		}
		Face->ResetMarkers();
	}

	PreMeshingTasks();
	MeshEntities();
}

void FParametricMesher::PreMeshingTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParametricMesher::PreMeshingTasks);

	const TArray<TSharedPtr<FCriterion>>& Criteria = GetMeshModel().GetCriteria();

	// ============================================================================================================
	//      Apply Surface Criteria
	// ============================================================================================================
	const double MeshingTolerance = Tolerances.MeshingTolerance;
	bool bThinZone = bThinZoneMeshing;
	for (FTopologicalFace* Face : Faces)
	{ 
		if (Face == nullptr || Face->IsNotMeshable())
		{
			continue;
		}
		ApplyFaceCriteria(*Face, Criteria, MeshingTolerance, bThinZone);
		if (!Face->IsDeletedOrDegenerated())
		{
			Face->ComputeSurfaceSideProperties();
		}
	}
}

void FParametricMesher::MeshEntities()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParametricMesher::MeshEntities);

	// ============================================================================================================
	//      Find and sort quad surfaces 
	// ============================================================================================================

	TArray<ParametricMesherTool::FCostToFace> QuadTrimmedSurfaceSet;

	if (Faces.Num() > 1)
	{
		TArray<FTopologicalFace*> OtherEntities;

		IsolateQuadFace(QuadTrimmedSurfaceSet, OtherEntities);
	}

	// ============================================================================================================
	//      Mesh surfaces 
	// ============================================================================================================

	MeshSurfaceByFront(QuadTrimmedSurfaceSet);
}

void FParametricMesher::ApplyFaceCriteria(FTopologicalFace& Face, const TArray<TSharedPtr<FCriterion>>& Criteria, const double MeshingTolerance, bool bThinZoneMeshing)
{
	if (Face.IsApplyCriteria())
	{
		return;
	}

	if (!Face.ComputeCriteriaGridSampling())
	{
		// The face is considered as degenerate, the face is delete, the process is canceled
		return;
	}

	FCriteriaGrid Grid(Face);
	Grid.ComputeFaceMinMaxThicknessAlongIso();

	if (Grid.CheckIfIsDegenerate())
	{
		Face.Remove();
		return;
	}

	Face.InitDeltaUs();
	Face.ApplyCriteria(Criteria, Grid);

	if (bThinZoneMeshing)
	{
		Grid.ScaleGrid();

		FThinZone2DFinder ThinZoneFinder(Grid, Face);

		// Size (length of segment of the loop sampling) is equal to MinimalElementLength / ElementRatio
		// With this ratio each edges of the mesh should be defined by at least 3 segments. 
		// This should ensure to identified all thin zones according to the mesh size and minimizing the size of the loop sampling
		constexpr double ElementRatio = 3.;
		const double SizeVsElementLength = Face.GetEstimatedMinimalElementLength() / ElementRatio;

		const double FaceSize = Grid.GetCharacteristicThicknessOfFace();
		const double SizeVsLoop = FaceSize / (double)Face.LoopCount();

		const double SizeVsThickness = FaceSize / MeshConst::ElementRatioVsThickness;

		const bool bHasThinZones = ThinZoneFinder.SearchThinZones(FMath::Max(FMath::Min3(SizeVsElementLength, SizeVsLoop, SizeVsThickness), MeshingTolerance));
		if (bHasThinZones)
		{
			Face.SetHasThinZoneMarker();
			Face.MoveThinZones(ThinZoneFinder.GetThinZones());
		}
	}

	if (Face.IsDegenerated())
	{
		Face.Remove();
	}
}

void FParametricMesher::ApplyEdgeCriteria(FTopologicalEdge& Edge)
{
	FTopologicalEdge& ActiveEdge = *Edge.GetLinkActiveEdge();

	if (Edge.Length() < Edge.GetTolerance3D())
	{
		Edge.SetAsDegenerated();
	}

	if (ActiveEdge.IsApplyCriteria())
	{
		return;
	}

	Edge.ComputeCrossingPointCoordinates();
	Edge.InitDeltaUs();
	const TArray<double>& CrossingPointUs = Edge.GetCrossingPointUs();

	TArray<double> Coordinates;
	Coordinates.SetNum(CrossingPointUs.Num() * 2 - 1);
	Coordinates[0] = CrossingPointUs[0];
	for (int32 ICuttingPoint = 1; ICuttingPoint < Edge.GetCrossingPointUs().Num(); ICuttingPoint++)
	{
		Coordinates[2 * ICuttingPoint - 1] = (CrossingPointUs[ICuttingPoint - 1] + CrossingPointUs[ICuttingPoint]) * 0.5;
		Coordinates[2 * ICuttingPoint] = CrossingPointUs[ICuttingPoint];
	}

	TArray<FCurvePoint> Points3D;
	Edge.EvaluatePoints(Coordinates, 0, Points3D);

	const TArray<TSharedPtr<FCriterion>>& Criteria = GetMeshModel().GetCriteria();
	for (const TSharedPtr<FCriterion>& Criterion : Criteria)
	{
		Criterion->ApplyOnEdgeParameters(Edge, CrossingPointUs, Points3D);
	}

	Edge.SetApplyCriteriaMarker();
	ActiveEdge.SetApplyCriteriaMarker();
}

void FParametricMesher::Mesh(FTopologicalFace& Face)
{
	FParametricFaceMesher FaceMesher(Face, MeshModel, Tolerances, bThinZoneMeshing);
	FaceMesher.Mesh();
}

void FParametricMesher::IsolateQuadFace(TArray<ParametricMesherTool::FCostToFace>& QuadSurfaces, TArray<FTopologicalFace*>& OtherSurfaces) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParametricMesher::IsolateQuadFace);

	TArray<FTopologicalFace*> FlatQuadsAndTriangles;
	FlatQuadsAndTriangles.Reserve(Faces.Num());
	QuadSurfaces.Reserve(Faces.Num() * 2);
	OtherSurfaces.Reserve(Faces.Num());

	for (FTopologicalFace* FacePtr : Faces)
	{
		if (FacePtr == nullptr)
		{
			continue;
		}

		FTopologicalFace& Face = *FacePtr;
		Face.DefineSurfaceType();
		switch (Face.GetQuadType())
		{
		case EQuadType::Quadrangular:
			double LocalMinCurvature;
			double LocalMaxCurvature;
			FMath::GetMinMax(Face.GetCurvature(EIso::IsoU).Max, Face.GetCurvature(EIso::IsoV).Max, LocalMinCurvature, LocalMaxCurvature);
			if (LocalMaxCurvature > MeshConst::MinCurvature)
			{
				QuadSurfaces.Emplace(LocalMaxCurvature, &Face);
				if (LocalMinCurvature > MeshConst::MinCurvature)
				{
					QuadSurfaces.Emplace(LocalMinCurvature, &Face);
				}
			}
			else
			{
				FlatQuadsAndTriangles.Add(&Face);
				OtherSurfaces.Add(&Face);
			}
			break;
		case EQuadType::Triangular:
			FlatQuadsAndTriangles.Add(&Face);
			OtherSurfaces.Add(&Face);
			break;
		case EQuadType::Unset:
		default:
			OtherSurfaces.Add(&Face);
		}
	}

	Algo::Sort(QuadSurfaces, [](ParametricMesherTool::FCostToFace& SurfaceA, ParametricMesherTool::FCostToFace& SurfaceB)
		{
			return (SurfaceA.Cost > SurfaceB.Cost);
		}
	);
}

void FParametricMesher::LinkQuadSurfaceForMesh(TArray<ParametricMesherTool::FCostToFace>& QuadTrimmedSurfaceSet, TArray<TArray<FTopologicalFace*>>& OutStrips)
{
	const double GeometricTolerance = 20. * MeshModel.GetGeometricTolerance();

	OutStrips.Reserve(QuadTrimmedSurfaceSet.Num());

	for (ParametricMesherTool::FCostToFace& Quad : QuadTrimmedSurfaceSet)
	{
		FTopologicalFace* Face = Quad.Face;
		const FSurfaceCurvature& Curvatures = Face->GetCurvatures();

		EIso Axe = (!RealCompare(Quad.Cost, Curvatures[EIso::IsoU].Max)) ? EIso::IsoU : EIso::IsoV;

		if (Axe == EIso::IsoU)
		{
			if (Face->HasMarker1())
			{
				continue;
			}
			Face->SetMarker1();
		}
		else
		{
			if (Face->HasMarker2())
			{
				continue;
			}
			Face->SetMarker2();
		}

		TArray<FTopologicalFace*>* QuadStrip = &OutStrips.Emplace_GetRef();
		QuadStrip->Reserve(QuadTrimmedSurfaceSet.Num());
		QuadStrip->Add(Face);

		const TArray<FEdge2DProperties>& SideProperties = Face->GetSideProperties();

		int32 StartSideIndex = 0;
		for (; StartSideIndex < 4; StartSideIndex++)
		{
			if (SideProperties[StartSideIndex].IsoType == Axe)
			{
				break;
			}
		}
		if (StartSideIndex == 4)
		{
			continue;
		}

		bool bFirstStep = true;
		int32 SideIndex = StartSideIndex;

		while (Face)
		{
			int32 EdgeIndex = Face->GetStartEdgeIndexOfSide(SideIndex);
			double SideLength = Face->GetSideProperties()[SideIndex].Length3D;
			TSharedPtr<FTopologicalEdge> Edge = Face->GetLoops()[0]->GetEdges()[EdgeIndex].Entity;

			Face = nullptr;
			FTopologicalEdge* NextEdge = Edge->GetFirstTwinEdge();
			if (NextEdge)
			{
				Face = NextEdge->GetLoop()->GetFace();
			}

			if (Face && (Face->GetQuadType() == EQuadType::Quadrangular || Face->GetQuadType() == EQuadType::Triangular))
			{
				// check side length
				int32 LocalEdgeIndex = Face->GetLoops()[0]->GetEdgeIndex(*NextEdge);
				SideIndex = Face->GetSideIndex(LocalEdgeIndex);
				double OtherSideLength = Face->GetSideProperties()[SideIndex].Length3D;

				FMath::GetMinMax(OtherSideLength, SideLength);
				if (SideLength - OtherSideLength > GeometricTolerance)
				{
					Face = nullptr;
				}
			}
			else
			{
				Face = nullptr;
			}

			if (Face)
			{
				// Set as processed in a direction
				const TArray<FEdge2DProperties>& LocalSideProperties = Face->GetSideProperties();
				if (LocalSideProperties[SideIndex].IsoType == EIso::IsoU)
				{
					if (Face->HasMarker1())
					{
						Face = nullptr;
					}
					else
					{
						Face->SetMarker1();
					}
				}
				else
				{
					if (Face->HasMarker2())
					{
						Face = nullptr;
					}
					else
					{
						Face->SetMarker2();
					}
				}
			}

			if (Face)
			{
				// it's a quad or a tri => add
				if (Face->GetQuadType() != EQuadType::Other)
				{
					QuadStrip->Add(Face);
				}

				if (Face->GetQuadType() == EQuadType::Triangular)
				{
					// stop
					Face = nullptr;
				}
			}

			if (!Face)
			{
				if (bFirstStep)
				{
					bFirstStep = false;
					Face = (*QuadStrip)[0];
					SideIndex = (StartSideIndex + 2) % 4;
					continue;
				}
				else
				{
					break;
				}
			}

			// find opposite side
			SideIndex = (SideIndex + 2) % 4;
		}

		if (QuadStrip->Num() == 1)
		{
			OutStrips.Pop();
		}
	}

	for (FTopologicalFace* Face : Faces)
	{
		if (Face == nullptr)
		{
			continue;
		}
		Face->ResetMarkers();
	}
}

void FParametricMesher::MeshSurfaceByFront(TArray<ParametricMesherTool::FCostToFace>& QuadTrimmedSurfaceSet)
{
	// WaitingMarker : Surfaces that have to be meshed are set WaitingMarker
	// Marker1 : Surfaces added in CandidateSurfacesForMesh
	// Marker2 : Surfaces added in SecondChoiceOfCandidateSurfacesForMesh

	for (FTopologicalFace* Face : Faces)
	{
		if (Face == nullptr || Face->IsDeletedOrDegenerated())
		{
			continue;
		}
		Face->SetWaitingMarker();
	}

	const double GeometricTolerance = 20. * MeshModel.GetGeometricTolerance();

	TArray<FTopologicalFace*> CandidateFacesForMesh; // first in first out
	CandidateFacesForMesh.Reserve(100);

	TArray<FTopologicalFace*> SecondChoiceOfCandidateFacesForMesh; // first in first out
	SecondChoiceOfCandidateFacesForMesh.Reserve(100);

	static bool bStop = false;

	TFunction<void(FTopologicalFace&)> MeshFace = [&](FTopologicalFace& Face)
	{
		Mesh(Face);

		if (Face.HasMarker1())
		{
			CandidateFacesForMesh.RemoveSingle(&Face);
		}
		if (Face.HasMarker2())
		{
			SecondChoiceOfCandidateFacesForMesh.RemoveSingle(&Face);
		}

		const TSharedPtr<FTopologicalLoop>& Loop = Face.GetLoops()[0];
		for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
		{
			const FTopologicalEdge& Edge = *OrientedEdge.Entity;
			Edge.SetMarker1();

			for (FTopologicalEdge* NextEdge : Edge.GetTwinEntities())
			{
				if (NextEdge->HasMarker1())
				{
					continue;
				}

				FTopologicalFace* NextFace = NextEdge->GetFace();
				if ((NextFace == nullptr) || !NextFace->IsWaiting())
				{
					// not in the scope of surface to mesh
					continue;
				}

				int32 EdgeIndex;
				int32 LoopIndex;
				NextFace->GetEdgeIndex(*NextEdge, LoopIndex, EdgeIndex);
				if (LoopIndex > 0)
				{
					continue;
				}

				int32 SideIndex = NextFace->GetSideIndex(*NextEdge);
				if (SideIndex == -1)
				{
					// The face is not a quad
					continue;
				}

				FEdge2DProperties& SideProperty = NextFace->GetSideProperty(SideIndex);

				double EdgeLength = NextEdge->Length();
				SideProperty.MeshedLength += EdgeLength;
				NextFace->AddMeshedLength(EdgeLength);
				if ((SideProperty.Length3D - SideProperty.MeshedLength) < GeometricTolerance)
				{
					if (!SideProperty.bIsMesh)
					{
						SideProperty.bIsMesh = true;
						NextFace->MeshedSideNum()++;
					}

					if (!NextFace->HasMarker1())
					{
						NextFace->SetMarker1();
						CandidateFacesForMesh.Add(NextFace);
					}
				}
				else
				{
					if (!NextFace->HasMarker2())
					{
						NextFace->SetMarker2();
						SecondChoiceOfCandidateFacesForMesh.Add(NextFace);
					}
				}
			}
		}
	};

	TFunction<void(FTopologicalFace&)> MeshFacesByFront = [&](FTopologicalFace& Face)
	{
		if (Face.IsNotMeshable())
		{
			return;
		}

		MeshFace(Face);

		while (CandidateFacesForMesh.Num() || SecondChoiceOfCandidateFacesForMesh.Num())
		{
			// the candidate are sorted according to the number of meshed side 
			Algo::Sort(CandidateFacesForMesh, [](FTopologicalFace* Surface1, FTopologicalFace* Surface2) { return Surface1->MeshedSideNum() > Surface2->MeshedSideNum(); });

			int32 IndexOfBestCandidate = -1;
			double CandidateMeshedSideRatio = 0;

			// The first choice will be done in the first set of surface with the max meshed side numbers.
			if (CandidateFacesForMesh.Num())
			{
				int32 MaxMeshedSideNum = CandidateFacesForMesh[0]->MeshedSideNum();

				// next face with side well meshed are preferred
				int32 Index = 0;
				for (; Index < CandidateFacesForMesh.Num(); ++Index)
				{
					FTopologicalFace* CandidateSurface = CandidateFacesForMesh[Index];
					if (CandidateSurface->IsNotMeshable())
					{
						CandidateFacesForMesh.RemoveAt(Index);
						--Index;
						continue;
					}

					if (CandidateSurface->MeshedSideNum() < MaxMeshedSideNum)
					{
						break;
					}

					if (CandidateMeshedSideRatio < CandidateSurface->MeshedSideRatio())
					{
						CandidateMeshedSideRatio = CandidateSurface->MeshedSideRatio();
						IndexOfBestCandidate = Index;
					}
				}

				// if no candidate has been selected, the choice is done on all next surfaces
				if (IndexOfBestCandidate == -1)
				{
					for (; Index < CandidateFacesForMesh.Num(); ++Index)
					{
						FTopologicalFace* CandidateSurface = CandidateFacesForMesh[Index];
						if (CandidateSurface->IsNotMeshable())
						{
							CandidateFacesForMesh.RemoveAt(Index);
							--Index;
							continue;
						}

						if (CandidateMeshedSideRatio < CandidateSurface->MeshedSideRatio())
						{
							CandidateMeshedSideRatio = CandidateSurface->MeshedSideRatio();
							IndexOfBestCandidate = Index;
						}
					}
				}

				if (IndexOfBestCandidate >= 0)
				{
					MeshFace(*CandidateFacesForMesh[IndexOfBestCandidate]);
					continue;
				}
			}

			for (int32 Index = 0; Index < SecondChoiceOfCandidateFacesForMesh.Num(); ++Index)
			{
				FTopologicalFace* CandidateSurface = SecondChoiceOfCandidateFacesForMesh[Index];
				if (CandidateSurface->IsNotMeshable())
				{
					SecondChoiceOfCandidateFacesForMesh.RemoveAt(Index);
					--Index;
					continue;
				}

				if (CandidateMeshedSideRatio < CandidateSurface->MeshedSideRatio())
				{
					CandidateMeshedSideRatio = CandidateSurface->MeshedSideRatio();
					IndexOfBestCandidate = Index;
				}
			}

			if (IndexOfBestCandidate >= 0)
			{
				MeshFace(*SecondChoiceOfCandidateFacesForMesh[IndexOfBestCandidate]);
			}
		}
	};

	// the front is initialized with quad surface
	for (const ParametricMesherTool::FCostToFace& Quad : QuadTrimmedSurfaceSet)
	{
		FTopologicalFace* Face = Quad.Face;
		MeshFacesByFront(*Face);
	}

	// the the other surface
	for (FTopologicalFace* Face : Faces)
	{
		if (Face != nullptr && Face->IsMeshable())
		{
			MeshFacesByFront(*Face);
		}
	}

}

} // namespace