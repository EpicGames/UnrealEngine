// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/Types.h"
#include "Mesh/Meshers/ParametricMesherConstantes.h"

namespace UE::CADKernel
{
class FCriterion;
class FGrid;
class FModelMesh;
class FThinZoneSide;
class FTopologicalEntity;
class FTopologicalEdge;
class FTopologicalFace;
class FTopologicalLoop;
class FTopologicalShapeEntity;

namespace ParametricMesherTool
{

struct FCostToFace
{
	double Cost;
	FTopologicalFace* Face;

	FCostToFace(double NewCost, FTopologicalFace* NewFace)
		: Cost(NewCost)
		, Face(NewFace)
	{
	}
};

}

constexpr double ConstMinCurvature = 0.001;


class CADKERNEL_API FParametricMesher
{
protected:

	FMeshingTolerances Tolerances;
	bool bThinZoneMeshing = false;

	FModelMesh& MeshModel;

	TArray<FTopologicalFace*> Faces;

#ifdef CADKERNEL_DEV
	bool bDisplay = false;
#endif

public:

	FParametricMesher(FModelMesh& InMeshModel, double GeometricTolerance, bool bActivateThinZoneMeshing);

	const FModelMesh& GetMeshModel() const
	{
		return MeshModel;
	}

	FModelMesh& GetMeshModel()
	{
		return MeshModel;
	}

	void MeshEntities(TArray<FTopologicalShapeEntity*>& InEntities);

	void MeshEntity(FTopologicalShapeEntity& InEntity)
	{
		TArray<FTopologicalShapeEntity*> Entities;
		Entities.Add(&InEntity);
		MeshEntities(Entities);
	}

	void Mesh(FTopologicalFace& Face);
	//void Mesh(FTopologicalEdge& InEdge, const FTopologicalFace& CarrierFace, bool bFinalMeshing = true);
	//void Mesh(FTopologicalVertex& Vertex);

	//void MeshFaceLoops(FGrid& Grid);

	//void MeshThinZoneEdges(FTopologicalFace& Face);
	//void MeshThinZoneSide(FThinZoneSide& Side, bool bFinalMeshing);

	//void DefineImposedCuttingPointsBasedOnOtherSideMesh(FTopologicalFace& Face, FThinZoneSide& Side1, FThinZoneSide& Side2, bool Last);

	/**
	 * @return false if the process fails i.e. the grid is degenerated or else. 
	 */
	//bool GenerateCloud(FGrid& Grid);

protected:

	/**
	 * ApplyFaceCriteria, ComputeSurfaceSideProperties
	 */
	void PreMeshingTasks();
	void MeshEntities();


	void IsolateQuadFace(TArray<ParametricMesherTool::FCostToFace>& QuadSurfaces, TArray<FTopologicalFace*>& OtherSurfaces) const;

	void LinkQuadSurfaceForMesh(TArray<ParametricMesherTool::FCostToFace>& QuadTrimmedSurfaceSet, TArray<TArray<FTopologicalFace*>>& OutStrips);
	void MeshSurfaceByFront(TArray<ParametricMesherTool::FCostToFace>& QuadTrimmedSurfaceSet);

	void ApplyEdgeCriteria(FTopologicalEdge& Edge);
	static void ApplyFaceCriteria(FTopologicalFace& Face, const TArray<TSharedPtr<FCriterion>>& Criteria, const double, bool);
};

} // namespace UE::CADKernel

