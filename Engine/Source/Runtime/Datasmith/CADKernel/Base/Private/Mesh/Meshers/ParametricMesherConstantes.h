// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

namespace UE::CADKernel
{
namespace MeshConst
{
constexpr double MinCurvature = 0.001;
constexpr double GeometricToMeshingToleranceFactor = 10.;
constexpr double ElementRatioVsThickness = 20.;

}

class FMeshingTolerances
{
public:
	const double GeometricTolerance;
	const double GeometricToleranceSqrt2;
	const double SquareGeometricTolerance;
	const double SquareGeometricTolerance2;
	const double MeshingTolerance;
	const double SquareMeshingTolerance;

	FMeshingTolerances(double InGeometricTolerance3D)
		: GeometricTolerance(InGeometricTolerance3D)
		, GeometricToleranceSqrt2(InGeometricTolerance3D * UE_DOUBLE_SQRT_2)
		, SquareGeometricTolerance(FMath::Square(GeometricTolerance))
		, SquareGeometricTolerance2(2. * SquareGeometricTolerance)
		, MeshingTolerance(GeometricTolerance * MeshConst::GeometricToMeshingToleranceFactor)
		, SquareMeshingTolerance(FMath::Square(MeshingTolerance))
	{
	}
};

} // namespace UE::CADKernel

