// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/FaceMesh.h"

#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLoop.h"
#include "Topo/TopologicalVertex.h"

namespace UE::CADKernel
{

void FFaceMesh::GetNodeIdToCoordinates(TMap<int32, const FVector*>& NodeIdToCoordinates) const
{
	const FTopologicalFace& Face = (const FTopologicalFace&) GetGeometricEntity();

	for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
	{
		const TArray<FOrientedEdge>& Edges = Loop->GetEdges();

		for (const FOrientedEdge& Edge : Edges)
		{
			TSharedRef<FTopologicalEdge> ActiveEdge = Edge.Entity->GetLinkActiveEdge();
			if (ActiveEdge->IsDegenerated()) 
			{
				continue;
			}

			const FVertexMesh* StartVertexMesh = ActiveEdge->GetStartVertex()->GetMesh();
			if(StartVertexMesh)
			{
				NodeIdToCoordinates.Add(StartVertexMesh->GetMesh(), &StartVertexMesh->GetNodeCoordinates()[0]);
			}

			const FVertexMesh* EndVertexMesh = ActiveEdge->GetEndVertex()->GetMesh();
			if (EndVertexMesh)
			{
				NodeIdToCoordinates.Add(EndVertexMesh->GetMesh(), &EndVertexMesh->GetNodeCoordinates()[0]);
			}

			const FEdgeMesh* EdgeMesh = ActiveEdge->GetMesh();

			TArray<int32> NodeIds = EdgeMesh->EdgeVerticesIndex;
			const FVector* EdgeNodeCoordinates = EdgeMesh->GetNodeCoordinates().GetData();
			if(EdgeNodeCoordinates)
			{
				for (int32 Index = 1; Index <NodeIds.Num() - 1; Index++)
				{
					NodeIdToCoordinates.Add(NodeIds[Index], EdgeNodeCoordinates + Index - 1);
				}
			}
		}
	}

	const FVector* Coordinates = GetNodeCoordinates().GetData();
	for (int32 Index = 0; Index < GetNodeCoordinates().Num(); ++Index)
	{
		NodeIdToCoordinates.Add(StartNodeId+Index, Coordinates + Index);
	}
}

void FFaceMesh::InverseOrientation()
{
	for (int32 Index = 0; Index <TrianglesVerticesIndex.Num(); Index += 3)
	{
		Swap(TrianglesVerticesIndex[Index], TrianglesVerticesIndex[Index+1]);
	}

	for (FVector3f& Normal : Normals)
	{
		Normal *= -1.;
	}
}

}

