// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/Grid.h"

#include "Geo/Sampling/SurfacicSampling.h"
#include "Geo/Surfaces/SurfaceUtilities.h"
#include "Mesh/Meshers/MesherTools.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Utils/Util.h"
#include "Utils/ArrayUtils.h"

namespace UE::CADKernel
{

FGrid::FGrid(FTopologicalFace& InFace, FModelMesh& InMeshModel)
	: FGridBase(InFace)
	, CoordinateGrid(InFace.GetCuttingPointCoordinates())
	, FaceTolerance(InFace.GetIsoTolerances())
	, MinimumElementSize(Tolerance3D * 2.)
	, MeshModel(InMeshModel)
{
}

void FGrid::ProcessPointCloud()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::ProcessPointCloud)

	if (!GetMeshOfLoops())
	{
		return;
	}

	FindInnerFacePoints();

	FindPointsCloseToLoop();

	RemovePointsCloseToLoop();

	// Removed of Thin zone boundary (the last boundaries). In case of thin zone, the number of 2d boundary will be biggest than 3d boundary one.
	// Only EGridSpace::UniformScaled is needed.
	FaceLoops2D[EGridSpace::UniformScaled].SetNum(FaceLoops3D.Num());
}

void FGrid::DefineCuttingParameters()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::DefineCuttingParameters)

	FCuttingGrid PreferredCuttingParametersFromLoops;
	GetPreferredUVCuttingParametersFromLoops(PreferredCuttingParametersFromLoops);

	DefineCuttingParameters(EIso::IsoU, PreferredCuttingParametersFromLoops);
	DefineCuttingParameters(EIso::IsoV, PreferredCuttingParametersFromLoops);

	CuttingSize = CoordinateGrid.Count();
}

void FGrid::DefineCuttingParameters(EIso Iso, FCuttingGrid& Neighbors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::DefineCuttingParameters)

	const FSurfacicBoundary& Boundary = Face.GetBoundary();

	if (Neighbors[Iso].Num())
	{
		FMesherTools::ComputeFinalCuttingPointsWithPreferredCuttingPoints(Face.GetCrossingPointCoordinates(Iso), Face.GetCrossingPointDeltaMaxs(Iso), Neighbors[Iso], Boundary[Iso], CoordinateGrid[Iso]);
	}
	else
	{
		TArray<FCuttingPoint> Extremities;
		Extremities.Reserve(2);
		Extremities.Emplace(Boundary[Iso].Min, ECoordinateType::VertexCoordinate, -1, 0.001);
		Extremities.Emplace(Boundary[Iso].Max, ECoordinateType::VertexCoordinate, -1, 0.001);
		FMesherTools::ComputeFinalCuttingPointsWithImposedCuttingPoints(Face.GetCrossingPointCoordinates(Iso), Face.GetCrossingPointDeltaMaxs(Iso), Extremities, CoordinateGrid[Iso]);
	}

	// #cadkernel_check: Why does this only apply to planar surfaces?
	FSurface& CarrierSurface = *Face.GetCarrierSurface();
	if (SurfaceUtilities::IsPlanar(CarrierSurface))
	{
		FCoordinateGrid FaceNotDerivableCoordinates;
		CarrierSurface.LinesNotDerivables(Face.GetBoundary(), 1, FaceNotDerivableCoordinates);

		ArrayUtils::Complete(CoordinateGrid[Iso], FaceNotDerivableCoordinates[Iso], CarrierSurface.GetIsoTolerance(Iso));
	}

	CuttingCount[Iso] = CoordinateGrid.IsoCount(Iso);
}

void FGrid::GetPreferredUVCuttingParametersFromLoops(FCuttingGrid& CuttingParametersFromLoops)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::GetPreferredUVCuttingParametersFromLoops)

	int32 nbPoints = 0;
	for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			nbPoints += Edge.Entity->GetOrCreateMesh(MeshModel).GetNodeCoordinates().Num() + 1;
		}
	}

	CuttingParametersFromLoops[EIso::IsoU].Reserve(nbPoints);
	CuttingParametersFromLoops[EIso::IsoV].Reserve(nbPoints);

	for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
	{
		for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
		{
			const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;

			TArray<double> ProjectedPointCoords;
			TSharedRef<FTopologicalEdge> ActiveEdge = Edge->GetLinkActiveEdge();
			if (ActiveEdge->IsMeshed())
			{
				const FMesh* EdgeMesh = ActiveEdge->GetMesh();
				if (!EdgeMesh)
				{
					continue;
				}

				const TArray<FVector>& EdgeMeshNodes = EdgeMesh->GetNodeCoordinates();
				if (EdgeMeshNodes.Num() == 0)
				{
					continue;
				}

				ProjectedPointCoords.Reserve(EdgeMeshNodes.Num() + 2);
				bool bSameDirection = Edge->IsSameDirection(*ActiveEdge);

				Edge->ProjectTwinEdgePoints(EdgeMeshNodes, bSameDirection, ProjectedPointCoords);
				ProjectedPointCoords.Insert(Edge->GetStartCurvilinearCoordinates(), 0);
				ProjectedPointCoords.Add(Edge->GetEndCurvilinearCoordinates());
			}
			else // Add Vertices
			{
				ProjectedPointCoords.Add(Edge->GetBoundary().GetMin());
				ProjectedPointCoords.Add(Edge->GetBoundary().GetMax());
			}

			TArray<FVector2d> EdgePoints2D;
			Edge->Approximate2DPoints(ProjectedPointCoords, EdgePoints2D);

			for (int32 Index = 0; Index < EdgePoints2D.Num(); ++Index)
			{
				CuttingParametersFromLoops[EIso::IsoU].Emplace(EdgePoints2D[Index].X, ECoordinateType::OtherCoordinate);
				CuttingParametersFromLoops[EIso::IsoV].Emplace(EdgePoints2D[Index].Y, ECoordinateType::OtherCoordinate);
			}
		}
	}

	TFunction<void(TArray<FCuttingPoint>&, double)> SortAndRemoveDuplicated = [](TArray<FCuttingPoint>& Neighbours, double Tolerance)
	{
		if (Neighbours.Num() == 0)
		{
			return;
		}

		Algo::Sort(Neighbours, [](const FCuttingPoint& Point1, const FCuttingPoint& Point2) -> bool
			{
				return (Point1.Coordinate) < (Point2.Coordinate);
			}
		);

		int32 NewIndex = 0;
		for (int32 Index = 1; Index < Neighbours.Num() - 1; ++Index)
		{
			if (FMath::IsNearlyEqual(Neighbours[Index].Coordinate, Neighbours[NewIndex].Coordinate, Tolerance))
			{
				continue;
			}
			NewIndex++;
			Neighbours[NewIndex] = Neighbours[Index];
		}
		if (FMath::IsNearlyEqual(Neighbours.Last().Coordinate, Neighbours[NewIndex].Coordinate, Tolerance))
		{
			Neighbours[NewIndex] = Neighbours.Last();
		}
		else
		{
			NewIndex++;
		}
		Neighbours.SetNum(NewIndex);
	};

	SortAndRemoveDuplicated(CuttingParametersFromLoops[EIso::IsoU], FaceTolerance[EIso::IsoU]);
	SortAndRemoveDuplicated(CuttingParametersFromLoops[EIso::IsoV], FaceTolerance[EIso::IsoV]);
}

bool FGrid::GeneratePointCloud()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::GeneratePointCloud)

	if (CheckIf2DGridIsDegenerate())
	{
		return false;
	}

	NodeMarkers.Init(ENodeMarker::None, CuttingSize);

	EvaluatePointGrid(CoordinateGrid, true);

	CountOfInnerNodes = CuttingSize;

	if (!ScaleGrid())
	{
		return false;
	}

	return true;
}

void FGrid::FindPointsCloseToLoop()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::FindPointsCloseToLoop)

	int32 IndexLoop = 0;
	int32 Index[2] = { 1, 1 };
	int32 GlobalIndex = 1;

	const FVector2d* PointA = nullptr;
	const FVector2d* PointB = nullptr;

	// Find start index
	TFunction<void(EIso)> FindPointAIndex = [&](EIso Iso)
	{
		Index[Iso] = 1;
		for (; Index[Iso] < CuttingCount[Iso] - 1; ++Index[Iso])
		{
			if (UniformCuttingCoordinates[Iso][Index[Iso]] + DOUBLE_SMALL_NUMBER > (*PointA)[Iso])
			{
				break;
			}
		}
	};

	TFunction<void()> SetCellCloseToLoop = [&]()
	{
		SetCloseToLoop(GlobalIndex);
		SetCloseToLoop(GlobalIndex - 1);
		SetCloseToLoop(GlobalIndex - 1 - CuttingCount[EIso::IsoU]);
		SetCloseToLoop(GlobalIndex - CuttingCount[EIso::IsoU]);
	};

	TFunction<void(EIso)> Increase = [&](EIso Iso)
	{
		if (Index[Iso] < CuttingCount[Iso] - 1)
		{
			Index[Iso]++;
			GlobalIndex += Iso == EIso::IsoU ? 1 : CuttingCount[EIso::IsoU];
		}
	};

	TFunction<void(EIso)> Decrease = [&](EIso Iso)
	{
		if (Index[Iso] > 1) //-V547
		{
			Index[Iso]--;
			GlobalIndex -= Iso == EIso::IsoU ? 1 : CuttingCount[EIso::IsoU];
		}
	};

	TFunction<void()> IncreaseU = [&]()
	{
		Increase(EIso::IsoU);
	};

	TFunction<void()> IncreaseV = [&]()
	{
		Increase(EIso::IsoV);
	};

	TFunction<void()> DecreaseU = [&]()
	{
		Decrease(EIso::IsoU);
	};

	TFunction<void()> DecreaseV = [&]()
	{
		Decrease(EIso::IsoV);
	};

	TFunction<bool(const double, const double)> IsReallyBigger = [](const double FirstValue, const double SecondValue) ->bool
	{
		return FirstValue - DOUBLE_SMALL_NUMBER > SecondValue;
	};

	TFunction<bool(const double, const double)> IsReallySmaller = [](const double FirstValue, const double SecondValue) ->bool
	{
		return FirstValue + DOUBLE_SMALL_NUMBER < SecondValue;
	};

	double Slope;
	double Origin;

	TFunction<void(EIso, const int32, const int32, TFunction<void()>, TFunction<void()>)> FindIntersection = [&](EIso MainIso, const int32 DeltaIso, const int32 DeltaOther, TFunction<void()> OffsetIndexIfBigger, TFunction<void()> OffsetIndexIfSmaller)
	{
		TFunction<bool(const double, const double)> TestAlongIso = DeltaIso ? IsReallyBigger : IsReallySmaller;
		TFunction<bool(const double, const double)> TestAlongOther = DeltaOther ? IsReallyBigger : IsReallySmaller;

		EIso OtherIso = Other(MainIso);
		while (TestAlongIso(UniformCuttingCoordinates[MainIso][Index[MainIso] - DeltaIso], (*PointB)[MainIso]) || TestAlongOther(UniformCuttingCoordinates[OtherIso][Index[OtherIso] - DeltaOther], (*PointB)[OtherIso]))
		{
			double CoordinateOther = Slope * UniformCuttingCoordinates[MainIso][Index[MainIso] - DeltaIso] + Origin;
			if (IsReallyBigger(CoordinateOther, UniformCuttingCoordinates[OtherIso][Index[OtherIso] - DeltaOther]))
			{
				OffsetIndexIfBigger();
			}
			else if (IsReallySmaller(CoordinateOther, UniformCuttingCoordinates[OtherIso][Index[OtherIso] - DeltaOther]))
			{
				OffsetIndexIfSmaller();
			}
			else // IsNearlyEqual
			{
				OffsetIndexIfBigger();
				OffsetIndexIfSmaller();
			}
			SetCellCloseToLoop();
		}
	};

	TFunction<bool(EIso)> FindIntersectionIsoStrip = [&](EIso MainIso) -> bool
	{
		EIso OtherIso = Other(MainIso);
		if ((UniformCuttingCoordinates[OtherIso][Index[OtherIso] - 1] < (*PointB)[OtherIso]) && ((*PointB)[OtherIso] < UniformCuttingCoordinates[OtherIso][Index[OtherIso]]))
		{
			if ((UniformCuttingCoordinates[MainIso][Index[MainIso] - 1] < (*PointB)[MainIso]) && ((*PointB)[MainIso] < UniformCuttingCoordinates[MainIso][Index[MainIso]]))
			{
				PointA = PointB;
				return true;
			}

			if ((*PointA)[MainIso] < (*PointB)[MainIso])
			{
				while (UniformCuttingCoordinates[MainIso][Index[MainIso]] < (*PointB)[MainIso])
				{
					Increase(MainIso);
					SetCellCloseToLoop();
				}
			}
			else
			{
				while (UniformCuttingCoordinates[MainIso][Index[MainIso] - 1] > (*PointB)[MainIso])
				{
					Decrease(MainIso);
					SetCellCloseToLoop();
				}
			}
			PointA = PointB;
			return true;
		}
		return false;
	};

	double ABv = 0.;
	double ABu = 0.;
	for (const TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
	{
		PointA = &Loop.Last();

		// Find start index
		FindPointAIndex(EIso::IsoU);
		FindPointAIndex(EIso::IsoV);

		GlobalIndex = Index[EIso::IsoV] * CuttingCount[EIso::IsoU] + Index[EIso::IsoU];
		SetCellCloseToLoop();

		for (int32 BIndex = 0; BIndex < Loop.Num(); ++BIndex)
		{
			PointB = &Loop[BIndex];

			// Horizontal case
			if (FindIntersectionIsoStrip(EIso::IsoU))
			{
				continue;
			}

			if (FindIntersectionIsoStrip(EIso::IsoV))
			{
				continue;
			}

			ABv = PointB->Y - PointA->Y;
			ABu = PointB->X - PointA->X;
			if (FMath::Abs(ABu) > FMath::Abs(ABv))
			{
				Slope = ABv / ABu;
				Origin = PointA->Y - Slope * PointA->X;
				if (ABu > 0)
				{
					if (ABv > 0)
					{
						FindIntersection(EIso::IsoU, 0, 0, IncreaseV, IncreaseU);
					}
					else
					{
						FindIntersection(EIso::IsoU, 0, 1, IncreaseU, DecreaseV);
					}
				}
				else
				{
					if (ABv > 0)
					{
						FindIntersection(EIso::IsoU, 1, 0, IncreaseV, DecreaseU);
					}
					else
					{
						FindIntersection(EIso::IsoU, 1, 1, DecreaseU, DecreaseV);
					}
				}
			}
			else
			{
				Slope = ABu / ABv;
				Origin = PointA->X - Slope * PointA->Y;
				if (ABu > 0)
				{
					if (ABv > 0)
					{
						FindIntersection(EIso::IsoV, 0, 0, IncreaseU, IncreaseV);
					}
					else
					{
						FindIntersection(EIso::IsoV, 1, 0, IncreaseU, DecreaseV);
					}
				}
				else
				{
					if (ABv > 0)
					{
						FindIntersection(EIso::IsoV, 0, 1, IncreaseV, DecreaseU);
					}
					else
					{
						FindIntersection(EIso::IsoV, 1, 1, DecreaseV, DecreaseU);
					}
				}
			}
			PointA = PointB;
		}
	}
}

void FGrid::RemovePointsCloseToLoop()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::RemovePointsCloseToLoop)

	struct FGridSegment
	{
		const FVector2d* StartPoint;
		const FVector2d* EndPoint;
		double StartPointWeight;
		double EndPointWeight;
		double UMin;
		double VMin;
		double UMax;
		double VMax;

		FGridSegment(const FVector2d& SPoint, const FVector2d& EPoint)
			: StartPoint(&SPoint)
			, EndPoint(&EPoint)
		{
			StartPointWeight = StartPoint->X + StartPoint->Y;
			EndPointWeight = EndPoint->X + EndPoint->Y;
			if (StartPointWeight > EndPointWeight)
			{
				Swap(StartPointWeight, EndPointWeight);
				Swap(StartPoint, EndPoint);
			}
			if (StartPoint->X < EndPoint->X)
			{
				UMin = StartPoint->X;
				UMax = EndPoint->X;
			}
			else
			{
				UMin = EndPoint->X;
				UMax = StartPoint->X;
			}
			if (StartPoint->Y < EndPoint->Y)
			{
				VMin = StartPoint->Y;
				VMax = EndPoint->Y;
			}
			else
			{
				VMin = EndPoint->Y;
				VMax = StartPoint->Y;
			}
		}
	};

	TArray<FGridSegment> LoopSegments;
	{
		int32 SegmentNum = 0;
		for (const TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
		{
			SegmentNum += Loop.Num();
		}
		LoopSegments.Reserve(SegmentNum);

		for (TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
		{
			for (int32 Index = 0; Index < Loop.Num() - 1; ++Index)
			{
				LoopSegments.Emplace(Loop[Index], Loop[Index + 1]);
			}
			LoopSegments.Emplace(Loop.Last(), Loop[0]);
		}

		Algo::Sort(LoopSegments, [](const FGridSegment& Seg1, const FGridSegment& Seg2) -> bool
			{
				return (Seg1.EndPointWeight) < (Seg2.EndPointWeight);
			}
		);
	}

	// Sort point border grid
	TArray<double> GridPointWeight;
	TArray<int32> IndexOfPointsNearAndInsideLoop;
	TArray<int32> SortedPointIndexes;
	{
		IndexOfPointsNearAndInsideLoop.Reserve(CuttingSize);
		for (int32 Index = 0; Index < CuttingSize; ++Index)
		{
			if (IsNodeInsideAndCloseToLoop(Index))
			{
				IndexOfPointsNearAndInsideLoop.Add(Index);
			}
		}

		GridPointWeight.Reserve(IndexOfPointsNearAndInsideLoop.Num());
		SortedPointIndexes.Reserve(IndexOfPointsNearAndInsideLoop.Num());
		for (const int32& Index : IndexOfPointsNearAndInsideLoop)
		{
			GridPointWeight.Add(Points2D[EGridSpace::UniformScaled][Index].X + Points2D[EGridSpace::UniformScaled][Index].Y);
		}
		for (int32 Index = 0; Index < IndexOfPointsNearAndInsideLoop.Num(); ++Index)
		{
			SortedPointIndexes.Add(Index);
		}
		SortedPointIndexes.Sort([&GridPointWeight](const int32& Index1, const int32& Index2) { return GridPointWeight[Index1] < GridPointWeight[Index2]; });
	}

	// only used to reduce the search of neighborhood
	double DeltaUVMax = 0;
	{
		double MaxDeltaU = 0;
		for (int32 Index = 0; Index < CuttingCount[EIso::IsoU] - 1; ++Index)
		{
			MaxDeltaU = FMath::Max(MaxDeltaU, FMath::Abs(UniformCuttingCoordinates[EIso::IsoU][Index + 1] - UniformCuttingCoordinates[EIso::IsoU][Index]));
		}

		double MaxDeltaV = 0;
		for (int32 Index = 0; Index < CuttingCount[EIso::IsoV] - 1; ++Index)
		{
			MaxDeltaV = FMath::Max(MaxDeltaV, FMath::Abs(UniformCuttingCoordinates[EIso::IsoV][Index + 1] - UniformCuttingCoordinates[EIso::IsoV][Index]));
		}
		DeltaUVMax = FMath::Max(MaxDeltaU, MaxDeltaV);
	}

	// Find DeltaU and DeltaV around a cutting point defined by its index
	TFunction<void(const int32, double&, double&)> GetDeltaUV = [&](const int32 Index, double& DeltaU, double& DeltaV)
	{
		int32 IndexU = Index % CuttingCount[EIso::IsoU];
		int32 IndexV = Index / CuttingCount[EIso::IsoU];

		if (IndexU == 0)
		{
			DeltaU = FMath::Abs(UniformCuttingCoordinates[EIso::IsoU][1] - UniformCuttingCoordinates[EIso::IsoU][0]);
		}
		else if (IndexU == CuttingCount[EIso::IsoU] - 1)
		{
			DeltaU = FMath::Abs(UniformCuttingCoordinates[EIso::IsoU][CuttingCount[EIso::IsoU] - 1] - UniformCuttingCoordinates[EIso::IsoU][CuttingCount[EIso::IsoU] - 2]);
		}
		else
		{
			DeltaU = FMath::Abs(UniformCuttingCoordinates[EIso::IsoU][IndexU + 1] - UniformCuttingCoordinates[EIso::IsoU][IndexU - 1]) * .5;
		}

		if (IndexV == 0)
		{
			DeltaV = FMath::Abs(UniformCuttingCoordinates[EIso::IsoV][1] - UniformCuttingCoordinates[EIso::IsoV][0]);
		}
		else if (IndexV == CuttingCount[EIso::IsoV] - 1)
		{
			DeltaV = FMath::Abs(UniformCuttingCoordinates[EIso::IsoV][CuttingCount[EIso::IsoV] - 1] - UniformCuttingCoordinates[EIso::IsoV][CuttingCount[EIso::IsoV] - 2]);
		}
		else
		{
			DeltaV = FMath::Abs(UniformCuttingCoordinates[EIso::IsoV][IndexV + 1] - UniformCuttingCoordinates[EIso::IsoV][IndexV - 1]) * .5;
		}
	};

	int32 SegmentIndex = 0;
	SegmentIndex = 0;
	for (const int32& SortedIndex : SortedPointIndexes)
	{
		int32 Index = IndexOfPointsNearAndInsideLoop[SortedIndex];
		const FVector2d& Point2D = Points2D[EGridSpace::UniformScaled][Index];

		double DeltaU;
		double DeltaV;
		GetDeltaUV(Index, DeltaU, DeltaV);

		//find the first segment that could be close to the point			
		for (; SegmentIndex < LoopSegments.Num(); ++SegmentIndex)
		{
			if (GridPointWeight[SortedIndex] < LoopSegments[SegmentIndex].EndPointWeight + DeltaUVMax)
			{
				break;
			}
		}

		for (int32 SegmentIndex2 = SegmentIndex; SegmentIndex2 < LoopSegments.Num(); ++SegmentIndex2)
		{
			const FGridSegment& Segment = LoopSegments[SegmentIndex2];

			if (GridPointWeight[SortedIndex] < Segment.StartPointWeight - DeltaUVMax)
			{
				continue;
			}
			if (Point2D.X + DeltaU < Segment.UMin)
			{
				continue;
			}
			if (Point2D.X - DeltaU > Segment.UMax)
			{
				continue;
			}
			if (Point2D.Y + DeltaV < Segment.VMin)
			{
				continue;
			}
			if (Point2D.Y - DeltaV > Segment.VMax)
			{
				continue;
			}

			double Coordinate;
			FVector2d Projection = ProjectPointOnSegment<FVector2d>(Point2D, *Segment.StartPoint, *Segment.EndPoint, Coordinate, /*bRestrictCoodinateToInside*/ true);

			// If Projected point is in the oval center on Point2D => the node is too close
			FVector2d ProjectionToPoint = (Point2D - Projection).GetAbs();

			if (ProjectionToPoint.X > DeltaU * 0.05 || ProjectionToPoint.Y > DeltaV * 0.05)
			{
				continue;
			}

			SetTooCloseToLoop(Index);
			break;
		}
	}
}

/**
 * For the surface normal at a StartPoint of the 3D degenerated curve (Not degenerated in 2d)
 * The normal is swap if StartPoint is too close to the Boundary
 * The norm of the normal is defined as 1/20 of the parallel boundary Length
 */
void ScaleAndSwap(FVector2d& Normal, const FVector2d& StartPoint, const FSurfacicBoundary& Boundary)
{
	Normal.Normalize();
	FVector2d MainDirection = Normal;
	MainDirection.X /= Boundary[EIso::IsoU].Length();
	MainDirection.Y /= Boundary[EIso::IsoV].Length();

	TFunction<void(const EIso)> SwapAndScale = [&](const EIso Iso)
	{
		if (MainDirection[Iso] > 0)
		{
			if (FMath::IsNearlyEqual(Boundary[Iso].Max, StartPoint[Iso]))
			{
				Normal *= -1.;
			}
		}
		else
		{
			if (FMath::IsNearlyEqual(Boundary[IsoU].Min, StartPoint[Iso]))
			{
				Normal *= -1.;
			}
		}
		Normal *= Boundary[Iso].Length() / 20;
	};

	if (MainDirection.X > MainDirection.Y)
	{
		SwapAndScale(EIso::IsoU);
	}
	else
	{
		SwapAndScale(EIso::IsoV);
	}
}

/**
 * Displace loop nodes inside to avoid that the nodes are outside the surface Boundary, so outside the grid
 */
void SlightlyDisplacedPolyline(TArray<FVector2d>& D2Points, const FSurfacicBoundary& Boundary)
{
	FVector2d Normal;
	for (int32 Index = 0; Index < D2Points.Num() - 1; ++Index)
	{
		const FVector2d Tangent = D2Points[Index + 1] - D2Points[Index];
		Normal = FVector2d(-Tangent.Y, Tangent.X);

		ScaleAndSwap(Normal, D2Points[Index], Boundary);
		D2Points[Index] += Normal;
	}
	D2Points.Last() += Normal;
}

void FGrid::GetMeshOfLoop(const FTopologicalLoop& Loop)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::GetMeshOfLoop)

	int32 LoopNodeCount = 0;

	for (const FOrientedEdge& Edge : Loop.GetEdges())
	{
		LoopNodeCount += Edge.Entity->GetLinkActiveEdge()->GetMesh()->GetNodeCount() + 2;
	}

	TArray<FVector2d>& Loop2D = FaceLoops2D[EGridSpace::Default2D].Emplace_GetRef();
	Loop2D.Empty(LoopNodeCount);

	TArray<FVector>& Loop3D = FaceLoops3D.Emplace_GetRef();
	Loop3D.Reserve(LoopNodeCount);

	TArray<FVector3f>& LoopNormals = NormalsOfFaceLoops.Emplace_GetRef();
	LoopNormals.Reserve(LoopNodeCount);

	TArray<int32>& LoopIds = NodeIdsOfFaceLoops.Emplace_GetRef();
	LoopIds.Reserve(LoopNodeCount);

	for (const FOrientedEdge& OrientedEdge : Loop.GetEdges())
	{
		const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;
		const TSharedRef<FTopologicalEdge>& ActiveEdge = Edge->GetLinkActiveEdge();

		bool bSameDirection = Edge->IsSameDirection(*ActiveEdge);

		TArray<double> EdgeCuttingPointCoordinates;
		{
			const TArray<FCuttingPoint>& CuttingPoints = Edge->GetCuttingPoints();
			if (!CuttingPoints.IsEmpty())
			{
				GetCuttingPointCoordinates(CuttingPoints, EdgeCuttingPointCoordinates);
			}
		}

		FSurfacicPolyline CuttingPolyline(true);

		if (Edge->IsDegenerated())
		{
			if (EdgeCuttingPointCoordinates.IsEmpty())
			{
				int32 CuttingPointCount = 2;
				for (const FTopologicalEdge* TwinEdge : Edge->GetTwinEntities())
				{
					int32 TwinCuttingPointCount = TwinEdge->GetCuttingPoints().Num();
					if (TwinCuttingPointCount > CuttingPointCount)
					{
						CuttingPointCount = TwinCuttingPointCount;
					}
				}
				FMesherTools::FillCuttingPointCoordinates(Edge->GetBoundary(), CuttingPointCount, EdgeCuttingPointCoordinates);
			}

			Swap(CuttingPolyline.Coordinates, EdgeCuttingPointCoordinates);
			Edge->Approximate2DPoints(CuttingPolyline.Coordinates, CuttingPolyline.Points2D);

			CuttingPolyline.Points3D.Init(ActiveEdge->GetStartBarycenter(), CuttingPolyline.Coordinates.Num());

			TArray<FVector2d> D2Points = CuttingPolyline.Points2D;
			const FSurfacicBoundary& Boundary = Edge->GetCurve()->GetCarrierSurface()->GetBoundary();
			// to compute the normals, the 2D points are slightly displaced perpendicular to the curve
			SlightlyDisplacedPolyline(D2Points, Boundary);
			Edge->GetCurve()->GetCarrierSurface()->EvaluateNormals(D2Points, CuttingPolyline.Normals);
		}
		else
		{
			if (EdgeCuttingPointCoordinates.IsEmpty())
			{
				const TArray<FVector>& MeshVertex3D = ActiveEdge->GetOrCreateMesh(MeshModel).GetNodeCoordinates();
				TArray<double> ProjectedPointCoords;

				CuttingPolyline.Coordinates.Reserve(MeshVertex3D.Num() + 2);
				if(MeshVertex3D.Num())
				{
					Edge->ProjectTwinEdgePoints(MeshVertex3D, bSameDirection, CuttingPolyline.Coordinates);
				}
				CuttingPolyline.Coordinates.Insert(Edge->GetBoundary().GetMin(), 0);
				CuttingPolyline.Coordinates.Add(Edge->GetBoundary().GetMax());

				// check if there are coincident coordinates
				bool bProjectionFailed = false;
				for (int32 Index = 0; Index < CuttingPolyline.Coordinates.Num() - 1;)
				{
					double Diff = CuttingPolyline.Coordinates[Index++];
					Diff -= CuttingPolyline.Coordinates[Index];
					if (Diff > -DOUBLE_SMALL_NUMBER)
					{
						bProjectionFailed = true;
						break;
					}
				}
				if (bProjectionFailed)
				{
					FMesherTools::FillCuttingPointCoordinates(Edge->GetBoundary(), MeshVertex3D.Num() + 2, CuttingPolyline.Coordinates);
				}
			}
			else
			{
				Swap(CuttingPolyline.Coordinates, EdgeCuttingPointCoordinates);
			}

			// #cadkernel_check: Why does this only applies to planar surfaces
			const FSurface& CarrierSurface = *OrientedEdge.Entity->GetLoop()->GetFace()->GetCarrierSurface();
			if (SurfaceUtilities::IsPlanar(CarrierSurface))
			{
				// Make sure 'not derivable coordinates' are part of the polyline's coordinates
				TSharedPtr<FRestrictionCurve> Curve = OrientedEdge.Entity->GetCurve();
				TArray<double> NotDerivableCoordinates;
				Curve->FindNotDerivableCoordinates(Curve->GetBoundary(), 1, NotDerivableCoordinates);
				ArrayUtils::Complete(CuttingPolyline.Coordinates, NotDerivableCoordinates, Curve->GetMinLinearTolerance());

				// Remove duplicates
				TArray<double> CachedCoordinates(CuttingPolyline.Coordinates);
				ArrayUtils::RemoveDuplicates(CuttingPolyline.Coordinates, Curve->GetMinLinearTolerance());
				ensureCADKernel(CuttingPolyline.Coordinates.Num() > 1);
			}
			
			Edge->ApproximatePolyline(CuttingPolyline);
		}

		TArray<int32> EdgeVerticesIndex;
		if (Edge->IsDegenerated())
		{
			EdgeVerticesIndex.Init(ActiveEdge->GetStartVertex()->GetLinkActiveEntity()->GetOrCreateMesh(MeshModel).GetMesh(), CuttingPolyline.Coordinates.Num());
			if (OrientedEdge.Direction == Front)
			{
				EdgeVerticesIndex[0] = ActiveEdge->GetEndVertex()->GetLinkActiveEntity()->GetOrCreateMesh(MeshModel).GetMesh();
			}
			else
			{
				EdgeVerticesIndex.Last() = ActiveEdge->GetEndVertex()->GetLinkActiveEntity()->GetOrCreateMesh(MeshModel).GetMesh();
			}
		}
		else if (Edge->IsVirtuallyMeshed())
		{
			// @see FParametricMesher::Mesh(FTopologicalEdge& InEdge, FTopologicalFace& Face)
			int32 NodeCount = CuttingPolyline.Coordinates.Num();
			EdgeVerticesIndex.Reserve(NodeCount);
			int32 MiddleNodeIndex = NodeCount / 2;
			int32 Index = 0;
			{
				int32 StartVertexMeshId = Edge->GetStartVertex()->GetLinkActiveEntity()->GetOrCreateMesh(MeshModel).GetMesh();
				for (; Index < MiddleNodeIndex; ++Index)
				{
					EdgeVerticesIndex.Add(StartVertexMeshId);
				}
			}
			{
				int32 EndVertexMeshId = Edge->GetEndVertex()->GetLinkActiveEntity()->GetOrCreateMesh(MeshModel).GetMesh();
				for (; Index < NodeCount; ++Index)
				{
					EdgeVerticesIndex.Add(EndVertexMeshId);
				}
			}
		}
		else
		{
			EdgeVerticesIndex = ActiveEdge->GetOrCreateMesh(MeshModel).EdgeVerticesIndex;

			// #cadkernel_check: Adding the 'not derivable coordinates may introduce new points
			//		Whys is EdgeVerticesIndex taken from the ActiveEdge?
			//		Should the EdgeVerticesIndex be regenerated?
			int32 LastID = EdgeVerticesIndex.Last();
			while (EdgeVerticesIndex.Num() < CuttingPolyline.Size())
			{
				EdgeVerticesIndex.Add(LastID);
			}
		}

		if (OrientedEdge.Direction != EOrientation::Front)
		{
			CuttingPolyline.Reverse();
		}

		if (bSameDirection != (OrientedEdge.Direction == EOrientation::Front))
		{
			Algo::Reverse(EdgeVerticesIndex);
		}

		ensureCADKernel(CuttingPolyline.Size() > 1);

		Loop2D.Append(MoveTemp(CuttingPolyline.Points2D));
		// Ignore last added vertex as it is equal to first of next edge in loop
		Loop2D.SetNum(Loop2D.Num() - 1, EAllowShrinking::No); 

		int32 LastIndex = Loop3D.Num();
		Loop3D.Append(MoveTemp(CuttingPolyline.Points3D));
		Loop3D[LastIndex] = (ActiveEdge->GetStartVertex((OrientedEdge.Direction == EOrientation::Front) == bSameDirection)->GetLinkActiveEntity()->GetBarycenter());
		// Ignore last added vertex as it is equal to first of next edge in loop
		Loop3D.SetNum(Loop3D.Num() - 1, EAllowShrinking::No);

		LoopNormals.Append(MoveTemp(CuttingPolyline.Normals));
		// Ignore last added normal as it is equal to first of next edge in loop
		LoopNormals.SetNum(LoopNormals.Num() - 1, EAllowShrinking::No);

		LoopIds.Append(MoveTemp(EdgeVerticesIndex));
		// Ignore last added index as it is equal to first of next edge in loop
		LoopIds.SetNum(LoopIds.Num() - 1, EAllowShrinking::No);
	}

	if (Loop2D.Num() < 3) // degenerated loop
	{
		FaceLoops2D[EGridSpace::Default2D].Pop();
		FaceLoops3D.Pop();
		NormalsOfFaceLoops.Pop();
		NodeIdsOfFaceLoops.Pop();
	}
}

void FGrid::GetMeshOfThinZone(const FThinZone2D& ThinZone)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::GetMeshOfThinZone)

	// ThinZones are identified during "ApplyCriteria". "ApplyCriteria" use the FCriteriaGrid, The EGridSpace::UniformScaled are not the same between FCriteriaGrid and FGrid
	// This is the reason why, to get the ThinZone in the FGrid UniformScaled space, we need to get the mesh of the thinZone defined by the node ids, 
	// And then, from this, we retrieve the thinZone points in Grid UniformScaled space.

	// Return the mesh defined by the node index
	TFunction<void(const FThinZoneSide&, TArray<int32>&)> GetThinZoneSideMesh = [this](const FThinZoneSide& Side, TArray<int32>& MeshIndices)
	{
		FAddMeshNodeFunc AddMeshNode = [&MeshIndices](int32 NodeIndice, const FVector2d& MeshNode2D, double MeshingTolerance, const FEdgeSegment& EdgeSegment, const FPairOfIndex& OppositeNodeIndices)
		{
			if (MeshIndices.IsEmpty() || MeshIndices.Last() != NodeIndice)
			{
				MeshIndices.Add(NodeIndice);
			}
		};

		FReserveContainerFunc Reserve = [&MeshIndices](int32 MeshVertexCount)
		{
			MeshIndices.Reserve(MeshIndices.Num() + MeshVertexCount);
		};
		const bool bWithTolerance = false;
		Side.GetExistingMeshNodes(GetFace(), MeshModel, Reserve, AddMeshNode, bWithTolerance);
	};

	TFunction<bool(int32, int32&, int32&)> FindLoopAndNodeIndex = [&NodeIdsOfLoops = NodeIdsOfFaceLoops](int32 NodeId, int32& LoopIndex, int32& LoopNodeIndex)
	{
		bool bFind = false;
		for (LoopIndex = 0; LoopIndex < NodeIdsOfLoops.Num(); ++LoopIndex)
		{
			TArray<int32>& NodeIdsOfLoop = NodeIdsOfLoops[LoopIndex];
			for (LoopNodeIndex = 0; LoopNodeIndex < NodeIdsOfLoop.Num(); ++LoopNodeIndex)
			{
				if (NodeIdsOfLoop[LoopNodeIndex] == NodeId)
				{
					return true;
				}
			}
		}
		LoopIndex = -1;
		LoopNodeIndex = -1;
		return false;
	};

	TFunction<void(const TArray<int32>&, TArray<FVector2d>&)> GetThinZoneMeshCoordinates = [&NodeIdsOfLoops = NodeIdsOfFaceLoops, &Loops2D = FaceLoops2D, &FindLoopAndNodeIndex](const TArray<int32>& ThinZoneNodeIds, TArray<FVector2d>& ThinZonePoints)
	{
		ThinZonePoints.Reset(ThinZoneNodeIds.Num());

		int32 LoopIndex = 0;
		int32 LoopNodeIndex = 0;

		for (int32 NodeId : ThinZoneNodeIds)
		{
			if (LoopIndex >= 0)
			{
				const int32 NextLoopNodeIndex = (LoopNodeIndex == NodeIdsOfLoops[LoopIndex].Num() - 1) ? 0 : LoopNodeIndex + 1;
				if (NodeIdsOfLoops[LoopIndex][NextLoopNodeIndex] == NodeId)
				{
					LoopNodeIndex = NextLoopNodeIndex;
				}
				else
				{
					const int32 PrevLoopNodeIndex = LoopNodeIndex ? LoopNodeIndex - 1 : NodeIdsOfLoops[LoopIndex].Num() - 1;
					if (NodeIdsOfLoops[LoopIndex][PrevLoopNodeIndex] == NodeId)
					{
						LoopNodeIndex = PrevLoopNodeIndex;
					}
					else
					{
						LoopIndex = -1;
					}
				}
			}

			if (LoopIndex < 0 && !FindLoopAndNodeIndex(NodeId, LoopIndex, LoopNodeIndex))
			{
				continue;
			}

			ThinZonePoints.Emplace(Loops2D[EGridSpace::UniformScaled][LoopIndex][LoopNodeIndex]);
		}
	};

	TFunction<void(TArray<FVector2d>&)> RemoveDuplicatedNode = [](TArray<FVector2d>& ThinZoneMesh)
	{
		if (ThinZoneMesh.Num() > 1)
		{
			// Remove Duplicated Node
			constexpr double Tolerance = DOUBLE_SMALL_NUMBER;
			for (int32 Index = ThinZoneMesh.Num() - 1; Index > 0; --Index)
			{
				double SquareDist = FVector2d::DistSquared(ThinZoneMesh[Index - 1], ThinZoneMesh[Index]);
				if (SquareDist < Tolerance)
				{
					ThinZoneMesh.RemoveAt(Index);
				}
			}
			if (ThinZoneMesh.Num() > 2)
			{
				double SquareDist = FVector2d::DistSquared(ThinZoneMesh[0], ThinZoneMesh.Last());
				if (SquareDist < Tolerance)
				{
					ThinZoneMesh.RemoveAt(ThinZoneMesh.Num() - 1);
				}
			}
		}
	};

	const bool FirstSideIsClosed = ThinZone.GetFirstSide().IsClosed();
	const bool SecondSideIsClosed = ThinZone.GetSecondSide().IsClosed();

	TArray<int32> ThinZoneNodeIds;
	GetThinZoneSideMesh(ThinZone.GetFirstSide(), ThinZoneNodeIds);
	const int32 FirstSideSize = ThinZoneNodeIds.Num();

	if (FirstSideSize < 2)
	{
		return;
	}

	if (FirstSideIsClosed && SecondSideIsClosed)
	{
		TArray<FVector2d>& ThinZoneMesh = FaceLoops2D[EGridSpace::UniformScaled].Emplace_GetRef();
		GetThinZoneMeshCoordinates(ThinZoneNodeIds, ThinZoneMesh);
		RemoveDuplicatedNode(ThinZoneMesh);
		ThinZoneNodeIds.Reset();
	}

	GetThinZoneSideMesh(ThinZone.GetSecondSide(), ThinZoneNodeIds);
	const int32 SecondSideSize = (FirstSideIsClosed && SecondSideIsClosed) ? ThinZoneNodeIds.Num() : ThinZoneNodeIds.Num() - FirstSideSize;
	if (SecondSideSize < 2)
	{
		if (FirstSideIsClosed && SecondSideIsClosed)
		{
			FaceLoops2D[EGridSpace::UniformScaled].Pop();
		}
		return;
	}

	TArray<FVector2d>& ThinZoneMesh = FaceLoops2D[EGridSpace::UniformScaled].Emplace_GetRef();
	GetThinZoneMeshCoordinates(ThinZoneNodeIds, ThinZoneMesh);
	RemoveDuplicatedNode(ThinZoneMesh);

	if (ThinZoneMesh.Num() < 4)
	{
		FaceLoops2D[EGridSpace::UniformScaled].Pop();
	}
}

bool FGrid::GetMeshOfLoops()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::GetMeshOfLoops)

	int32 LoopCount = Face.GetLoops().Num();
	FaceLoops2D[EGridSpace::Default2D].Reserve(LoopCount);

	FaceLoops3D.Reserve(LoopCount);
	NormalsOfFaceLoops.Reserve(LoopCount);
	NodeIdsOfFaceLoops.Reserve(LoopCount);

	// Outer loops is processed first
	for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
	{
		if (Loop->IsExternal())
		{
			GetMeshOfLoop(*Loop);
			break;
		}
	}

	for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
	{
		if (!Loop->IsExternal())
		{
			GetMeshOfLoop(*Loop);
		}
	}

	if (CheckIfExternalLoopIsDegenerate())
	{
		return false;
	}

	ScaleLoops();
 
	if (Face.HasThinZone())
	{
		for (const FThinZone2D& ThinZone : Face.GetThinZones())
		{
			GetMeshOfThinZone(ThinZone);
		}
	}

	// Fit loops to Surface bounds.
	const FSurfacicBoundary& Bounds = Face.GetBoundary();
	for (TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::Default2D])
	{
		for (FVector2d& Point : Loop)
		{
			Bounds.MoveInsideIfNot(Point);
		}
	}

	FSurfacicBoundary UniformScaleBounds;
	UniformScaleBounds[IsoU].Set(UniformCuttingCoordinates[IsoU][0], UniformCuttingCoordinates[IsoU].Last());
	UniformScaleBounds[IsoV].Set(UniformCuttingCoordinates[IsoV][0], UniformCuttingCoordinates[IsoV].Last());

	// Fit loops in UniformScaled to UniformScale bounds.
	for (TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
	{
		for (FVector2d& Point : Loop)
		{
			UniformScaleBounds.MoveInsideIfNot(Point);
		}
	}

	return true;
}

void FGrid::ScaleLoops()
{
	int32 ThinZoneNum = 0;
	if (Face.HasThinZone())
	{
		ThinZoneNum = Face.GetThinZones().Num();
	}

	FaceLoops2D[EGridSpace::Scaled].SetNum(FaceLoops2D[EGridSpace::Default2D].Num());
	FaceLoops2D[EGridSpace::UniformScaled].Reserve(FaceLoops2D[EGridSpace::Default2D].Num() + ThinZoneNum);
	FaceLoops2D[EGridSpace::UniformScaled].SetNum(FaceLoops2D[EGridSpace::Default2D].Num());

	for (int32 IndexBoudnary = 0; IndexBoudnary < FaceLoops2D[EGridSpace::Default2D].Num(); ++IndexBoudnary)
	{
		const TArray<FVector2d>& Loop = FaceLoops2D[EGridSpace::Default2D][IndexBoudnary];
		TArray<FVector2d>& ScaledLoop = FaceLoops2D[EGridSpace::Scaled][IndexBoudnary];
		TArray<FVector2d>& UniformScaledLoop = FaceLoops2D[EGridSpace::UniformScaled][IndexBoudnary];

		ScaledLoop.SetNum(Loop.Num());
		UniformScaledLoop.SetNum(Loop.Num());

		int32 IndexU = 0;
		int32 IndexV = 0;
		for (int32 Index = 0; Index < Loop.Num(); ++Index)
		{
			const FVector2d& Point = Loop[Index];

			ArrayUtils::FindCoordinateIndex(CoordinateGrid[EIso::IsoU], Point.X, IndexU);
			ArrayUtils::FindCoordinateIndex(CoordinateGrid[EIso::IsoV], Point.Y, IndexV);

			ComputeNewCoordinate(Points2D[EGridSpace::Scaled], IndexU, IndexV, Point, ScaledLoop[Index]);
			ComputeNewCoordinate(Points2D[EGridSpace::UniformScaled], IndexU, IndexV, Point, UniformScaledLoop[Index]);
		}
	}
}

bool FGrid::CheckIf2DGridIsDegenerate() const
{
	TFunction<bool(EIso)> IsDegenerate = [&](EIso Iso) -> bool
	{
		double MaxDelta = 0;
		for (int32 Index = 1; Index < CoordinateGrid[Iso].Num(); ++Index)
		{
			double Delta = CoordinateGrid[Iso][Index] - CoordinateGrid[Iso][Index - 1];
			if (Delta > MaxDelta)
			{
				MaxDelta = Delta;
			}
		}
		return MaxDelta < FaceTolerance[Iso];
	};

	if (IsDegenerate(EIso::IsoU) || IsDegenerate(EIso::IsoV))
	{
		SetAsDegenerated();
		return true;
	}
	return false;
}

void FGrid::FindInnerFacePoints()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::CADKernel::FGrid::FindInnerFacePoints)

	// #cadkernel_check: This could use the TPolygon class???

	// FindInnerDomainPoints: Inner Points <-> bIsOfInnerDomain = true
	// For each points count the number of intersection with the boundary in the four directions U+ U- V+ V-
	// It for each the number is pair, the point is outside,
	// If in 3 directions the point is inner, the point is inner else we have a doubt so it preferable to consider it outside. 
	// Most of the time, there is a doubt if the point is to close of the boundary. So it will be removed be other criteria

	TFunction<void(TArray<char>&, int32)> AddIntersection = [](TArray<char>& Intersect, int32 Index)
	{
		Intersect[Index] = Intersect[Index] > 0 ? 0 : 1;
	};

	TArray<char> VForwardIntersectCount; // we need to know if intersect is pair 0, 2, 4... intersection of impair 1, 3, 5... false is pair, true is impair 
	TArray<char> VBackwardIntersectCount;
	TArray<char> UForwardIntersectCount;
	TArray<char> UBackwardIntersectCount;
	TArray<char> IntersectLoop;

	IntersectLoop.Init(0, CuttingSize);
	NodeMarkers.Init(ENodeMarker::IsInside, CuttingSize);

	VForwardIntersectCount.Init(0, CuttingSize);
	VBackwardIntersectCount.Init(0, CuttingSize);
	UForwardIntersectCount.Init(0, CuttingSize);
	UBackwardIntersectCount.Init(0, CuttingSize);

	// Loop node too close to one of CoordinateU or CoordinateV are moved a little to avoid floating error of comparison 
	// This step is necessary instead of all points could be considered outside...
	const double SmallToleranceU = DOUBLE_SMALL_NUMBER;
	const double SmallToleranceV = DOUBLE_SMALL_NUMBER;

	{
		int32 IndexV = 0;
		int32 IndexU = 0;
		for (TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
		{
			for (FVector2d& Point : Loop)
			{
				while (IndexV != 0 && (Point.Y < UniformCuttingCoordinates[EIso::IsoV][IndexV]))
				{
					IndexV--;
				}
				for (; IndexV < CuttingCount[EIso::IsoV]; ++IndexV)
				{
					if (Point.Y + SmallToleranceV < UniformCuttingCoordinates[EIso::IsoV][IndexV])
					{
						break;
					}
					if (Point.Y - SmallToleranceV > UniformCuttingCoordinates[EIso::IsoV][IndexV])
					{
						continue;
					}

					if (IndexV == 0)
					{
						Point.Y += SmallToleranceV;
					}
					else
					{
						Point.Y -= SmallToleranceV;
					}

					break;
				}
				if (IndexV == CuttingCount[EIso::IsoV])
				{
					IndexV--;
				}

				while (IndexU != 0 && (Point.X < UniformCuttingCoordinates[EIso::IsoU][IndexU]))
				{
					IndexU--;
				}
				for (; IndexU < CuttingCount[EIso::IsoU]; ++IndexU)
				{
					if (Point.X + SmallToleranceU < UniformCuttingCoordinates[EIso::IsoU][IndexU])
					{
						break;
					}
					if (Point.X - SmallToleranceU > UniformCuttingCoordinates[EIso::IsoU][IndexU])
					{
						continue;
					}

					if (IndexU == 0)
					{
						Point.X += SmallToleranceU;
					}
					else
					{
						Point.X -= SmallToleranceU;
					}
					break;
				}
				if (IndexU == CuttingCount[EIso::IsoU])
				{
					IndexU--;
				}
			}
		}
	}

	// Intersection along U axis
	for (const TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
	{
		const FVector2d* FirstSegmentPoint = &Loop.Last();
		for (const FVector2d& LoopPoint : Loop)
		{
			const FVector2d* SecondSegmentPoint = &LoopPoint;
			double UMin = FirstSegmentPoint->X;
			double VMin = FirstSegmentPoint->Y;
			double Umax = SecondSegmentPoint->X;
			double Vmax = SecondSegmentPoint->Y;
			FMath::GetMinMax(UMin, Umax);
			FMath::GetMinMax(VMin, Vmax);

			// AB^AP = ABu*APv - ABv*APu
			// AB^AP = ABu*(Pv-Av) - ABv*(Pu-Au)
			// AB^AP = Pv*ABu - Pu*ABv + Au*ABv - Av*ABu
			// AB^AP = Pv*ABu - Pu*ABv + AuABVMinusAvABu
			FVector2d PointA;
			FVector2d PointB;
			if (FirstSegmentPoint->Y < SecondSegmentPoint->Y)
			{
				PointA = *FirstSegmentPoint;
				PointB = *SecondSegmentPoint;
			}
			else
			{
				PointA = *SecondSegmentPoint;
				PointB = *FirstSegmentPoint;
			}
			double ABv = PointB.Y - PointA.Y;
			double ABu = PointB.X - PointA.X;
			double AuABVMinusAvABu = PointA.X * ABv - PointA.Y * ABu;

			int32 IndexV = 0;
			int32 Index = 0;
			for (; IndexV < CuttingCount[EIso::IsoV]; ++IndexV)
			{
				if (UniformCuttingCoordinates[EIso::IsoV][IndexV] >= VMin)
				{
					break;
				}
				Index += CuttingCount[EIso::IsoU];
			}

			for (; IndexV < CuttingCount[EIso::IsoV]; ++IndexV)
			{
				if (UniformCuttingCoordinates[EIso::IsoV][IndexV] > Vmax)
				{
					break;
				}

				for (int32 IndexU = 0; IndexU < CuttingCount[EIso::IsoU]; ++IndexU, ++Index)
				{
					//Index = IndexV * NumU + IndexU;
					if (IntersectLoop[Index])
					{
						continue;
					}

					if (UniformCuttingCoordinates[EIso::IsoU][IndexU] < UMin)
					{
						AddIntersection(UForwardIntersectCount, Index);
					}
					else if (UniformCuttingCoordinates[EIso::IsoU][IndexU] > Umax)
					{
						AddIntersection(UBackwardIntersectCount, Index);
					}
					else
					{
						double APvectAB = UniformCuttingCoordinates[EIso::IsoV][IndexV] * ABu - UniformCuttingCoordinates[EIso::IsoU][IndexU] * ABv + AuABVMinusAvABu;
						if (APvectAB > DOUBLE_SMALL_NUMBER)
						{
							AddIntersection(UForwardIntersectCount, Index);
						}
						else if (APvectAB < DOUBLE_SMALL_NUMBER)
						{
							AddIntersection(UBackwardIntersectCount, Index);
						}
						else
						{
							IntersectLoop[Index] = 1;
						}
					}
				}
			}
			FirstSegmentPoint = SecondSegmentPoint;
		}
	}

	// Intersection along V axis
	for (const TArray<FVector2d>& Loop : FaceLoops2D[EGridSpace::UniformScaled])
	{
		const FVector2d* FirstSegmentPoint = &Loop.Last();
		for (const FVector2d& LoopPoint : Loop)
		{
			const FVector2d* SecondSegmentPoint = &LoopPoint;
			double UMin = FirstSegmentPoint->X;
			double VMin = FirstSegmentPoint->Y;
			double Umax = SecondSegmentPoint->X;
			double Vmax = SecondSegmentPoint->Y;
			FMath::GetMinMax(UMin, Umax);
			FMath::GetMinMax(VMin, Vmax);

			// AB^AP = ABu*APv - ABv*APu
			// AB^AP = ABu*(Pv-Av) - ABv*(Pu-Au)
			// AB^AP = Pv*ABu - Pu*ABv + Au*ABv - Av*ABu
			// AB^AP = Pv*ABu - Pu*ABv + AuABVMinusAvABu
			FVector2d PointA;
			FVector2d PointB;
			if (FirstSegmentPoint->X < SecondSegmentPoint->X)
			{
				PointA = *FirstSegmentPoint;
				PointB = *SecondSegmentPoint;
			}
			else
			{
				PointA = *SecondSegmentPoint;
				PointB = *FirstSegmentPoint;
			}

			double ABu = PointB.X - PointA.X;
			double ABv = PointB.Y - PointA.Y;
			double AuABVMinusAvABu = PointA.X * ABv - PointA.Y * ABu;
			int32 Index = 0;
			for (int32 IndexU = 0; IndexU < CuttingCount[EIso::IsoU]; ++IndexU)
			{
				if (UniformCuttingCoordinates[EIso::IsoU][IndexU] < UMin)
				{
					continue;
				}

				if (UniformCuttingCoordinates[EIso::IsoU][IndexU] >= Umax)
				{
					continue;
				}

				Index = IndexU;
				for (int32 IndexV = 0; IndexV < CuttingCount[EIso::IsoV]; ++IndexV, Index += CuttingCount[EIso::IsoU])
				{
					if (IntersectLoop[Index])
					{
						continue;
					}

					if (UniformCuttingCoordinates[EIso::IsoV][IndexV] < VMin)
					{
						AddIntersection(VForwardIntersectCount, Index);
					}
					else if (UniformCuttingCoordinates[EIso::IsoV][IndexV] > Vmax)
					{
						AddIntersection(VBackwardIntersectCount, Index);
					}
					else
					{
						double APvectAB = UniformCuttingCoordinates[EIso::IsoV][IndexV] * ABu - UniformCuttingCoordinates[EIso::IsoU][IndexU] * ABv + AuABVMinusAvABu;
						if (APvectAB > DOUBLE_SMALL_NUMBER)
						{
							AddIntersection(VBackwardIntersectCount, Index);
						}
						else if (APvectAB < DOUBLE_SMALL_NUMBER)
						{
							AddIntersection(VForwardIntersectCount, Index);
						}
						else
						{
							IntersectLoop[Index] = 1;
						}
					}
				}
			}
			FirstSegmentPoint = SecondSegmentPoint;
		}
	}

	for (int32 Index = 0; Index < CuttingSize; ++Index)
	{
		if (IntersectLoop[Index])
		{
			ResetInsideLoop(Index);
			CountOfInnerNodes--;
			continue;
		}

		int32 IsInside = 0;
		if (UForwardIntersectCount[Index] > 0)
		{
			IsInside++;
		}
		if (UBackwardIntersectCount[Index] > 0)
		{
			IsInside++;
		}
		if (VForwardIntersectCount[Index] > 0)
		{
			IsInside++;
		}
		if (VBackwardIntersectCount[Index] > 0)
		{
			IsInside++;
		}
		if (IsInside < 3)
		{
			ResetInsideLoop(Index);
			CountOfInnerNodes--;
		}
	}
}

bool FGrid::CheckIfExternalLoopIsDegenerate() const
{
	if (FaceLoops2D[EGridSpace::Default2D].Num() == 0)
	{
		SetAsDegenerated();
		return true;
	}

	// if the external boundary is composed by only 2 points, the mesh of the surface is only an edge.
	// The grid is degenerated.
	if (FaceLoops2D[EGridSpace::Default2D][0].Num() < 3)
	{
		SetAsDegenerated();
		return true;
	}

	return false;
}

} // NS CADKernel

