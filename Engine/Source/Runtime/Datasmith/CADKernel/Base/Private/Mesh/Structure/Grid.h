// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Math/Point.h"
#include "Mesh/MeshEnum.h"
#include "Mesh/Structure/GridBase.h"
#include "Mesh/Structure/ThinZone2D.h"


namespace UE::CADKernel
{

struct FCuttingPoint;
struct FCuttingGrid;

class FIsoNode;
class FIsoSegment;
class FLoopNode;
class FModelMesh;
class FThinZone2DFinder;
class FTopologicalFace;


enum class ENodeMarker : uint8
{
	None = 0x00u,  // No flags.

	IsInside = 0x01u,
	IsInsideButTooCloseToLoop = 0x02u,  // node inside the loop but too close to the loop to be include in the mesh
	IsCloseToLoop = 0x04u,

	All = 0xFFu
};
ENUM_CLASS_FLAGS(ENodeMarker);


class FGrid : public FGridBase
{
protected:

	/*
	 * Cutting coordinates of the face respecting the meshing criteria
	 */
	FCoordinateGrid& CoordinateGrid;

	virtual const FCoordinateGrid& GetCoordinateGrid() const override
	{
		return CoordinateGrid;
	}

	const FSurfacicTolerance FaceTolerance;
	const double MinimumElementSize;

	FModelMesh& MeshModel;


	/**
	 * 2D Coordinate of Loop's nodes in each space
	 */
	TArray<TArray<FVector2d>> FaceLoops2D[EGridSpace::EndGridSpace];

	/**
	 * 3D Coordinate of Loop nodes in each space
	 */
	TArray<TArray<FVector>> FaceLoops3D;

	/**
	 * Surface Normal at each boundary nodes
	 */
	TArray<TArray<FVector3f>> NormalsOfFaceLoops;

	TArray<TArray<int32>> NodeIdsOfFaceLoops;

	/**
	 * count of node inside the face i.e. node inside the external loop and outer the inner loops
	 */
	int32 CountOfInnerNodes = 0;

	double MinOfMaxElementSize = 0;

	/**
	 * Array to flag each points as inside, close, ... the face
	 * @see IsNode...,
	 */
	TArray<ENodeMarker> NodeMarkers;

public:
#ifdef CADKERNEL_DEV
	FGridChronos Chronos;
#endif

	FGrid(FTopologicalFace& InFace, FModelMesh& InShellMesh);

#ifdef CADKERNEL_DEBUG
	virtual ~FGrid()
	{
		Close3DDebugSession(bDisplay);
	}
#else
	virtual ~FGrid() = default;
#endif

	// ======================================================================================================================================================================================================================
	// Meshing tools =========================================================================================================================================================================================================
	// ======================================================================================================================================================================================================================

	/**
	 * Defines the cutting coordinate of the grid according to mesh criteria and the existing mesh of bordering edges (loop's edges)
	 * @see GetPreferredUVCoordinatesFromNeighbours
	 */
	void DefineCuttingParameters();
	void DefineCuttingParameters(EIso Iso, FCuttingGrid& Neighbors);

	/**
	 * Computes the 2d Points, 3D points and Normals of the grid.
	 * call of DefineCuttingParameters is mandatory called before
	 * @return false if the grid is degenerated
	 */
	bool GeneratePointCloud();

	/**
	 * Process the generated point cloud to compute the scaled parametric spaces and to identify outer points of the face
	 * @see IsNodeInsideFace
	 * @see Points2D
	 */
	void ProcessPointCloud();

protected:

	/**
	 * Projects loop's points in the scaled parametric spaces
	 * @see FaceLoops2D
	 * @see ProcessPointClound (called in ProcessPointClound)
	 */
	void ScaleLoops();

	/**
	 * @see ProcessPointClound (called in ProcessPointClound)
	 */
	void FindInnerFacePoints();

	/**
	 * Define if an inner node is close to a loop i.e. if the loop cross the cell [[IndexU - 1, IndexU], [IndexV - 1, IndexV]], each corner node of the cell is close to the loop
	 *
	 * This algorithm is inspired of Bresenham's line algorithm
	 * The loop is traversed segment by segment
	 * For each segment, according to the slop of the segment, each cell intersecting the segment is selected (FindIntersectionsCaseSlope_) i.e. each corner node of the cell is flagged IsCloseToLoop (SetCellCloseToLoop)
	 * @see SlopeUtils
	 * @see IsNodeCloseToLoop
	 * @see ProcessPointCloud (called in ProcessPointClound)
	 */
	void FindPointsCloseToLoop();

	/**
	 * Find all node close to the boundary.
	 * For each node, the 3d distance to the boundary is estimated
	 * If this distance is bigger than the Tolerance, the node is remove
	 * Removing a node can make its neighbor removable,
	 * Each removed node make its direct neighbor removable, the process is recursive as long as it is possible to remove a node
	 *
	 * @see ProcessPointClound (called in ProcessPointClound)
	 */
	void RemovePointsCloseToLoop();

	/**
	 * Gets the cutting coordinates of the existing mesh of bordering edges (loop's edges)
	 * @see DefineCuttingParameters (called in DefineCuttingParameters)
	 */
	void GetPreferredUVCuttingParametersFromLoops(FCuttingGrid& CuttingFromLoops);

	/**
	 * @ return false if the mesh of the loop is degenerated
	 */
	bool GetMeshOfLoops();
	void GetMeshOfLoop(const FTopologicalLoop& Loop);
	void GetMeshOfThinZone(const FThinZone2D& ThinZone);

	// Meshing tools =========================================================================================================================================================================================================
	// ======================================================================================================================================================================================================================

	/**
	 * @return true if the grid is not consistent to build a mesh as composed of only two border nodes.
	 */
	bool CheckIfExternalLoopIsDegenerate() const;

	/**
	 * @return true if along an Iso, all (U(i+1) - U(i)) < FaceTolerance
	 */
	bool CheckIf2DGridIsDegenerate() const;

public:

	// ======================================================================================================================================================================================================================
	// GET Methodes =========================================================================================================================================================================================================
	// ======================================================================================================================================================================================================================

	constexpr const int32 GetCuttingCount(EIso Iso) const
	{
		return CuttingCount[Iso];
	}

	/**
	 * Return the number of points (inner and outer points) of the grid (i.e. CuttingCount[IsoU] x CuttingCount[IsoV]).
	 */
	const int32 GetTotalCuttingCount() const
	{
		return CuttingSize;
	}

	const double GetTolerance(EIso Iso) const
	{
		return FaceTolerance[Iso];
	}

	/**
	 * @return true if the node is inner the external loop and outer the inner loops and not too close
	 */
	const bool IsNodeInsideAndMeshable(int32 Index) const
	{
		return (NodeMarkers[Index] & ENodeMarker::IsInside) == ENodeMarker::IsInside;
	}

	const bool IsNodeInsideButTooCloseToLoop(int32 Index) const
	{
		return (NodeMarkers[Index] & ENodeMarker::IsInsideButTooCloseToLoop) == ENodeMarker::IsInsideButTooCloseToLoop;
	}

	/**
	 * @return true if the bits EPointMarker::IsInside, EPointMarker::IsInsideButTooCloseToLoop and EPointMarker::IsCloseToLoop are alls equal to 0
	 */
	const bool IsNodeFarFromFace(int32 Index) const
	{
		constexpr ENodeMarker IsInsideAndClose = ENodeMarker::IsInside | ENodeMarker::IsInsideButTooCloseToLoop | ENodeMarker::IsCloseToLoop;
		return (NodeMarkers[Index] & IsInsideAndClose) == ENodeMarker::None;
	}

	/**
	 * @return true if the bits EPointMarker::IsInside, EPointMarker::IsInsideButTooCloseToLoop and EPointMarker::IsCloseToLoop are alls equal to 0
	 */
	const bool IsNodeOutsideFace(int32 Index) const
	{
		constexpr ENodeMarker IsInsideAndClose = ENodeMarker::IsInside | ENodeMarker::IsInsideButTooCloseToLoop;
		return (NodeMarkers[Index] & IsInsideAndClose) == ENodeMarker::None;
	}

	const bool IsNodeInsideAndCloseToLoop(int32 Index) const
	{
		constexpr ENodeMarker IsInsideAndClose = ENodeMarker::IsInside | ENodeMarker::IsCloseToLoop;
		return (NodeMarkers[Index] & IsInsideAndClose) == IsInsideAndClose;
	}

	/**
	 * @return true if the bits EPointMarker::IsInside and EPointMarker::IsInsideButTooCloseToLoop are booth equal to 0 and EPointMarker::IsCloseToLoop  is equal to 1
	 */
	const bool IsNodeOusideFaceButClose(int32 Index) const
	{
		constexpr ENodeMarker IsInsideAndClose = ENodeMarker::IsInside | ENodeMarker::IsInsideButTooCloseToLoop | ENodeMarker::IsCloseToLoop;
		return (NodeMarkers[Index] & IsInsideAndClose) == ENodeMarker::IsCloseToLoop;
	}

	void SetNodeInside(int32 Index)
	{
		NodeMarkers[Index] |= ENodeMarker::IsInside;
	}

	/**
	 * @return the count of inner nodes
	 */
	const int32 InnerNodesCount() const
	{
		return CountOfInnerNodes;
	}

	/**
	 * @return true if the node is close to a loop i.e. the loop cross the space [[IndexU - 1, IndexU + 1], [IndexV - 1, IndexV + 1]]
	 */
	const bool IsNodeCloseToLoop(int32 Index) const
	{
		return (NodeMarkers[Index] & ENodeMarker::IsCloseToLoop) == ENodeMarker::IsCloseToLoop;
	}

	const bool IsNodeTooCloseToLoop(int32 Index) const
	{
		return (NodeMarkers[Index] & ENodeMarker::IsInsideButTooCloseToLoop) == ENodeMarker::IsInsideButTooCloseToLoop;
	}

	void SetCloseToLoop(int32 Index)
	{
		NodeMarkers[Index] |= ENodeMarker::IsCloseToLoop;
	}

	void SetTooCloseToLoop(int32 Index)
	{
		NodeMarkers[Index] |= ENodeMarker::IsInsideButTooCloseToLoop;
		NodeMarkers[Index] &= ~ENodeMarker::IsInside;
	}

	void ResetInsideLoop(int32 Index)
	{
		NodeMarkers[Index] &= ~ENodeMarker::IsInside;
	}

	void SetInner2DPoint(EGridSpace Space, int32 Index, const FVector2d& NewCoordinate)
	{
		Points2D[(int32)Space][Index] = NewCoordinate;
	}

	/**
	 * @return the normal of the surface at the point at the Index of the grid
	 */
	const FVector3f& GetPointNormal(int32 IndexU, int32 IndexV) const
	{
		return Normals[GobalIndex(IndexU, IndexV)];
	}

	/**
	 * @return the normal of the surface at the point at the Index of the grid
	 */
	const FVector3f& GetPointNormal(int32 Index) const
	{
		return Normals[Index];
	}

	const TArray<double>& GetCuttingCoordinatesAlongIso(EIso Iso) const
	{
		return CoordinateGrid[Iso];
	}

	const FCoordinateGrid& GetCuttingCoordinates() const
	{
		return CoordinateGrid;
	}

	/**
	 * @return the array of normal of the points of the grid
	 */
	TArray<FVector3f>& GetNormals()
	{
		return Normals;
	}

	const TArray<TArray<int32>>& GetNodeIdsOfFaceLoops() const
	{
		return NodeIdsOfFaceLoops;
	}

	const FVector2d& GetLoop2DPoint(EGridSpace Space, int32 LoopIndex, int32 Index) const
	{
		return FaceLoops2D[(int32)Space][LoopIndex][Index];
	}

	void SetLoop2DPoint(EGridSpace Space, int32 LoopIndex, int32 Index, const FVector2d& NewCoordinate)
	{
		FaceLoops2D[(int32)Space][LoopIndex][Index] = NewCoordinate;
	}

	const FVector& GetLoop3DPoint(int32 LoopIndex, int32 Index) const
	{
		return FaceLoops3D[LoopIndex][Index];
	}

	const int32 GetLoopCount() const
	{
		return FaceLoops2D[0].Num();
	}

	/**
	 * @return the array of array of 2d points of the loops according to the defined grid space
	 */
	const TArray<TArray<FVector2d>>& GetLoops2D(EGridSpace Space) const
	{
		return FaceLoops2D[(int32)Space];
	}

	/**
	 * @return the array of array of 3d points of the loops
	 */
	TArray<TArray<FVector>>& GetLoops3D()
	{
		return FaceLoops3D;
	}

	/**
	 * @return the array of array of 3d points of the loops
	 */
	const TArray<TArray<FVector>>& GetLoops3D() const
	{
		return FaceLoops3D;
	}

	/**
	 * @return the array of array of normal of the points of the loops
	 */
	const TArray<TArray<FVector3f>>& GetLoopNormals() const
	{
		return NormalsOfFaceLoops;
	}

	/**
	 * @return the Index of the position in the arrays of a point [IndexU, IndexV] of the grid
	 */
	void UVIndexFromGlobalIndex(int32 GLobalIndex, int32& OutIndexU, int32& OutIndexV) const
	{
		OutIndexU = GLobalIndex % CuttingCount[EIso::IsoU];
		OutIndexV = GLobalIndex / CuttingCount[EIso::IsoU];
	}

	/**
	 * @return the minimal size of the mesh elements according to 3d geometric tolerance
	 */
	double GetMinElementSize() const
	{
		return MinimumElementSize;
	}
};

}
