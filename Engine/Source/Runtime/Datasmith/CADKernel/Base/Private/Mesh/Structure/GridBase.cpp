// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/GridBase.h"

#include "Geo/Sampling/SurfacicSampling.h"
#include "Mesh/Structure/Grid.h"
#include "Topo/TopologicalFace.h"
#include "Utils/ArrayUtils.h"

//#define DEBUG_GRID
//#define DEBUG_GETPREFERREDUVCOORDINATESFROMNEIGHBOURS
namespace UE::CADKernel
{

FGridBase::FGridBase(FTopologicalFace& InFace)
	: Face(InFace)
	, Tolerance3D(InFace.GetCarrierSurface()->Get3DTolerance())
{
#ifdef DEBUG_ONLY_SURFACE_TO_DEBUG
	bDisplay = (InFace.GetId() == FaceToDebug);
	Open3DDebugSession(bDisplay, FString::Printf(TEXT("Grid %d"), InFace.GetId()));
#endif
}

bool FGridBase::ScaleGrid()
{
	const FCoordinateGrid& CoordinateGrid = GetCoordinateGrid();

	TFunction<double(const TArray<double>&)> GetMean = [](const TArray<double>& Lengths)
	{
		double MeanLength = 0;
		for (double Length : Lengths)
		{
			MeanLength += Length;
		}
		MeanLength /= Lengths.Num();
		return MeanLength;
	};

	TFunction<double(const TArray<double>&, const double)> StandardDeviation = [](const TArray<double>& Lengths, const double MeanLength)
	{
		double StandardDeviation = 0;
		for (double Length : Lengths)
		{
			StandardDeviation += FMath::Square(Length - MeanLength);
		}
		StandardDeviation /= Lengths.Num();
		StandardDeviation = sqrt(StandardDeviation);
		return StandardDeviation;
	};

	TFunction<void(const TArray<double>&, const double, TArray<double>&)> ScaleCoordinates = [](const TArray<double>& InCoordinates, const double ScaleFactor, TArray<double>& OutCoordinatesScaled)
	{
		OutCoordinatesScaled.Reserve(InCoordinates.Num());

		for (double Coordinate : InCoordinates)
		{
			OutCoordinatesScaled.Add(Coordinate * ScaleFactor);
		}
	};

	TFunction<int32(const TArray<double>&, const double)> GetMiddleIndex = [](const TArray<double>& Coordinates, double Middle)
	{
		int32 StartIndexUp = 1;
		for (; StartIndexUp < Coordinates.Num(); ++StartIndexUp)
		{
			if (Coordinates[StartIndexUp] > Middle)
			{
				break;
			}
		}
		return StartIndexUp;
	};

	for (int32 Index = EGridSpace::Default2D + 1; Index < EGridSpace::EndGridSpace; ++Index)
	{
		Points2D[Index].SetNum(CuttingSize);
	}

	TArray<double> LengthsV;
	LengthsV.SetNum(CuttingCount[EIso::IsoU]);
	for (int32 IndexU = 0; IndexU < CuttingCount[EIso::IsoU]; ++IndexU)
	{
		int32 Index = IndexU;
		double Length = 0;
		for (int32 IndexV = 1; IndexV < CuttingCount[EIso::IsoV]; ++IndexV)
		{
			Length += FVector::Distance(Points3D[Index], Points3D[Index + CuttingCount[EIso::IsoU]]);
			Index += CuttingCount[EIso::IsoU];
		}
		LengthsV[IndexU] = Length;
	}

	TArray<double> LengthsU;
	LengthsU.SetNum(CuttingCount[EIso::IsoV]);
	for (int32 IndexV = 0, Index = 0; IndexV < CuttingCount[EIso::IsoV]; ++IndexV)
	{
		double Length = 0;
		for (int32 IndexU = 1; IndexU < CuttingCount[EIso::IsoU]; ++IndexU)
		{
			Length += FVector::Distance(Points3D[Index], Points3D[Index + 1]);
			Index++;
		}
		Index++;
		LengthsU[IndexV] = Length;
	}

	double MeanLengthV = GetMean(LengthsV);
	double MeanLengthU = GetMean(LengthsU);
	if(MeanLengthV < Tolerance3D || MeanLengthU < Tolerance3D)
	{
		SetAsDegenerated();
		return false;
	}

	double FactorV = MeanLengthV / (CoordinateGrid[EIso::IsoV].Last() - CoordinateGrid[EIso::IsoV][0]);
	double FactorU = MeanLengthU / (CoordinateGrid[EIso::IsoU].Last() - CoordinateGrid[EIso::IsoU][0]);

	ScaleCoordinates(CoordinateGrid[EIso::IsoU], FactorU, UniformCuttingCoordinates[EIso::IsoU]);
	ScaleCoordinates(CoordinateGrid[EIso::IsoV], FactorV, UniformCuttingCoordinates[EIso::IsoV]);

	{
		int32 NumUV = 0;
		for (int32 IPointV = 0; IPointV < CuttingCount[EIso::IsoV]; ++IPointV)
		{
			for (int32 IPointU = 0; IPointU < CuttingCount[EIso::IsoU]; ++IPointU, ++NumUV)
			{
				Points2D[EGridSpace::UniformScaled][NumUV].Set(UniformCuttingCoordinates[EIso::IsoU][IPointU], UniformCuttingCoordinates[EIso::IsoV][IPointV]);
			}
		}
	}

	double StandardDeviationU = StandardDeviation(LengthsU, MeanLengthU);
	double StandardDeviationV = StandardDeviation(LengthsV, MeanLengthV);

	if (StandardDeviationV > StandardDeviationU)
	{
		double MiddleV = (CoordinateGrid[EIso::IsoV].Last() + CoordinateGrid[EIso::IsoV][0]) * 0.5;

		FCoordinateGrid Grid;
		Grid[EIso::IsoU] = CoordinateGrid[EIso::IsoU];
		Grid[EIso::IsoV].Add(MiddleV);

		FSurfacicSampling MiddlePoints;
		Face.EvaluatePointGrid(Grid, MiddlePoints);

		int32 StartIndexUp = GetMiddleIndex(CoordinateGrid[EIso::IsoV], MiddleV);
		int32 StartIndexDown = StartIndexUp - 1;

		int32 NumUV = 0;
		for (int32 IPointU = 0; IPointU < CuttingCount[EIso::IsoU]; ++IPointU)
		{
			double Length = 0;
			FVector LastPoint = MiddlePoints.Points3D[IPointU];
			for (int32 IPointV = StartIndexUp; IPointV < CuttingCount[EIso::IsoV]; ++IPointV)
			{
				NumUV = IPointV * CuttingCount[EIso::IsoU] + IPointU;
				Length += FVector::Distance(LastPoint, Points3D[NumUV]);
				Points2D[EGridSpace::Scaled][NumUV].Set(Points2D[EGridSpace::UniformScaled][NumUV].X, Length);
				LastPoint = Points3D[NumUV];
			}

			Length = 0;
			LastPoint = MiddlePoints.Points3D[IPointU];
			for (int32 IPointV = StartIndexDown; IPointV >= 0; --IPointV)
			{
				NumUV = IPointV * CuttingCount[EIso::IsoU] + IPointU;
				Length -= FVector::Distance(LastPoint, Points3D[NumUV]);
				Points2D[EGridSpace::Scaled][NumUV].Set(Points2D[EGridSpace::UniformScaled][NumUV].X, Length);
				LastPoint = Points3D[NumUV];
			}
		}
	}
	else
	{
		double MiddleU = (CoordinateGrid[EIso::IsoU].Last() + CoordinateGrid[EIso::IsoU][0]) * 0.5;

		FCoordinateGrid Grid;
		Grid[EIso::IsoU].Add(MiddleU);
		Grid[EIso::IsoV] = CoordinateGrid[EIso::IsoV];

		FSurfacicSampling MiddlePoints;
		Face.EvaluatePointGrid(Grid, MiddlePoints);

		int32 StartIndexUp = GetMiddleIndex(CoordinateGrid[EIso::IsoU], MiddleU);
		int32 StartIndexDown = StartIndexUp - 1;

		int32 NumUV = 0;
		for (int32 IPointV = 0; IPointV < CuttingCount[EIso::IsoV]; ++IPointV)
		{
			double Length = 0;
			FVector LastPoint = MiddlePoints.Points3D[IPointV];
			for (int32 IPointU = StartIndexUp; IPointU < CuttingCount[EIso::IsoU]; ++IPointU)
			{
				NumUV = IPointV * CuttingCount[EIso::IsoU] + IPointU;
				Length += FVector::Distance(LastPoint, Points3D[NumUV]);
				Points2D[EGridSpace::Scaled][NumUV].Set(Length, Points2D[EGridSpace::UniformScaled][NumUV].Y);
				LastPoint = Points3D[NumUV];
			}

			Length = 0;
			LastPoint = MiddlePoints.Points3D[IPointV];
			for (int32 IPointU = StartIndexDown; IPointU >= 0; --IPointU)
			{
				NumUV = IPointV * CuttingCount[EIso::IsoU] + IPointU;
				Length -= FVector::Distance(LastPoint, Points3D[NumUV]);
				Points2D[EGridSpace::Scaled][NumUV].Set(Length, Points2D[EGridSpace::UniformScaled][NumUV].Y);
				LastPoint = Points3D[NumUV];
			}
		}
	}

	return true;
}

void FGridBase::TransformPoints(EGridSpace DestinationSpace, const TArray<FVector2d>& InPointsToScale, TArray<FVector2d>& OutTransformedPoints) const
{
	const FCoordinateGrid& CoordinateGrid = GetCoordinateGrid();

	OutTransformedPoints.SetNum(InPointsToScale.Num());

	int32 IndexU = 0;
	int32 IndexV = 0;
	for (int32 Index = 0; Index < InPointsToScale.Num(); ++Index)
	{
		const FVector2d& Point = InPointsToScale[Index];

		ArrayUtils::FindCoordinateIndex(CoordinateGrid[EIso::IsoU], Point.X, IndexU);
		ArrayUtils::FindCoordinateIndex(CoordinateGrid[EIso::IsoV], Point.Y, IndexV);

		ComputeNewCoordinate(Points2D[DestinationSpace], IndexU, IndexV, Point, OutTransformedPoints[Index]);
	}
}

void FGridBase::EvaluatePointGrid(const FCoordinateGrid& CoordGrid, bool bWithNormals)
{
	CuttingSize = CoordGrid.Count();

	FSurfacicSampling Grid;
	Face.EvaluatePointGrid(CoordGrid, Grid, bWithNormals);

	Points2D[EGridSpace::Default2D] = MoveTemp(Grid.Points2D);
	Points3D = MoveTemp(Grid.Points3D);

	if (bWithNormals)
	{
		Normals = MoveTemp(Grid.Normals);
	}
}

#ifdef CADKERNEL_DEV
void FGrid::PrintTimeElapse() const
{
	Chronos.PrintTimeElapse();
}
#endif

}


