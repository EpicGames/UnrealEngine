// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/HaveStates.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Math/Point.h"
#include "Mesh/MeshEnum.h"


namespace UE::CADKernel
{

struct FCuttingPoint;
struct FCuttingGrid;

class FIsoNode;
class FIsoSegment;
class FLoopNode;
class FModelMesh;
class FThinZone2DFinder;
class FTopologicalFace;

class FGridBase : public FHaveStates
{

protected:

	FTopologicalFace& Face;
	const double Tolerance3D;

	/**
	 * grid point cloud size
	 */
	int32 CuttingCount[2] = { 0 , 0 };
	int32 CuttingSize = 0;

	FCoordinateGrid UniformCuttingCoordinates;

	/**
	 * 2D Coordinate of grid nodes in each space
	 */
	TArray<FVector2d> Points2D[EGridSpace::EndGridSpace];

	/**
	 * 3D Coordinate of inner nodes
	 */
	TArray<FVector> Points3D;

	/**
	 * Surface Normal at each inner nodes
	 */
	TArray<FVector3f> Normals;

public:
	FGridBase(FTopologicalFace& InFace);

	virtual ~FGridBase()
	{
#ifdef DEBUG_ONLY_SURFACE_TO_DEBUG
		Close3DDebugSession(bDisplay);
#endif
	}

	/**
	 * Convert an array of points of "DefaultParametric" space into a scaled parametric space
	 * @see FThinZone2DFinder::BuildBoundarySegments()
	 */
	void TransformPoints(EGridSpace DestinationSpace, const TArray<FVector2d>& InPointsToScale, TArray<FVector2d>& OutScaledPoints) const;

	/**
	 * Builds the scaled parametric spaces
	 * @see Points2D
	 * @see GeneratePointCloud (ended GeneratePointCloud process)
	 * @return false if the scaled grid is degenerated
	 */
	bool ScaleGrid();

	double GetTolerance() const
	{
		return Tolerance3D;
	}

protected:

	virtual const FCoordinateGrid& GetCoordinateGrid() const = 0;

	void EvaluatePointGrid(const FCoordinateGrid& CoordinateGrid, bool bWithNormals);

	/**
	 * Convert Coordinate of "DefaultParametric" space into a scaled parametric space
	 * @see ScaleLoops
	 */
	void ComputeNewCoordinate(const TArray<FVector2d>& NewGrid, int32 IndexU, int32 IndexV, const FVector2d& InPoint, FVector2d& OutNewScaledPoint) const
	{
		const FCoordinateGrid& CoordinateGrid = GetCoordinateGrid();

		const FVector2d& PointU0V0 = NewGrid[(IndexV + 0) * CuttingCount[EIso::IsoU] + (IndexU + 0)];
		const FVector2d& PointU1V0 = NewGrid[(IndexV + 0) * CuttingCount[EIso::IsoU] + (IndexU + 1)];
		const FVector2d& PointU0V1 = NewGrid[(IndexV + 1) * CuttingCount[EIso::IsoU] + (IndexU + 0)];
		const FVector2d& PointU1V1 = NewGrid[(IndexV + 1) * CuttingCount[EIso::IsoU] + (IndexU + 1)];

		const double U1MinusU0 = CoordinateGrid[EIso::IsoU][IndexU + 1] - CoordinateGrid[EIso::IsoU][IndexU];
		const double U0MinusU  = CoordinateGrid[EIso::IsoU][IndexU] - InPoint.X;
		const double V1MinusV0 = CoordinateGrid[EIso::IsoV][IndexV + 1] - CoordinateGrid[EIso::IsoV][IndexV];
		const double V0MinusV  = CoordinateGrid[EIso::IsoV][IndexV] - InPoint.Y;

		OutNewScaledPoint =
			PointU0V0 +
			(PointU0V0 - PointU1V0) * (U0MinusU / U1MinusU0) +
			(PointU0V0 - PointU0V1) * (V0MinusV / V1MinusV0) +
			(PointU0V0 - PointU0V1 + PointU1V1 - PointU1V0) * ((U0MinusU * V0MinusV) / (U1MinusU0 * V1MinusV0));
	};

public:

	/**
	 * @return the FVector2d (parametric coordinates) of the point at the Index of the grid in the defined grid space
	 * @see EGridSpace
	 * @see Points2D
	 */
	const FVector2d& GetInner2DPoint(EGridSpace Space, int32 Index) const
	{
		return Points2D[(int32)Space][Index];
	}

	/**
	 * @return the FVector2d (parametric coordinates) of the point at the Index of the grid in the defined grid space @see EGridSpace
	 */
	const FVector2d& GetInner2DPoint(EGridSpace Space, int32 IndexU, int32 IndexV) const
	{
		return Points2D[(int32)Space][GobalIndex(IndexU, IndexV)];
	}

	/**
	 * @return the FVector (3D coordinates) of the point at the Index of the grid
	 */
	const FVector& GetInner3DPoint(int32 Index) const
	{
		return Points3D[Index];
	}

	constexpr const TArray<double>& GetUniformCuttingCoordinatesAlongIso(EIso Iso) const
	{
		return UniformCuttingCoordinates[Iso];
	}

	const FCoordinateGrid& GetUniformCuttingCoordinates() const
	{
		return UniformCuttingCoordinates;
	}

	/**
	 * @return the array of 3d points of the grid
	 */
	TArray<FVector>& GetInner3DPoints()
	{
		return Points3D;
	}

	/**
	 * @return the array of 3d points of the grid
	 */
	const TArray<FVector>& GetInner3DPoints() const
	{
		return Points3D;
	}

	/**
	 * @return the array of 2d points of the grid in the defined space
	 */
	const TArray<FVector2d>& GetInner2DPoints(EGridSpace Space) const
	{
		return Points2D[(int32)Space];
	}

	const FTopologicalFace& GetFace() const
	{
		return Face;
	}

	FTopologicalFace& GetFace()
	{
		return Face;
	}

	/**
	 * @return the Index of the position in the arrays of a point [IndexU, IndexV] of the grid
	 */
	int32 GobalIndex(int32 IndexU, int32 IndexV) const
	{
		return IndexV * CuttingCount[EIso::IsoU] + IndexU;
	}

#ifdef CADKERNEL_DEBUG

	// ======================================================================================================================================================================================================================
	// Display Methodes   ================================================================================================================================================================================================
	// ======================================================================================================================================================================================================================
	mutable bool bDisplay = true;

	void DisplayIsoNode(EGridSpace Space, const int32 PointIndex, FIdent Ident = 0, EVisuProperty Property = EVisuProperty::BluePoint) const;
	virtual void DisplayGridPoints(EGridSpace DisplaySpace) const;
	void DisplayInnerPoints(TCHAR* Message, EGridSpace DisplaySpace) const;
#endif

#ifdef CADKERNEL_DEV
	void PrintTimeElapse() const;
#endif
};

}
