// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/LoopCleaner.h"

#include "Mesh/Meshers/IsoTriangulator.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Structure/Grid.h"
#include "Topo/TopologicalFace.h"

#ifdef CADKERNEL_DEV
#include "Mesh/Meshers/MesherReport.h"
#endif

namespace UE::CADKernel
{

FLoopCleaner::FLoopCleaner(FIsoTriangulator& Triangulator)
	: Grid(Triangulator.Grid)
	, Tolerances(Triangulator.Tolerances)
	, LoopNodes(Triangulator.LoopNodes)
	, LoopSegments(Triangulator.LoopSegments)
	, IsoSegmentFactory(Triangulator.IsoSegmentFactory)
	, bDisplay(Triangulator.bDisplay)
	, LoopSegmentsIntersectionTool(Grid, Triangulator.Tolerances.GeometricTolerance)
	, GetNext(LoopCleanerImpl::GetNextNodeImpl)
	, GetPrevious(LoopCleanerImpl::GetPreviousNodeImpl)
	, GetFirst(LoopCleanerImpl::GetFirstNode)
	, GetSecond(LoopCleanerImpl::GetSecondNode)
{
}

bool FLoopCleaner::Run()
{
	if (!CleanLoops())
	{
#ifdef CADKERNEL_DEV
		FMesherReport::GetLogs().AddRemoveSelfIntersectionFailure();
#endif
		return false;
	}

	// Two step: 
	// first intersection between inner loops and outer loop are processed 
	if (!UncrossLoops(/*bAddProcessedLoop*/ false))
	{
#ifdef CADKERNEL_DEV
		FMesherReport::GetLogs().AddRemoveCrossingLoopsFailure();
#endif
		return false;
	}

	// second, inner loop are added to LoopSegmentsIntersectionTool, so intersection between inner loop can be fixed
	if (!UncrossLoops(/*bAddProcessedLoop*/ true))
	{
#ifdef CADKERNEL_DEV
		FMesherReport::GetLogs().AddRemoveCrossingLoopsFailure();
#endif
		return false;
	}

	// a new call of CleanLoops because UncrossLoops could have generated auto intersection.
	if (!CleanLoops())
	{
#ifdef CADKERNEL_DEV
		FMesherReport::GetLogs().AddRemoveSelfIntersectionFailure();
#endif
		return false;
	}

	return true;
}

bool FLoopCleaner::CleanLoops()
{
	FindBestLoopExtremity();

#ifdef DEBUG_CLEAN_LOOPS		
	if (bDisplay)
	{
		Grid.DisplayIsoSegments(TEXT("Loops Orientation"), EGridSpace::UniformScaled, LoopSegments, false, true, false, EVisuProperty::BlueCurve);
		Grid.DisplayIsoSegments(TEXT("Loops Init"), EGridSpace::UniformScaled, LoopSegments, true, false, false, EVisuProperty::BlueCurve);
		Grid.DisplayIsoNodes(TEXT("BestStartNodeOfLoop"), EGridSpace::UniformScaled, (const TArray<const FIsoNode*>) BestStartNodeOfLoops, EVisuProperty::BluePoint);
		Wait(bDisplay);
	}
#endif

	// for each loop, start by the best node, find all intersections
	LoopIndex = -1;
	NextLoopFirstSegmentIndex = 0;

	GetNext = LoopCleanerImpl::GetNextNodeImpl;
	GetFirst = LoopCleanerImpl::GetFirstNode;
	GetSecond = LoopCleanerImpl::GetSecondNode;

	for (FLoopNode* StartNode : BestStartNodeOfLoops)
	{
		if (StartNode->IsDelete())
		{
			continue;
		}

		LoopIndex++;

		StartSegmentIndex = NextLoopFirstSegmentIndex;
		UpdateNextLoopFirstSegmentIndex(LoopIndex);

		if (!FindAndRemoveCoincidence(StartNode))
		{
			if (StartSegmentIndex == 0)
			{
				UE_LOGF(LogCADKernelBase, Warning, "The outer loop of the surface %d is degenerated. The mesh of this surface is canceled.\n", Grid.GetFace().GetId());
				return false;
			}
			continue;
		}

		Intersections.Empty(5);

		GetLoopNodeStartingFrom(StartNode, NodesOfLoop);

		if (!RemoveLoopPicks())
		{
			continue;
		}

#ifdef DEBUG_CLEAN_LOOPS		
		Grid.DisplayGridPolyline(TEXT("LoopIntersections: remove loop's picks"), EGridSpace::UniformScaled, NodesOfLoop, true, EVisuProperty::YellowCurve);
		if(bDisplay)
		{
			Wait(bDisplay);
		}
#endif

		FLoopNode** StartNodePtr = NodesOfLoop.FindByPredicate([](FLoopNode* Node) { return !Node->IsDelete(); });
		if (StartNodePtr == nullptr || *StartNodePtr == nullptr || (*StartNodePtr)->IsDelete())
		{
			continue;
		}

		StartNode = *StartNodePtr;

		if (!FindAndRemoveCoincidence(StartNode))
		{
			if (StartSegmentIndex == 0)
			{
				UE_LOGF(LogCADKernelBase, Warning, "The outer loop of the surface %d is degenerated. The mesh of this surface is canceled.\n", Grid.GetFace().GetId());
				return false;
			}
			continue;
		}

#ifdef DEBUG_CLEAN_LOOPS		
		Grid.DisplayGridPolyline(TEXT("Loop: start"), EGridSpace::UniformScaled, NodesOfLoop, true, EVisuProperty::YellowCurve);
		//Wait(bDisplay);
#endif

		FindLoopIntersections();

		if(Intersections.Num())
		{
			if (!RemoveSelfIntersectionsOfLoop())
			{
				UE_LOGF(LogCADKernelBase, Warning, "Loop intersections of the surface %d cannot be fixed. The mesh of this surface is canceled.\n", Grid.GetFace().GetId());
				return false;
			}

			Intersections.SetNum(0);
			FindLoopIntersections();
			if (Intersections.Num())
			{
				UE_LOGF(LogCADKernelBase, Warning, "Loop intersections of the surface %d cannot be fixed. The mesh of this surface is canceled.\n", Grid.GetFace().GetId());
				return false;
			}
#ifdef DEBUG_CLEAN_LOOPS		
			Grid.DisplayGridPolyline(TEXT("LoopIntersections: remove loop's self intersections"), EGridSpace::UniformScaled, NodesOfLoop, true, EVisuProperty::YellowCurve);
			//Wait(bDisplay);
#endif
		}

		if (!RemoveLoopPicks())
		{
			UE_LOGF(LogCADKernelBase, Warning, "Loop picks of the surface %d cannot be fixed. The mesh of this surface is canceled.\n", Grid.GetFace().GetId());
			return false;
		}

		if (NodesOfLoopCount == 0)
		{
			continue;
		}

#ifdef DEBUG_CLEAN_LOOPS		
		Grid.DisplayGridPolyline(TEXT("LoopIntersections: remove loop's picks (step2)"), EGridSpace::UniformScaled, NodesOfLoop, true, EVisuProperty::YellowCurve);
		//Wait(bDisplay);
#endif

		FixLoopOrientation();
	}

	if (!CheckMainLoopConsistency())
	{
		return false;
	}

	return true;
}

bool FLoopCleaner::UncrossLoops(bool bAddProcessedLoop)
{
	if (Grid.GetLoopCount() == 1)
	{
		return true;
	}

#ifdef DEBUG_UNCROSS_LOOPS
	int32 Iteration = 0;
	int32 MaxIndex = 10000;
	F3DDebugSession _(bDisplay, TEXT("FixIntersectionBetweenLoops"));
#endif

	TArray<int32> SegmentAlreadyProcessed;

	LoopSegmentsIntersectionTool.Empty(LoopSegments.Num());

	TArray<TArray<FIsoSegment*>> SegmentsOfLoops;
	{
		int32 LoopCount = ((FLoopNode&)LoopSegments.Last()->GetFirstNode()).GetLoopIndex();
		SegmentsOfLoops.Reserve(LoopCount + 1);
	}

	{
		TArray<FIsoSegment*>* SegmentsOfLoop = &SegmentsOfLoops.Emplace_GetRef();
		int32 IndexOfLoop = 0;
		for (FIsoSegment* Segment : LoopSegments)
		{
			if (IndexOfLoop != ((FLoopNode&)Segment->GetFirstNode()).GetLoopIndex())
			{
				SegmentsOfLoop = &SegmentsOfLoops.Emplace_GetRef();
				IndexOfLoop = ((FLoopNode&)Segment->GetFirstNode()).GetLoopIndex();
			}
			SegmentsOfLoop->Add(Segment);
		}
	}
	for (FIsoSegment* Segment : SegmentsOfLoops[0])
	{
		LoopSegmentsIntersectionTool.AddSegment(*Segment);
	}

	for (int32 Index = 1; Index < SegmentsOfLoops.Num(); ++Index)
	{
		TArray<FIsoSegment*>& SegmentsOfLoop = SegmentsOfLoops[Index];

		SegmentAlreadyProcessed.Init(0, SegmentsOfLoop.Num());

#ifdef DEBUG_UNCROSS_LOOPS
		if (bDisplay)
		{
			F3DDebugSession B(*FString::Printf(TEXT("Before Move Inside")));
			Grid.DisplayIsoSegments(EGridSpace::UniformScaled, SegmentsOfLoop, true, false, false, EVisuProperty::BlueCurve);
			Wait();
		}
#endif

#ifdef DEBUG_UNCROSS_LOOPS
		F3DDebugSession _(*FString::Printf(TEXT("Loop %d"), Index));
#endif
		int32 LoopSize = SegmentsOfLoop.Num();
		for (int32 SegmentIndex = 0; SegmentIndex < LoopSize; ++SegmentIndex)
		{

			FIsoSegment& Segment = *SegmentsOfLoop[SegmentIndex];
			if (Segment.IsDelete())
			{
				continue;
			}

			if (const FIsoSegment* IntersectingSegment = LoopSegmentsIntersectionTool.FindIntersectingSegment(Segment))
			{
				SegmentAlreadyProcessed[SegmentIndex]++;
				if (SegmentAlreadyProcessed[SegmentIndex] > 5)
				{
					// the segment is processed more than 5 times... infinite loop... the loop process is canceled
					break;
				}

#ifdef DEBUG_UNCROSS_LOOPS2
				++Iteration;
				if (bDisplay && Index > MaxIndex)
				{
					F3DDebugSession _(*FString::Printf(TEXT("Intersection %d"), Iteration));
					LoopSegmentsIntersectionTool.Display(bDisplay, TEXT("IntersectionTool"), EVisuProperty::BlueCurve);
					{
						F3DDebugSession B(*FString::Printf(TEXT("Loop")));
						Grid.DisplayIsoSegments(EGridSpace::UniformScaled, SegmentsOfLoop, true, false, false, EVisuProperty::BlueCurve);
					}
					{
						F3DDebugSession C(*FString::Printf(TEXT("Segment to be processed %d %d"), Index, Iteration));
						Grid.DisplayIsoSegment(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::YellowCurve);
					}
					{
						F3DDebugSession C(TEXT("Intersecting Segments"));
						Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *IntersectingSegment, 0, EVisuProperty::RedCurve/*, true*/);
					}
				}
#endif

				if (!TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(*IntersectingSegment, Segment))
				{
					if (!TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(Segment, const_cast<FIsoSegment&>(*IntersectingSegment)))
					{
						RemoveIntersectionByMovingOutsideSegmentNodeInside(*IntersectingSegment, Segment, /*((FLoopNode&)Segment.GetFirstNode()).GetLoopIndex() && */ false);
					}
				}

#ifdef DEBUG_UNCROSS_LOOPS2
				if (bDisplay && Index > MaxIndex)
				{
					F3DDebugSession B(*FString::Printf(TEXT("After fix")));
					Grid.DisplayIsoSegments(EGridSpace::UniformScaled, SegmentsOfLoop, true, false, false, EVisuProperty::YellowCurve);
					//Grid.DisplayGridPolyline(TEXT("After fix"), EGridSpace::UniformScaled, LoopNodes, true, EVisuProperty::YellowCurve);
					//Wait();
				}
#endif
				if (SegmentIndex > 0)
				{
					SegmentIndex--;
				}
				while (SegmentIndex > 0 && SegmentsOfLoop[SegmentIndex]->IsDelete())
				{
					SegmentIndex--;
				}
				SegmentIndex--;
			}
		}

#ifdef DEBUG_UNCROSS_LOOPS
		if(bDisplay)
		{
			F3DDebugSession B(*FString::Printf(TEXT("Before RemoveLoopPicks")));
			Grid.DisplayIsoSegments(EGridSpace::UniformScaled, SegmentsOfLoop, true, false, false, EVisuProperty::BlueCurve);
			Wait();
		}
#endif

		RemoveLoopPicks(SegmentsOfLoop);

#ifdef DEBUG_UNCROSS_LOOPS
		if (bDisplay)
		{
			F3DDebugSession B(*FString::Printf(TEXT("After RemoveLoopPicks")));
			Grid.DisplayIsoSegments(EGridSpace::UniformScaled, SegmentsOfLoop, true, false, false, EVisuProperty::BlueCurve);
			Wait();
		}
#endif

		if(bAddProcessedLoop)
		{
			for (FIsoSegment* Segment : SegmentsOfLoop)
			{
				if (Segment->IsDelete())
				{
					continue;
				}
				LoopSegmentsIntersectionTool.AddSegment(*Segment);
			}
		}
	}

#ifdef DEBUG_UNCROSS_LOOPS
	if (bDisplay && Iteration > MaxIndex)
	{
		Grid.DisplayGridPolyline(TEXT("After UncrossLoops"), EGridSpace::UniformScaled, LoopNodes, true, EVisuProperty::YellowCurve);
		Wait();
	}
#endif

	return true;
}

bool FLoopCleaner::TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(const FIsoSegment& IntersectingSegment, FIsoSegment& Segment)
{
	FLoopNode* Node = nullptr;
	FLoopNode* PreviousNode = nullptr;
	FLoopNode* NextNode = nullptr;

	FSegment2D IntersectingSegment2D(IntersectingSegment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), IntersectingSegment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid));
	FSegment2D Segment2D(Segment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), Segment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid));

	double IntersectingSegmentSlope = ComputeOrientedSlope(IntersectingSegment2D.Point0, IntersectingSegment2D.Point1, 0);
	double SegmentSlope = ComputeUnorientedSlope(Segment2D.Point1, Segment2D.Point0, IntersectingSegmentSlope);
	if (SegmentSlope > 2)
	{
		SegmentSlope = 4 - SegmentSlope;
	}

	// if the segment and IntersectingSegment are parallel, segment are moved inside
	if (SegmentSlope < 0.01)
	{
		double StartPointSquareDistance = SquareDistanceOfPointToSegment<FVector2d>(Segment2D.Point0, IntersectingSegment2D.Point0, IntersectingSegment2D.Point1);
		double EndPointSquareDistance = SquareDistanceOfPointToSegment<FVector2d>(Segment2D.Point1, IntersectingSegment2D.Point0, IntersectingSegment2D.Point1);
		if (StartPointSquareDistance < Tolerances.SquareGeometricTolerance2 && EndPointSquareDistance < Tolerances.SquareGeometricTolerance2)
		{
			OffsetSegment(Segment, Segment2D, IntersectingSegment2D);
			return true;
		}
	}

	// check if the intersection is not at the extremity
	{
		double Coordinate = 0;
		FindIntersectionOfSegments2D(Segment2D, IntersectingSegment2D, Coordinate);
		if (FMath::IsNearlyZero(Coordinate, DOUBLE_KINDA_SMALL_NUMBER))
		{
			// can add a test to offset the outside node and not the node a 
			OffsetNode((FLoopNode&)Segment.GetFirstNode(), IntersectingSegment2D);
			return true;
		}
		else if (FMath::IsNearlyEqual(Coordinate, 1, DOUBLE_KINDA_SMALL_NUMBER))
		{
			OffsetNode((FLoopNode&)Segment.GetSecondNode(), IntersectingSegment2D);
			return true;
		}
	}

	double OrientedSlope = ComputeOrientedSlope(IntersectingSegment2D.Point0, Segment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid), IntersectingSegmentSlope);
	if (OrientedSlope >= 0)
	{
		Node = (FLoopNode*)&Segment.GetSecondNode();
		PreviousNode = (FLoopNode*)&Segment.GetFirstNode();
		NextNode = (FLoopNode*)&Node->GetNextNode();
	}
	else
	{
		Node = (FLoopNode*)&Segment.GetFirstNode();
		PreviousNode = (FLoopNode*)&Segment.GetSecondNode();
		NextNode = (FLoopNode*)&Node->GetPreviousNode();
	}

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("Intersecting Segments"));
		Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::BlueCurve);
		Display(EGridSpace::UniformScaled, *Node, *NextNode, 0, EVisuProperty::BlueCurve);
		Display(EGridSpace::UniformScaled, IntersectingSegment, 0, EVisuProperty::RedCurve);
		Display(EGridSpace::UniformScaled, *Node, 0, EVisuProperty::RedPoint);
		Wait(false);
	}
#endif

	FSegment2D NextSegment2D(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid));
	if (!DoIntersect(NextSegment2D, IntersectingSegment2D))
	{
		return false;
	}

	FSegment2D  PreviousSegment2D(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid));

	FVector2d Intersection1 = FindIntersectionOfSegments2D(PreviousSegment2D, IntersectingSegment2D);
	FVector2d Intersection2 = FindIntersectionOfSegments2D(NextSegment2D, IntersectingSegment2D);

	double Coordinate;
	FVector2d ProjectedPoint = ProjectPointOnSegment<FVector2d>(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), Intersection1, Intersection2, Coordinate);

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("ProjectedPoint"));
		DisplayPoint(ProjectedPoint, EVisuProperty::RedPoint);
	}
#endif

	FVector2d SegmentTangent = IntersectingSegment2D.Point1 - IntersectingSegment2D.Point0;
	SegmentTangent.Normalize();
	FVector2d MoveDirection = FVector2d(-SegmentTangent.Y, SegmentTangent.X);

	MoveDirection *= Tolerances.GeometricTolerance;

	FVector2d NewCoordinate = ProjectedPoint + MoveDirection;

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("NewCoordinate"));
		DisplayPoint(NewCoordinate, EVisuProperty::BluePoint);
	}
#endif

	if ((FVector2d::DistSquared(PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid), NewCoordinate) < Tolerances.SquareGeometricTolerance2) ||
		(FVector2d::DistSquared(NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid), NewCoordinate) < Tolerances.SquareGeometricTolerance2))
	{
		if(RemoveNodeOfLoop(*Node))
		{
			return true;
		}
	}
	else
	{
		Node->Set2DPoint(EGridSpace::UniformScaled, Grid, NewCoordinate);
		FIsoSegment* NextSegment = Node->GetSegmentConnectedTo(NextNode);
		LoopSegmentsIntersectionTool.Update(&Segment);
		LoopSegmentsIntersectionTool.Update(NextSegment);

#ifdef DEBUG_TWO_CONSECUTIVE_INTERSECTING
		if (bDisplay)
		{
			F3DDebugSession _(TEXT("New position"));
			DisplayPoint(Intersection1, EVisuProperty::RedPoint);
			DisplayPoint(Intersection2, EVisuProperty::RedPoint);
			Display(EGridSpace::UniformScaled, *Node, EVisuProperty::RedPoint);
			Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::GreenCurve/*, true*/);
			Display(EGridSpace::UniformScaled, *Node, *NextNode, 0, EVisuProperty::GreenCurve/*, true*/);
			Wait(false);
		}
#endif
	}

	return true;
}

void FLoopCleaner::OffsetSegment(FIsoSegment& Segment, FSegment2D& Segment2D, FSegment2D& IntersectingSegment2D)
{
	FVector2d SegmentTangent = IntersectingSegment2D.Point1 - IntersectingSegment2D.Point0;
	SegmentTangent.Normalize();
	FVector2d MoveDirection = FVector2d(-SegmentTangent.Y, SegmentTangent.X);
	MoveDirection *= Tolerances.GeometricTolerance;

	FVector2d NewPoint0 = Segment2D.Point0 + MoveDirection;
	FVector2d NewPoint1 = Segment2D.Point1 + MoveDirection;

	Segment.GetFirstNode().Set2DPoint(EGridSpace::UniformScaled, Grid, NewPoint0);
	Segment.GetSecondNode().Set2DPoint(EGridSpace::UniformScaled, Grid, NewPoint1);
}

void FLoopCleaner::OffsetNode(FLoopNode& Node, FSegment2D& IntersectingSegment2D)
{
	FVector2d SegmentTangent = IntersectingSegment2D.Point1 - IntersectingSegment2D.Point0;
	SegmentTangent.Normalize();
	FVector2d MoveDirection = FVector2d(-SegmentTangent.Y, SegmentTangent.X);
	MoveDirection *= Tolerances.GeometricTolerance;

	FVector2d NewPoint = Node.Get2DPoint(EGridSpace::UniformScaled, Grid) + MoveDirection;
	Node.Set2DPoint(EGridSpace::UniformScaled, Grid, NewPoint);
}

void FLoopCleaner::RemoveIntersectionByMovingOutsideSegmentNodeInside(const FIsoSegment& IntersectingSegment, const FIsoSegment& Segment, bool bIsSameInnerLoop)
{
	const FIsoNode* Nodes[2][2] = { {nullptr, nullptr}, {nullptr, nullptr} };
	Nodes[0][0] = &IntersectingSegment.GetFirstNode();
	Nodes[0][1] = &IntersectingSegment.GetSecondNode();
	Nodes[1][0] = &Segment.GetFirstNode();
	Nodes[1][1] = &Segment.GetSecondNode();

	FVector2d IntersectingPoints[2];
	IntersectingPoints[0] = IntersectingSegment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid);
	IntersectingPoints[1] = IntersectingSegment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid);

	FVector2d SegmentPoints[2];
	SegmentPoints[0] = Segment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid);
	SegmentPoints[1] = Segment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid);

	double SquareLengthIntersectingSegment = FVector2d::DistSquared(IntersectingPoints[0], IntersectingPoints[1]);
	double SquareLengthSegment = FVector2d::DistSquared(SegmentPoints[0], SegmentPoints[1]);

	if (SquareLengthSegment > 10 * SquareLengthIntersectingSegment)
	{
		return RemoveIntersectionByMovingOutsideSegmentNodeInside(Segment, const_cast<FIsoSegment&>(IntersectingSegment), bIsSameInnerLoop);
	}

	bool bFirstNodeIsOutside = false;
	FVector2d PointToMove = Segment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid);

	// Is Second node, the outside node ?
	{
		double OrientedSlope = ComputeOrientedSlope(IntersectingPoints[0], IntersectingPoints[1], PointToMove);
		if (bIsSameInnerLoop)
		{
			OrientedSlope *= -1;
		}

		if (OrientedSlope > 0)
		{
			PointToMove = Segment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid);
			bFirstNodeIsOutside = true;
		}
	}

#ifdef DEBUG_CLOSED_OUSIDE_POINT
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("Outside Point"));
		DisplayPoint(PointToMove, bFirstNodeIsOutside ? EVisuProperty::GreenPoint : EVisuProperty::YellowPoint);
	}
#endif

	double Coordinate;
	FVector2d ProjectedPoint = ProjectPointOnSegment<FVector2d>(PointToMove, IntersectingPoints[0], IntersectingPoints[1], Coordinate);

	FVector2d MoveDirection = IntersectingPoints[1] - IntersectingPoints[0];
	MoveDirection.Normalize();
	MoveDirection = FVector2d(-MoveDirection.Y, MoveDirection.X);
	MoveDirection *= bIsSameInnerLoop ? -Tolerances.GeometricTolerance : Tolerances.GeometricTolerance;

	FVector2d NewCoordinate = ProjectedPoint + MoveDirection;

	FLoopNode& Node = (FLoopNode&) const_cast<FIsoNode&>(!bFirstNodeIsOutside ? Segment.GetSecondNode() : Segment.GetFirstNode());
	FLoopNode& PreviousNode = Node.GetPreviousNode();
	FLoopNode& NextNode = Node.GetNextNode();

	if ((FVector2d::DistSquared(PreviousNode.Get2DPoint(EGridSpace::UniformScaled, Grid), NewCoordinate) < Tolerances.SquareGeometricTolerance2) ||
		(FVector2d::DistSquared(NextNode.Get2DPoint(EGridSpace::UniformScaled, Grid), NewCoordinate) < Tolerances.SquareGeometricTolerance2))
	{
		RemoveNodeOfLoop(Node);
		return;
	}

	Node.Set2DPoint(EGridSpace::UniformScaled, Grid, NewCoordinate);

#ifdef DEBUG_CLOSED_OUSIDE_POINT
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("New Segs"));
		DisplayPoint(NewCoordinate, EVisuProperty::BluePoint);
		Display(EGridSpace::UniformScaled, Segment, 0, EVisuProperty::BlueCurve);
		Display(EGridSpace::UniformScaled, IntersectingSegment, 0, EVisuProperty::RedCurve);
	}
#endif
}

void FLoopCleaner::FixLoopOrientation()
{
	FLoopNode const* const* StartNode = NodesOfLoop.FindByPredicate([](const FLoopNode* Node) { return !Node->IsDelete(); });
	if (StartNode == nullptr)
	{
		return;
	}

	EOrientation Orientation = GetLoopOrientation(*StartNode);
	if (Orientation == EOrientation::Back)
	{
#ifdef DEBUG_LOOP_ORIENTATION
		F3DDebugSession Q(Grid.bDisplay, TEXT("FixLoopOrientation"));
		Grid.DisplayIsoNodes(TEXT("Nodes of loop Before orientation"), EGridSpace::UniformScaled, (const TArray<const FIsoNode*>&) NodesOfLoop, EVisuProperty::BluePoint);
		Grid.DisplayIsoSegments(TEXT("Before orientation"), EGridSpace::UniformScaled, LoopSegments, true, true);
		Wait(false);
#endif

		SwapSubLoopOrientation(StartSegmentIndex, NextLoopFirstSegmentIndex);

#ifdef DEBUG_LOOP_ORIENTATION
		Grid.DisplayIsoSegments(TEXT("After orientation"), EGridSpace::UniformScaled, LoopSegments, true, true);
		Wait(false);
#endif
	}
}

bool FLoopCleaner::RemoveSelfIntersectionsOfLoop()
{
	using namespace LoopCleanerImpl;

	// All pinches are removed
	for (int32 IntersectionIndex = Intersections.Num() - 1; IntersectionIndex >= 0; --IntersectionIndex)
	{
		const TPair<double, double>& Intersection = Intersections[IntersectionIndex];

		bool bIntersectionKeyIsExtremity = FMath::IsNearlyEqual(Intersection.Key, (int32)(Intersection.Key + 0.5));
		bool bIntersectionValueIsExtremity = FMath::IsNearlyEqual(Intersection.Value, (int32)(Intersection.Value + 0.5));

		if (bIntersectionKeyIsExtremity && bIntersectionValueIsExtremity)
		{
			FPinchIntersectionContext Context(Intersection);
			if(!Fill(Context))
			{
				return false;
			}

			if (IsAPinch(Context))
			{
				DisconnectCoincidentNodes(Context);
				Intersections.RemoveAt(IntersectionIndex);
			}
		}
	}

	Algo::Sort(Intersections, [&](const TPair<double, double>& Intersection1, const TPair<double, double>& Intersection2)
		{
			return (Intersection1.Key < Intersection2.Key);
		});

#ifdef DEBUG_REMOVE_LOOP_INTERSECTIONS		
	Grid.DisplayGridPolyline(TEXT("RemoveSelfIntersectionsOfLoop start"), EGridSpace::UniformScaled, NodesOfLoop, true, EVisuProperty::BlueCurve);
	Wait(bDisplay);
#endif

	for (int32 IntersectionIndex = 0; IntersectionIndex < Intersections.Num(); )
	{
		const TPair<double, double>& Intersection = Intersections[IntersectionIndex];

		FLoopNode* Segment0End = GetNodeAt(NextIndex((int32)Intersection.Key));
		FLoopNode* Segment1Start = GetNodeAt((int32)Intersection.Value);

		if (Segment0End == nullptr || Segment1Start == nullptr)
		{
			++IntersectionIndex;
			continue;
		}


#ifdef DEBUG_REMOVE_LOOP_INTERSECTIONS		
		F3DDebugSession _(bDisplay, TEXT("Intersected Segments"));
		if(bDisplay)
		{
			DisplayIntersection(Intersection);
			Wait();
		}
#endif

		bool bIntersectionForward = true;

		int32 NextIntersectionIndex = IntersectionIndex + 1;
		int32 IntersectionCount = 1;
		for (; NextIntersectionIndex < Intersections.Num(); ++NextIntersectionIndex)
		{
			if (Intersections[NextIntersectionIndex].Value > Intersection.Value)
			{
				break;
			}
			++IntersectionCount;

#ifdef DEBUG_REMOVE_LOOP_INTERSECTIONS		
			DisplayIntersection(Intersections[NextIntersectionIndex]);
#endif
		}

		if (IntersectionCount == 1)
		{
			NextIntersectionIndex = IntersectionIndex + 1;
			if (Intersections.IsValidIndex(NextIntersectionIndex) && Intersections[NextIntersectionIndex].Key <= Intersection.Value)
			{
				bIntersectionForward = false;
				++IntersectionCount;
#ifdef DEBUG_REMOVE_LOOP_INTERSECTIONS
				DisplayIntersection(Intersections[NextIntersectionIndex]);
#endif
			}
		}

		// Stating from this point 
		// the process must not delete node after the first intersection
		if (IntersectionCount == 1)
		{
			if (!RemoveIntersection(Intersections[IntersectionIndex]))
			{
				return false;
			}
			IntersectionIndex++;
		}
		else if (!bIntersectionForward)
		{
			if (!RemoveOutgoingLoop(Intersections[IntersectionIndex], Intersections[IntersectionIndex + 1]))
			{
				return false;
			}
			IntersectionIndex += 2;
		}
		else
		{
			if (!RemoveIntersectionsOfSubLoop(IntersectionIndex, IntersectionCount))
			{
				return false;
			}
			IntersectionIndex += IntersectionCount;
		}

#ifdef DEBUG_REMOVE_LOOP_INTERSECTIONS		
		Grid.DisplayGridPolyline(TEXT("RemoveSelfIntersectionsOfLoop end"), EGridSpace::UniformScaled, NodesOfLoop, true, EVisuProperty::BlueCurve);
#endif
	}

	return true;
}

bool FLoopCleaner::RemoveOutgoingLoop(const TPair<double, double>& Intersection, const TPair<double, double>& NextIntersection)
{
	const TPair<double, double> OutSideLoop(Intersection.Value, NextIntersection.Value);
	//if (IsSubLoopBiggerThanMainLoop(NodesOfLoop, OutSideLoop, bForward))
	//{
	//	return false;
	//}

	TFunction<FVector2d(double, FLoopNode*, FLoopNode*)> IntersectingPoint = [&](double Coordinate, FLoopNode* Start, FLoopNode* End) -> FVector2d
	{
		const FVector2d& StartPoint = Start->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& EndPoint = End->Get2DPoint(EGridSpace::UniformScaled, Grid);

		int32 StartIndex = (int32)Coordinate;
		Coordinate -= StartIndex;
		FVector2d Intersection = PointOnSegment<FVector2d>(StartPoint, EndPoint, Coordinate);

		//{
		//	F3DDebugSession _(TEXT("Intersected Point"));
		//	DisplayPoint(StartPoint, EVisuProperty::RedPoint);
		//	DisplayPoint(EndPoint, EVisuProperty::RedPoint);
		//	DisplayPoint(Intersection, EVisuProperty::BluePoint);
		//}

		return Intersection;
	};

	FLoopNode* TmpNode = GetNodeAt(NextIndex((int32)Intersection.Value));
	FLoopNode* EndNode = GetNodeAt((int32)NextIntersection.Value);
	if (TmpNode == nullptr || EndNode == nullptr)
	{
		return false;
	}

	FLoopNode* StartNode = GetPrevious(TmpNode);
	FVector2d FirstIntersection = IntersectingPoint(Intersection.Value, StartNode, TmpNode);

	TmpNode = GetNext(EndNode);
	FVector2d SecondIntersection = IntersectingPoint(NextIntersection.Value, EndNode, TmpNode);

	FVector2d MiddlePoint = (FirstIntersection + SecondIntersection) * 0.5;

	TmpNode = GetNext(StartNode);
	while (TmpNode && (TmpNode != EndNode) && !TmpNode->IsDelete())
	{
		if (!RemoveNodeOfLoop(*TmpNode))
		{
			return false;
		}
		TmpNode = GetNext(StartNode);
	}

	FVector2d MoveDirection = SecondIntersection - FirstIntersection;
	double Length = MoveDirection.Length();
	if (FMath::IsNearlyZero(Length))
	{
		FLoopNode* StartSegment = GetNodeAt((int32)Intersection.Key);
		if (StartSegment == nullptr)
		{
			return false;
		}
		MoveDirection = GetNext(StartSegment)->Get2DPoint(EGridSpace::UniformScaled, Grid) - StartSegment->Get2DPoint(EGridSpace::UniformScaled, Grid);
		Length = MoveDirection.Length();
	}

	MoveDirection /= Length;
	MoveDirection = FVector2d(-MoveDirection.Y, MoveDirection.X);
	MoveDirection *= Tolerances.GeometricTolerance;
	MiddlePoint += MoveDirection;

	if (!EndNode->IsDelete())
	{
		EndNode->Set2DPoint(EGridSpace::UniformScaled, Grid, MiddlePoint);
	}

	return true;
}

bool FLoopCleaner::RemoveIntersectionsOfSubLoop(int32 IntersectionIndex, int32 IntersectionCount)
{
	using namespace LoopCleanerImpl;

	TFunction<void(FLoopNode*, const FVector2d&, FVector2d&)> MoveNodeToProjection = [&](FLoopNode* NodeToProject, const FVector2d& PointToProject, FVector2d& ProjectedPoint)
	{
		FVector2d MoveDirection = ProjectedPoint - PointToProject;
		MoveDirection.Normalize();
		MoveDirection *= Tolerances.GeometricTolerance;
		ProjectedPoint += MoveDirection;
		NodeToProject->Set2DPoint(EGridSpace::UniformScaled, Grid, ProjectedPoint);
	};

	TFunction<void(FLoopNode*, const FVector2d&, const FVector2d&)> ProjectNodeOnSegment = [&](FLoopNode* NodeToProject, const FVector2d& Point0, const FVector2d& Point1)
	{
		const FVector2d& PointToProject = NodeToProject->Get2DPoint(EGridSpace::UniformScaled, Grid);

		FVector2d ProjectedPoint;
		double Coordinate;
		ProjectedPoint = ProjectPointOnSegment<FVector2d>(PointToProject, Point0, Point1, Coordinate);

		MoveNodeToProjection(NodeToProject, PointToProject, ProjectedPoint);
	};

	TFunction<bool(const int32, const int32, const int32)> ProjectNodesOnSegment = [&](const int32 StartIndex, const int32 EndIndex, const int32 SegmnentEndIndex) -> bool
	{
		FLoopNode* Node = GetNodeAt(StartIndex);
		FLoopNode* StopNode = GetNodeAt(EndIndex);
		FLoopNode* EndSegment = GetNodeAt(SegmnentEndIndex);
		if (Node == nullptr || StopNode == nullptr || EndSegment == nullptr)
		{
			return false;
		}

		StopNode = GetNext(StopNode);
		FLoopNode* StartSegment = GetPrevious(EndSegment);

		const FVector2d& EndPoint = EndSegment->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& StartPoint = StartSegment->Get2DPoint(EGridSpace::UniformScaled, Grid);

		for (; Node != StopNode; Node = GetNext(Node))
		{
			ProjectNodeOnSegment(Node, StartPoint, EndPoint);
		}

		for (Node = GetNodeAt(StartIndex); Node != StopNode; )
		{
			if (Node == nullptr)
			{
				return false;
			}

			FLoopNode* NodeToProcess = Node;
			Node = GetNext(Node);
			if (Node == nullptr || Node->IsDelete())
			{
				return false;
			}
			CheckAndRemovePick(NodeToProcess->GetPreviousNode().Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToProcess->Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToProcess->GetNextNode().Get2DPoint(EGridSpace::UniformScaled, Grid), *NodeToProcess);
		}
		return true;
	};

	TFunction<void(FLoopNode*, FLoopNode*)> MoveNodeBehindOther = [&](FLoopNode* NodeToMove, FLoopNode* Node1Side1)
	{
		FLoopNode* Node0Side1 = GetPrevious(Node1Side1);
		const FVector2d& Point0 = Node0Side1->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& Point1 = Node1Side1->Get2DPoint(EGridSpace::UniformScaled, Grid);

		const FVector2d& PointToMove = NodeToMove->Get2DPoint(EGridSpace::UniformScaled, Grid);

		FVector2d MoveDirection = Point1 - Point0;
		MoveDirection.Normalize();
		MoveDirection = FVector2d(-MoveDirection.Y, MoveDirection.X);
		MoveDirection *= Tolerances.GeometricTolerance;

		FVector2d NewCoordinate = Point1 + MoveDirection;
		NodeToMove->Set2DPoint(EGridSpace::UniformScaled, Grid, NewCoordinate);
	};

	for (int32 Index = IntersectionCount - 1; Index >= 0; --Index)
	{
		int32 SecondIntersectionIndex = IntersectionIndex + Index;

#ifdef DEBUG_REMOVE_INTERSECTIONS		
		Grid.DisplayGridPolyline(FString::Printf(TEXT("RemoveIntersectionsOfSubLoop Step %d"), Index), EGridSpace::UniformScaled, NodesOfLoop, true);
		Grid.DisplayNodes(TEXT("RemoveIntersectionsOfSubLoop Nodes"), EGridSpace::UniformScaled, (TArray<const FIsoNode*>) NodesOfLoop, EVisuProperty::BluePoint);
#endif


		if (Index > 0)
		{
#ifdef DEBUG_REMOVE_INTERSECTIONS		
			DisplayIntersection(Intersections[SecondIntersectionIndex]);
			DisplayIntersection(Intersections[SecondIntersectionIndex - 1]);
			//Wait();
#endif
			const TPair<double, double>& SecondIntersection = Intersections[SecondIntersectionIndex];
			const TPair<double, double>& FirstIntersection = Intersections[SecondIntersectionIndex - 1];

			int32 Side0NodeCount = (int32)SecondIntersection.Key - (int32)FirstIntersection.Key;
			int32 Side1NodeCount = (int32)FirstIntersection.Value - (int32)SecondIntersection.Value;

			int32 IndexSide0 = NextIndex((int32)FirstIntersection.Key);
			int32 IndexSide1 = NextIndex((int32)SecondIntersection.Value);

			if (Side0NodeCount == 0)
			{
				if (!ProjectNodesOnSegment(IndexSide1, FitNodeIndex((int32)FirstIntersection.Value), IndexSide0))
				{
					return false;
				}
			}
			else if (Side1NodeCount == 0)
			{
				if (!ProjectNodesOnSegment(IndexSide0, FitNodeIndex((int32)SecondIntersection.Key), IndexSide1))
				{
					return false;
				}
			}
			else if (Side0NodeCount == 1 && Side1NodeCount == 1)
			{
				FLoopNode* NodeSide0 = GetNodeAt(IndexSide0);
				FLoopNode* NodeSide1 = GetNodeAt(IndexSide1);
				if (NodeSide0 == nullptr || NodeSide1 == nullptr)
				{
					return false;
				}


				double SlopeSide0 = ComputeUnorientedSlope(GetPrevious(NodeSide0)->Get2DPoint(EGridSpace::UniformScaled, Grid), NodeSide0->Get2DPoint(EGridSpace::UniformScaled, Grid), GetNext(NodeSide0)->Get2DPoint(EGridSpace::UniformScaled, Grid));
				double SlopeSide1 = ComputeUnorientedSlope(GetPrevious(NodeSide1)->Get2DPoint(EGridSpace::UniformScaled, Grid), NodeSide1->Get2DPoint(EGridSpace::UniformScaled, Grid), GetNext(NodeSide1)->Get2DPoint(EGridSpace::UniformScaled, Grid));

				if (SlopeSide0 < SlopeSide1)
				{
					MoveNodeBehindOther(NodeSide1, NodeSide0);
				}
				else
				{
					MoveNodeBehindOther(NodeSide0, NodeSide1);
				}
			}
			else
			{
				FLoopSection IntersectingSection;
				IntersectingSection.Key = GetNodeAt(IndexSide0);
				IntersectingSection.Value = GetNodeAt((int32)SecondIntersection.Key);
				if(IntersectingSection.Key == nullptr || IntersectingSection.Value == nullptr)
				{
					return false;
				}

				IntersectingSection.Key = GetPrevious(IntersectingSection.Key);
				IntersectingSection.Value = GetNext(IntersectingSection.Value);
				if (IntersectingSection.Key == nullptr || IntersectingSection.Key->IsDelete() || IntersectingSection.Value == nullptr || IntersectingSection.Value->IsDelete())
				{
					return false;
				}

				FLoopSection OppositeSection;
				OppositeSection.Key = GetNodeAt(IndexSide1);
				OppositeSection.Value = GetNodeAt((int32)FirstIntersection.Value);
				if (OppositeSection.Key == nullptr || OppositeSection.Value == nullptr)
				{
					return false;
				}

				OppositeSection.Key = GetPrevious(OppositeSection.Key);
				OppositeSection.Value = GetNext(OppositeSection.Value);

				if (OppositeSection.Key == nullptr || OppositeSection.Key->IsDelete() || OppositeSection.Value == nullptr || OppositeSection.Value->IsDelete())
				{
					return false;
				}

				int32 IntersectingSectionCount = (int32)FirstIntersection.Value - IndexSide1;
				int32 OppositeSectionCount = (int32)SecondIntersection.Key - IndexSide0;
				if (OppositeSectionCount < IntersectingSectionCount)
				{
					Swap(IntersectingSection, OppositeSection);
				}

				if(!MoveIntersectingSectionBehindOppositeSection(IntersectingSection, OppositeSection))
				{
					return false;
				}

#ifdef DEBUG_REMOVE_INTERSECTIONS		
				Grid.DisplayGridPolyline(TEXT("RemoveIntersectionsOfSubLoop start"), EGridSpace::UniformScaled, NodesOfLoop, true);
#endif

				// Check if there is no more intersection, otherwise fix last intersections by moving node of OppositeSection
				// As we don't know the existence of intersection, the process checks and fixes
				{
					LoopSegmentsIntersectionTool.Empty(NodesOfLoopCount);
					for (FLoopNode* Node = IntersectingSection.Key; Node != IntersectingSection.Value; Node = GetNext(Node))
					{
						if (Node->IsDelete())
						{
							// should not happen => so the process is canceled
							return false;
						}
						
						FIsoSegment* Segment = Node->GetSegmentConnectedTo(GetNext(Node));
						if(!Segment)
						{
							// should not happen => so the process is canceled
							return false;
						}

						LoopSegmentsIntersectionTool.AddSegment(*Segment);
					}
				}

				for (FLoopNode* Node = OppositeSection.Key; Node != OppositeSection.Value; Node = GetNext(Node))
				{
					if (Node->IsDelete() || OppositeSection.Value->IsDelete())
					{
						// should not happen => so the process is canceled
						return false;
					}

					if (const FIsoSegment* Intersection = LoopSegmentsIntersectionTool.FindIntersectingSegment(*Node, *GetNext(Node)))
					{
						MoveNodeBehindSegment(*Intersection, *GetNext(Node));
					}
				}

#ifdef DEBUG_REMOVE_INTERSECTIONS		
				Grid.DisplayGridPolyline(TEXT("RemoveIntersectionsOfSubLoop second step"), EGridSpace::UniformScaled, NodesOfLoop, true);
#endif

				// Remove the pick if needed
				for (FLoopNode* Node = OppositeSection.Key; Node != OppositeSection.Value;)
				{
					FLoopNode* NodeToProceed = Node;
					Node = GetNext(Node);
					if (CheckAndRemovePick(NodeToProceed->GetPreviousNode().Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToProceed->Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToProceed->GetNextNode().Get2DPoint(EGridSpace::UniformScaled, Grid), *NodeToProceed))
					{
						Node = GetPrevious(Node);
					}
				}

#ifdef DEBUG_REMOVE_INTERSECTIONS		
				Grid.DisplayGridPolyline(TEXT("RemoveIntersectionsOfSubLoop after remove pick"), EGridSpace::UniformScaled, NodesOfLoop, true);
				Wait(bDisplay);
#endif
			}
			--Index;
		}
		else
		{
			if (!RemoveIntersection(Intersections[IntersectionIndex]))
			{
				return false;
			}
		}
	}

	return true;
}

bool FLoopCleaner::RemovePickOrCoincidenceBetween(FLoopNode* StartNode, FLoopNode* StopNode)
{
	if (StartNode->IsDelete() || StopNode->IsDelete())
	{
		// should not happen => so the process is canceled
		return false;
	}

	for (FLoopNode* Node = StartNode; Node != StopNode; )
	{
		FLoopNode* NodeToBeProcessed = Node;
		Node = GetNext(Node);
		if (Node == nullptr || Node->IsDelete())
		{
			return false;
		}

		const FVector2d& PreviousPoint = NodeToBeProcessed->GetPreviousNode().Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d& PointToBeProcessed = NodeToBeProcessed->Get2DPoint(EGridSpace::UniformScaled, Grid);
		if (CheckAndRemovePick(PreviousPoint, NodeToBeProcessed->Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToBeProcessed->GetNextNode().Get2DPoint(EGridSpace::UniformScaled, Grid), *NodeToBeProcessed) ||
			CheckAndRemoveCoincidence(PreviousPoint, PointToBeProcessed, *NodeToBeProcessed))
		{
			Node = GetPrevious(Node);
		}
	}
	return true;
}

bool FLoopCleaner::RemoveIntersection(TPair<double, double>& Intersection)
{
	int32 SubLoopSegmentCount = (int32)Intersection.Value - (int32)Intersection.Key;
	const int32 OtherSubLoopSegmentCount = NodesOfLoopCount - (int32)Intersection.Value + (int32)Intersection.Key;

	bool bIntersectionKeyIsExtremity = FMath::IsNearlyEqual(Intersection.Key, (int32)(Intersection.Key + 0.5));
	bool bIntersectionValueIsExtremity = FMath::IsNearlyEqual(Intersection.Value, (int32)(Intersection.Value + 0.5));

	if(bIntersectionKeyIsExtremity && bIntersectionValueIsExtremity)
	{   
		LoopCleanerImpl::FPinchIntersectionContext Context(Intersection);
		if (!Fill(Context))
		{
			return false;
		}

		if (IsAPinch(Context))
		{
			return DisconnectCoincidentNodes(Context);
		}
		else
		{
			return DisconnectCrossingSegments(Context);
		}
	}

	/*
	else if(bIntersectionKeyIsExtremity || bIntersectionValueIsExtremity)
	{
		// Case:  ________
		//          _/\_
		return MovePickBehind(NodesOfLoop, Intersection, bIntersectionKeyIsExtremity);
	}
	*/

	if (OtherSubLoopSegmentCount < SubLoopSegmentCount)
	{
		Swap(Intersection.Value, Intersection.Key);
		SubLoopSegmentCount = OtherSubLoopSegmentCount;
	}

#ifdef DEBUG_REMOVE_UNIQUE_INTERSECTION
	Grid.DisplayGridPolyline(TEXT("RemoveIntersection start"), EGridSpace::UniformScaled, NodesOfLoop, true);
	Wait(bDisplay);
#endif

	switch (SubLoopSegmentCount)
	{
	case 0:
		ensureCADKernel(false);
		break;

	case 1:
	case 2:
		// At least 2 nodes are behind the intersection, remove the node making the intersection
		if (!RemoveOuterNode(Intersection))
		{
			return false;
		}
		break;

	case 3:
		// 3 nodes are behind the intersection, remove the node making the intersection
		//    ______a  c       ______a__b
		//           \/ \o	            \o
		//    ______d/\b/	  ______d__c/
		//	    			  
		if (!SwapNodes(Intersection))
		{
			return false;
		}
		break;

	default:
		if (!TryToSwapSegmentsOrRemoveLoop(Intersection))
		{
			return false;
		}
		break;
	}

#ifdef DEBUG_REMOVE_UNIQUE_INTERSECTION
	Grid.DisplayGridPolyline(TEXT("RemoveIntersection end"), EGridSpace::UniformScaled, NodesOfLoop, true);
	Wait(bDisplay);
#endif
	return true;
};

bool FLoopCleaner::MoveIntersectingSectionBehindOppositeSection(LoopCleanerImpl::FLoopSection IntersectingSection, LoopCleanerImpl::FLoopSection OppositeSection)
{
	using namespace LoopCleanerImpl;

	FLoopNode* FirstNodeIntersectingSection = IntersectingSection.Key;
	FLoopNode* LastNodeIntersectingSection = IntersectingSection.Value;

	FLoopNode* FirstNodeOppositeSection = OppositeSection.Key;
	FLoopNode* LastNodeOppositeSection = OppositeSection.Value;

	int32 OppositeSectionCount = 1;
	for (FLoopNode* SegmentNode = FirstNodeOppositeSection; SegmentNode != LastNodeOppositeSection; SegmentNode = GetNext(SegmentNode), ++OppositeSectionCount);

	TArray<const FVector2d*> OppositeSectionPoint;
	OppositeSectionPoint.Reserve(OppositeSectionCount);

	for (FLoopNode* SegmentNode = FirstNodeOppositeSection; SegmentNode != LastNodeOppositeSection; SegmentNode = GetNext(SegmentNode))
	{
		OppositeSectionPoint.Add(&SegmentNode->Get2DPoint(EGridSpace::UniformScaled, Grid));
	}
	OppositeSectionPoint.Add(&LastNodeOppositeSection->Get2DPoint(EGridSpace::UniformScaled, Grid));

#ifdef DEBUG_MOVE_INTERSECTING_SECTION_BEHIND_OPPOSITE_SECTION
	{
		F3DDebugSession _(TEXT("IntersectingSection"));
		for (FLoopNode* Node = FirstNodeIntersectingSection; Node != GetNext(LastNodeIntersectingSection); Node = GetNext(Node))
		{
			DisplayPoint(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), EVisuProperty::PurplePoint);
		}
	}
	{
		F3DDebugSession _(TEXT("OppositeSection"));
		for (FLoopNode* Node = FirstNodeOppositeSection; Node != GetNext(LastNodeOppositeSection); Node = GetNext(Node))
		{
			DisplayPoint(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), EVisuProperty::YellowPoint);
		}
		Wait();
	}
#endif

	double Coordinate;
	FLoopNode* NextNode = nullptr;
	for (FLoopNode* Node = GetNext(FirstNodeIntersectingSection); Node != LastNodeIntersectingSection;)
	{
		FLoopNode* NodeToBeProcessed = Node;
		Node = GetNext(Node);

		FVector2d  CandiatePosition;
		double MinSquareDistance = HUGE_VALUE;
		FVector2d PointToProject = NodeToBeProcessed->Get2DPoint(EGridSpace::UniformScaled, Grid);
		for (int32 Index = 1; Index < OppositeSectionCount; ++Index)
		{
			FVector2d ProjectedPoint = ProjectPointOnSegment<FVector2d>(PointToProject, *OppositeSectionPoint[Index - 1], *OppositeSectionPoint[Index], Coordinate);
			double SquareDistance = FVector2d::DistSquared(PointToProject, ProjectedPoint);
			if (SquareDistance < MinSquareDistance)
			{
				MinSquareDistance = SquareDistance;
				CandiatePosition = ProjectedPoint;
			}
		}
		MoveNode(*NodeToBeProcessed, CandiatePosition);
	}

	for (FLoopNode* Node = GetNext(FirstNodeIntersectingSection); Node != LastNodeIntersectingSection; )
	{
		FLoopNode* NodeToProceed = Node;
		Node = GetNext(Node);
		if (CheckAndRemovePick(NodeToProceed->GetPreviousNode().Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToProceed->Get2DPoint(EGridSpace::UniformScaled, Grid), NodeToProceed->GetNextNode().Get2DPoint(EGridSpace::UniformScaled, Grid), *NodeToProceed))
		{
			Node = GetPrevious(Node);
		}
	}
	return true;
}

void FLoopCleaner::MoveNodeBehindSegment(const FIsoSegment& IntersectingSegment, FLoopNode& NodeToMove)
{
	const FIsoNode* Nodes[2] = { nullptr, nullptr };
	Nodes[0] = &IntersectingSegment.GetFirstNode();
	Nodes[1] = &IntersectingSegment.GetSecondNode();

	FVector2d IntersectingPoints[2];
	IntersectingPoints[0] = IntersectingSegment.GetFirstNode().Get2DPoint(EGridSpace::UniformScaled, Grid);
	IntersectingPoints[1] = IntersectingSegment.GetSecondNode().Get2DPoint(EGridSpace::UniformScaled, Grid);

	bool bEndNodeIsOutside = true;
	FVector2d PointToMove = NodeToMove.Get2DPoint(EGridSpace::UniformScaled, Grid);

#ifdef DEBUG_CLOSED_OUSIDE_POINT
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("Outside Point"));
		DisplayPoint(PointToMove, bEndNodeIsOutside ? EVisuProperty::GreenPoint : EVisuProperty::YellowPoint);
	}
#endif

	double Coordinate;
	FVector2d ProjectedPoint = ProjectPointOnSegment<FVector2d>(PointToMove, IntersectingPoints[0], IntersectingPoints[1], Coordinate);
	MoveNode(NodeToMove, ProjectedPoint);
}

bool FLoopCleaner::Fill(LoopCleanerImpl::FPinchIntersectionContext& Context)
{
	TFunction<bool(int32, const int32)> FillData = [&](int32 Index, const int32 Side) -> bool
	{
		Context.Points[Side].SetNum(3);
		Context.Nodes[Side][1] = GetNodeAt(Index);
		if (Context.Nodes[Side][1] == nullptr)
		{
			return false;
		}

		Context.Nodes[Side][0] = &Context.Nodes[Side][1]->GetPreviousNode();
		Context.Nodes[Side][2] = &Context.Nodes[Side][1]->GetNextNode();
		if (Context.Nodes[Side][0] == nullptr || Context.Nodes[Side][0]->IsDelete() || Context.Nodes[Side][2] == nullptr || Context.Nodes[Side][2]->IsDelete())
		{
			return false;
		}

		Context.Points[Side][0] = &Context.Nodes[Side][0]->Get2DPoint(EGridSpace::UniformScaled, Grid);
		Context.Points[Side][1] = &Context.Nodes[Side][1]->Get2DPoint(EGridSpace::UniformScaled, Grid);
		Context.Points[Side][2] = &Context.Nodes[Side][2]->Get2DPoint(EGridSpace::UniformScaled, Grid);
		return true;
	};

	if (!FillData((int32)(Context.Intersection.Key + 0.5), 0))
	{
		return false;
	}

	if (!FillData((int32)(Context.Intersection.Value + 0.5), 1))
	{
		return false;
	}
	return true;
}

bool FLoopCleaner::IsAPinch(const LoopCleanerImpl::FPinchIntersectionContext& Context) const
{
	return ArePointsInsideSectorABC(*Context.Points[0][2], *Context.Points[0][1], *Context.Points[0][0], Context.Points[1]);
}

bool FLoopCleaner::DisconnectCoincidentNodes(const LoopCleanerImpl::FPinchIntersectionContext& Context)
{
	double Slope00 = ComputeSlope(*Context.Points[0][1], *Context.Points[0][0]);
	double Slope02 = ComputeSlope(*Context.Points[0][1], *Context.Points[0][2]);
	double Slope10 = ComputeSlope(*Context.Points[1][1], *Context.Points[1][0]);
	double Slope12 = ComputeSlope(*Context.Points[1][1], *Context.Points[1][2]);

	double MediumSlope0 = TransformIntoPositiveSlope((Slope00 + Slope02) * 0.5);
	double MediumSlope1 = TransformIntoPositiveSlope((Slope10 + Slope12) * 0.5);

	// #cadkernel_check: Why is this constant equal to 30????
	constexpr double ScaleFactor = 30.;
	const double Scale = Tolerances.GeometricTolerance * ScaleFactor;
	FVector2d P1 = SlopeToVector(MediumSlope1).GetSafeNormal() * Scale;
	FVector2d P0 = SlopeToVector(MediumSlope0).GetSafeNormal() * Scale;

	P0 += *Context.Points[0][1];
	P1 += *Context.Points[1][1];

	Context.Nodes[0][1]->Set2DPoint(EGridSpace::UniformScaled, Grid, P0);
	Context.Nodes[1][1]->Set2DPoint(EGridSpace::UniformScaled, Grid, P1);

#ifdef DEBUG_DISCONNECT_COINCIDENT_NODES
	F3DDebugSession _(TEXT("DisconnectCoincidentNodes"));
	DisplaySegment(*Context.Points[0][1], P0, 0, EVisuProperty::BlueCurve);
	DisplaySegment(*Context.Points[1][1], P1, 0, EVisuProperty::RedCurve);
	Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Context.Nodes[0][1], 0, EVisuProperty::BluePoint);
	Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Context.Nodes[1][1], 0, EVisuProperty::RedPoint);
	Wait();
#endif

	return true;
}


/**
 * CrossingCase:
 *           --<-o    o->--       --<-o   o--<-
 *           |    \  /    |       |    \ /    |
 *           |     oo     |   =>  |     o     |
 *           |    /  \    |       |     o     |
 *           -->-o    o-<--       |    / \    |
 *                                -->-o   o->--
 */
//#define DEBUG_DISCONNECT_CROSSING_SEGMENTS
bool FLoopCleaner::DisconnectCrossingSegments(LoopCleanerImpl::FPinchIntersectionContext& Context)
{

#ifdef DEBUG_DISCONNECT_CROSSING_SEGMENTS
	{
		F3DDebugSession C(bDisplay, TEXT("DisconnectCrossingSegments"));
		if (bDisplay)
		{
			Grid.DisplayIsoSegments(TEXT("Loops Orientation"), EGridSpace::UniformScaled, LoopSegments, false, true, EVisuProperty::BlueCurve);
			Wait();
		}
	}
#endif

	FIsoSegment* Segment01_02 = Context.Nodes[0][1]->GetSegmentConnectedTo(Context.Nodes[0][2]);
	FIsoSegment* Segment10_11 = Context.Nodes[1][0]->GetSegmentConnectedTo(Context.Nodes[1][1]);

	Context.Nodes[0][2]->DisconnectSegment(*Segment01_02);
	Context.Nodes[1][0]->DisconnectSegment(*Segment10_11);

	Context.Nodes[0][2]->ConnectSegment(*Segment10_11);
	Context.Nodes[1][0]->ConnectSegment(*Segment01_02);

	Segment10_11->ReplaceNode(*Context.Nodes[1][0], *Context.Nodes[0][2]);
	Segment01_02->ReplaceNode(*Context.Nodes[0][2], *Context.Nodes[1][0]);

	Swap(Context.Nodes[0][2], Context.Nodes[1][0]);
	Swap(Context.Points[0][2], Context.Points[1][0]);

	DisconnectCoincidentNodes(Context);

#ifdef DEBUG_DISCONNECT_CROSSING_SEGMENTS
	F3DDebugSession _(bDisplay, TEXT("DisconnectCrossingSegments"));
	if (bDisplay)
	{
		{
			F3DDebugSession _(TEXT("New Segment from 00"));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Context.Nodes[0][0], Context.Nodes[0][0]->GetNextNode(), 0, EVisuProperty::YellowCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, Context.Nodes[0][0]->GetNextNode(), Context.Nodes[0][0]->GetNextNode().GetNextNode(), 0, EVisuProperty::YellowCurve);
		}
		{
			F3DDebugSession _(TEXT("New Segment fron 10"));
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Context.Nodes[1][0], Context.Nodes[1][0]->GetNextNode(), 0, EVisuProperty::YellowCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, Context.Nodes[1][0]->GetNextNode(), Context.Nodes[1][0]->GetNextNode().GetNextNode(), 0, EVisuProperty::YellowCurve);
		}
		Grid.DisplayIsoSegments(TEXT("Loops Orientation"), EGridSpace::UniformScaled, LoopSegments, false, true, EVisuProperty::BlueCurve);
		Wait();
	}
#endif

	int32 Segment01_02Index = LoopSegments.IndexOfByKey(Segment01_02);
	int32 Segment10_11Index = LoopSegments.IndexOfByKey(Segment10_11);

#ifdef DEBUG_REMOVE_UNIQUE_INTERSECTION
	Grid.DisplayIsoSegments(TEXT("Loops Orientation"), EGridSpace::UniformScaled, LoopSegments, false, true, false, EVisuProperty::BlueCurve);
	Grid.DisplayGridPolyline(TEXT("RemoveIntersection end"), EGridSpace::UniformScaled, NodesOfLoop, true);
	Wait(bDisplay);
#endif

	SwapSubLoopOrientation(Segment01_02Index + 1, Segment10_11Index);

#ifdef DEBUG_REMOVE_UNIQUE_INTERSECTION
	Grid.DisplayIsoSegments(TEXT("Loops Orientation"), EGridSpace::UniformScaled, LoopSegments, false, true, false, EVisuProperty::BlueCurve);
	Grid.DisplayGridPolyline(TEXT("RemoveIntersection end"), EGridSpace::UniformScaled, NodesOfLoop, true);
	Wait(bDisplay);
#endif

	return true;
}

// Case  ______a  c       ______a__b
//              \/ \o	            \o
//       ______d/\b/	  ______d__c/
//	    			  
bool FLoopCleaner::SwapNodes(const TPair<double, double>& Intersection)
{
	FLoopNode* Node0 = GetNodeAt(NextIndex((int32)Intersection.Key));
	if (Node0 == nullptr)
	{
		return false;
	}

	FLoopNode* Pick = GetNext(Node0);
	FLoopNode* Node1 = GetNext(Pick);

	const FVector2d Point0Copy = Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& Point1 = Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);

	Node0->Set2DPoint(EGridSpace::UniformScaled, Grid, Point1);
	Node1->Set2DPoint(EGridSpace::UniformScaled, Grid, Point0Copy);
	return true;
};

bool FLoopCleaner::TryToSwapSegmentsOrRemoveLoop(const TPair<double, double>& Intersection)
{
	//using namespace IsoTriangulatorImpl;

	const int32 Segment0StartIndex = NextIndex((int32)Intersection.Key);
	const int32 Segment1EndIndex = (int32)Intersection.Value;

	FLoopNode* Segment0_Node1 = GetNodeAt(Segment0StartIndex);
	FLoopNode* Segment1_Node0 = GetNodeAt(Segment1EndIndex);
	if (Segment0_Node1 == nullptr || Segment1_Node0 == nullptr)
	{
		return false;
	}

	FLoopNode* Segment0_Node0 = GetPrevious(Segment0_Node1);
	FLoopNode* Segment1_Node1 = GetNext(Segment1_Node0);
	if (Segment0_Node0 == nullptr || Segment0_Node0->IsDelete() || Segment1_Node1 == nullptr || Segment1_Node1->IsDelete())
	{
		return false;
	}

	const FVector2d& Segment0_Point0 = Segment0_Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& Segment0_Point1 = Segment0_Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& Segment1_Point0 = Segment1_Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d& Segment1_Point1 = Segment1_Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);

#ifdef DEBUG_SWAP_SEGMENTS_OR_REMOVE
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("Intersected Segments"));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment0_Node0, *Segment0_Node1, 0, EVisuProperty::RedCurve);
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment1_Node0, *Segment1_Node1, 0, EVisuProperty::RedCurve);
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Segment0_Node0, 0, EVisuProperty::RedPoint);
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Segment1_Node0, 0, EVisuProperty::RedPoint);
		Wait(false);
	}
#endif

	double Slope = ComputeSlope(Segment0_Point0, Segment0_Point1);
	Slope = ComputeUnorientedSlope(Segment1_Point0, Segment1_Point1, Slope);

	if (Slope < 2)
	{
		FIsoSegment* Segment0 = Segment0_Node0->GetSegmentConnectedTo(Segment0_Node1);
		FIsoSegment* Segment1 = Segment1_Node0->GetSegmentConnectedTo(Segment1_Node1);
		if (Segment0 == nullptr || Segment1 == nullptr)
		{
			return false;
		}

		if (!LoopSegmentsIntersectionTool.DoesIntersect(Segment0->GetFirstNode(), Segment1->GetFirstNode())
		&& !LoopSegmentsIntersectionTool.DoesIntersect(Segment0->GetSecondNode(), Segment1->GetSecondNode()))
		{
			SwapSegments(*Segment0, *Segment1);

#ifdef DEBUG_SWAP_SEGMENTS_OR_REMOVE
			if (bDisplay)
			{
				F3DDebugSession _(TEXT("New Segments"));
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment0_Node0, Segment0_Node0->GetNextNode(), 0, EVisuProperty::BlueCurve);
				Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment1_Node1, Segment1_Node1->GetPreviousNode(), 0, EVisuProperty::BlueCurve);
				Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Segment0_Node0, 0, EVisuProperty::BluePoint);
				Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Segment1_Node0, 0, EVisuProperty::BluePoint);
			}
#endif
			return true;
		}
	}

	return RemoveSubLoop(Segment0_Node0, Segment1_Node1);
}

void FLoopCleaner::SwapSegments(FIsoSegment& IntersectingSegment, FIsoSegment& Segment)
{
	int32 IntersectingSegmentIndex = LoopSegments.IndexOfByKey(&IntersectingSegment);
	int32 SegmentIndex = LoopSegments.IndexOfByKey(&Segment);

	IntersectingSegment.GetSecondNode().DisconnectSegment(IntersectingSegment);
	Segment.GetFirstNode().DisconnectSegment(Segment);

	FIsoNode& Node = IntersectingSegment.GetSecondNode();
	IntersectingSegment.SetSecondNode(Segment.GetFirstNode());
	Segment.SetFirstNode(Node);

	Segment.GetFirstNode().ConnectSegment(Segment);
	IntersectingSegment.GetSecondNode().ConnectSegment(IntersectingSegment);

#ifdef DEBUG_BY_SWAPPING_SEGMENTS
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("New Segments"));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, Segment, SegmentIndex, EVisuProperty::RedCurve/*, true*/);
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, IntersectingSegment, IntersectingSegmentIndex, EVisuProperty::RedCurve/*, true*/);
		Wait();
	}
#endif

#ifdef DEBUG_BY_SWAPPING_SEGMENTS
	if (bDisplay)
	{
		Grid.DisplayIsoSegments(TEXT("Before orientation"), EGridSpace::UniformScaled, LoopSegments, false, true);
		Wait();
	}
#endif

	SwapSubLoopOrientation(IntersectingSegmentIndex + 1, SegmentIndex);

#ifdef DEBUG_BY_SWAPPING_SEGMENTS
	if (bDisplay)
	{
		Grid.DisplayIsoSegments(TEXT("After orientation"), EGridSpace::UniformScaled, LoopSegments, false, true);
		Wait();
	}
#endif

	LoopSegmentsIntersectionTool.Update(&IntersectingSegment);
	LoopSegmentsIntersectionTool.Update(&Segment);
}

void FLoopCleaner::SwapSubLoopOrientation(int32 FirstSegmentIndex, int32 LastSegmentIndex)
{
	FirstSegmentIndex = FitSegmentIndex(FirstSegmentIndex);
	LastSegmentIndex = FitSegmentIndex(LastSegmentIndex);

	TArray<FIsoSegment*> Segments;
	Segments.Reserve(SegmentCount);

	int32 Index = FirstSegmentIndex;
	do
	{
		LoopSegments[Index]->SwapOrientation();
		Segments.Add(LoopSegments[Index]);
		Index = NextSegmentIndex(Index);
	}
	while(Index != LastSegmentIndex);

	Index = FirstSegmentIndex;
	for (int32 ReverseIndex = Segments.Num() - 1; ReverseIndex >= 0; --ReverseIndex)
	{
		LoopSegments[Index] = Segments[ReverseIndex];
		Index = NextSegmentIndex(Index);
	}
}

bool FLoopCleaner::RemoveSubLoop(FLoopNode* StartNode, FLoopNode* EndNode)
{
	FLoopNode* Node = GetNext(StartNode);
	while (Node && (Node != EndNode) && !Node->IsDelete())
	{
		if (!RemoveNodeOfLoop(*Node))
		{
			return false;
		}

		Node = GetNext(StartNode);
	}

	return true;
}

void FLoopCleaner::MoveNode(FLoopNode& NodeToMove, FVector2d& NewPostion)
{
	const FVector2d& PointToMove = NodeToMove.Get2DPoint(EGridSpace::UniformScaled, Grid);

	FVector2d ProjectedDirection = NewPostion - PointToMove;
	ProjectedDirection.Normalize();
	ProjectedDirection *= Tolerances.GeometricTolerance;

	FVector2d NewCoordinate = NewPostion + ProjectedDirection;

	FLoopNode& PreviousNode = NodeToMove.GetPreviousNode();
	FLoopNode& NextNode = NodeToMove.GetNextNode();

	if ((FVector2d::DistSquared(PreviousNode.Get2DPoint(EGridSpace::UniformScaled, Grid), NewCoordinate) < Tolerances.SquareGeometricTolerance2) ||
		(FVector2d::DistSquared(NextNode.Get2DPoint(EGridSpace::UniformScaled, Grid), NewCoordinate) < Tolerances.SquareGeometricTolerance2))
	{
		RemoveNodeOfLoop(NodeToMove);
	}
	else
	{
		NodeToMove.Set2DPoint(EGridSpace::UniformScaled, Grid, NewCoordinate);
	}

#ifdef DEBUG_MOVE_NODE
	if (bDisplay)
	{
		{
			F3DDebugSession _(TEXT("Point To Move"));
			DisplayPoint(PointToMove, EVisuProperty::YellowPoint);
		}
		{
			F3DDebugSession _(TEXT("Projected Point"));
			DisplayPoint(NewPostion, EVisuProperty::GreenPoint);
		}
		{
			F3DDebugSession _(TEXT("New Position"));
			DisplayPoint(NewCoordinate, EVisuProperty::BluePoint);
			Wait(false);
		}
	}
#endif
}

void FLoopCleaner::FindLoopIntersections()
{
	LoopSegmentsIntersectionTool.Empty(NodesOfLoopCount);

	FLoopNode* const* const StartNodePtr = NodesOfLoop.FindByPredicate([](const FLoopNode* Node) { return !Node->IsDelete(); });
	if (StartNodePtr == nullptr || *StartNodePtr == nullptr || (*StartNodePtr)->IsDelete())
	{
		return;
	}

	FLoopNode* StartNode = *StartNodePtr;

	TArray<const FIsoSegment*> IntersectedSegments;

	SegmentCount = 1;
	int32 SegmentIndex = 1;
	FLoopNode* Node = nullptr;
	FLoopNode* NextNode = nullptr;

	TFunction<void()> FindSegmentIntersection = [&]()
	{
		FIsoSegment* Segment = Node->GetSegmentConnectedTo(NextNode);
		if (Segment == nullptr)
		{
			return;
		}

		if (LoopSegmentsIntersectionTool.FindIntersectingSegments(*Node, *NextNode, IntersectedSegments))
		{
			for (const FIsoSegment* IntersectedSegment : IntersectedSegments)
			{
				const FLoopNode* IntersectedSegmentFirstNode = GetFirst(IntersectedSegment);
				if (IntersectedSegmentFirstNode == nullptr)
				{
					continue;
				}

				int32 IntersectionIndex = 0;
				FLoopNode* TmpNode = StartNode;
				while (TmpNode != IntersectedSegmentFirstNode)
				{
					++IntersectionIndex;
					TmpNode = GetNext(TmpNode);
				}

				FSegment2D SegmentPoints(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid));
				FSegment2D IntersectedSegmentPoints(GetFirst(IntersectedSegment)->Get2DPoint(EGridSpace::UniformScaled, Grid), GetSecond(IntersectedSegment)->Get2DPoint(EGridSpace::UniformScaled, Grid));

				double SegmentIntersectionCoordinate;
				FVector2d IntersectionPoint = FindIntersectionOfSegments2D(SegmentPoints, IntersectedSegmentPoints, SegmentIntersectionCoordinate);

				const double IntersectedSegmentPointsSquareLength = IntersectedSegmentPoints.SquaredLength();
				const double SquareDistanceToIntersectionPoint = FVector2d::DistSquared(IntersectedSegmentPoints[0], IntersectionPoint);
				if (SquareDistanceToIntersectionPoint > IntersectedSegmentPointsSquareLength)
				{
					// the IntersectionPoint is not inside IntersectedSegmentPoints
					continue;
				}

				double IntersectedSegmentIntersectionCoordinate = (IntersectedSegmentPointsSquareLength > DOUBLE_KINDA_SMALL_NUMBER_SQUARE) ? FMath::Sqrt(SquareDistanceToIntersectionPoint / IntersectedSegmentPointsSquareLength) : 0.;
				IntersectedSegmentIntersectionCoordinate += IntersectionIndex;

				SegmentIntersectionCoordinate += SegmentIndex;
				if (FMath::IsNearlyEqual(SegmentIntersectionCoordinate, SegmentCount))
				{
					SegmentIntersectionCoordinate = IntersectedSegmentIntersectionCoordinate;
					IntersectedSegmentIntersectionCoordinate = SegmentCount;
				}

				if (IntersectedSegmentIntersectionCoordinate > SegmentIntersectionCoordinate)
				{
					SegmentIntersectionCoordinate += NodesOfLoopCount;
				}

				if ((SegmentIntersectionCoordinate - IntersectedSegmentIntersectionCoordinate) > NodesOfLoopCount * 0.5)
				{
					Swap(SegmentIntersectionCoordinate, IntersectedSegmentIntersectionCoordinate);
					SegmentIntersectionCoordinate += NodesOfLoopCount;
				}

				while (IntersectedSegmentIntersectionCoordinate >= NodesOfLoopCount)
				{
					IntersectedSegmentIntersectionCoordinate -= NodesOfLoopCount;
					SegmentIntersectionCoordinate -= NodesOfLoopCount;
				}

 				if (Intersections.Num() && FMath::IsNearlyEqual(Intersections.Last().Key, IntersectedSegmentIntersectionCoordinate) && FMath::IsNearlyEqual(Intersections.Last().Value, SegmentIntersectionCoordinate))
				{
					continue;
				}

				Intersections.Emplace(IntersectedSegmentIntersectionCoordinate, SegmentIntersectionCoordinate);

#ifdef DEBUG_FIND_LOOP_INTERSECTIONS		
				if (bDisplay)
				{
					F3DDebugSession _(*FString::Printf(TEXT("Intersection %f %f"), IntersectedSegmentIntersectionCoordinate, SegmentIntersectionCoordinate));
					{
						F3DDebugSession A(TEXT("Seg 1"));
						Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Node, *NextNode, 0, EVisuProperty::BlueCurve);
						Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Node, 0, EVisuProperty::BluePoint);
						Grid.DisplayIsoNode(EGridSpace::UniformScaled, *NextNode, 0, EVisuProperty::BluePoint);
					}
					{
						F3DDebugSession A(TEXT("IntersectedSegment"));
						Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *IntersectedSegment, 0, EVisuProperty::RedCurve);
						Grid.DisplayIsoNode(EGridSpace::UniformScaled, IntersectedSegment->GetFirstNode(), 0, EVisuProperty::RedPoint);
						Grid.DisplayIsoNode(EGridSpace::UniformScaled, IntersectedSegment->GetSecondNode(), 0, EVisuProperty::RedPoint);
					}
					{
						F3DDebugSession A(TEXT("IntersectionPoint"));
						DisplayPoint(IntersectionPoint * DisplayScale, EVisuProperty::YellowPoint);
					}
					Wait();
				}
#endif
			}
		}

		if (Segment != nullptr)
		{
			LoopSegmentsIntersectionTool.AddSegment(*Segment);
		}
	};

	LoopSegmentsIntersectionTool.Empty(NodesOfLoopCount);

	const int32 LimitOfNodesOfLoopCount = NodesOfLoopCount + 5;
	Node = GetNext(StartNode);
	for (; Node != StartNode && SegmentCount < LimitOfNodesOfLoopCount; Node = GetNext(Node), ++SegmentCount);

	if (SegmentCount == LimitOfNodesOfLoopCount)
	{
		return;
	}

	Node = StartNode;
	NextNode = GetNext(Node);

#ifdef DEBUG_FIND_LOOP_INTERSECTIONS		
	if (bDisplay)
	{
		F3DDebugSession _(TEXT("Start Node"));
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Node, *NextNode, 0, EVisuProperty::BlueCurve);
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Node, 0, EVisuProperty::RedPoint);
	}
#endif

	FIsoSegment* StartToEndSegment = StartNode->GetSegmentConnectedTo(NextNode);
	if (StartToEndSegment != nullptr)
	{
		LoopSegmentsIntersectionTool.AddSegment(*StartToEndSegment);
	}

	for (Node = NextNode; Node != StartNode; Node = NextNode, ++SegmentIndex)
	{
		if (Node == nullptr || Node->IsDelete())
		{
			return;
		}

		NextNode = GetNext(Node);
		if (NextNode == nullptr || NextNode->IsDelete())
		{
			return;
		}
		FindSegmentIntersection();
	}

}

bool FLoopCleaner::RemoveOuterNode(const TPair<double, double>& Intersection)
{
	FLoopNode* Node0 = GetNodeAt(NextIndex((int32)Intersection.Key));
	if (Node0 == nullptr)
	{
		return false;
	}

	FLoopNode* PreviousNode = GetPrevious(Node0);
	FLoopNode* Node1 = GetNext(Node0);
	FLoopNode* NextNode = GetNext(Node1);

	const FVector2d* PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* Point0 = &Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* Point1 = &Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

	double Slope0 = ComputeUnorientedSlope(*Point0, *PreviousPoint, *Point1);
	double Slope1 = ComputeUnorientedSlope(*Point1, *PreviousPoint, *NextPoint);

	if (Slope0 < Slope1)
	{
		return RemoveNodeOfLoop(*Node0);
	}
	else
	{
		return RemoveNodeOfLoop(*Node1);
	}
}

bool FLoopCleaner::RemoveLoopPicks()
{
	for (FLoopNode* Node : NodesOfLoop)
	{
		if (Node == nullptr || Node->IsDelete())
		{
			continue;
		}

		if (!RemovePickRecursively(Node, &Node->GetNextNode()))
		{
			return false;
		}
	}
	return UpdateNodesOfLoop();
}

bool FLoopCleaner::RemoveLoopPicks(TArray<FIsoSegment*>& Loop)
{
	for (FIsoSegment* Segment : Loop)
	{
		if (Segment == nullptr || Segment->IsDelete())
		{
			continue;
		}

		if (!RemovePickRecursively((FLoopNode*)&Segment->GetFirstNode(), (FLoopNode*)&Segment->GetSecondNode()))
		{
			return false;
		}
	}
	return UpdateNodesOfLoop();
}

bool FLoopCleaner::FindAndRemoveCoincidence(FLoopNode*& StartNode)
{
#ifdef DEBUG_FIND_AND_REMOVE_COINCIDENCE
	F3DDebugSession A(bDisplay, ("FindAndRemoveCoincidence"));
#endif

	FLoopNode* PreviousNode = StartNode;
	const FVector2d* PreviousPoint = &StartNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	for (FLoopNode* Node = GetNext(StartNode); ; Node = GetNext(Node))
	{
		const FVector2d& Point = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);

		if (CheckAndRemoveCoincidence(*PreviousPoint, Point, *PreviousNode))
		{
			if (Node->IsDelete())
			{
				return false;
			}

			if (PreviousNode == StartNode)
			{
				StartNode = Node;
#ifdef DEBUG_FIND_AND_REMOVE_COINCIDENCE
				if(bDisplay)
				{
					F3DDebugSession B(TEXT("RemoveCoincidence"));
					DisplayPoint2DWithScale(*PreviousPoint, EVisuProperty::RedPoint);
				}
#endif
				continue;
			}
#ifdef DEBUG_FIND_AND_REMOVE_COINCIDENCE
			if (bDisplay)
			{
				F3DDebugSession B(TEXT("RemoveCoincidence"));
				DisplayPoint2DWithScale(*PreviousPoint, EVisuProperty::RedPoint);
			}
		}
		else if(bDisplay)
		{
			F3DDebugSession B(TEXT("Seg"));
			DisplaySegmentWithScale(*PreviousPoint, Point, 1, EVisuProperty::BlueCurve);
#endif
		}
	
		if (Node == StartNode)
		{
			break;
		}

		PreviousNode = Node;
		PreviousPoint = &Point;
	}

	return true;
}

bool FLoopCleaner::RemovePickRecursively(FLoopNode* Node0, FLoopNode* Node1)
{
	if (Node0->IsDelete() || Node1->IsDelete())
	{
		return false;
	}

	FLoopNode* PreviousNode = &Node0->GetPreviousNode();
	FLoopNode* NextNode = &Node1->GetNextNode();

	if (PreviousNode->IsDelete() || NextNode->IsDelete())
	{
		return false;
	}

	const FVector2d* PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* Point0 = &Node0->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* Point1 = &Node1->Get2DPoint(EGridSpace::UniformScaled, Grid);
	const FVector2d* NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

	while (true)
	{
		if (CheckAndRemovePick(*PreviousPoint, *Point0, *Point1, *Node0))
		{
			if (PreviousNode->IsDelete())
			{
				break;
			}

			if (CheckAndRemoveCoincidence(*PreviousPoint, *Point1, *Node1))
			{
				if (PreviousNode->IsDelete())
				{
					break;
				}

				Point1 = NextPoint;
 				Node1 = NextNode;
				NextNode = &NextNode->GetNextNode();
				NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
			}

			Point0 = PreviousPoint;
			Node0 = PreviousNode;
			PreviousNode = &PreviousNode->GetPreviousNode();
			PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);

			continue;
		}

		if (CheckAndRemovePick(*Point0, *Point1, *NextPoint, *Node1))
		{
			if (NextNode->IsDelete())
			{
				break;
			}

			if (CheckAndRemoveCoincidence(*Point0, *NextPoint, *Node0))
			{
				if (PreviousNode->IsDelete())
				{
					break;
				}

				Point0 = PreviousPoint;
				Node0 = PreviousNode;
				PreviousNode = &PreviousNode->GetPreviousNode();
				PreviousPoint = &PreviousNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
			}

			Point1 = NextPoint;
			Node1 = NextNode;
			NextNode = &NextNode->GetNextNode();
			NextPoint = &NextNode->Get2DPoint(EGridSpace::UniformScaled, Grid);
			continue;
		}
		break;
	}

	return true;
}

bool FLoopCleaner::RemoveNodeOfLoop(FLoopNode& NodeToRemove)
{
	if (NodeToRemove.GetConnectedSegments().Num() != 2)
	{
		return false;
	}

	FLoopNode& PreviousNode = NodeToRemove.GetPreviousNode();
	FLoopNode& NextNode = NodeToRemove.GetNextNode();

	FIsoSegment* Segment = PreviousNode.GetSegmentConnectedTo(&NodeToRemove);
	if (Segment == nullptr)
	{
		return false;
	}

	FIsoSegment* SegmentToDelete = NextNode.GetSegmentConnectedTo(&NodeToRemove);
	if (SegmentToDelete == nullptr)
	{
		return false;
	}

	NextNode.DisconnectSegment(*SegmentToDelete);
	NextNode.ConnectSegment(*Segment);
	Segment->SetSecondNode(NextNode);

	if (&NextNode.GetNextNode() == &NextNode.GetPreviousNode())
	{
		NextNode.DisconnectSegment(*Segment);
		PreviousNode.DisconnectSegment(*Segment);
		RemoveSegmentOfLoops(Segment);
		IsoSegmentFactory.DeleteEntity(Segment);

		FIsoSegment* ThirdSegment = PreviousNode.GetSegmentConnectedTo(&NextNode);
		if (!ThirdSegment)
		{
			return false;
		}

		NextNode.DisconnectSegment(*ThirdSegment);
		PreviousNode.DisconnectSegment(*ThirdSegment);

		RemoveSegmentOfLoops(ThirdSegment);
		IsoSegmentFactory.DeleteEntity(ThirdSegment);

		NextNode.Delete();
		PreviousNode.Delete();

		LoopSegmentsIntersectionTool.Remove(Segment);
		LoopSegmentsIntersectionTool.Remove(ThirdSegment);
	}

	RemoveSegmentOfLoops(SegmentToDelete);
	IsoSegmentFactory.DeleteEntity(SegmentToDelete);
	NodeToRemove.Delete();

	LoopSegmentsIntersectionTool.Remove(SegmentToDelete);
	if (!Segment->IsDelete())
	{
		LoopSegmentsIntersectionTool.Update(Segment);
	}

	return true;
}

void FLoopCleaner::FindBestLoopExtremity()
{
	double UMin = HUGE_VAL;
	double UMax = -HUGE_VAL;
	double VMin = HUGE_VAL;
	double VMax = -HUGE_VAL;

	FLoopNode* ExtremityNodes[4] = { nullptr, nullptr, nullptr, nullptr };

	FLoopNode* BestNode = nullptr;
	double OptimalSlope = 9.;
	LoopIndex = 0;

	BestStartNodeOfLoops.Reset(Grid.GetLoopCount());

	TFunction<void(FLoopNode*)> CompareWithSlopeAt = [&](FLoopNode* Node)
	{
		FLoopNode& PreviousNode = Node->GetPreviousNode();
		FLoopNode& NextNode = Node->GetNextNode();

		if (FVector2d::DistSquared(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), PreviousNode.Get2DPoint(EGridSpace::UniformScaled, Grid)) < DOUBLE_SMALL_NUMBER ||
			FVector2d::DistSquared(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), NextNode.Get2DPoint(EGridSpace::UniformScaled, Grid)) < DOUBLE_SMALL_NUMBER)
		{
			return;
		}

		double Slope = ComputePositiveSlope(Node->Get2DPoint(EGridSpace::UniformScaled, Grid), PreviousNode.Get2DPoint(EGridSpace::UniformScaled, Grid), NextNode.Get2DPoint(EGridSpace::UniformScaled, Grid));

		if ((Slope > OptimalSlope) == (LoopIndex == 0))
		{
			OptimalSlope = Slope;
			BestNode = Node;
		}
	};

	TFunction<void()> FindLoopExtremity = [&]()
	{
		BestNode = nullptr;
		OptimalSlope = (LoopIndex == 0) ? -1 : 9.;

		for (FLoopNode* Node : ExtremityNodes)
		{
			if (Node->IsDelete())
			{
				continue;
			}
			CompareWithSlopeAt(Node);
		}
		if(BestNode)
		{
			BestStartNodeOfLoops.Add(BestNode);
		}

		// init for next loop
		UMin = HUGE_VAL;
		UMax = -HUGE_VAL;
		VMin = HUGE_VAL;
		VMax = -HUGE_VAL;

		for (FLoopNode*& Node : ExtremityNodes)
		{
			Node = nullptr;
		}
	};

	// For each loops, Find the best loop extremity
	// The end of the loop is know when Node.GetLoopIndex() change
	int32 Index = 0;
	for (FLoopNode& Node : LoopNodes)
	{
		if (Node.IsDelete())
		{
			continue;
		}

		if (Node.GetLoopIndex() != LoopIndex)
		{
			FindLoopExtremity();
			LoopIndex = Node.GetLoopIndex();
		}
		Index++;
		const FVector2d& Point = Node.Get2DPoint(EGridSpace::UniformScaled, Grid);

		if (Point.X > UMax)
		{
			UMax = Point.X;
			ExtremityNodes[0] = &Node;
		}
		if (Point.X < UMin)
		{
			UMin = Point.X;
			ExtremityNodes[1] = &Node;
		}

		if (Point.Y > VMax)
		{
			VMax = Point.Y;
			ExtremityNodes[2] = &Node;
		}
		if (Point.Y < VMin)
		{
			VMin = Point.Y;
			ExtremityNodes[3] = &Node;
		}
	}
	FindLoopExtremity();
}

bool FLoopCleaner::CheckMainLoopConsistency()
{
	int32 OuterNodeCount = 0;
	for (const FLoopNode& Node : LoopNodes)
	{
		if (Node.GetLoopIndex() != 0)
		{
			break;
		}

		if (!Node.IsDelete())
		{
			++OuterNodeCount;
			if (OuterNodeCount > 2)
			{
				return true;
			}
		}
	}
	return false;
}

EOrientation FLoopCleaner::GetLoopOrientation(const FLoopNode* StartNode)
{
	using namespace IsoTriangulatorImpl;
	double UMin = HUGE_VAL;
	double UMax = -HUGE_VAL;
	double VMin = HUGE_VAL;
	double VMax = -HUGE_VAL;

	const FLoopNode* ExtremityNodes[4] = { nullptr, nullptr, nullptr, nullptr };

	LoopIndex = StartNode->GetLoopIndex();
	double OptimalSlope = (LoopIndex == 0) ? -1 : 9.;

	double MaxFrontSlope = 4;
	double MinBackSlope = 4;

#ifdef DEBUG_LOOP_ORIENTATION
	F3DDebugSession _(Grid.bDisplay, TEXT("GetLoopOrientation"));
#endif

	TFunction<void(int32)> CompareWithSlopeAt = [&ExtremityNodes, &MaxFrontSlope , &MinBackSlope, &Grid = Grid](int32 Index)
	{
		const FLoopNode* Node = ExtremityNodes[Index];
		const FLoopNode& PreviousNode = Node->GetPreviousNode();
		const FLoopNode& NextNode = Node->GetNextNode();

		const FVector2d Point = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d PreviousPoint = PreviousNode.Get2DPoint(EGridSpace::UniformScaled, Grid);
		const FVector2d NextPoint = NextNode.Get2DPoint(EGridSpace::UniformScaled, Grid);

		double Slope = ComputePositiveSlope(Point, PreviousPoint, NextPoint);

#ifdef DEBUG_LOOP_ORIENTATION
		F3DDebugSession A(Grid.bDisplay, FString::Printf(TEXT("Slope %f"), Slope));
		Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Node);
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Node, PreviousNode);
		Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Node, NextNode);
#endif

		if (FMath::IsNearlyEqual(Slope, 4.))
		{
			switch (Index)
			{
			case 0:
			{
				// UMax
				Slope += PreviousPoint.Y < NextPoint.Y ? 0.1 : -0.1;
				break;
			}
			case 1:
			{
				// UMin
				Slope += PreviousPoint.Y > NextPoint.Y ? 0.1 : -0.1;
				break;
			}
			case 2:
			{
				// VMax
				Slope += PreviousPoint.X > NextPoint.X ? 0.1 : -0.1;
				break;
			}
			case 3:
			{
				// VMin
				Slope += PreviousPoint.X < NextPoint.X ? 0.1 : -0.1;
				break;
			}
			default:
				break;
			}
		}
		
		if (Slope > 4.)
		{
			if (MaxFrontSlope < Slope) MaxFrontSlope = Slope;
		}
		else
		{
			if (MinBackSlope > Slope) MinBackSlope = Slope;
		}
	};

	// Find extremities
	const FLoopNode* Node = StartNode;
	do
	{
		const FVector2d& Point = Node->Get2DPoint(EGridSpace::UniformScaled, Grid);

		if (Point.X > UMax)
		{
			UMax = Point.X;
			ExtremityNodes[0] = Node;
		}
		if (Point.X < UMin)
		{
			UMin = Point.X;
			ExtremityNodes[1] = Node;
		}

		if (Point.Y > VMax)
		{
			VMax = Point.Y;
			ExtremityNodes[2] = Node;
		}
		if (Point.Y < VMin)
		{
			VMin = Point.Y;
			ExtremityNodes[3] = Node;
		}
		Node = LoopCleanerImpl::GetNextConstNodeImpl(Node);
	} while (Node != StartNode);

	for (int32 Index = 0; Index<4; ++Index)
	{
		CompareWithSlopeAt(Index);
	}

	MaxFrontSlope = 8. - MaxFrontSlope;

	if (LoopIndex == 0)
	{
		return MaxFrontSlope < MinBackSlope ? EOrientation::Front : EOrientation::Back;
	}
	else
	{
		return MaxFrontSlope > MinBackSlope ? EOrientation::Front : EOrientation::Back;
	}
}

}
