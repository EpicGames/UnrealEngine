// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Core/Factory.h"
#include "Math/SlopeUtils.h"
#include "Mesh/Meshers/IsoTriangulator/IntersectionSegmentTool.h"
#include "Mesh/Meshers/IsoTriangulator/IsoNode.h"
#include "Mesh/Meshers/IsoTriangulator/IsoSegment.h"
#include "Mesh/Meshers/ParametricMesherConstantes.h"



namespace UE::CADKernel
{
class FGrid;
class FIsoSegment;
class FIsoTriangulator;
class FLoopNode;
class FMeshingTolerances;

namespace LoopCleanerImpl
{
typedef TFunction<double(const FVector2d&, const FVector2d&, double)> GetSlopeMethod;
typedef TFunction<FLoopNode* (FLoopNode*)> GetNextNodeMethod;
typedef TFunction<const FLoopNode* (const FLoopNode*)> GetNextConstNodeMethod;
typedef TFunction<const FLoopNode* (const FIsoSegment*)> GetSegmentToNodeMethod;
typedef TPair<FLoopNode*, FLoopNode*> FLoopSection;

struct FPinchIntersectionContext
{
	const TPair<double, double>& Intersection;
	FLoopNode* Nodes[2][3];
	TArray<const FVector2d*> Points[2];

	FPinchIntersectionContext(const TPair<double, double>& InIntersection)
		: Intersection(InIntersection)
	{
	}
};

inline FLoopNode* GetNextNodeImpl(FLoopNode* Node)
{
	return &Node->GetNextNode();
}

inline FLoopNode* GetPreviousNodeImpl(FLoopNode* Node)
{
	return &Node->GetPreviousNode();
}

inline const FLoopNode* GetNextConstNodeImpl(const FLoopNode* Node)
{
	return &Node->GetNextNode();
}

inline const FLoopNode* GetPreviousConstNodeImpl(const FLoopNode* Node)
{
	return &Node->GetPreviousNode();
}

inline const FLoopNode* GetFirstNode(const FIsoSegment* Segment)
{
	return (const FLoopNode*)&Segment->GetFirstNode();
};

inline const FLoopNode* GetSecondNode(const FIsoSegment* Segment)
{
	return (const FLoopNode*)&Segment->GetSecondNode();
};

inline void RemoveDeletedNodes(TArray<FLoopNode*>& NodesOfLoop)
{
	int32 Index = NodesOfLoop.IndexOfByPredicate([](FLoopNode* Node) { return Node->IsDelete(); });
	if (Index == INDEX_NONE)
	{
		return;
	}
	int32 NewIndex = Index;
	for (; Index < NodesOfLoop.Num(); ++Index)
	{
		if (!NodesOfLoop[Index]->IsDelete())
		{
			NodesOfLoop[NewIndex++] = NodesOfLoop[Index];
		}
	}
	NodesOfLoop.SetNum(NewIndex);
}

}


class FLoopCleaner
{
private:
	FGrid& Grid;
	const FMeshingTolerances& Tolerances;

	TArray<FLoopNode>& LoopNodes;
	TArray<FIsoSegment*>& LoopSegments;
	TFactory<FIsoSegment>& IsoSegmentFactory;

	bool bDisplay;

	TArray<FLoopNode*> BestStartNodeOfLoops;

	// Fields for the processing loop 
	/** Index of the processing loop */
	int32 LoopIndex;  

	/**
	 * Array of the processed loop's nodes
	 * Warning: some nodes can be deleted -> check with Node.IsDelete
	 */
	TArray<FLoopNode*> NodesOfLoop;

	/**
	 * This value doesn't need to be updated because the count of nodes doesn't change
	 * Deleted nodes are not removed of NodesOfLoop array
	 */
	int32 NodesOfLoopCount;

	/** Index of first segment of the processing loop */
	int32 StartSegmentIndex;
	int32 SegmentCount;

	/** 
	 * Index of first segment of the next loop 
	 * Warning, this index need to be updated with the deletion of a segment
	 * Use RemoveSegmentOfLoops to delete a segment
	 * and use UpdateNextLoopFirstSegmentIndex in case of doubt
	 */
	int32 NextLoopFirstSegmentIndex;

	bool bLoopOrientation;
	TArray<TPair<double, double>> Intersections;
	FIntersectionSegmentTool LoopSegmentsIntersectionTool;

	LoopCleanerImpl::GetNextNodeMethod GetNext;
	LoopCleanerImpl::GetNextNodeMethod GetPrevious;
	LoopCleanerImpl::GetSegmentToNodeMethod GetFirst;
	LoopCleanerImpl::GetSegmentToNodeMethod GetSecond;

public:

	FLoopCleaner(FIsoTriangulator& Triangulator);

	bool Run();

private:

	bool CleanLoops();
	bool UncrossLoops(bool bAddProcessedLoop);

	/**
	 * For each loop, find the best starting node i.e. a node well oriented
	 * All extremity nodes have to be identified, then one of the four nodes is chosen 
	 */
	void FindBestLoopExtremity();

	//
	// ========= CleanLoops methods ==========
	//

	/**
	 * @return false if the process failed => the surface cannot be meshed
	 */
	bool RemoveLoopPicks();
	bool RemoveLoopPicks(TArray<FIsoSegment*>& Loop);

	bool RemovePickRecursively(FLoopNode* Node0, FLoopNode* Node1);
	bool FindAndRemoveCoincidence(FLoopNode*& StartNode);

	/**
	 * @return true if the node has been deleted
	 */
	bool CheckAndRemovePick(const FVector2d& PreviousPoint, const FVector2d& NodeToRemovePoint, const FVector2d& NextPoint, FLoopNode& NodeToRemove)
	{
		double Slope = ComputeUnorientedSlope(NodeToRemovePoint, PreviousPoint, NextPoint);
		bool bRemoveNode = false;
		if (Slope < 0.1)
		{
			double SquareDistance1 = SquareDistanceOfPointToSegment<FVector2d>(PreviousPoint, NodeToRemovePoint, NextPoint);
			double SquareDistance2 = SquareDistanceOfPointToSegment<FVector2d>(NextPoint, NodeToRemovePoint, PreviousPoint);
			double MinSquareDistance = FMath::Min(SquareDistance1, SquareDistance2);
			if (MinSquareDistance < Tolerances.SquareGeometricTolerance2)
			{
				bRemoveNode = true;
			}
		}

		if (bRemoveNode)
		{
			return RemoveNodeOfLoop(NodeToRemove);
		}
		return false;
	};

	/**
	 * @return true if the node has been deleted
	 */
	bool CheckAndRemoveCoincidence(const FVector2d& Point0, const FVector2d& Point1, FLoopNode& NodeToRemove)
	{
		double SquareDistance = FVector2d::DistSquared(Point0, Point1);
		if (SquareDistance < Tolerances.SquareGeometricTolerance2)
		{
			return RemoveNodeOfLoop(NodeToRemove);
		}
		return false;
	};

	/**
	 * @return true if the node has been deleted
	 */
	bool RemoveNodeOfLoop(FLoopNode& NodeToRemove);

	void FindLoopIntersections();

	/**
	 * @return false if the process failed => the surface cannot be meshed
	 */
	bool RemoveSelfIntersectionsOfLoop();

	bool RemoveIntersection(TPair<double, double>& Intersection);
	bool RemoveOutgoingLoop(const TPair<double, double>& Intersection, const TPair<double, double>& NextIntersection);
	bool RemoveIntersectionsOfSubLoop(int32 IntersectionIndex, int32 IntersectionCount);
	bool RemoveOuterNode(const TPair<double, double>& Intersection);
	bool SwapNodes(const TPair<double, double>& Intersection);

	/**
	 * @return false if the process failed => the surface cannot be meshed
	 */
	bool RemovePickOrCoincidenceBetween(FLoopNode* StartNode, FLoopNode* StopNode);

	/**
	 * Two cases:
	 *    - the segments of the intersection a closed parallel and in same orientation. in this case, the sub-loop is a long pick. The sub-Loop is delete
	 *    - the loop is an inner loop closed to the border:
	 *       _____________                _____________
	 *   |  /             \			  |__/             \
	 *    \/               |     =>                     |
	 *    /\               |		   __               |
	 *   |  \_____________/			  |  \_____________/
	 *
	 */
	bool TryToSwapSegmentsOrRemoveLoop(const TPair<double, double>& Intersection);
	void SwapSubLoopOrientation(int32 FirstSegmentIndex, int32 LastSegmentIndex);
	bool RemoveSubLoop(FLoopNode* StartNode, FLoopNode* EndNode);

	bool MoveIntersectingSectionBehindOppositeSection(LoopCleanerImpl::FLoopSection IntersectingSection, LoopCleanerImpl::FLoopSection OppositeSection);
	void MoveNodeBehindSegment(const FIsoSegment& IntersectingSegment, FLoopNode& NodeToMove);
	void MoveNode(FLoopNode& NodeToMove, FVector2d& NewPosition);

	void FixLoopOrientation();
	EOrientation GetLoopOrientation(const FLoopNode* StartNode);

	//
	// ========= UncrossLoops methods ==========
	//

	/**
	 * Segment and the segment next (or before Segment) are intersecting IntersectingSegment
	 * The common node is moved
	 */
	bool TryToRemoveIntersectionOfTwoConsecutiveIntersectingSegments(const FIsoSegment& IntersectingSegment, FIsoSegment& Segment);
	void RemoveIntersectionByMovingOutsideSegmentNodeInside(const FIsoSegment& IntersectingSegment, const FIsoSegment& Segment, bool bIsSameInnerLoop);

	//bool TryToRemoveSelfIntersectionByMovingTheClosedOusidePoint(const FIsoSegment& Segment0, const FIsoSegment& Segment1);
	//bool TryToRemoveIntersectionByMovingTheClosedOusidePoint(const FIsoSegment& Segment0, const FIsoSegment& Segment1);

	void OffsetSegment(FIsoSegment& Segment, FSegment2D& Segment2D, FSegment2D& IntersectingSegment2D);
	void OffsetNode(FLoopNode& Node, FSegment2D& IntersectingSegment2D);


	//
	// ========= Commun methods ==========
	//

	void SwapSegments(FIsoSegment& Segment0, FIsoSegment& Segment1);

	bool IsAPinch(const LoopCleanerImpl::FPinchIntersectionContext& Contex) const;

	/**
	 * OppositeCase:
	 *        __________        ___________ 
	 *       |__     ___|      |__      ___|
	 *          \   /             \    /
	 *           o o        =>     o  o
	 *        __/   \___		__/    \___
	 *       
	 */
	bool DisconnectCoincidentNodes(const LoopCleanerImpl::FPinchIntersectionContext& Contex);

	/**
	 * CrossingCase:
	 *           --<-    ->--       --<-    --<-
	 *           |   \  /   |       |   \  /   |
	 *           |    oo    |   =>  |    o     |
	 *           |   /  \   |       |          |
	 *           -->-    -<--       |     o    |
	 * 								|   /  \   |
	 *                              -->-    ->--
	 */
	bool DisconnectCrossingSegments(LoopCleanerImpl::FPinchIntersectionContext& Context);

	/**
	 * Case:  o_________o          o_________o
	 *             o         =>
	 *          __/ \___		        o
	 *								 __/ \___
	 */
	bool MovePickBehind(const TPair<double, double>& Intersection, bool bKeyIsExtremity);


	//
	// ========= Other ==========
	//

	bool CheckMainLoopConsistency();

	FLoopNode* GetNodeAt(int32 Index)
	{
		while (Index >= NodesOfLoopCount)
		{
			Index -= NodesOfLoopCount;
		}
		
		FLoopNode* Node = NodesOfLoop[Index];
		if (Node->IsDelete())
		{
#ifdef GET_NODE_AT	
			Wait();
#endif
			return nullptr;
		}

		return Node;
	};

	int32 NextSegmentIndex(int32 StartIndex)
	{
		++StartIndex;
		return FitSegmentIndex(StartIndex);
	};

	int32 NextIndex(int32 StartIndex)
	{
		++StartIndex;
		return FitNodeIndex(StartIndex);
	};

	int32 FitNodeIndex(int32 Index)
	{
		while (Index >= NodesOfLoopCount)
		{
			Index -= NodesOfLoopCount;
		}
		return Index;
	};

	int32 FitSegmentIndex(int32 Index)
	{
		while (Index >= NextLoopFirstSegmentIndex)
		{
			Index -= SegmentCount;
		}
		return Index;
	};

	bool Fill(LoopCleanerImpl::FPinchIntersectionContext& Context);

	void GetLoopNodeStartingFrom(FLoopNode* StartNode, TArray<FLoopNode*>& Loop)
	{
		FLoopNode* Node = StartNode;
		Loop.Empty(LoopNodes.Num());
		Loop.Add(StartNode);
		for (Node = GetNext(Node); Node != StartNode; Node = GetNext(Node))
		{
			Loop.Add(Node);
		}
	}

	bool UpdateNodesOfLoop()
	{
		LoopCleanerImpl::RemoveDeletedNodes(NodesOfLoop);
		NodesOfLoopCount = NodesOfLoop.Num();
		if (NodesOfLoopCount < 3)
		{
			return false;
		}
		return true;
	}

	/*
	 * Update NextLoopFirstSegmentIndex i.e. the Index of the first segment of the next loop (for iteration purpose) 
	 */
	void UpdateNextLoopFirstSegmentIndex(int32 NewLoopIndex)
	{
		NextLoopFirstSegmentIndex = LoopSegments.IndexOfByPredicate([&](FIsoSegment* Segment) {
			return ((FLoopNode&)Segment->GetFirstNode()).GetLoopIndex() > NewLoopIndex;
			});
		if (NextLoopFirstSegmentIndex == INDEX_NONE)
		{
			NextLoopFirstSegmentIndex = LoopSegments.Num();
		}
		SegmentCount = NextLoopFirstSegmentIndex - StartSegmentIndex;
	}

	void RemoveSegmentOfLoops(FIsoSegment* Segment)
	{
		LoopSegments.RemoveSingle(Segment);
		NextLoopFirstSegmentIndex--;
		SegmentCount--;
	}

#ifdef CADKERNEL_DEBUG
	void DisplayIntersection(const TPair<double, double>& Intersection)
	{
		if (bDisplay)
		{
			F3DDebugSession _(FString::Printf(TEXT("Intersection %f %f"), Intersection.Key, Intersection.Value));

			FLoopNode* Segment0End = GetNodeAt(NextIndex((int32)Intersection.Key));
			FLoopNode* Segment1Start = GetNodeAt((int32)Intersection.Value);

			if (Segment0End == nullptr || Segment1Start == nullptr)
			{
				return;
			}

			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment0End, *GetPrevious(Segment0End), 0, EVisuProperty::RedCurve);
			Grid.DisplayIsoSegment(EGridSpace::UniformScaled, *Segment1Start, *GetNext(Segment1Start), 0, EVisuProperty::RedCurve);
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *GetPrevious(Segment0End), 0, EVisuProperty::RedPoint);
			Grid.DisplayIsoNode(EGridSpace::UniformScaled, *Segment1Start, 0, EVisuProperty::RedPoint);
		}
	};
#endif

};

}