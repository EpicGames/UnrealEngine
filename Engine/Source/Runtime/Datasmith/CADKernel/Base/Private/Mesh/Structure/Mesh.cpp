// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/Mesh.h"

#include "Geo/Sampling/PolylineTools.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Topo/TopologicalEntity.h"
#include "Topo/TopologicalEdge.h"

namespace UE::CADKernel
{

TArray<double> FEdgeMesh::GetElementLengths() const
{
	const FTopologicalEdge& Edge = static_cast<const FTopologicalEdge&>(TopologicalEntity);
	const TArray<FVector>& MeshInnerNodes = GetNodeCoordinates();

	const FVector& StartNode = Edge.GetStartVertex()->GetCoordinates();
	const FVector& EndNode = Edge.GetEndVertex()->GetCoordinates();
	return PolylineTools::ComputePolylineSegmentLengths(StartNode, MeshInnerNodes, EndNode);
}

int32 FMesh::RegisterCoordinates()
{
	ModelMesh.RegisterCoordinates(NodeCoordinates, StartNodeId, MeshModelIndex);
	LastNodeIndex = StartNodeId + (int32)NodeCoordinates.Num();
	return StartNodeId;
}


#ifdef CADKERNEL_DEV
FInfoEntity& FMesh::GetInfo(FInfoEntity& Info) const
{
	return FEntityGeom::GetInfo(Info)
		.Add(TEXT("Geometric Entity"), (FEntity&) GetGeometricEntity())
		.Add(TEXT("Mesh model"), (FEntity&) GetMeshModel())
		.Add(TEXT("Node Num"), (int32) NodeCoordinates.Num());
}
#endif

}