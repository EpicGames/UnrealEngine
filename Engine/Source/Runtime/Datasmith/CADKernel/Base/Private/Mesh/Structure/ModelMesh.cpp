// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/ModelMesh.h"

#include "Mesh/Criteria/Criterion.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/Mesh.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEntity.h"

namespace UE::CADKernel
{

void FModelMesh::AddCriterion(TSharedPtr<FCriterion>& Criterion)
{
	Criteria.Add(Criterion);
	switch (Criterion->GetCriterionType())
	{
	case ECriterion::MinSize:
		MinSize = Criterion->Value();
		break;
	case ECriterion::MaxSize:
		MaxSize = Criterion->Value();
		break;
	case ECriterion::Angle:
		MaxAngle = Criterion->Value();
		break;
	case ECriterion::Sag:
		Sag = Criterion->Value();
		break;
	case ECriterion::CADCurvature:
		QuadAnalyse = true;
		break;
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FModelMesh::GetInfo(FInfoEntity& Info) const
{
	return FEntityGeom::GetInfo(Info)
		.Add(TEXT("Surface Meshes"), FaceMeshes)
		.Add(TEXT("Edge Meshes"), EdgeMeshes)
		.Add(TEXT("Vertex Meshes"), VertexMeshes);
}
#endif

const FVertexMesh* FModelMesh::GetMeshOfVertexNodeId(const int32 Ident) const
{
	for (const FVertexMesh* VertexMesh : VertexMeshes)
	{
		if (Ident == VertexMesh->GetStartVertexId())
		{
			return VertexMesh;
		}
	}
	return nullptr;
}

void FModelMesh::GetNodeCoordinates(TArray<FVector>& NodeCoordinates) const
{
	NodeCoordinates.Reserve(LastIdUsed + 1);

	for (const TArray<FVector>* PointArray : GlobalPointCloud)
	{
		NodeCoordinates.Insert(*PointArray, NodeCoordinates.Num());
	}
}

void FModelMesh::GetNodeCoordinates(TArray<FVector3f>& NodeCoordinates) const
{
	NodeCoordinates.Reserve(LastIdUsed);

	for (const TArray<FVector>* PointArray : GlobalPointCloud)
	{
		for (const FVector& Point : *PointArray)
		{
			NodeCoordinates.Emplace(Point.X, Point.Y, Point.Z);
		}
	}
}

const TArray<FMesh*>& FModelMesh::GetMeshes() const
{
	if (FaceMeshes.Num())
	{
		return (TArray<FMesh*>&) FaceMeshes;
	}
	if (EdgeMeshes.Num())
	{
		return (TArray<FMesh*>&) EdgeMeshes;
	}
	return (TArray<FMesh*>&) VertexMeshes;
}

int32 FModelMesh::GetTriangleCount() const
{
	int32 TriangleCount = 0;
	for (const FFaceMesh* FaceMesh : FaceMeshes)
	{
		TriangleCount += FaceMesh->TrianglesVerticesIndex.Num() / 3;
	}
	return TriangleCount;
}

} // namespace UE::CADKernel

