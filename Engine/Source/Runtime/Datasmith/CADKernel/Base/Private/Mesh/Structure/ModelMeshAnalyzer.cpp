// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/ModelMeshAnalyzer.h"

#include "Math/Geometry.h"
#include "Math/MathConst.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/Mesh.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEntity.h"

#include "Templates/UnrealTemplate.h"

namespace UE::CADKernel
{

void FModelMeshAnalyzer::BuildMesh()
{
	ModelMesh.GetNodeCoordinates(NodeCoordinates);

	int32 TriangleCount = 0;
	for (const FFaceMesh* FaceMesh : ModelMesh.GetFaceMeshes())
	{
		TriangleCount += FaceMesh->TrianglesVerticesIndex.Num() / 3;
	}

	Triangles.Reserve(TriangleCount);
	Edges.Reserve(TriangleCount*3);

	TMap<uint32, Analyzer::FEdge*> EdgeMap;
	EdgeMap.Reserve(TriangleCount * 3);

	TFunction<Analyzer::FEdge*(uint32, uint32)> FindOrBuildEdge = [&](uint32 VertexIndex0, uint32 VertexIndex1) -> Analyzer::FEdge*
	{
		uint32 EdgeVertexIndex0;
		uint32 EdgeVertexIndex1;
		FMath::GetMinMax(VertexIndex0, VertexIndex1, EdgeVertexIndex0, EdgeVertexIndex1);
		uint32 EdgeHash = HashCombine(EdgeVertexIndex0, EdgeVertexIndex1);
		Analyzer::FEdge** EdgePtr = EdgeMap.Find(EdgeHash);
		if (EdgePtr != nullptr)
		{
			return *EdgePtr;
		}
		Analyzer::FEdge& Edge = Edges.Emplace_GetRef(EdgeVertexIndex0, EdgeVertexIndex1);
		EdgeMap.Add(EdgeHash, &Edge);

		return &Edge;
	};

	for (const FFaceMesh* FaceMesh : ModelMesh.GetFaceMeshes())
	{
		const TArray<int32>& TrianglesVerticesIndex = FaceMesh->TrianglesVerticesIndex;
		const TArray<int32>& VerticesGlobalIndex = FaceMesh->VerticesGlobalIndex;
		const TArray<FVector3f>& FaceNormals = FaceMesh->Normals;

		for (int32 Index = 0; Index < FaceMesh->TrianglesVerticesIndex.Num(); Index += 3)
		{
			uint32 Vertices[3];
			Vertices[0] = VerticesGlobalIndex[TrianglesVerticesIndex[Index + 0]];
			Vertices[1] = VerticesGlobalIndex[TrianglesVerticesIndex[Index + 1]];
			Vertices[2] = VerticesGlobalIndex[TrianglesVerticesIndex[Index + 2]];

			Analyzer::FEdge* TriangleEdges[3];
			TriangleEdges[0] = FindOrBuildEdge(Vertices[0], Vertices[1]);
			TriangleEdges[1] = FindOrBuildEdge(Vertices[1], Vertices[2]);
			TriangleEdges[2] = FindOrBuildEdge(Vertices[2], Vertices[0]);

			FVector3f Normals[3];
			for (uint32 NormalI = 0; NormalI < 3; ++NormalI)
			{
				Normals[NormalI] = FaceNormals[TrianglesVerticesIndex[Index + NormalI]];
			}

			Analyzer::FTriangle& Triangle = Triangles.Emplace_GetRef(Vertices, TriangleEdges, Normals, FaceMesh);

			for (uint32 EdgeI = 0; EdgeI < 3; ++EdgeI)
			{
				TriangleEdges[EdgeI]->AddTriangle(Triangle);
			}
		}
	}

	VertexToEdges.SetNum(NodeCoordinates.Num());
	for (Analyzer::FEdge& Edge : Edges)
	{
		if (Edge.TriangleCount != 2)
		{
			VertexToEdges[Edge.VertexIndices[0]].Add(&Edge);
			VertexToEdges[Edge.VertexIndices[1]].Add(&Edge);
		}
	}
}

void FModelMeshAnalyzer::ComputeBorderCount(int32& OutBorderCount, int32& OutNonManifoldCount) const
{
	OutBorderCount = 0;
	OutNonManifoldCount = 0;
	for (const Analyzer::FEdge& Edge : Edges)
	{
		switch (Edge.TriangleCount)
		{
		case 1:
			OutBorderCount++;
			break;

		case 2:
			break;

		default:
			OutNonManifoldCount++;
			break;
		}
	}
}

void FModelMeshAnalyzer::ComputeMeshGapCount(int32& OutCycleCount, int32& OutChainCount) const
{
	OutCycleCount = 0;
	OutChainCount = 0;
	for (const Analyzer::FEdge& Edge : Edges)
	{
		if (Edge.TriangleCount == 2)
		{
			continue;
		}

		if (Edge.HasMarker1())
		{
			continue;
		}
		Edge.SetMarker1();

		uint32 FirstVertex = Edge.VertexIndices[0];
		uint32 NextVertex = Edge.VertexIndices[1];
		
		const Analyzer::FEdge* NextEdge = &Edge;

		bool bIsCycle = true;
		while (NextVertex != FirstVertex)
		{
			if (VertexToEdges[NextVertex].Num() != 2)
			{
				if (!bIsCycle)
				{
					break;
				}
				else
				{
					Swap(NextVertex, FirstVertex);
					bIsCycle = false;
				}
			}
			else
			{
				NextEdge = VertexToEdges[NextVertex][0] == NextEdge ? VertexToEdges[NextVertex][1] : VertexToEdges[NextVertex][0];
				NextEdge->SetMarker1();
				NextVertex = NextEdge->OtherVertexIndex(NextVertex);
			}
		}

		if (bIsCycle)
		{
			OutCycleCount++;
		}
		else
		{
			OutChainCount++;
		}
	}

	for (const Analyzer::FEdge& Edge : Edges)
	{
		Edge.ResetMarker1();
	}

}

double FModelMeshAnalyzer::ComputeMaxAngle() const 
{
	double MaxAngle = 0;

	for (const Analyzer::FEdge& Edge : Edges)
	{
		if (Edge.TriangleCount != 2)
		{
			continue;
		}

		if (Edge.Triangles[0]->FaceMesh != Edge.Triangles[1]->FaceMesh)
		{
			continue;
		}

		FVector Normal0 = Edge.Triangles[0]->ComputeNormal(NodeCoordinates);
		FVector Normal1 = Edge.Triangles[1]->ComputeNormal(NodeCoordinates);

		double Angle = FVectorUtil::ComputeAngle(Normal0, Normal1);
		if (Angle > MaxAngle)
		{
			MaxAngle = Angle;
		}
	}
	return MaxAngle;
}

bool FModelMeshAnalyzer::CheckOrientation() const
{
	return true;
}

FVector Analyzer::FTriangle::ComputeNormal(const TArray<FVector>& NodeCoordinates) const
{
	return UE::CADKernel::FTriangle(NodeCoordinates[Vertices[0]], NodeCoordinates[Vertices[1]], NodeCoordinates[Vertices[2]]).ComputeNormal();
}

} // namespace UE::CADKernel

