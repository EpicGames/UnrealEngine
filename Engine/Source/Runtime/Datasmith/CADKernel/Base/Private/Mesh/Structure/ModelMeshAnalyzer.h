// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/HaveStates.h"
#include "Core/Types.h"

#include "Math/Point.h"

namespace UE::CADKernel
{

class FModelMesh;
class FFaceMesh;

namespace Analyzer
{

struct FEdge;

struct FTriangle
{
	uint32 Vertices[3];
	FVector3f Normals[3];
	FEdge* Edges[3];
	const FFaceMesh* FaceMesh;


	FTriangle(uint32* InVertices, FEdge** InEdges, FVector3f* InNormals, const FFaceMesh* InFaceMesh)
	: FaceMesh(InFaceMesh)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			Vertices[Index] = InVertices[Index];
		}
		for (int32 Index = 0; Index < 3; ++Index)
		{
			Edges[Index] = InEdges[Index];
		}
		for (int32 Index = 0; Index < 3; ++Index)
		{
			Normals[Index] = InNormals[Index];
		}
	}

	FVector ComputeNormal(const TArray<FVector>& NodeCoordinates) const;
};

struct FEdge : public FHaveStates
{
	static const int32 MaxTraingleStoredCount = 2;
	uint32 VertexIndices[2];
	FTriangle* Triangles[MaxTraingleStoredCount]; // n first adjacent triangles
	uint32 TriangleCount = 0;

	FEdge(uint32 Index0, uint32 Index1)
	{
		VertexIndices[0] = Index0;
		VertexIndices[1] = Index1;
	}

	void AddTriangle(FTriangle& Triangle)
	{
		if (TriangleCount < MaxTraingleStoredCount)
		{
			Triangles[TriangleCount] = &Triangle;
		}
		TriangleCount++;
	}

	uint32 OtherVertexIndex(uint32 Index) const
	{
		return Index == VertexIndices[0] ? VertexIndices[1] : VertexIndices[0];
	}
};

}

class CADKERNEL_API FModelMeshAnalyzer
{
	const FModelMesh& ModelMesh;

	TArray<FVector> NodeCoordinates;
	TArray<Analyzer::FEdge> Edges;
	TArray<Analyzer::FTriangle> Triangles;
	TArray<TArray<Analyzer::FEdge*>> VertexToEdges;  // Vertex to border or non manifold edges 

public:
	FModelMeshAnalyzer(const FModelMesh& InModelMesh)
		: ModelMesh(InModelMesh)
	{
	}

	void BuildMesh();

	void ComputeBorderCount(int32& OutBorderCount, int32& OutNonManifoldCount) const;
	void ComputeMeshGapCount(int32& OutCycleCount, int32& OutChainCount) const;
	double ComputeMaxAngle() const;
	bool CheckOrientation() const;
};

}

