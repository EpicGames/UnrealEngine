// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/Structure/ThinZone2DFinder.h"

#include "Math/Point.h"
#include "Mesh/Structure/Grid.h"
#include "Mesh/Structure/EdgeSegment.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLoop.h"

namespace UE::CADKernel
{
FIdent FEdgeSegment::LastId = 0;

void FThinZone2DFinder::FindCloseSegments()
{
	TFunction<double()> ComputeMaxSegmentLength = [&]()-> double
	{
		double MaxLength = 0;
		for (const FEdgeSegment* Segment : LoopSegments)
		{
			double Length = Segment->GetLength();
			if (Length > MaxLength)
			{
				MaxLength = Length;
			}
		}
		MaxLength *= 1.01;
		return MaxLength;
	};

	const double MaxSegmentLength = ComputeMaxSegmentLength();
	const double MaxSpace = FMath::Max(MaxSegmentLength, FinderTolerance * 1.01);
	const double MaxSquareSpace = 4 * FMath::Square(MaxSpace);
	const double MaxSpacePlusLength = 1.1 * (MaxSpace + MaxSegmentLength);

	// Copy of loop segments to generate a sorted array of segments
	TArray<FEdgeSegment*> SortedThinZoneSegments = LoopSegments;
	Algo::Sort(SortedThinZoneSegments, [](FEdgeSegment* SegmentA, FEdgeSegment* SegmentB)
		{
			return (SegmentA->GetAxeMin() < SegmentB->GetAxeMin());
		});

#ifdef DEBUG_FIND_CLOSE_SEGMENTS
	F3DDebugSession _(bDisplay, ("Close Segments"));
	int32 SegmentID = 102;
	int32 SegmentIndex = 0;
#endif

	// For each segment, the nearest segment is search
	// If segment is from an inner loop, the nearest could not be from the same loop
	int32 SegmentToBeginIndex = 0;
	for (FEdgeSegment* Segment : SortedThinZoneSegments)
	{
		FTopologicalLoop* SegmentLoop = nullptr;
		if (Segment->IsInner())
		{
			SegmentLoop = Segment->GetEdge()->GetLoop();
		}
		else
		{
			SegmentLoop = nullptr;
		}

		const FVector2d SegmentMiddle = Segment->GetCenter();

		FVector2d ClosePoint;
		FEdgeSegment* CloseSegment = nullptr;

		//     DistanceMin between Candidate and Segment with Candidate is before 
		//     DistanceMin = (Segment.SegmentMin - Candidate.SegmentMax) = Segment.SegmentMin - (Candidate.SegmentMin + MaxSegmentLength) 
		//
		//     Candidate                              Segment
		//     *---*                                  x-----x
		//     | Candidate.SegmentMin                 | Segment.SegmentMin
		//         | Candidate.SegmentMax = Candidate.SegmentMin + MaxSegmentLength
		//
		//     We want DistanceMin < MaxSpace
		//     Segment.SegmentMin - (Candidate.SegmentMin + MaxSegmentLength) < MaxSpace
		//     Candidate.SegmentMin > Segment.SegmentMin - MaxSegmentLength - MaxSpace

		//     DistanceMin between Candidate and Segment, if Candidate is after
		//     DistanceMin = (Candidate.SegmentMin - Segment.SegmentMax) = Candidate.SegmentMin - (Segment.SegmentMin + MaxSegmentLength) 
		//
		//     Segment                                Candidate
		//     *--x--*                                  x-----x
		//        | Segment.Middle                      | Candidate.SegmentMin
		//
		//     We want DistanceMin < MaxSpace + MaxSegmentLength
		//     Candidate.SegmentMin - Segment.Middle < MaxSpace + MaxSegmentLength
		//     Candidate.SegmentMin < Segment.Middle + MaxSpace + MaxSegmentLength
		//
		//     MaxSpacePlusLength = MaxSpace + MaxSegmentLength;

		const double SegmentMinMin = Segment->GetAxeMin() - MaxSpacePlusLength;
		const double SegmentMiddleMax = FVectorUtil::DiagonalAxisCoordinate(SegmentMiddle) + MaxSpacePlusLength;
		double MinSquareThickness = HUGE_VALUE;

#ifdef DEBUG_FIND_CLOSE_SEGMENTS_
		if (SegmentIndex == SegmentID)
		{
			F3DDebugSession _(bDisplay, FString::Printf(TEXT("Segment %d"), SegmentIndex));
			ThinZone::DisplayEdgeSegment(Segment, EVisuProperty::RedCurve);
			Wait();
		}
#endif
		for (int32 CandidateIndex = SegmentToBeginIndex; CandidateIndex < SortedThinZoneSegments.Num(); ++CandidateIndex)
		{

			FEdgeSegment* Candidate = SortedThinZoneSegments[CandidateIndex];
			if (Candidate == Segment)
			{
				continue;
			}

			// Inner boundary are not check for thin inner boundary
			if (SegmentLoop == Candidate->GetEdge()->GetLoop())
			{
				continue;
			}


			const double CandidateSegmentMin = Candidate->GetAxeMin();
			// If min point of candidate segment - maxSpace (= max length of the segments + Tolerance) is smaller than Min point of current segment then the distance between both segments cannot be smaller the Tolerance
			if (CandidateSegmentMin < SegmentMinMin)
			{
				SegmentToBeginIndex = CandidateIndex;
				continue;
			}

			// If Min point of current segment + maxSpace is smaller than Min point of candidate segment then the distance between both segments cannot be smaller the Tolerance.
			// As segments are sorted, next segments are not close to the current segment
			if (CandidateSegmentMin > SegmentMiddleMax)
			{
				break;
			}

			const FVector2d& FirstPointCandidate = Candidate->GetExtemity(ELimit::Start);

			// if the distance of FirstPointCandidate with the middle of current segment is biggest than MaxSpace, then the projection of the point cannot be smaller than the tolerance, 
			{
				const double SquareDistance = FVector2d::DistSquared(SegmentMiddle, FirstPointCandidate);
				if (SquareDistance > MaxSquareSpace)
				{
					continue;
				}
			}

			// check the angle between segments. As they are opposite, the angle has to be close to PI i.e. slope has to be close to 4 => slope > Pi/2			
			{
				const double Slope = Segment->ComputeUnorientedSlopeOf(Candidate);
				if (Slope < Slope::ThreeQuaterPiSlope)
				{
					continue;
				}
			}

			const FVector2d& SecondPointCandidate = Candidate->GetExtemity(ELimit::End);
			double Coordinate;
			const FVector2d Projection = ProjectPointOnSegment<FVector2d>(SegmentMiddle, FirstPointCandidate, SecondPointCandidate, Coordinate, true);

#ifdef DEBUG_FIND_CLOSE_SEGMENTS_
			if (SegmentIndex == SegmentID)
			{
				F3DDebugSession _(bDisplay, FString::Printf(TEXT("Candidate %d %d Dist2"), SegmentIndex, CandidateIndex));
				ThinZone::DisplayEdgeSegmentAndProjection(Segment, Candidate, EVisuProperty::BlueCurve, EVisuProperty::BlueCurve, EVisuProperty::RedCurve);
			}
#endif

			const double SquareDistance = FVector2d::DistSquared(SegmentMiddle, Projection);
			if (SquareDistance > SquareFinderTolerance)
			{
				continue;
			}

			if (MinSquareThickness > SquareDistance)
			{
				// check the angle between segment and Middle-Projection. If Slope is positive, Middle-Projection is on the outer side 
				const double Slope = Candidate->ComputeOrientedSlopeOf(SegmentMiddle, Projection);
				if (Slope < 0)
				{
#ifdef DEBUG_FIND_CLOSE_SEGMENTS_
					F3DDebugSession _(bDisplay, FString::Printf(TEXT("Candidate %f outer side criteria"), Slope));
					ThinZone::DisplayEdgeSegmentAndProjection(Segment, Candidate, EVisuProperty::RedCurve, EVisuProperty::RedCurve, EVisuProperty::YellowCurve);
#endif
					continue;
				}

				MinSquareThickness = SquareDistance;
				ClosePoint = Projection;
				CloseSegment = Candidate;
			}
		}

		if (CloseSegment)
		{
			Segment->SetCloseSegment(CloseSegment, MinSquareThickness);

#ifdef DEBUG_FIND_CLOSE_SEGMENTS
			if (bDisplay)
			{
				if (SegmentIndex == SegmentID)
				{
					Wait();
				}

				F3DDebugSession _(bDisplay, FString::Printf(TEXT("Close Segment %d"), SegmentIndex));
				ThinZone::DisplayEdgeSegmentAndProjection(Segment, CloseSegment, EVisuProperty::BlueCurve, EVisuProperty::BlueCurve, EVisuProperty::RedCurve);
			}
#endif
		}
#ifdef DEBUG_FIND_CLOSE_SEGMENTS
		SegmentIndex++;
#endif
	}
}

void FThinZone2DFinder::FindCloseSegments(const TArray<FEdgeSegment*>& Segments, const TArray< const TArray<FEdgeSegment*>*>& OppositeSides)
{
#ifdef DEBUG_FIND_CLOSE_SEGMENTS
	F3DDebugSession _(TEXT("FindCloseSegments"));
	{
		F3DDebugSession _(TEXT("OppositeSide"));
		for (const TArray<FEdgeSegment*>* OppositeSidePtr : OppositeSides)
		{
			ThinZone::DisplayThinZoneSide(*OppositeSidePtr, 0, EVisuProperty::YellowCurve);
		}
	}
#endif

	double MinSquareThickness;
	TFunction<FEdgeSegment* (const FEdgeSegment*, bool)> FindCloseSegment = [&OppositeSides, &MinSquareThickness](const FEdgeSegment* Segment, const bool bForce)
	{
		const FVector2d SegmentMiddle = Segment->GetCenter();

		FVector2d ClosePoint;
		FEdgeSegment* CloseSegment = nullptr;

		MinSquareThickness = HUGE_VALUE;
		for (const TArray<FEdgeSegment*>* OppositeSidePtr : OppositeSides)
		{
			const TArray<FEdgeSegment*>& OppositeSide = *OppositeSidePtr;
			for (FEdgeSegment* Candidate : OppositeSide)
			{
				if (!bForce)
				{
					// check the angle between segments. As they are opposite, the Angle as to be close to Pi i.e. Slop ~= 4
					double Slope = Segment->ComputeOrientedSlopeOf(Candidate);
					if (Slope > -Slope::RightSlope && Slope < Slope::RightSlope)
					{
						continue;
					}
				}

				const FVector2d& FirstPointCandidate = Candidate->GetExtemity(ELimit::Start);
				const FVector2d& SecondPointCandidate = Candidate->GetExtemity(ELimit::End);

				double Coordinate;
				const FVector2d Projection = ProjectPointOnSegment<FVector2d>(SegmentMiddle, FirstPointCandidate, SecondPointCandidate, Coordinate, true);

				const double SquareDistance = FVector2d::DistSquared(SegmentMiddle, Projection);
				if (MinSquareThickness > SquareDistance)
				{
					MinSquareThickness = SquareDistance;
					ClosePoint = Projection;
					CloseSegment = Candidate;
				}
			}
		}
		return CloseSegment;
	};

	int32 SegmentToBeginIndex = 0;
	for (FEdgeSegment* Segment : Segments)
	{
		if (Segment->GetCloseSegment())
		{
			continue;
		}

		constexpr bool bForceFind = true;
		FEdgeSegment* CloseSegment = FindCloseSegment(Segment, !bForceFind);
		if (!CloseSegment)
		{
			CloseSegment = FindCloseSegment(Segment, bForceFind);
		}
		if (CloseSegment)
		{
			Segment->SetCloseSegment(CloseSegment, MinSquareThickness);
		}
	}
}

void FThinZone2DFinder::LinkCloseSegments()
{
	TArray<FEdgeSegment*> ThinZoneSegments;
	ThinZoneSegments.Reserve(LoopSegments.Num());

	FIdent ChainIndex = Ident::Undefined;
	FTopologicalLoop* Loop = nullptr;
	bool bThinZone = false;

	// Add in ThinZoneSegments all segments of thin zone 
	for (FEdgeSegment* EdgeSegment : LoopSegments)
	{
		if (Loop != EdgeSegment->GetEdge()->GetLoop())
		{
			Loop = EdgeSegment->GetEdge()->GetLoop();
			if (bThinZone)
			{
				ChainIndex++;
				bThinZone = false;
			}
		}

		if (const FEdgeSegment* CloseSegment = EdgeSegment->GetCloseSegment())
		{
			// Pick case i.e. all the segments are connected, 
			//    _____ . ____ .
			//   /
			//  . --- . ---- . --- .
			// As soon as a CloseSegment chain index is set and is equal to index, this mean that the segment is on the other side of the pick.
			if (CloseSegment->GetChainIndex() == ChainIndex)
			{
				ChainIndex++; // new chain
			}

			EdgeSegment->SetChainIndex(ChainIndex);
			ThinZoneSegments.Add(EdgeSegment);
			bThinZone = true;
		}
		else if (bThinZone)
		{
			ChainIndex++;  // new chain
			bThinZone = false;
		}
	}

#ifdef DEBUG_LINK_CLOSE_SEGMENTS
	{
		DisplaySegmentsOfThinZone();
	}
#endif

	if (ThinZoneSegments.IsEmpty())
	{
		return;
	}

	// Fill ThinZoneSides
	{
		// Reserve
		ThinZoneSides.SetNum(ChainIndex);
		ChainIndex = Ident::Undefined;
		int32 SegmentCount = 0;
		for (FEdgeSegment* EdgeSegment : ThinZoneSegments)
		{
			if (ChainIndex != EdgeSegment->GetChainIndex())
			{
				if (SegmentCount)
				{
					TArray<FEdgeSegment*>& ThinZoneSide = ThinZoneSides[ChainIndex];
					ThinZoneSide.Reserve(ThinZoneSide.Max() + SegmentCount);
					SegmentCount = 0;
				}
				ChainIndex = EdgeSegment->GetChainIndex();
			}
			SegmentCount++;
		}
		ThinZoneSides.Emplace_GetRef().Reserve(SegmentCount);

		// Fill
		ChainIndex = Ident::Undefined;
		TArray<FEdgeSegment*>* ThinZoneSide = nullptr;
		for (FEdgeSegment* EdgeSegment : ThinZoneSegments)
		{
			if (ChainIndex != EdgeSegment->GetChainIndex())
			{
				ChainIndex = EdgeSegment->GetChainIndex();
				ThinZoneSide = &ThinZoneSides[ChainIndex];
			}
			ThinZoneSide->Add(EdgeSegment);
		}
	}
}

void FThinZone2DFinder::CheckIfCloseSideOfThinSidesAreNotDegenerated()
{
#ifdef DEBUG_CHECK_THIN_SIDE
	F3DDebugSession A(bDisplay, TEXT("Check Thin Sides"));
#endif

	for (TArray<FEdgeSegment*>& Side : ThinZoneSides)
	{
		if (Side.IsEmpty())
		{
			continue;
		}

		if (!CheckIfCloseSideOfThinSideIsNotDegenerated(Side))
		{
#ifdef DEBUG_CHECK_THIN_SIDE
			if (bDisplay)
			{
				F3DDebugSession _(TEXT("Delete Thin Zone"));
				ThinZone::DisplayThinZoneSide(Side, 0, EVisuProperty::RedCurve);
			}
#endif
			for (FEdgeSegment* Segment : Side)
			{
				Segment->ResetCloseData();
				Segment->SetChainIndex(Ident::Undefined);
			}
			Side.Empty();
		}
	}
#ifdef DEBUG_CHECK_THIN_SIDE
	Wait(bDisplay);
#endif
}

/**
 * The main idea of this algorithm is to check that the Side closed area is not limited to a point (Tol = 1/3 of the close segment)
 * If the close side is defined by at least 3 segments, it's not degenerated.
 * If the (2) segments of the close side are not in the same side -> it's not degenerated.
 * Each extremities of the segments of the side is projected on the close segment(s)
 * If the max distance between the projected points is smaller than the tolerance, the side is degenerated
 */
bool FThinZone2DFinder::CheckIfCloseSideOfThinSideIsNotDegenerated(TArray<FEdgeSegment*>& Side)
{
#ifdef DEBUG_CHECK_THIN_SIDE_
	F3DDebugSession _(bDisplay, ("Check Thin Zone"));
#endif

	const FEdgeSegment* CloseSegments[3] = { nullptr, nullptr, nullptr };
	TFunction<FVector2d(const FVector2d&)> GetOppositeSegment = [&CloseSegments](const FVector2d& Candidate) -> FVector2d
	{
		FVector2d OutOppositePoint = FVector2d::ZeroVector;
		double MinSquareThickness = HUGE_VALUE;
		for (int32 Index = 0; CloseSegments[Index] != nullptr; ++Index)
		{
			const FEdgeSegment* CloseSegment = CloseSegments[Index];
			const FVector2d& FirstPointCandidate = CloseSegment->GetExtemity(ELimit::Start);
			const FVector2d& SecondPointCandidate = CloseSegment->GetExtemity(ELimit::End);

			double Coordinate;
			const FVector2d Projection = ProjectPointOnSegment<FVector2d>(Candidate, FirstPointCandidate, SecondPointCandidate, Coordinate, true);

			const double SquareDistance = FVector2d::DistSquared(Candidate, Projection);
			if (MinSquareThickness > SquareDistance)
			{
				MinSquareThickness = SquareDistance;
				OutOppositePoint = Projection;
			}
		}
		return OutOppositePoint;
	};

	if (Side.IsEmpty())
	{
		// side is degenerated
		return false;
	}

	// find the close segments
	for (const FEdgeSegment* Segment : Side)
	{
		if (Segment == nullptr)
		{
			continue;
		}

		const FEdgeSegment* CloseSegment = Segment->GetCloseSegment();
		if (!CloseSegments[0])
		{
			CloseSegments[0] = CloseSegment;
			continue;
		}
		if (CloseSegments[0] != CloseSegment)
		{
			if (!CloseSegments[1])
			{
				CloseSegments[1] = CloseSegment;
			}
			else if (CloseSegments[1] != CloseSegment)
			{
				CloseSegments[2] = CloseSegment;
				break;
			}
		}
	}

	// More than two segments -> not degenerated
	if (CloseSegments[2] != nullptr)
	{
		return true;
	}

	if (CloseSegments[0] != nullptr && CloseSegments[1] != nullptr)
	{
		// Case two chain -> not degenerated 
		// 
		//          Chain 0     Side     Chain 1
		//                \      \ /     /
		//     Close[0] -> \      #     / <- Close[1]
		//                  \          /
		//                   
		if (CloseSegments[0]->GetChainIndex() != CloseSegments[1]->GetChainIndex())
		{
			return true;
		}

		// Case the close segments are not connected -> not degenerated 
		//                       Side     Chain
		//                \      \ /     /
		//     Close[0] -> \      #     / <- Close[1]
		//                  \          /
		//                   #--------#
		if (CloseSegments[0]->GetPrevious() != CloseSegments[1] && CloseSegments[0]->GetNext() != CloseSegments[1])
		{
			return true;
		}
	}

	TFunction<double()> ComputeTolerance = [&CloseSegments]() -> double
	{
		if (CloseSegments[0] == nullptr)
		{
			return 0;
		}
		double CloseLength = CloseSegments[0]->GetLength();

		const double Ratio = CloseSegments[1] ? 12. : 6.;
		if (CloseSegments[1])
		{
			CloseLength += CloseSegments[1]->GetLength();
		}
		return FMath::Square(CloseLength / Ratio);
	};
	const double ToleranceCloseSquare = ComputeTolerance();

	if (Side[0] == nullptr)
	{
		return false;
	}

	const FVector2d& StartPoint = Side[0]->GetExtemity(Start);
	FVector2d FirstOppositePoint = GetOppositeSegment(StartPoint);

#ifdef DEBUG_CHECK_THIN_SIDE_
	{
		F3DDebugSession _(bDisplay, ("Seg"));
		DisplaySegmentWithScale(StartPoint, FirstOppositePoint, 0, EVisuProperty::BlueCurve);
		DisplayPoint2DWithScale(StartPoint, EVisuProperty::BluePoint);
		DisplayPoint2DWithScale(StartPoint, EVisuProperty::RedPoint);
	}
#endif

	for (const FEdgeSegment* Segment : Side)
	{
		if (Segment == nullptr)
		{
			continue;
		}
		const FVector2d& NextPoint = Segment->GetExtemity(End);
		FVector2d OppositePoint = GetOppositeSegment(NextPoint);
		const double SquarewDistance = FVector2d::DistSquared(OppositePoint, FirstOppositePoint);

#ifdef DEBUG_CHECK_THIN_SIDE_
		{
			F3DDebugSession _(bDisplay, TEXT("Seg"));
			DisplaySegmentWithScale(NextPoint, OppositePoint, 0, EVisuProperty::BlueCurve);
			DisplayPoint2DWithScale(NextPoint, EVisuProperty::BluePoint);
			DisplayPoint2DWithScale(OppositePoint, EVisuProperty::RedPoint);
			Wait();
		}
#endif

		if (SquarewDistance > ToleranceCloseSquare)
		{
			return true;
		}
	}

	// Degenerated... 
	// Check if at least one segment of the side is the close segment of the other side
	const FEdgeSegment* CloseOfCloseSegments[2] = { CloseSegments[0] ? CloseSegments[0]->GetCloseSegment() : nullptr, CloseSegments[1] ? CloseSegments[1]->GetCloseSegment() : nullptr };
	for (const FEdgeSegment* Segment : Side)
	{
		if (Segment && (Segment == CloseOfCloseSegments[0] || Segment == CloseOfCloseSegments[1]))
		{
			return true;
		}
	}

	return false;
}

void FThinZone2DFinder::ImproveThinSide()
{
#ifdef DEBUG_THIN_ZONES_IMPROVE
	if (bDisplay)
	{
		F3DDebugSession _(bDisplay, ("Improve Thin Zone"));
		Wait(false);
	}
#endif

	TFunction<TArray<FEdgeSegment*>(const FEdgeSegment*)> GetComplementary = [this](const FEdgeSegment* EdgeSegment) ->TArray<FEdgeSegment*>
	{
		// We allow to extend a thin zone up to 4 times the local thickness if it lets to link another ThinZone
		constexpr double ComplementaryFactor = 4.;

		TArray<FEdgeSegment*> ComplementaryEdges;
		ComplementaryEdges.Reserve(100);

		double Length = 0;
		double MaxLength = this->FinderTolerance;

		for (FEdgeSegment* PreviousSegment = EdgeSegment->GetPrevious();; PreviousSegment = PreviousSegment->GetPrevious())
		{
			if (PreviousSegment->GetCloseSegment())
			{
				break;
			}
			Length += PreviousSegment->GetLength();
			ComplementaryEdges.Add(PreviousSegment);

			if (Length > MaxLength)
			{
				ComplementaryEdges.Empty();
				break;
			}
		}

		Algo::Reverse(ComplementaryEdges);
		return MoveTemp(ComplementaryEdges);
	};

	TFunction<void(TArray<FEdgeSegment*>&, TArray<FEdgeSegment*>&)> MergeChains = [&ThinZoneSides = ThinZoneSides](TArray<FEdgeSegment*>& ThinZoneSide, TArray<FEdgeSegment*>& ComplementaryEdges)
	{
		FEdgeSegment* PreviousSegment = ComplementaryEdges[0]->GetPrevious();
		const FIdent ChainId = ThinZoneSide[0]->GetChainIndex();

		TArray<FEdgeSegment*> NewChain = MoveTemp(ThinZoneSides[PreviousSegment->GetChainIndex()]);
		NewChain.Reserve(NewChain.Num() + ThinZoneSide.Num() + ComplementaryEdges.Num());
		NewChain.Append(ComplementaryEdges);
		NewChain.Append(ThinZoneSide);
		ThinZoneSide = MoveTemp(NewChain);

		for (FEdgeSegment* Segment : ThinZoneSide)
		{
			Segment->SetChainIndex(ChainId);
		}
	};

	TFunction<void(TArray<FEdgeSegment*>&)> MergeWithPreviousChain = [&ThinZoneSides = ThinZoneSides](TArray<FEdgeSegment*>& ThinZoneSide)
	{
		const FEdgeSegment* PreviousSegment = ThinZoneSide[0]->GetPrevious();
		const FIdent ChainId = ThinZoneSide[0]->GetChainIndex();

		TArray<FEdgeSegment*> NewChain = MoveTemp(ThinZoneSides[PreviousSegment->GetChainIndex()]);
		NewChain.Reserve(NewChain.Num() + ThinZoneSide.Num());
		NewChain.Append(ThinZoneSide);
		ThinZoneSide = MoveTemp(NewChain);

		for (FEdgeSegment* Segment : ThinZoneSide)
		{
			Segment->SetChainIndex(ChainId);
		}
	};

	TFunction<void(const FEdgeSegment*, TArray<FEdgeSegment*>&)> SetChainIndex = [](const FEdgeSegment* ThinZoneSegment, TArray<FEdgeSegment*>& Complementary)
	{
		const FIdent ChainId = ThinZoneSegment->GetChainIndex();
		for (FEdgeSegment* Segment : Complementary)
		{
			Segment->SetChainIndex(ChainId);
		}
	};

	TFunction<void(TArray<FEdgeSegment*>&, TArray<FEdgeSegment*>&)> MergeSideWithComplementaryAtStart = [SetChainIndex](TArray<FEdgeSegment*>& ThinZoneSide, TArray<FEdgeSegment*>& Complementary)
	{
		SetChainIndex(ThinZoneSide[0], Complementary);
		Complementary.Reserve(Complementary.Num() + ThinZoneSide.Num());
		Complementary.Append(MoveTemp(ThinZoneSide));
		ThinZoneSide = MoveTemp(Complementary);
	};

	TFunction<void(TArray<FEdgeSegment*>&, TArray<FEdgeSegment*>&)> MergeSideWithComplementaryAtEnd = [SetChainIndex](TArray<FEdgeSegment*>& ThinZoneSide, TArray<FEdgeSegment*>& Complementary)
	{
		SetChainIndex(ThinZoneSide[0], Complementary);
		ThinZoneSide.Reserve(Complementary.Num() + ThinZoneSide.Num());
		ThinZoneSide.Append(MoveTemp(Complementary));
	};

	TFunction<void(TArray<FEdgeSegment*>&)> SetMarker1ToComplementarySegments = [&](TArray<FEdgeSegment*>& Complementary)
	{
		for (FEdgeSegment* Segment : Complementary)
		{
			Segment->SetMarker1();
		}
	};

	// two adjacent chains can be:
	// - connected. In this case, the both chains are merged
	// - separated by a small chain slightly too far to be considered as close. If this kind of chain is find, the three chains are merged.
	for (TArray<FEdgeSegment*>& Side : ThinZoneSides)
	{
		if (Side.IsEmpty())
		{
			continue;
		}

#ifdef DEBUG_THIN_ZONES_IMPROVE
		static int32 IndexSide = 0;
		if (bDisplay)
		{
			ThinZone::DisplayThinZoneSide(Side, IndexSide++, EVisuProperty::BlueCurve, false);
			Wait();
		}
#endif

		const FEdgeSegment* Segment0Side = Side[0];
		if (Segment0Side->GetPrevious()->GetCloseSegment() != nullptr)
		{
			// Connected case
			// 	             
			//                             Segment0Side->GetPrevious()            
			//                Segment0Side |           
			//                  Side(n)  | | Side(n-1)       Side(n-1) = ThinZoneSides[n-1]
			//             #--------------#-----------#
			//	          
			//             #--------------------------#
			//                           | |
			// Seg0Side->GetCloseSegment() | 
			//                             Segment0Side->GetPrevious()->GetCloseSegment()

			FIdent CloseOfPreviousSideIndex = Segment0Side->GetPrevious()->GetCloseSegment()->GetChainIndex();
			if (CloseOfPreviousSideIndex != Side[0]->GetChainIndex() && CloseOfPreviousSideIndex == Segment0Side->GetCloseSegment()->GetChainIndex())
			{
				MergeWithPreviousChain(Side);
#ifdef DEBUG_THIN_ZONES_IMPROVE
				if (bDisplay)
				{
					ThinZone::DisplayThinZoneSide(Side, 10000, EVisuProperty::BlueCurve, false);
					Wait();
				}
#endif
				continue;
			}
			continue;
		}

		TArray<FEdgeSegment*> Complementary = GetComplementary(Segment0Side);
		if (Complementary.Num())
		{
#ifdef DEBUG_THIN_ZONES_IMPROVE
			if (bDisplay)
			{
				ThinZone::DisplayThinZoneSide(Complementary, -1, EVisuProperty::RedCurve, false);
				Wait();
			}
#endif

			// Case2 : separated by a small chain slightly too far to be considered as close
			//
			//                  Seg0Side             Complementary[0]
			//                  |                    | PreviousComplementary[0]            Side(n)   = ThinZoneSides[n]
			//       Side(n  )  |     Complementary  | |  Side(n-1)                        Side(n-1) = ThinZoneSides[n-1]
			//    #--------------#--------------------#-----------#
			//
			//    #-----------------------------------------------#
			//                  |                      |
			//                  |                      ClosePreviousComplementary[0]
			//                  Seg0Side->GetCloseSegment()
			const FEdgeSegment* PreviousComplementary = Complementary[0]->GetPrevious();
			const FEdgeSegment* ClosePreviousComplementary = PreviousComplementary->GetCloseSegment();

			if (ClosePreviousComplementary->GetChainIndex() == Ident::Undefined)
			{
				continue;
			}

			// Case Side(n-1) and Side(n) are closed together
			// 
			//                               PreviousComplementary = Complementary[0]->Previous = PreviousSide.Last 
			//          Side(n)              | 
			//    #---------------------------#
			//                                |
			//                                |  Complementary
			//    #---------------------------#
			//              Side(n-1)        |       
			//                               ClosePreviousComplementary
			if (ClosePreviousComplementary->GetChainIndex() == Segment0Side->GetChainIndex())
			{
				// Check the slope
				double SlopeAtPreviousComplementary = Complementary[0]->ComputeUnorientedSlopeOf(PreviousComplementary);
				if (SlopeAtPreviousComplementary < 1)
				{
					TArray<FEdgeSegment*>& PreviousSide = ThinZoneSides[PreviousComplementary->GetChainIndex()];
					MergeSideWithComplementaryAtEnd(PreviousSide, Complementary);
					SetMarker1ToComplementarySegments(Complementary);
#ifdef DEBUG_THIN_ZONES_IMPROVE
					if (bDisplay)
					{
						ThinZone::DisplayThinZoneSide(PreviousSide, 80000, EVisuProperty::BlueCurve, false);
						Wait();
					}
#endif
					continue;
				}

				const FEdgeSegment* LastComplementary = Complementary.Last();
				const FEdgeSegment* NextComplementary = LastComplementary->GetNext();
				double SlopeAtNextComplementary = LastComplementary->ComputeUnorientedSlopeOf(NextComplementary);
				if (SlopeAtNextComplementary < 1)
				{
					MergeSideWithComplementaryAtStart(Side, Complementary);
					SetMarker1ToComplementarySegments(Complementary);

#ifdef DEBUG_THIN_ZONES_IMPROVE
					if (bDisplay)
					{
						ThinZone::DisplayThinZoneSide(Side, 70000, EVisuProperty::BlueCurve, false);
						Wait();
					}
#endif
					continue;
				}

				continue;
			}

			FIdent CloseSideIndex = Segment0Side->GetCloseSegment()->GetChainIndex();
			if (CloseSideIndex == ClosePreviousComplementary->GetChainIndex())
			{
				MergeChains(Side, Complementary);
				SetMarker1ToComplementarySegments(Complementary);

#ifdef DEBUG_THIN_ZONES_IMPROVE
				if (bDisplay)
				{
					ThinZone::DisplayThinZoneSide(Side, 20000, EVisuProperty::BlueCurve, false);
					Wait();
				}
#endif

				continue;
			}

			//                  PreviousComplementary[0]
			//                  | Complementary[0]     Seg0Side            Side(n)   = ThinZoneSides[n]
			//       Side(n-1)  | |   Complementary    |  Side(n)          Side(n-1) = ThinZoneSides[n-1]
			//    #--------------#--------------------#-----------#  ->
			//
			//    #--------------#--------------------#-----------#  <-
			//                  |  ComplementaryClose  |
			//                  |                      Seg0Side->GetCloseSegment() == PreviousComplementaryClose[0]
			//                  ClosePreviousComplementary[0]
			TArray<FEdgeSegment*> ComplementaryClose = GetComplementary(ClosePreviousComplementary);
			if (ComplementaryClose.Num())
			{
				const FEdgeSegment* PreviousComplementaryClose = ComplementaryClose[0]->GetPrevious(); // == Seg0Side->GetCloseSegment()
				const FEdgeSegment* CloseOfPreviousComplementaryClose = PreviousComplementaryClose->GetCloseSegment(); // == Seg0Side

				if (CloseOfPreviousComplementaryClose->GetChainIndex() == Segment0Side->GetChainIndex())
				{
					MergeChains(Side, Complementary);
					SetMarker1ToComplementarySegments(Complementary);

					TArray<FEdgeSegment*>& SideOfClosePreviousComplementary = ThinZoneSides[ClosePreviousComplementary->GetChainIndex()];
					MergeChains(SideOfClosePreviousComplementary, ComplementaryClose);
					SetMarker1ToComplementarySegments(ComplementaryClose);

#ifdef DEBUG_THIN_ZONES_IMPROVE			
					if (bDisplay)
					{

						ThinZone::DisplayThinZoneSide(Side, 30000, EVisuProperty::BlueCurve, false);
						Wait();
					}
#endif

					continue;
				}
			}
		}
		continue;
	}

	TFunction<const TArray<const TArray<FEdgeSegment*>*>(const TArray<FEdgeSegment*>&)> FindOppositeSides = [&](const TArray<FEdgeSegment*>& Side) -> const TArray<const TArray<FEdgeSegment*>*>
	{
		TArray<int32> OppositeSideIndexes;
		OppositeSideIndexes.Reserve(5);
		int32 LastIndex = -1;
		for (const FEdgeSegment* Seg : Side)
		{
			const FEdgeSegment* Opposite = Seg->GetCloseSegment();
			if (Opposite != nullptr)
			{
				const int32 Index = Opposite->GetChainIndex();
				if (Index != LastIndex)
				{
					OppositeSideIndexes.AddUnique(Index);
					LastIndex = Index;
				}
			}
		}

		TArray<const TArray<FEdgeSegment*>*> OppositeSides;
		OppositeSides.Reserve(OppositeSideIndexes.Num());
		for (int32 Index : OppositeSideIndexes)
		{
			if (Index < 0)
			{
				continue;
			}
			OppositeSides.Add(&ThinZoneSides[Index]);
		}

		return MoveTemp(OppositeSides);
	};

#ifdef DEBUG_SEARCH_THIN_ZONES_
	if (bDisplay)
	{
		ThinZone::DisplayThinZoneSides2(ThinZoneSides);
		Wait();
	}
#endif

	// Finalize:
	// Find close segments to the added segments
	for (TArray<FEdgeSegment*>& Side : ThinZoneSides)
	{
		if (!Algo::AllOf(Side, [](const FEdgeSegment* Seg) { return Seg->GetCloseSegment(); }))
		{
			const TArray<const TArray<FEdgeSegment*>*> OppositeSides = FindOppositeSides(Side);
			FindCloseSegments(Side, OppositeSides);
		}
	}

#ifdef DEBUG_SEARCH_THIN_ZONES_
	if (bDisplay)
	{
		ThinZone::DisplayThinZoneSides2(ThinZoneSides);
		Wait();
	}
#endif


#ifdef DEBUG_THIN_ZONES_IMPROVE_
	if (bDisplay)
	{
		ThinZone::DisplayThinZoneSides(ThinZoneSides);
		Wait(false);
	}
#endif
}

void FThinZone2DFinder::SplitThinSide()
{
	TArray<TArray<FEdgeSegment*>> NewThinZoneSides;
	FIdent NewSideIndex = ThinZoneSides.Num();

	for (TArray<FEdgeSegment*>& ThinZoneSide : ThinZoneSides)
	{
		if (ThinZoneSide.IsEmpty())
		{
			continue;
		}

		FIdent CloseSideIndex = ThinZoneSide[0]->GetCloseSegment()->GetChainIndex();
		if (!Algo::AllOf(ThinZoneSide, [&CloseSideIndex](const FEdgeSegment* EdgeSegment)
			{
				if (!EdgeSegment->GetCloseSegment())
				{
					return false;
				}
				return EdgeSegment->GetCloseSegment()->GetChainIndex() == CloseSideIndex; 
			}))
		{
			int32 Index = 0;

			const FEdgeSegment* FirstSegment = ThinZoneSide[0];
			const FEdgeSegment* LastSegment = ThinZoneSide.Last();
			const bool bIsCloseZone = (LastSegment->GetNext() == FirstSegment);

			for (; Index < ThinZoneSide.Num(); ++Index)
			{
				if (!ThinZoneSide[Index]->GetCloseSegment() || ThinZoneSide[Index]->GetCloseSegment()->GetChainIndex() != CloseSideIndex)
				{
					break;
				}
			}

			const int32 LastIndexFirstSide = Index;
			while (Index < ThinZoneSide.Num())
			{
				if (!ThinZoneSide[Index]->GetCloseSegment())
				{
					continue;
				}

				CloseSideIndex = ThinZoneSide[Index]->GetCloseSegment()->GetChainIndex();
				TArray<FEdgeSegment*>& NewThinZoneSide = NewThinZoneSides.Emplace_GetRef();
				for (; Index < ThinZoneSide.Num(); ++Index)
				{
					FEdgeSegment* Segment = ThinZoneSide[Index];
					if (!Segment->GetCloseSegment() || Segment->GetCloseSegment()->GetChainIndex() != CloseSideIndex)
					{
						break;
					}
					NewThinZoneSide.Add(Segment);
					Segment->SetChainIndex(NewSideIndex);
				}
				++NewSideIndex;
			}

			ThinZoneSide.SetNum(LastIndexFirstSide);
			if (bIsCloseZone && (FirstSegment->GetCloseSegment()->GetChainIndex() == LastSegment->GetCloseSegment()->GetChainIndex()))
			{
				--NewSideIndex;
				TArray<FEdgeSegment*>& NewThinZoneSide = NewThinZoneSides.Last();
				for (FEdgeSegment* Segment : ThinZoneSide)
				{
					Segment->SetChainIndex(NewSideIndex);
				}
				NewThinZoneSide += MoveTemp(ThinZoneSide);
			}
		}
	}

	for (TArray<FEdgeSegment*>& ThinZoneSide : NewThinZoneSides)
	{
		TArray<FEdgeSegment*>& NewThinZoneSide = ThinZoneSides.Emplace_GetRef();
		NewThinZoneSide = MoveTemp(ThinZoneSide);
	}

}

namespace ThinZone2DFinderTool
{
void AlignStartEndPoints(TArray<FEdgeSegment*>& ClosedSide, TArray<FEdgeSegment*>& OpenSide)
{
	FEdgeSegment* OpenSideFirst = OpenSide[0];
	FEdgeSegment* OpenSideLast = OpenSide.Last();

	TArray<FEdgeSegment*> NewClosedSide;
	NewClosedSide.Reserve(ClosedSide.Num());

	FEdgeSegment* CloseSideSegment = OpenSideLast->GetCloseSegment();
	FEdgeSegment* CloseSideEnd = OpenSideFirst->GetCloseSegment();
	while (CloseSideSegment != CloseSideEnd)
	{
		NewClosedSide.Add(CloseSideSegment);
		CloseSideSegment = CloseSideSegment->GetNext();
	}
	ClosedSide = MoveTemp(NewClosedSide);
}
}

void FThinZone2DFinder::BuildThinZone()
{
	// The number of ThinZone should be less than the number of ThinZoneSides, 
	ThinZones.Reserve(ThinZoneSides.Num());

	for (TArray<FEdgeSegment*>& FirstSide : ThinZoneSides)
	{
		if (FirstSide.IsEmpty())
		{
			continue;
		}

		const FEdgeSegment* FirstSideSegment = FirstSide[0];
		const FIdent OppositeChainIndex = FirstSideSegment->GetCloseSegment()->GetChainIndex();
		if (OppositeChainIndex == Ident::Undefined)
		{
			continue;
		}

		TArray<FEdgeSegment*>& SecondSide = ThinZoneSides[FirstSideSegment->GetCloseSegment()->GetChainIndex()];
		if (SecondSide.IsEmpty())
		{
			continue;
		}

		if (SecondSide[0]->GetCloseSegment()->GetChainIndex() != FirstSideSegment->GetChainIndex())
		{
			continue;
		}

		const bool FirstSideIsClosed = (FirstSide[0]->GetPrevious() == FirstSide.Last());
		const bool SecondSideIsClosed = (SecondSide[0]->GetPrevious() == SecondSide.Last());

		if (FirstSideIsClosed ^ SecondSideIsClosed)
		{
			if (!SecondSideIsClosed)
			{
				ThinZone2DFinderTool::AlignStartEndPoints(FirstSide, SecondSide);
			}
			else
			{
				ThinZone2DFinderTool::AlignStartEndPoints(SecondSide, FirstSide);
			}
		}

		BuildThinZone(FirstSide, SecondSide);

		FirstSide.Empty(); // to avoid to rebuild a second thin zone with SecondSide & FirstSide
		SecondSide.Empty();
	}
}

void FThinZone2DFinder::GetThinZoneSideConnectionsLength(const TArray<FEdgeSegment*>& FirstSide, const TArray<FEdgeSegment*>& SecondSide, double InMaxLength, double* OutLengthBetweenExtremities, TArray<const FTopologicalEdge*>* OutPeakEdges)
{
	using FGetNeighborFunc = TFunction<const FEdgeSegment* (const FEdgeSegment*)>;
	using FComputeLengthBetweenExtremitiesFunc = TFunction<double(const FEdgeSegment*, const FEdgeSegment*, FGetNeighborFunc, TArray<const FTopologicalEdge*>&)>;

	FComputeLengthBetweenExtremitiesFunc ComputeLengthBetweenExtremities = [&InMaxLength](const FEdgeSegment* Start, const FEdgeSegment* End, FGetNeighborFunc GetNext, TArray<const FTopologicalEdge*>& PeakEdges) -> double
	{
		PeakEdges.Reserve(10);

		double LengthBetweenExtremities = 0;
		const FEdgeSegment* Segment = GetNext(Start);
		const FTopologicalEdge* Edge = nullptr;

		while (Segment != End)
		{
			if (Edge != Segment->GetEdge())
			{
				Edge = Segment->GetEdge();
				PeakEdges.Add(Edge);
			}

			LengthBetweenExtremities += Segment->GetLength();
			if (LengthBetweenExtremities > InMaxLength)
			{
				LengthBetweenExtremities = HUGE_VALUE;
				break;
			}
			Segment = GetNext(Segment);
		}
		return LengthBetweenExtremities;
	};

	OutLengthBetweenExtremities[0] = ComputeLengthBetweenExtremities(FirstSide[0], SecondSide.Last(), [](const FEdgeSegment* Segment) { return Segment->GetPrevious(); }, OutPeakEdges[0]);
	OutLengthBetweenExtremities[1] = ComputeLengthBetweenExtremities(FirstSide.Last(), SecondSide[0], [](const FEdgeSegment* Segment) { return Segment->GetNext(); }, OutPeakEdges[1]);
}

void FThinZone2DFinder::BuildThinZone(const TArray<FEdgeSegment*>& FirstSide, const TArray<FEdgeSegment*>& SecondSide)
{
	if (FirstSide.IsEmpty() || SecondSide.IsEmpty())
	{
		return;
	}

	TFunction<void(const TArray<FEdgeSegment*>&, double&, double&)> ComputeThicknessAndLength = [](const TArray<FEdgeSegment*>& Side, double& MaxThickness, double& SideLength)
	{
		SideLength = 0;
		double SquareMaxThickness = 0;

		for (const FEdgeSegment* Segment : Side)
		{
			const double SquareThickness = Segment->GetCloseSquareDistance();
			const double SegmentLength = Segment->GetLength();
			SideLength += SegmentLength;
			if (SquareMaxThickness < SquareThickness)
			{
				SquareMaxThickness = SquareThickness;
			}
		}
		MaxThickness = FMath::Sqrt(SquareMaxThickness);
	};

	EThinZone2DType Zone2DType = EThinZone2DType::BetweenLoops;
	TArray<const FTopologicalEdge*> PeakEdges[2];

	if (!FirstSide[0]->IsInner() && !SecondSide[0]->IsInner())
	{
		double MaxThicknessSide1 = 0;
		double MaxThicknessSide2 = 0;
		double Side1Length;
		double Side2Length;

		ComputeThicknessAndLength(FirstSide, MaxThicknessSide1, Side1Length);
		ComputeThicknessAndLength(SecondSide, MaxThicknessSide2, Side2Length);

		const double MaxThickness = FMath::Max(MaxThicknessSide1, MaxThicknessSide2);
		const double ThinZoneLength = Side1Length + Side2Length;

		double LengthBetweenExtremity[2] = { HUGE_VALUE, HUGE_VALUE };
		GetThinZoneSideConnectionsLength(FirstSide, SecondSide, 3. * MaxThickness, LengthBetweenExtremity, PeakEdges);

		//                     Side 0 
		//       #-------------------------------------# 
		//      /
		//     /  <- LengthBetweendSidesToBePeak  
		//    /
		//   #-----------------------------------------# 
		//                     Side 1 
		//
		// Two EdgeSegments are close if, among other things, the shortest distance between the both EdgeSegments make an angle smallest than 45 deg with the EdgeSegments.
		// So the MaxLengthBetweendSidesToBePeak is theoretically MaxThickness x Sqrt(2)...
		// It's simplified to MaxThickness x 2.
		const double MaxLengthBetweendSidesToBeAPeak = MaxThickness * 2.;
		const double MinThinZoneLengthToBeGlobal = ExternalLoopLength - 2. * MaxLengthBetweendSidesToBeAPeak;

		if (ThinZoneLength > MinThinZoneLengthToBeGlobal || (LengthBetweenExtremity[0] < MaxLengthBetweendSidesToBeAPeak && LengthBetweenExtremity[1] < MaxLengthBetweendSidesToBeAPeak))
		{
			Zone2DType = EThinZone2DType::Global;
		}
		else if (LengthBetweenExtremity[0] < MaxLengthBetweendSidesToBeAPeak)
		{
			if (ThinZoneLength < MaxThickness * 3.)
			{
				// The thin zone is too small => void
				return;
			}
			else
			{
				Zone2DType = EThinZone2DType::PeakStart;
			}
		}
		else if (LengthBetweenExtremity[1] < MaxLengthBetweendSidesToBeAPeak)
		{
			if (ThinZoneLength < MaxThickness * 3.)
			{
				// The thin zone is too small => void
				return;
			}
			else
			{
				Zone2DType = EThinZone2DType::PeakEnd;
			}
		}
		else
		{
			Zone2DType = EThinZone2DType::Butterfly;
		}
	}

	FThinZone2D& Zone = ThinZones.Emplace_GetRef(FirstSide, SecondSide);

	Zone.SetCategory(Zone2DType);
	if (Zone2DType == EThinZone2DType::PeakStart)
	{
		Zone.SetPeakEdgesMarker(PeakEdges[0]);
	}
	else if (Zone2DType == EThinZone2DType::PeakEnd)
	{
		Zone.SetPeakEdgesMarker(PeakEdges[1]);
	}
	Zone.AddToEdge();
}

bool FThinZone2DFinder::SearchThinZones(double InTolerance)
{
	SetTolerance(InTolerance);

	if (!BuildLoopSegments())
	{
		Face.SetAsDegenerated();
		return false;
	}

	FindCloseSegments();

	LinkCloseSegments();

	CheckIfCloseSideOfThinSidesAreNotDegenerated();

	ImproveThinSide();

	SplitThinSide();

	BuildThinZone();

	return (ThinZones.Num() > 0);
}

bool FThinZone2DFinder::BuildLoopSegments()
{
	const double GeometricTolerance = Grid.GetTolerance();
	const double WishedSegmentLength = FinderTolerance / 5.;
	const TArray<TSharedPtr<FTopologicalLoop>>& Loops = Face.GetLoops();

	{
		double Length = 0;
		ExternalLoopLength = -1.;
		for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
		{
			const double LoopLength = Loop->Length();
			Length += LoopLength;

			if (Loop->IsExternal())
			{
				ExternalLoopLength = LoopLength;
			}
		}

		const int32 SegmentNum = (int32)(1.2 * Length / WishedSegmentLength);
		LoopSegments.Empty(SegmentNum);
	}

	const TSharedPtr<FTopologicalLoop> OuterLoop = Loops[0];
	for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
	{
		const bool bIsInnerLoop = (Loop != OuterLoop);
		const TArray<FOrientedEdge>& Edges = Loop->GetEdges();

		FEdgeSegment* FirstSegment = nullptr;
		FEdgeSegment* PrecedingSegment = nullptr;

		for (const FOrientedEdge& Edge : Edges)
		{
			TArray<double> Coordinates;
			// limits the count of segment to 1000 per edge
			const double EdgeLength = Edge.Entity->Length();
			const double SegmentLength = FMath::Min(FMath::Max(EdgeLength / 1000., WishedSegmentLength), EdgeLength);
			Edge.Entity->Sample(SegmentLength, Coordinates);

			TArray<FVector2d> ScaledPoints;
			{
				TArray<FVector2d> Points;
				Edge.Entity->Approximate2DPoints(Coordinates, Points);
				Grid.TransformPoints(EGridSpace::UniformScaled, Points, ScaledPoints);
			}

			// Remove duplicated points (vs GeometricTolerance) of the end
			{
				int32 EndPointIndex = ScaledPoints.Num() - 1;
				while (EndPointIndex > 0 && FVector2d::Distance(ScaledPoints[EndPointIndex], ScaledPoints[EndPointIndex - 1]) < GeometricTolerance)
				{
					EndPointIndex--;
					ScaledPoints.RemoveAt(EndPointIndex);
					Coordinates.RemoveAt(EndPointIndex);
				}

				if (ScaledPoints.Num() == 1)
				{
					continue;
				}
			}

			// Remove duplicated points (vs GeometricTolerance)
			{
				for (int32 PointIndex = 1; PointIndex < ScaledPoints.Num(); PointIndex++)
				{
					while (PointIndex < ScaledPoints.Num() && FVector2d::Distance(ScaledPoints[PointIndex - 1], ScaledPoints[PointIndex]) < GeometricTolerance)
					{
						ScaledPoints.RemoveAt(PointIndex);
						Coordinates.RemoveAt(PointIndex);
					}
				}

				if (ScaledPoints.Num() == 1)
				{
					continue;
				}
			}

			const int32 LastPointIndex = ScaledPoints.Num() - 1;

			const int32 Increment = (Edge.Direction == EOrientation::Front) ? 1 : -1;
			int32 ISegment2 = (Edge.Direction == EOrientation::Front) ? 1 : LastPointIndex - 1;

			for (; ISegment2 >= 0 && ISegment2 <= LastPointIndex; ISegment2 += Increment)
			{
				const int32 ISegment1 = ISegment2 - Increment;
				FEdgeSegment& CurrentSeg = SegmentFatory.New();
				CurrentSeg.SetBoundarySegment(bIsInnerLoop, Edge.Entity.Get(), Coordinates[ISegment1], Coordinates[ISegment2], ScaledPoints[ISegment1], ScaledPoints[ISegment2]);

				LoopSegments.Add(&CurrentSeg);
				if (!FirstSegment)
				{
					FirstSegment = PrecedingSegment = &CurrentSeg;
				}
				else
				{
					PrecedingSegment->SetNext(&CurrentSeg);
				}
				PrecedingSegment = &CurrentSeg;
			}
		}

		if (PrecedingSegment)
		{
			PrecedingSegment->SetNext(FirstSegment);
		}

		if (!bIsInnerLoop && LoopSegments.Num() < 2)
		{
			return false;
		}
	}

	return true;
}

FThinZoneSide::FThinZoneSide(FThinZoneSide* InFrontSide, const TArray<FEdgeSegment*>& InSegments)
	: FrontSide(*InFrontSide)
	, SideLength(-1)
	, MediumThickness(-1)
	, MaxThickness(-1)
{
	const int32 SegmentCount = InSegments.Num();
	Segments.Reserve(SegmentCount);
	for (const FEdgeSegment* Segment : InSegments)
	{
		Segments.Emplace(*Segment);
	}
}

void FThinZone2D::Finalize()
{
	TFunction<void(TArray<FEdgeSegment>&, TMap<int32, FEdgeSegment*>&)> BuildMap = [](TArray<FEdgeSegment>& Segments, TMap<int32, FEdgeSegment*>& Map)
	{
		for (FEdgeSegment& Segment : Segments)
		{
			Map.Emplace(Segment.GetId(), &Segment);
		}
	};

	TFunction<void(TArray<FEdgeSegment>&, TMap<int32, FEdgeSegment*>&)> UpdateReference = [](TArray<FEdgeSegment>& Segments, TMap<int32, FEdgeSegment*>& Map)
	{
		for (FEdgeSegment& Segment : Segments)
		{
			Segment.UpdateReferences(Map);
		}
	};

	TMap<int32, FEdgeSegment*> NewSegmentMap;

	TArray<FEdgeSegment>& SideASegments = SideA.GetSegments();
	TArray<FEdgeSegment>& SideBSegments = SideB.GetSegments();
	BuildMap(SideASegments, NewSegmentMap);
	BuildMap(SideBSegments, NewSegmentMap);

	UpdateReference(SideASegments, NewSegmentMap);
	UpdateReference(SideBSegments, NewSegmentMap);

	SideA.ComputeThicknessAndLength();
	SideB.ComputeThicknessAndLength();

	Thickness = (SideA.GetThickness() + SideB.GetThickness()) * 0.5;
	MaxThickness = FMath::Max(SideA.GetMaxThickness(), SideB.GetMaxThickness());
}

int32 FThinZoneSide::GetImposedPointCount()
{
	TMap<FTopologicalEdge*, FLinearBoundary> ThinZoneOfEdges;

	FLinearBoundary* Boundary = nullptr;
	for (FEdgeSegment& Segment : Segments)
	{
		FTopologicalEdge* Edge = nullptr;
		if (Edge != Segment.GetEdge())
		{
			Edge = Segment.GetEdge();
			ThinZoneOfEdges.Emplace(Edge);
			Boundary = ThinZoneOfEdges.Find(Edge);
			Boundary->Set(Segment.GetCoordinate(Start), Segment.GetCoordinate(End));
		}
		else if (Boundary)
		{
			Boundary->ExtendTo(Segment.GetCoordinate(Start), Segment.GetCoordinate(End));
		}
	}

	int32 ImposedPointCount = 0;
	for (TPair<FTopologicalEdge*, FLinearBoundary> ThinZoneOfEdge : ThinZoneOfEdges)
	{
		FTopologicalEdge* Edge = ThinZoneOfEdge.Key;
		FLinearBoundary& ThinZoneBoundary = ThinZoneOfEdge.Value;

		const TArray<FImposedCuttingPoint>& ImposedCuttingPoints = Edge->GetImposedCuttingPoints();
		for (const FImposedCuttingPoint& CuttingPoint : ImposedCuttingPoints)
		{
			if (ThinZoneBoundary.Contains(CuttingPoint.Coordinate))
			{
				ImposedPointCount++;
			}
		}
	}
	return ImposedPointCount;
}

void FThinZoneSide::ComputeThicknessAndLength()
{
	SideLength = 0;
	double SquareMediumThickness = 0;
	double SquareMaxThickness = 0;

	for (const FEdgeSegment& Segment : Segments)
	{
		const double SquareThickness = Segment.GetCloseSquareDistance();
		const double SegmentLength = Segment.GetLength();
		SideLength += SegmentLength;
		SquareMediumThickness += SquareThickness * SegmentLength;
		if (SquareMaxThickness < SquareThickness)
		{
			SquareMaxThickness = SquareThickness;
		}
	}

	SquareMediumThickness /= SideLength;
	MediumThickness = FMath::Sqrt(SquareMediumThickness);
	MaxThickness = FMath::Sqrt(SquareMaxThickness);
}

void FThinZoneSide::GetEdges(TArray<FTopologicalEdge*>& OutEdges) const
{
	for (FTopologicalEdge* Edge : Edges)
	{
		if (!Edge->IsProcessed())
		{
			OutEdges.Add(Edge);
			Edge->SetProcessedMarker();
		}
	}
}

void FThinZoneSide::AddToEdge()
{
	FTopologicalEdge* Edge = nullptr;
	FLinearBoundary SideEdgeCoordinate;

	TFunction<void(FTopologicalEdge*, FLinearBoundary&) > SetEdge = [this](FTopologicalEdge* Edge, FLinearBoundary& SideEdgeCoordinate)
	{
		Edge->AddThinZone(this, SideEdgeCoordinate);
		Edge->SetThinZoneMarker();
		Edges.AddUnique(Edge);
	};

	for (FEdgeSegment& EdgeSegment : GetSegments())
	{
		double UMin = EdgeSegment.GetCoordinate(ELimit::Start);
		double UMax = EdgeSegment.GetCoordinate(ELimit::End);
		FMath::GetMinMax(UMin, UMax);

		if (Edge != EdgeSegment.GetEdge())
		{
			if (Edge)
			{
				SetEdge(Edge, SideEdgeCoordinate);
			}

			Edge = EdgeSegment.GetEdge();
			SideEdgeCoordinate.Set(UMin, UMax);
		}
		else
		{
			SideEdgeCoordinate.ExtendTo(UMin, UMax);
		}
	};
	SetEdge(Edge, SideEdgeCoordinate);
}

void FThinZoneSide::CheckEdgesZoneSide()
{
	ResetMarker1();
	ResetMarker2();
	for (FTopologicalEdge* Edge : Edges)
	{
		if (Edge->HasMarker1())
		{
			SetMarker1();
		}
		if (Edge->HasMarker2())
		{
			SetMarker2();
		}
	}
}

void FThinZoneSide::SetEdgesZoneSide(ESide Side)
{
	ResetMarkers();
	if (Side == ESide::First)
	{
		Algo::ForEach(Edges, [](FTopologicalEdge* Edge) { Edge->SetMarker1(); });
	}
	else
	{
		Algo::ForEach(Edges, [](FTopologicalEdge* Edge) { Edge->SetMarker2(); });
	}
}

void FThinZoneSide::CleanMesh()
{
	FTopologicalEdge* Edge = nullptr;
	for (FEdgeSegment& EdgeSegment : GetSegments())
	{
		if (Edge != EdgeSegment.GetEdge())
		{
			Edge = EdgeSegment.GetEdge();
			if (!Edge->IsMeshed())
			{
				Edge->RemovePreMesh();
			}
		}
	}
}

void FThinZone2D::SetPeakEdgesMarker(const TArray<const FTopologicalEdge*>& PeakEdges)
{
	for (const FTopologicalEdge* Edge : PeakEdges)
	{
		Edge->SetThinPeakMarker();
	}
}

EMeshingState FThinZoneSide::GetMeshingState() const
{
	EMeshingState MeshingState = NotMeshed;
	int32 EdgeCount = 0;
	int32 MeshedEdgeCount = 0;

	const FTopologicalEdge* Edge = nullptr;
	for (const FEdgeSegment& EdgeSegment : Segments)
	{
		if (Edge != EdgeSegment.GetEdge())
		{
			Edge = EdgeSegment.GetEdge();
			++EdgeCount;
			if (Edge->GetLinkActiveEdge()->IsMeshed())
			{
				++MeshedEdgeCount;
			}
		}
	}

	if (MeshedEdgeCount == 0)
	{
		return NotMeshed;
	}
	else if (MeshedEdgeCount != EdgeCount)
	{
		return PartiallyMeshed;
	}
	return FullyMeshed;
}

void FThinZone2D::AddToEdge()
{
	SideA.AddToEdge();
	SideB.AddToEdge();
}

void FThinZoneSide::GetExistingMeshNodes(const FTopologicalFace& Face, FModelMesh& MeshModel, FReserveContainerFunc& Reserve, FAddMeshNodeFunc& AddMeshNode, const bool bWithTolerance) const
{
	bool bEdgeIsForward = true;
	bool bEdgeIsClosed = false;

	int32 Index = 0;
	int32 Increment = 1;
	double SegmentUMin = 0.;
	double SegmentUMax = 0.;
	int32 EdgeMeshNodeCount = 0;

	TArray<double> CoordinatesOfMesh;
	TArray<int32> NodeIndices;
	TArray<FPairOfIndex> OppositeNodeIndices;
	TArray<double> EdgeElementLength;

	typedef TFunction<bool(double, double)> FCompareMethode;
	TFunction<void(double, FCompareMethode)> FindFirstIndex = [&CoordinatesOfMesh, &Index, &Increment](double ULimit, FCompareMethode Compare)
	{
		for (; Index >= 0 && Index < CoordinatesOfMesh.Num(); Index += Increment)
		{
			if (Compare(ULimit, CoordinatesOfMesh[Index]))
			{
				break;
			}
		}
	};

	TFunction<double(const int32)> ComputeTolerance = [&EdgeElementLength, &EdgeMeshNodeCount](const int32 Index)
	{
		if (Index == 0)
		{
			return EdgeElementLength[0] * ASixth;
		}
		else if (Index == EdgeMeshNodeCount - 1)
		{
			return EdgeElementLength[EdgeMeshNodeCount - 2] * ASixth;
		}
		return FMath::Min(EdgeElementLength[Index - 1], EdgeElementLength[Index]) * AThird;
	};

	TFunction<void(const FEdgeSegment&)> AddEdgeMeshNodeIfInclude = [&](const FEdgeSegment& EdgeSegment)
	{
		bool bAlreadyDone = false;
		for (;; Index += Increment)
		{
			if (Index < 0)
			{
				if (bAlreadyDone)
				{
					break;
				}
				bAlreadyDone = true;
				Index = EdgeMeshNodeCount - 1;
			}
			else if (Index == EdgeMeshNodeCount)
			{
				if (bAlreadyDone)
				{
					break;
				}
				bAlreadyDone = true;
				Index = 0;
			}

			const double MeshNodeCoordinate = CoordinatesOfMesh[Index];
			if (MeshNodeCoordinate < SegmentUMin || MeshNodeCoordinate > SegmentUMax)
			{
				break;
			}

			const double MeshingTolerance = bWithTolerance ? ComputeTolerance(Index) : 0;
			const FVector2d MeshNode2D = EdgeSegment.ComputeEdgePoint(MeshNodeCoordinate);
			AddMeshNode(NodeIndices[Index], MeshNode2D, MeshingTolerance, EdgeSegment, OppositeNodeIndices[Index]);
		}
	};

	// Count the existing mesh vertices on the Side
	{
		int32 MeshVertexCount = 0;
		const FTopologicalEdge* Edge = nullptr;
		for (const FEdgeSegment& EdgeSegment : GetSegments())
		{
			if (Edge != EdgeSegment.GetEdge())
			{
				Edge = EdgeSegment.GetEdge();
				MeshVertexCount++; // first vertex

				if (Edge->GetLinkActiveEdge()->IsMeshed())
				{
					const TArray<FVector>& EdgeMeshNodes = Edge->GetMesh()->GetNodeCoordinates();
					MeshVertexCount += EdgeMeshNodes.Num();
				}
				else if (Edge->IsPreMeshed())
				{
					const TArray<FCuttingPoint>& EdgeCuttingPoints = Edge->GetCuttingPoints();
					MeshVertexCount += EdgeCuttingPoints.Num();
				}
			}
		}
		MeshVertexCount++; // last vertex

		Reserve(MeshVertexCount);
	}

	// Get the existing mesh vertices and fill output data
	{
		FTopologicalEdge* Edge = nullptr;
		const FTopologicalEdge* ActiveEdge = nullptr;

		EOrientation EdgeOrientation = EOrientation::Front;

		for (const FEdgeSegment& EdgeSegment : GetSegments())
		{
			SegmentUMin = EdgeSegment.GetCoordinate(ELimit::Start);
			SegmentUMax = EdgeSegment.GetCoordinate(ELimit::End);
			FMath::GetMinMax(SegmentUMin, SegmentUMax);

			if (Edge != EdgeSegment.GetEdge())
			{
				Edge = EdgeSegment.GetEdge();
				ActiveEdge = &Edge->GetLinkActiveEdge().Get();
				bEdgeIsForward = EdgeSegment.IsForward();
				EdgeMeshNodeCount = 0;

				bEdgeIsClosed = Edge->IsClosed();

				if (!Edge->IsDegenerated())
				{
					if (ActiveEdge->IsPreMeshed())
					{
						{
							FAddCuttingPointFunc AddCuttingPoint = [&Edge](const double Coordinate, const ECoordinateType Type, const FPairOfIndex OppositNodeIndices, const double DeltaU)
							{
								Edge->AddTwinsCuttingPoint(Coordinate, DeltaU);
							};

							const bool bOnlyWithOppositeNode = false;
							Edge->TransferCuttingPointFromMeshedEdge(bOnlyWithOppositeNode, AddCuttingPoint);
						}

						const TArray<FCuttingPoint>& CuttingPoints = Edge->GetCuttingPoints();
						EdgeMeshNodeCount = CuttingPoints.Num();
						if (EdgeMeshNodeCount >= 2)
						{
							CoordinatesOfMesh.Empty(EdgeMeshNodeCount);
							OppositeNodeIndices.Empty(EdgeMeshNodeCount);
							for (int32 Cndex = 0; Cndex < EdgeMeshNodeCount; ++Cndex)
							{
								const FCuttingPoint& CuttingPoint = CuttingPoints[Cndex];
								CoordinatesOfMesh.Add(CuttingPoint.Coordinate);
								OppositeNodeIndices.Emplace(CuttingPoint.OppositNodeIndices);
							}

							if (ActiveEdge->IsMeshed())
							{
								NodeIndices.Empty(EdgeMeshNodeCount);
								const FEdgeMesh* EdgeMesh = Edge->GetMesh();
								if (!EdgeMesh)
								{
									continue;
								}

								NodeIndices.Add(Edge->GetStartVertex()->GetMesh()->GetMesh());

								const int32 InnerEdgeMeshNodeCount = EdgeMeshNodeCount - 2;
								int32 StartMeshVertexId = EdgeMesh->GetStartVertexId();
								if (Edge->IsSameDirection(*ActiveEdge))
								{
									for (int32 Cndex = 0; Cndex < InnerEdgeMeshNodeCount; ++Cndex)
									{
										NodeIndices.Add(StartMeshVertexId++);
									}
								}
								else
								{
									StartMeshVertexId += InnerEdgeMeshNodeCount;
									for (int32 Cndex = 0; Cndex < InnerEdgeMeshNodeCount; ++Cndex)
									{
										NodeIndices.Add(--StartMeshVertexId);
									}
								}

								NodeIndices.Add(Edge->GetEndVertex()->GetMesh()->GetMesh());
							}
							else
							{
								NodeIndices.Init(-1, EdgeMeshNodeCount);
								NodeIndices[0] = Edge->GetStartVertex()->GetMesh()->GetMesh();
								NodeIndices.Last() = Edge->GetEndVertex()->GetMesh()->GetMesh();
							}
						}
					}
					else
					{
						CoordinatesOfMesh.Reset(2);
						NodeIndices.Reset(2);
						OppositeNodeIndices.Reset(2);

						CoordinatesOfMesh.Add(Edge->GetBoundary().GetMin());
						CoordinatesOfMesh.Add(Edge->GetBoundary().GetMax());
						NodeIndices.Add(Edge->GetStartVertex()->GetMesh()->GetMesh());
						NodeIndices.Add(Edge->GetEndVertex()->GetMesh()->GetMesh());
						OppositeNodeIndices.Add(FPairOfIndex::Undefined);
						OppositeNodeIndices.Add(FPairOfIndex::Undefined);
						EdgeMeshNodeCount = 2;
					}

					if (bWithTolerance)
					{
						if (ActiveEdge->IsMeshed())
						{
							EdgeElementLength = Edge->GetMesh()->GetElementLengths();
							if (EdgeElementLength.Num() + 1 != EdgeMeshNodeCount)
							{
								EdgeElementLength = Edge->GetPreElementLengths();
							}
						}
						else
						{
							EdgeElementLength = Edge->GetPreElementLengths();
						}
					}

					if (bEdgeIsForward)
					{
						Index = 0;
						Increment = 1;
						FindFirstIndex(SegmentUMin, [](double Value1, double Value2) { return (Value1 < Value2); });
					}
					else
					{
						Index = EdgeMeshNodeCount - 1;
						Increment = -1;
						FindFirstIndex(SegmentUMax, [](double Value1, double Value2) { return (Value1 > Value2); });
					}
				}
			}

			if(EdgeMeshNodeCount)
			{
				AddEdgeMeshNodeIfInclude(EdgeSegment);
			}
		}
	}
}

}

