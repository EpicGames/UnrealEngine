// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/HaveStates.h"
#include "Core/Factory.h"
#include "Core/Types.h"
#include "Math/Point.h"

#include "Mesh/Structure/EdgeSegment.h"
#include "Mesh/Structure/GridBase.h"
#include "Mesh/Structure/ThinZone2D.h"

namespace UE::CADKernel
{
	class FTopologicalLoop;
	class FTopologicalEdge;
	class FGrid;
	class FThinZone2D;
	class FThinZone2DFinder;

	class FThinZone2DFinder
	{
		friend class FThinZone2D;

	protected:
		const FGridBase& Grid;
		const FTopologicalFace& Face;

		double FinderTolerance;
		double SquareFinderTolerance;

		TArray<FThinZone2D> ThinZones;

		TArray<FEdgeSegment*> LoopSegments;
		TArray<TArray<FEdgeSegment*>> ThinZoneSides;

		double ExternalLoopLength = 0.0;

		TFactory<FEdgeSegment> SegmentFatory;

	public:

		FThinZone2DFinder(const FGridBase& InGrid, const FTopologicalFace& InFace)
			: Grid(InGrid)
			, Face(InFace)
			, SegmentFatory()
		{
#ifdef DEBUG_CADKERNEL
			bDisplay = Grid.bDisplay;
#endif
		}

		bool SearchThinZones(double InFinderTolerance);

		TArray<FThinZone2D>& GetThinZones()
		{
			return ThinZones;
		}


	private:

		void SetTolerance(double InFinderTolerance)
		{
			FinderTolerance = InFinderTolerance;
			SquareFinderTolerance = FMath::Square(FinderTolerance);
		}

		/**
		 * @return false if the main loop is empty
		 */
		bool BuildLoopSegments();

		void FindCloseSegments();

		/**
		 * For Segment with Marker1 in Segments, find in OppositeSides the closest segment in the respect of the criteria
		 */
		void FindCloseSegments(const TArray<FEdgeSegment*>& Segments, const TArray<const TArray<FEdgeSegment*>*>& OppositeSides);

		/**
		 * Fill ThinZoneSides (TArray<TArray<FEdgeSegment*>>)
		 * Each TArray<FEdgeSegment*> is an array of connected segment defining a side of one (or more) thin zone
		 */
		void LinkCloseSegments();

		/**
		 * Two adjacent sides can be separated by a small chain of segments slightly too far to another side be considered as thin side. If this kind of chain is find, the three chains are merged.
		 * E.g. Side(n-1) and Side(n) are close from Side(k)
		 *                           _
		 *                          / \ <-- a small chain of segments slightly too far                          / \
		 *                         /   \												                       /   \
		 *    #-------------------#     #-----------------#              ====>		      #-------------------/     \-----------------#
		 *          Side(n-1)                  Side(n)									                      Side(n)                  
		 *    #-------------------------------------------#								  #-------------------------------------------#
		 *                      Side(k)                                                                       Side(k)  
		 */
		void ImproveThinSide();

		/**
		 * Check that the thin side is not opposite to a unique point
		 * 
		 *     #-----------------------#  a Side
		 * 
		 * 
		 *                                      #------------ 
		 *                                     /
		 *           The opposite unique point 
		 * @return false if the opposite is degenerated
		 */
		bool CheckIfCloseSideOfThinSideIsNotDegenerated(TArray<FEdgeSegment*>& Side);
		void CheckIfCloseSideOfThinSidesAreNotDegenerated();

		/**
		 * SplitThinSide splits chain in front of many other chain eg.
		 * Side A is close to Side B then Side C
		 * Side A has to be split in two
		 * 
		 *                      Side A
		 *     #----------------------------------------#
		 * 
		 *     #-----------------# #--------------------# 
		 *          Side B               Side C
		 */
		void SplitThinSide();

		void BuildThinZone();
		void BuildThinZone(const TArray<FEdgeSegment*>& FirstSide, const TArray<FEdgeSegment*>& SecondSide);
		static void GetThinZoneSideConnectionsLength(const TArray<FEdgeSegment*>& FirstSide, const TArray<FEdgeSegment*>& SecondSide, double InMaxLength, double* OutLengthBetweenExtremities, TArray<const FTopologicalEdge*>* OutPeakEdges);

	// ======================================================================================================================================================================================================================
	// Display Methodes   ================================================================================================================================================================================================
	// ======================================================================================================================================================================================================================
#ifdef CADKERNEL_DEV
		bool bDisplay = false;
		void DisplayCloseSegments();
		void DisplayLoopSegments();
		void DisplaySegmentsOfThinZone();
#endif
	};

#ifdef CADKERNEL_DEBUG
namespace ThinZone
{
static bool bDisplay = false;
void DisplayEdgeSegmentAndProjection(const FEdgeSegment* Segment, EVisuProperty SegColor, EVisuProperty OppositeColor, EVisuProperty ProjectionColor);
void DisplayEdgeSegmentAndProjection(const FEdgeSegment* Segment, const FEdgeSegment* Opposite, EVisuProperty SegColor, EVisuProperty OppositeColor, EVisuProperty ProjectionColor);
void DisplayEdgeSegment(const FEdgeSegment* EdgeSegment, EVisuProperty Color);
void DisplayEdgeSegment(const FEdgeSegment* EdgeSegment, EVisuProperty Color, int32 Index);
void DisplayThinZoneSidesAndCloses(const TArray<TArray<FEdgeSegment*>>& ThinZoneSides);
void DisplayThinZoneSides(const TArray<TArray<FEdgeSegment*>>& ThinZoneSides);
void DisplayThinZoneSides2(const TArray<TArray<FEdgeSegment*>>& ThinZoneSides);
void DisplayThinZoneSide2(const TArray<FEdgeSegment*>& Side, int32 Index, bool bSplitBySide = false, bool bSplitBySegment = false);
void DisplayThinZoneSide(const TArray<FEdgeSegment*>& Side, int32 Index, EVisuProperty Color, bool bSplitBySegment = false);
void DisplayThinZoneSideAndClose(const TArray<FEdgeSegment*>& Side, int32 Index, bool bSplitBySegment = false);
void DisplayThinZoneSide(const FThinZoneSide& Side, int32 Index, EVisuProperty Color, bool bSplitBySegment = false);
void DisplayThinZones(const TArray<FThinZone2D>& ThinZones);
}
#endif

}
