// Copyright Epic Games, Inc. All Rights Reserved.
#include "Topo/Body.h"

#include "Topo/Model.h"
#include "Topo/Shell.h"

namespace UE::CADKernel
{

void FBody::AddShell(TSharedRef<FShell> Shell)
{
	Shells.Add(Shell);
	Shell->SetHost(this);
}

void FBody::RemoveEmptyShell()
{
	TArray<TSharedPtr<FShell>> NewShells;
	NewShells.Reserve(Shells.Num());
	for (TSharedPtr<FShell> Shell : Shells)
	{
		if (!Shell->IsDeleted() && Shell->FaceCount() > 0)
		{
			NewShells.Emplace(Shell);
		}
		else
		{
			Shell->Delete();
			Shell->ResetHost();
		}
	}

	Shells = MoveTemp(NewShells);
}

void FBody::Remove(const FTopologicalShapeEntity* ShellToRemove)
{
	if (!ShellToRemove)
	{
		return;
	}

	int32 Index = Shells.IndexOfByPredicate([&](const TSharedPtr<FShell>& Shell) { return (Shell.Get() == ShellToRemove); });
	if(Index >= 0)
	{
		Shells.RemoveAt(Index);
	}
}

}
