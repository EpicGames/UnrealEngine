// Copyright Epic Games, Inc. All Rights Reserved.

#include "Topo/FaceAnalyzer.h"

#include "Math/Point.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLoop.h"

namespace UE::CADKernel
{

void FFaceAnalyzer::FindClosedSegments(Topo::FThinFaceContext& Context)
{
#ifdef DEBUG_THIN_FACE
	F3DDebugSession _(TEXT("Find Closed Segment"));
#endif
	{
		bool bEdgeIsConnectedToAnotherEdgeOfTheFace = false;
		const FTopologicalEdge* Edge = nullptr;
		for (Topo::FEdgeSegment* Segment : Context.LoopSegments)
		{
			if (Segment->GetEdge() != Edge)
			{
				Edge = Segment->GetEdge();
				bEdgeIsConnectedToAnotherEdgeOfTheFace = false;
				for (FTopologicalEdge* Twin : Edge->GetTwinEntities())
				{
					if (Twin == Edge)
					{
						continue;
					}
					if (Twin->GetFace() == &Face)
					{
						bEdgeIsConnectedToAnotherEdgeOfTheFace = true;
						break;
					}
				}
			}
			if (bEdgeIsConnectedToAnotherEdgeOfTheFace)
			{
				Segment->SetAsThinZone();
			}
		}
	}

	// For each segment, the nearest segment is search
	// If segment is from an inner loop, the nearest could not be from the same loop
	for (Topo::FEdgeSegment* Segment : Context.LoopSegments)
	{
		if (Segment->IsThinZone())
		{
			continue;
		}

		const FVector& Point = Segment->GetMiddle();

		Topo::FEdgeSegment* ClosedSegment = nullptr;

		double MinSquareDistance = HUGE_VALUE;

		for (Topo::FEdgeSegment* Candidate : Context.LoopSegments)
		{
			// to avoid to define cylinder or cone as a thin surface
			if (Candidate->IsThinZone())
			{
				continue;
			}

			const FVector& CenterPointCandidate = Candidate->GetMiddle();
			const double SquareDistanceToCenter = FVector::DistSquared(CenterPointCandidate, Point);

			// if the distance of Point with the middle of candidate segment is biggest than MaxSpace, then the projection of the point cannot be smaller than the tolerance, 
			if (SquareDistanceToCenter > MinSquareDistance)
			{
				continue;
			}

			const FVector& FirstPointCandidate = Candidate->GetExtemity(ELimit::Start);
			const FVector& SecondPointCandidate = Candidate->GetExtemity(ELimit::End);

			double Coordinate;
			FVector Projection = ProjectPointOnSegment<FVector>(Point, FirstPointCandidate, SecondPointCandidate, Coordinate, true);

			double SquareDistance = FVector::DistSquared(Point, Projection);
			if (SquareDistance > MinSquareDistance)
			{
				continue;
			}

			// check the angle between segments. As they are opposite, the cosAngle as to be close to -1
			double CosAngle = Segment->ComputeCosAngleOf(Candidate);
			if (CosAngle > -0.5) // Angle < 3Pi/4 (135 deg)
			{
				continue;
			}

			MinSquareDistance = SquareDistance;
			ClosedSegment = Candidate;
		}

		if (ClosedSegment)
		{
			Segment->SetClosedSegment(ClosedSegment, MinSquareDistance);

#ifdef DEBUG_THIN_FACE
			//F3DDebugSession _(TEXT("Segment"));
			DisplaySegment(Segment->GetExtemity(ELimit::Start), Segment->GetExtemity(ELimit::End), 0, EVisuProperty::BlueCurve);
			DisplaySegment(ClosedSegment->GetExtemity(ELimit::Start), ClosedSegment->GetExtemity(ELimit::End), 0, EVisuProperty::RedCurve);
			//Wait();
#endif
		}
	}
}


void FFaceAnalyzer::Analyze(Topo::FThinFaceContext& Context)
{
	if (Context.LoopSegments.IsEmpty())
	{
		return;
	}

	const FTopologicalEdge* Edge = Context.LoopSegments[0]->GetEdge();
	double MaxSquareDistance = 0;
	double MedSquareDistance = 0;
	double EdgeLength = 0;
	bool bIsThinZone = false;

	TFunction<void()> SetEdgeMaxGap = [&]()
	{
		if (EdgeLength * 1.2 > Edge->Length())
		{
			MedSquareDistance /= EdgeLength;
		}
		else if(!bIsThinZone)
		{
			MedSquareDistance = DOUBLE_BIG_NUMBER;
			MaxSquareDistance = DOUBLE_BIG_NUMBER;
		}

		if (bIsThinZone)
		{
			Context.EdgeSquareDistance.Add(0.);
			Context.EdgeMaxSquareDistance.Add(0.);
			return;
		}
		
		if (MaxSquareDistance < 2 * SquareTolerance)
		{
			Context.EdgeSquareDistance.Add(MedSquareDistance);
			Context.EdgeMaxSquareDistance.Add(MaxSquareDistance);
			Context.MaxSquareDistance = FMath::Max(Context.MaxSquareDistance, MaxSquareDistance);
			Context.ThinSideEdgeLength += Edge->Length();
		}
		else
		{
			Context.EdgeSquareDistance.Add(-1.);
			Context.EdgeMaxSquareDistance.Add(-1.);
			Context.OppositSideEdgeLength += Edge->Length();
		}
		Context.ExternalLoopLength += Edge->Length();
	};

	Context.EdgeSquareDistance.Reserve(Context.Loop.EdgeCount());
	Context.EdgeMaxSquareDistance.Reserve(Context.Loop.EdgeCount());

	for (Topo::FEdgeSegment* Segment : Context.LoopSegments)
	{
		if (Segment->GetEdge() != Edge)
		{
			SetEdgeMaxGap();

			Edge = Segment->GetEdge();
			MaxSquareDistance = 0.;
			MedSquareDistance = 0.;
			EdgeLength = 0.;
			bIsThinZone = false;
		}

		if (Segment->IsThinZone())
		{
			bIsThinZone = true;
		}
		else if (Segment->GetClosedSquareDistance() > 0)
		{
			MedSquareDistance += Segment->GetLength() * Segment->GetClosedSquareDistance();
			EdgeLength += Segment->GetLength();
			if (MaxSquareDistance < Segment->GetClosedSquareDistance())
			{
				MaxSquareDistance = Segment->GetClosedSquareDistance();
			}
		}
	}

	SetEdgeMaxGap();
}


void FFaceAnalyzer::BuildLoopSegments(Topo::FThinFaceContext& Context)
{
	double Length = 0;

	const TArray<FOrientedEdge>& Edges = Context.Loop.GetEdges();

	Context.LoopSegments.Empty();

	for (const FOrientedEdge& OrientedEdge : Edges)
	{
		const FTopologicalEdge* Edge = OrientedEdge.Entity.Get();
		if (Edge->IsDeletedOrDegenerated())
		{
			continue;
		}

		TArray<double> Coordinates;
		TArray<FVector> Points;
		Edge->GetCurve()->GetDiscretizationPoints(Edge->GetBoundary(), Coordinates, Points);

		if (OrientedEdge.Direction == EOrientation::Back)
		{
			Algo::Reverse(Coordinates);
			Algo::Reverse(Points);
		}
			
		int32 PointCount = Points.Num();

		Context.LoopSegments.Reserve(Context.LoopSegments.Num() + PointCount);

		for (int32 ISegment = 0; ISegment < PointCount - 1; ISegment++)
		{
			const int32 Index1 = ISegment;	
			const int32 Index2 = ISegment + 1;
			Topo::FEdgeSegment& CurrentSeg = Context.SegmentFatory.New();
			CurrentSeg.SetBoundarySegment(Edge, Coordinates[Index1], Coordinates[Index2], Points[Index1], Points[Index2]);
			Context.LoopSegments.Add(&CurrentSeg);
		}
	}
}

bool FFaceAnalyzer::IsThinFace(double& OutGapSize)
{
#ifdef DEBUG_THIN_FACE
	F3DDebugSession _(TEXT("Thin Surface"));
#endif
	FTopologicalLoop* Loop = Face.GetExternalLoop().Get();
	if (!Loop)
	{
		return true;
	}

	Topo::FThinFaceContext Context(*Loop);

	BuildLoopSegments(Context);

	FindClosedSegments(Context);

	Analyze(Context);

	if (Context.OppositSideEdgeLength < MaxOppositSideLength || (Context.ThinSideEdgeLength > Context.OppositSideEdgeLength && Context.MaxSquareDistance < SquareTolerance ) )
	{
		OutGapSize = sqrt(Context.MaxSquareDistance);
		return true;
	}

	OutGapSize = DOUBLE_BIG_NUMBER;
	return false;
}

}

