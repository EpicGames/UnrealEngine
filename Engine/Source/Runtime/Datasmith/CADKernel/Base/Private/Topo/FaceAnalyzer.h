// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/HaveStates.h"
#include "Core/Factory.h"
#include "Core/Types.h"
#include "Math/Point.h"
#include "Math/Geometry.h"


//#define DEBUG_THIN_FACE
//#define DISPLAY_THIN_FACE // to display found thin zone


namespace UE::CADKernel
{
class FTopologicalFace;
class FTopologicalEdge;
class FTopologicalLoop;

namespace Topo
{

class FEdgeSegment : public FHaveStates
{
private:
	const FTopologicalEdge* Edge = nullptr;
	double Coordinates[2] = {0. ,0.};
	FVector Points[2];
	FVector Middle;
	FVector Vector;

	FEdgeSegment* ClosedSegment = nullptr;

	double MiddleAxis = 0.;
	double SquareDistanceToClosedSegment = 0.;

	double Length;

	bool bIsThinZone = false;

#ifdef CADKERNEL_DEV
	FIdent Id;
	static FIdent LastId;
#endif

public:
	FEdgeSegment() = default;

	virtual ~FEdgeSegment() = default;

	void SetBoundarySegment(const FTopologicalEdge* InEdge, double InStartU, double InEndU, const FVector& InStartPoint, const FVector& InEndPoint)
	{
		Edge = InEdge;
		Coordinates[ELimit::Start] = InStartU;
		Coordinates[ELimit::End] = InEndU;
		Points[ELimit::Start] = InStartPoint;
		Points[ELimit::End] = InEndPoint;
		Middle = (Points[ELimit::Start] + Points[ELimit::End]) * 0.5;
		Vector = Points[ELimit::End] - Points[ELimit::Start];

		ClosedSegment = nullptr;

		SquareDistanceToClosedSegment = -1.;
		Length = FVector::Distance(Points[ELimit::Start], Points[ELimit::End]);

		MiddleAxis = FVectorUtil::DiagonalAxisCoordinate(Middle);

#ifdef CADKERNEL_DEV
		Id = LastId++;
#endif
	};

	const FVector GetVector() const
	{
		return Vector;
	}

#ifdef CADKERNEL_DEV
	FIdent GetId() const
	{
		return Id;
	}
#endif

	const FTopologicalEdge* GetEdge() const
	{
		return Edge;
	}

	double GetLength() const
	{
		return Length;
	}

	FVector GetMiddle() const
	{
		return Middle;
	}

	constexpr const FVector& GetExtemity(const ELimit Limit) const
	{
		return Points[Limit];
	}

	constexpr double GetCoordinate(const ELimit Limit) const
	{
		return Coordinates[Limit];
	}

	/**
	 * Compute the slope of the input Segment according to this.
	 */
	double ComputeCosAngleOf(const FEdgeSegment* Segment)
	{
		return FVectorUtil::ComputeCosinus(GetVector() , Segment->GetVector());
	}

	FEdgeSegment* GetClosedSegment() const
	{
		return ClosedSegment;
	}

	void SetClosedSegment(FEdgeSegment* InSegmentA, double InSquareDistance)
	{
		ClosedSegment = InSegmentA;
		SquareDistanceToClosedSegment = InSquareDistance;
	}

	double GetClosedSquareDistance() const
	{
		return SquareDistanceToClosedSegment;
	}

	FVector ProjectPoint(const FVector& PointToProject, double& SegmentU) const
	{
		return ProjectPointOnSegment<FVector>(PointToProject, Points[ELimit::Start], Points[ELimit::End], SegmentU, true);
	}

	bool IsThinZone()
	{
		return bIsThinZone;
	}

	void SetAsThinZone()
	{
		bIsThinZone = true;
	}
};

struct FThinFaceContext
{
	const FTopologicalLoop& Loop;

	TArray<Topo::FEdgeSegment*> LoopSegments;
	double ExternalLoopLength = 0.0;
	TFactory<Topo::FEdgeSegment> SegmentFatory;

	TArray<double> EdgeSquareDistance;
	TArray<double> EdgeMaxSquareDistance;

	double MaxSquareDistance = 0;
	double ThinSideEdgeLength = 0;
	double OppositSideEdgeLength = 0;

	FThinFaceContext(const FTopologicalLoop& InLoop)
		: Loop(InLoop)
	{
	}
};

}

class FFaceAnalyzer
{

protected:
	const double Tolerance;
	const double SquareTolerance;
	const double MaxOppositSideLength;

	FTopologicalFace& Face;

public:

	FFaceAnalyzer(FTopologicalFace& InFace, double InTol)
		: Tolerance(InTol)
		, SquareTolerance(FMath::Square(InTol))
		, MaxOppositSideLength(4. * InTol)
		, Face(InFace)
	{
 	}

	bool IsThinFace(double& OutGapSize);

private:

	void BuildLoopSegments(Topo::FThinFaceContext& Context);
	void FindClosedSegments(Topo::FThinFaceContext& Context);
	void Analyze(Topo::FThinFaceContext& Context);

#ifdef CADKERNEL_DEV
	void DisplayCloseSegments(Topo::FThinFaceContext& Context);
	void DisplayLoopSegments(Topo::FThinFaceContext& Context);
#endif
};
}
