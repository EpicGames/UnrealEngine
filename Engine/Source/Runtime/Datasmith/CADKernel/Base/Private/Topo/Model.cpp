// Copyright Epic Games, Inc. All Rights Reserved.
#include "Topo/Model.h"

#include "Core/EntityGeom.h"
#include "Topo/Body.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalVertex.h"

namespace UE::CADKernel
{

void FModel::AddEntity(TSharedRef<FTopologicalEntity> Entity)
{
	switch (Entity->GetEntityType())
	{
	case EEntity::Body:
		Add(StaticCastSharedRef<FBody>(Entity));
		break;
	default:
		break;
	}
}

bool FModel::Contains(TSharedPtr<FTopologicalEntity> Entity)
{
	switch(Entity->GetEntityType())
	{
	case EEntity::Body:
		return Bodies.Find(StaticCastSharedPtr<FBody>(Entity)) != INDEX_NONE;
	default:
		return false;
	}
}

int32 FModel::FaceCount() const
{
	int32 FaceCount = 0;
	for (const TSharedPtr<FBody>& Body : Bodies)
	{
		FaceCount += Body->FaceCount();
	}
	return FaceCount;
}

void FModel::RemoveEmptyBodies()
{
	TArray<TSharedPtr<FBody>> NewBodies;
	NewBodies.Reserve(Bodies.Num());

	for (const TSharedPtr<FBody>& Body : Bodies)
	{
		if (!Body->IsDeleted() && Body->ShellCount())
		{
			NewBodies.Add(Body);
		}
		else
		{
			Body->Delete();
			Body->ResetHost();
		}
	}

	Bodies = MoveTemp(NewBodies);
}

void FModel::GetFaces(TArray<FTopologicalFace*>& OutFaces)
{
	for (const TSharedPtr<FBody>& Body : Bodies)
	{
		Body->GetFaces(OutFaces);
	}
}

void FModel::PropagateBodyOrientation()
{
	for (TSharedPtr<FBody>& Body : Bodies)
	{
		Body->PropagateBodyOrientation();
	}
}

void FModel::CompleteMetaData()
{
	for (TSharedPtr<FBody>& Body : Bodies)
	{
		Body->CompleteMetaData();
	}
}

struct FBodyShell
{
	TSharedPtr<FBody> Body;
	TSharedPtr<FShell> Shell;

	FBodyShell(TSharedPtr<FBody> InBody, TSharedPtr<FShell> InShell)
	: Body(InBody)
	, Shell(InShell)
	{
	}

};

void FModel::CheckTopology() 
{
	TArray<FBodyShell> IsolatedBodies;
	IsolatedBodies.Reserve(Bodies.Num()*2);

	int32 ShellCount = 0;

	for (TSharedPtr<FBody> Body : Bodies)
	{
		for (TSharedPtr<FShell> Shell : Body->GetShells())
		{
			ShellCount++;
			TArray<FFaceSubset> SubShells;
			Shell->CheckTopology(SubShells);

			if (SubShells.Num() == 1 )
			{
				if (Shell->FaceCount() < 3 )
				{
					IsolatedBodies.Emplace(Body, Shell);
				}
				else
				{
					if (SubShells[0].BorderEdgeCount > 0 || SubShells[0].NonManifoldEdgeCount > 0)
					{
						UE_LOGF(LogCADKernelBase, Verbose, "Body %d shell %d is opened and has %d faces ", Body->GetId(), Shell->GetId(), Shell->FaceCount());
						UE_LOGF(LogCADKernelBase, Verbose, "and has %d border edges and %d nonManifold edges\n", SubShells[0].BorderEdgeCount, SubShells[0].NonManifoldEdgeCount);
					}
					else
					{
						UE_LOGF(LogCADKernelBase, Verbose, "Body %d shell %d is closed and has %d faces\n", Body->GetId(), Shell->GetId(), Shell->FaceCount());
					}
				}
			}
			else
			{
				UE_LOGF(LogCADKernelBase, Verbose, "Body %d shell %d has %d subshells\n", Body->GetId(), Shell->GetId(), SubShells.Num());
				for (const FFaceSubset& Subset : SubShells)
				{
					UE_LOGF(LogCADKernelBase, Verbose, "     - Subshell of %d faces %d border edges and %d nonManifold edges\n", Subset. Faces.Num(), Subset.BorderEdgeCount, Subset.NonManifoldEdgeCount);
				}
			}
		}
	}
}

void FModel::Orient()
{
	for (TSharedPtr<FBody> Body : Bodies)
	{
		Body->Orient();
	}
}

}