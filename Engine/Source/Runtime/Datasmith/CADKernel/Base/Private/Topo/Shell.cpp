// Copyright Epic Games, Inc. All Rights Reserved.
#include "Topo/Shell.h"

#include "Geo/GeoEnum.h"
#include "Topo/Body.h"
#include "Topo/TopologicalFace.h"

#ifdef CADKERNEL_DEV
#include "Topo/TopologyReport.h"
#endif

namespace UE::CADKernel
{

FShell::FShell(const TArray<TSharedPtr<FTopologicalFace>>& InTopologicalFaces, bool bIsInnerShell)
	: FTopologicalShapeEntity()
	, TopologicalFaces()
{
	TopologicalFaces.Reserve(InTopologicalFaces.Num());

	TArray<EOrientation> Orientations;
	Orientations.Reserve(InTopologicalFaces.Num());

	for (TSharedPtr<FTopologicalFace> Face : InTopologicalFaces)
	{
		TopologicalFaces.Emplace(Face, EOrientation::Front);
	}

	if (bIsInnerShell)
	{
		SetInner();
	}
}

FShell::FShell(const TArray<TSharedPtr<FTopologicalFace>>& InTopologicalFaces, const TArray<EOrientation>& InOrientations, bool bIsInnerShell)
	: FTopologicalShapeEntity()
{
	ensureCADKernel(InTopologicalFaces.Num() == InOrientations.Num());
	for (int32 Index = 0; Index < InTopologicalFaces.Num(); ++Index)
	{
		TSharedPtr<FTopologicalFace> Face = InTopologicalFaces[Index];
		EOrientation Orientation = InOrientations[Index];
		TopologicalFaces.Emplace(Face, Orientation);
	}

	if (bIsInnerShell)
	{
		SetInner();
	}
}

void FShell::RemoveFaces()
{
	for (FOrientedFace& Face : TopologicalFaces)
	{
		Face.Entity->ResetHost();
	}
	TopologicalFaces.Empty();
}

void FShell::RemoveDeletedOrDegeneratedFaces()
{
	TArray<FOrientedFace> NewTopologicalFaces;
	NewTopologicalFaces.Reserve(TopologicalFaces.Num());

	for (FOrientedFace& Face : TopologicalFaces)
	{
		if (!Face.Entity->IsDeletedOrDegenerated())
		{
			NewTopologicalFaces.Emplace(Face);
		}
		else
		{
			Face.Entity->ResetHost();
		}
	}
	TopologicalFaces = MoveTemp(NewTopologicalFaces);
}

void FShell::Empty()
{
	RemoveFaces();
	FTopologicalShapeEntity::Empty();
}

void FShell::Add(TArray<FTopologicalFace*>& Faces)
{
	TopologicalFaces.Reserve(TopologicalFaces.Num() + Faces.Num());

	for(FTopologicalFace* Face : Faces)
	{
		TSharedPtr<FTopologicalFace> FacePtr = StaticCastSharedRef<FTopologicalFace>(Face->AsShared());
		TopologicalFaces.Emplace(FacePtr, Face->IsBackOriented() ? EOrientation::Back : EOrientation::Front);
		Face->SetHost(this);
	}
}

void FShell::Add(TSharedRef<FTopologicalFace> InTopologicalFace, EOrientation Orientation)
{
	TSharedPtr<FTopologicalFace> Face = InTopologicalFace;
	TopologicalFaces.Emplace(Face, Orientation);

	Face->SetHost(this);
}

void FShell::Remove(const FTopologicalShapeEntity* FaceToRemove)
{
	if (!FaceToRemove)
	{
		return;
	}

	int32 Index = TopologicalFaces.IndexOfByPredicate([&](const FOrientedFace& Face) { return (Face.Entity.Get() == FaceToRemove); });
	if(Index >= 0)
	{
		TopologicalFaces.RemoveAt(Index);
	}
}

#ifdef CADKERNEL_DEV
FInfoEntity& FShell::GetInfo(FInfoEntity& Info) const
{
	return FTopologicalShapeEntity::GetInfo(Info)
		.Add(TEXT("TopologicalFaces"), (TArray<TOrientedEntity<FEntity>>&) TopologicalFaces);
}
#endif

void FShell::GetFaces(TArray<FTopologicalFace*>& Faces)
{
	for (FOrientedFace& Face : TopologicalFaces)
	{
		if (Face.Entity->HasMarker1())
		{
			continue;
		}

		Faces.Add(Face.Entity.Get());
		Face.Entity->SetMarker1();
	}
}

void FShell::Merge(TSharedPtr<FShell>& Shell)
{
	TopologicalFaces.Append(Shell->TopologicalFaces);
	Shell->TopologicalFaces.Empty();
}


void FShell::PropagateBodyOrientation()
{
	const bool bIsOuter = IsOuter();
	for (FOrientedFace& Face : TopologicalFaces)
	{
		if (bIsOuter != (Face.Direction == EOrientation::Front))
		{
			Face.Entity->SetBackOriented();
		}
		else
		{
			Face.Entity->ResetBackOriented();
		}
	}
}

void FShell::CompleteMetaData()
{
	CompleteMetaDataWithHostMetaData();
	for (FOrientedFace& Face : TopologicalFaces)
	{
		Face.Entity->CompleteMetaData();
	}
}


void FShell::UpdateShellOrientation()
{
	const bool bIsOuter = IsOuter();
	for (FOrientedFace& Face : TopologicalFaces)
	{
		if (bIsOuter == Face.Entity->IsBackOriented())
		{
			Face.Direction = EOrientation::Back;
		}
		else
		{
			Face.Direction = EOrientation::Front;
		}
	}
}

bool FShell::IsOpenShell()
{
	for (const FOrientedFace& OrientedFace : GetFaces())
	{
		const TSharedPtr<FTopologicalFace>& Face = OrientedFace.Entity;
		for (const TSharedPtr<FTopologicalLoop>& Loop : Face->GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;
				if (Edge->GetTwinEntityCount() == 1)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FShell::CheckTopology(TArray<FFaceSubset>& Subshells)
{
	// Processed1 : Surfaces added in CandidateSurfacesForMesh

	int32 TopologicalFaceCount = FaceCount();
	// Is closed ?
	// Is one shell ?

	int32 ProcessFaceCount = 0;

	TArray<FTopologicalFace*> Front;
	TFunction<void(const FTopologicalFace&, FFaceSubset&)> GetNeighboringFaces = [&](const FTopologicalFace& Face, FFaceSubset& Shell)
	{
		for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;
				if (Edge->HasMarker1())
				{
					continue;
				}
				Edge->SetMarker1();

				if (Edge->GetTwinEntityCount() == 1)
				{
					if (!Edge->IsDegenerated())
					{
						Shell.BorderEdgeCount++;
					}
					continue;
				}

				if (Edge->GetTwinEntityCount() > 2)
				{
					Shell.NonManifoldEdgeCount++;
				}

				for (FTopologicalEdge* NextEdge : Edge->GetTwinEntities())
				{
					if (NextEdge->HasMarker1())
					{
						continue;
					}
					NextEdge->SetMarker1();

					FTopologicalFace* NextFace = NextEdge->GetFace();
					if ((NextFace == nullptr) || NextFace->HasMarker1())
					{
						continue;
					}
					NextFace->SetMarker1();
					Front.Add(NextFace);
				}
			}
		}
	};
	
	TFunction<void(FFaceSubset&)> PropagateFront = [&](FFaceSubset& Shell)
	{
		while (Front.Num())
		{
			FTopologicalFace* Face = Front.Pop();
			Shell.Faces.Add(Face);
			GetNeighboringFaces(*Face, Shell);
		}
	};

	for (FOrientedFace& OrientedFace : GetFaces())
	{
		if (OrientedFace.Entity->HasMarker1())
		{
			continue;
		}
	
		FFaceSubset& Shell = Subshells.Emplace_GetRef();
		Shell.Faces.Reserve(TopologicalFaceCount - ProcessFaceCount);
		Front.Empty(TopologicalFaceCount);

		FTopologicalFace* Face = OrientedFace.Entity.Get();

		Front.Empty(TopologicalFaceCount);
		Face->SetMarker1();
		Front.Add(Face);
		PropagateFront(Shell);
		ProcessFaceCount += Shell.Faces.Num();

		if (ProcessFaceCount == TopologicalFaceCount)
		{
			break;
		}
	}

	ResetMarkersRecursively();
}

#ifdef CADKERNEL_DEV
void FShell::FillTopologyReport(FTopologyReport& Report) const
{
	Report.Add(this);
	for (const FOrientedFace& OrientedFace : GetFaces())
	{
		OrientedFace.Entity->FillTopologyReport(Report);
	}
}
#endif

int32 FShell::Orient()
{
	int32 TopologicalFaceCount = GetFaces().Num();

	if (TopologicalFaceCount < 2)
	{
		return 0;
	}

	TArray<FTopologicalFace*> Subshell;
	TArray<FTopologicalFace*> Front;

	int32 ShellSwappedFaceCount = 0;

	int32 SubshellFaceCount = 0;
	int32 SubshellSwappedFaceCount = 0;
	double BorderEdgesLength = 0;

	TFunction<void(const FTopologicalFace&)> GetAndOrientNeighboringFaces = [&](const FTopologicalFace& Face)
	{
		for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;
	
				if ((Edge->GetTwinEntityCount() != 2) || Edge->IsDegenerated())
				{
					if (Edge->GetTwinEntityCount() == 1)
					{
						BorderEdgesLength += Edge->Length();
					}
					continue;
				}

				FTopologicalEdge* NeighboringEdge = Edge->GetTwinEdge();
				if (NeighboringEdge == nullptr)
				{
					continue;
				}

				const FTopologicalLoop* NeighboringLoop = NeighboringEdge->GetLoop();
				if (NeighboringLoop == nullptr)
				{
					continue;
				}

				FTopologicalFace* NeighboringFace = NeighboringLoop->GetFace();
				if (NeighboringFace == nullptr)
				{
					continue;
				}

				if (NeighboringFace->IsNotToOrAlreadyProcess())
				{
					continue;
				}

				NeighboringFace->SetProcessedMarker();
				Front.Add(NeighboringFace);

				const FOrientedEdge* OrientedNeighboringEdge = NeighboringLoop->GetOrientedEdge(NeighboringEdge);
				if (OrientedNeighboringEdge == nullptr)
				{
					continue;
				}

				const bool EdgesAreWellOriented = (Edge->IsSameDirection(*NeighboringEdge) == (OrientedNeighboringEdge->Direction != OrientedEdge.Direction));
				const bool FacesHaveSameOrientation = Face.IsBackOriented() == NeighboringFace->IsBackOriented();

				if (EdgesAreWellOriented != FacesHaveSameOrientation)
				{
					NeighboringFace->SwapOrientation();
					SubshellSwappedFaceCount++;
				}
			}
		}
	};

	TFunction<void()> PropagateFront = [&]()
	{
		while (Front.Num())
		{
			FTopologicalFace* Face = Front.Pop();
			if (Face == nullptr)
			{
				continue;
			}

			Subshell.Add(Face);
			SubshellFaceCount++;

			GetAndOrientNeighboringFaces(*Face);
		}
	};

	TFunction<FBBoxWithNormal(const double)> GetSubshellBoundingBox = [&](const double ApproximationFactor) -> FBBoxWithNormal
	{
#ifdef DEBUG_GET_SHELL_BBOX
		F3DDebugSession _(TEXT("GetShellBoundingBox"));
#endif

		FBBoxWithNormal BBox;

		for (FTopologicalFace* Face : Subshell)
		{
			Face->UpdateBBox(3, ApproximationFactor, BBox);
		}

#ifdef DEBUG_GET_SHELL_BBOX
		{
			F3DDebugSession _(TEXT("BBox Face"));

			FAABB AABB(BBox.Min, BBox.Max);
			UE::CADKernel::DisplayAABB(AABB);

			for (int32 Index = 0; Index < 3; ++Index)
			{
				UE::CADKernel::DisplayPoint(BBox.MaxPoints[Index], EVisuProperty::YellowPoint);
				UE::CADKernel::DisplayPoint(BBox.MinPoints[Index], EVisuProperty::YellowPoint);
				UE::CADKernel::DisplaySegment(BBox.MaxPoints[Index], BBox.MaxPoints[Index] + BBox.MaxPointNormals[Index], 0, EVisuProperty::YellowCurve);
				UE::CADKernel::DisplaySegment(BBox.MinPoints[Index], BBox.MinPoints[Index] + BBox.MinPointNormals[Index], 0, EVisuProperty::YellowCurve);
			}
			Wait();
		}
#endif
		return BBox;
	};

	TFunction<void()> SwapSubshellOrientation = [&]()
	{
		for (FTopologicalFace* Face : Subshell)
		{
			Face->SwapOrientation();
		}
	};

	TFunction<void()> FixOpenShellOrientation = [&]()
	{
		if (SubshellSwappedFaceCount > (SubshellFaceCount / 2))
		{
			UE_LOGF(LogCADKernelBase, Verbose, "the open shell %ls (Id %d) is badly oriented. Its orientation is swapped", GetName(), GetId());
			SwapSubshellOrientation();
			ShellSwappedFaceCount += (SubshellFaceCount - SubshellSwappedFaceCount);
		}
		else
		{
			ShellSwappedFaceCount += SubshellSwappedFaceCount;
		}
	};

	TFunction<void()> CheckAndFixSubshellMainOrientation = [&]()
	{
		if (Subshell.Num() == 1)
		{
			return;
		}

		FBBoxWithNormal BBox = GetSubshellBoundingBox(100.);
		double BBoxLength = BBox.Length();

		bool bIsOpenShell = (BBoxLength < BorderEdgesLength);
		if(bIsOpenShell)
		{
			FixOpenShellOrientation();
			return;
		}

		bool bHasWrongOrientation = false;
		if (!BBox.CheckOrientation(bHasWrongOrientation))
		{
			FBBoxWithNormal BBox2 = GetSubshellBoundingBox(10.);
			if (!BBox.CheckOrientation(bHasWrongOrientation))
			{
				FixOpenShellOrientation();
				return;
			}
		}

		if (bHasWrongOrientation)
		{
#ifdef CADERNEL_DEV
			UE_LOGF(LogCADKernelBase, Verbose, "The closed shell %ls (Id %d) is badly oriented. Its orientation is swapped", *GetName(), GetId());
#endif
			SwapSubshellOrientation();
			ShellSwappedFaceCount += (SubshellFaceCount - SubshellSwappedFaceCount);
		}
		else
		{
			ShellSwappedFaceCount += SubshellSwappedFaceCount;
		}
	};

	for (FOrientedFace& Face : GetFaces())
	{
		Face.Entity->ResetMarkers();
		if(!Face.Entity->IsDeletedOrDegenerated())
		{
			Face.Entity->SetToProcessMarker();
		}
	}

	int32 ProcessFaceCount = 0;
	for (FOrientedFace& Face : GetFaces())
	{
		if (Face.Entity->IsNotToOrAlreadyProcess())
		{
			continue;
		}

		// Init data for the new subshell
		SubshellFaceCount = 0;
		SubshellSwappedFaceCount = 0;
		BorderEdgesLength = 0;
		Subshell.Empty(TopologicalFaceCount);
		Front.Empty(TopologicalFaceCount);

		Face.Entity->SetProcessedMarker();
		Front.Add(Face.Entity.Get());

		PropagateFront();

		CheckAndFixSubshellMainOrientation();

		ProcessFaceCount += Subshell.Num();

		if (ProcessFaceCount == TopologicalFaceCount)
		{
			break;
		}
	}

	UpdateShellOrientation();

	for (FOrientedFace& Face : GetFaces())
	{
		Face.Entity->ResetMarkers();
	}

	return ShellSwappedFaceCount;
}

namespace ShellTools
{

void UnlinkFromOther(TArray<FTopologicalFace*>& Faces, TArray<FTopologicalVertex*>& VerticesToLink)
{
	for (FTopologicalFace* Face : Faces)
	{
		Face->SetToProcessMarker();
	}

	VerticesToLink.Reserve(100);

	for (FTopologicalFace* Face : Faces)
	{
		TFunction<void(FTopologicalVertex&)> AddTwinVertex = [&Face, &VerticesToLink](FTopologicalVertex& Vertex)
		{
			for (FTopologicalVertex* TwinVertex : Vertex.GetTwinEntities())
			{
				if (!TwinVertex->IsProcessed() && TwinVertex->GetFace() == Face)
				{
					VerticesToLink.Add(TwinVertex);
					TwinVertex->SetProcessedMarker();
				}
			}
		};

		for (const TSharedPtr<FTopologicalLoop>& Loop : Face->GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				FTopologicalEdge* Edge = OrientedEdge.Entity.Get();
				if (!Edge)
				{
					continue;
				}

				for (FTopologicalEdge* TwinEdge : Edge->GetTwinEntities())
				{
					if (!TwinEdge || TwinEdge == Edge)
					{
						continue;
					}

					FTopologicalFace* NeighborFace = TwinEdge->GetFace();
					if (!NeighborFace)
					{
						continue;
					}

					if (!NeighborFace->IsToProcess())
					{
						FTopologicalVertex& StartVertex = Edge->GetStartVertex().Get();
						AddTwinVertex(StartVertex);

						FTopologicalVertex& EndVertex = Edge->GetEndVertex().Get();
						AddTwinVertex(EndVertex);

						Edge->Unlink();

						break;
					}
				}
			}
		}
	}

	for (FTopologicalFace* Face : Faces)
	{
		Face->ResetToProcessMarker();
	}

	for (FTopologicalVertex* Vertex : VerticesToLink)
	{
		Vertex->ResetProcessedMarker();
	}
}
}


}