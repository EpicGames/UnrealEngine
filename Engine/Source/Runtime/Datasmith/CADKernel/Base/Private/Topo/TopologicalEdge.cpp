// Copyright Epic Games, Inc. All Rights Reserved.

#include "Topo/TopologicalEdge.h"

#include "Geo/Curves/NURBSCurve.h"
#include "Geo/Curves/SegmentCurve.h"
#include "Geo/GeoEnum.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Geo/Sampler/SamplerOnParam.h"
#include "Math/BSpline.h"
#include "Math/Point.h"
#include "Math/SlopeUtils.h"
#include "Mesh/Structure/EdgeMesh.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalLoop.h"
#include "Topo/TopologicalLink.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalVertex.h"

namespace UE::CADKernel
{

FTopologicalEdge::FTopologicalEdge(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2, const FLinearBoundary& InBoundary)
	: StartVertex(InVertex1)
	, EndVertex(InVertex2)
	, Boundary(InBoundary)
	, Curve(InCurve)
	, Length3D(-1)
{
	ensureCADKernel(Boundary.IsValid());
	ensureCADKernel(Boundary.GetMin() < Boundary.GetMax());
}

FTopologicalEdge::FTopologicalEdge(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2)
	: StartVertex(InVertex1)
	, EndVertex(InVertex2)
	, Curve(InCurve)
	, Length3D(-1)
{
	Boundary = Curve->GetBoundary();
	ensureCADKernel(Boundary.IsValid());
}

FTopologicalEdge::FTopologicalEdge(const TSharedRef<FRestrictionCurve>& InCurve, const FLinearBoundary& InBoundary)
	: Boundary(InBoundary)
	, Curve(InCurve)
	, Length3D(-1)
{
	TArray<double> Coordinates = { Boundary.Min, Boundary.Max };
	TArray<FCurvePoint> Points;
	Curve->EvaluatePoints(Coordinates, Points);

	StartVertex = FTopologicalVertex::Make(Points[0].Point);
	EndVertex = FTopologicalVertex::Make(Points[1].Point);
}

FTopologicalEdge::FTopologicalEdge(const TSharedRef<FSurface>& InSurface, const FVector2d& InCoordinateVertex1, const TSharedRef<FTopologicalVertex>& InVertex1, const FVector2d& InCoordinateVertex2, const TSharedRef<FTopologicalVertex>& InVertex2)
	: StartVertex(InVertex1)
	, EndVertex(InVertex2)
	, Length3D(-1)
{
	TSharedRef<FSegmentCurve> Curve2D = FEntity::MakeShared<FSegmentCurve>(InCoordinateVertex1, InCoordinateVertex2, 2);
	Curve = FEntity::MakeShared<FRestrictionCurve>(InSurface, Curve2D);
}

void FTopologicalEdge::LinkVertex()
{
	StartVertex->AddConnectedEdge(*this);
	EndVertex->AddConnectedEdge(*this);

	if (IsDegenerated())
	{
		StartVertex->Link(*EndVertex);
	}
}

bool FTopologicalEdge::CheckVertices()
{
	TArray<double> Coordinates = { Boundary.Min, Boundary.Max };
	TArray<FVector> Points;
	Curve->Approximate3DPoints(Coordinates, Points);

	double ToleranceGeo = GetTolerance3D();
	TFunction<bool(TSharedPtr<FTopologicalVertex>, FVector)> CheckExtremityGap = [&](TSharedPtr<FTopologicalVertex> Vertex, FVector Point)
	{
		double GapToVertex = FVector::Distance(Vertex->GetCoordinates(), Point);
		return (GapToVertex < ToleranceGeo);
	};

	if (!CheckExtremityGap(StartVertex, Points[0]))
	{
		if (CheckExtremityGap(StartVertex, Points[1]) && CheckExtremityGap(EndVertex, Points[0]))
		{
			Swap(StartVertex, EndVertex);
			return true;
		}
		return false;
	}
	return CheckExtremityGap(EndVertex, Points[1]);
}

bool FTopologicalEdge::CheckIfDegenerated() const
{
	bool bDegeneration2D = false;
	bool bDegeneration3D = false;

	Curve->CheckIfDegenerated(Boundary, bDegeneration2D, bDegeneration3D, Length3D);

	Max2DTolerance = Length3D * FactorToComputeMaxTol;

	if (bDegeneration3D)
	{
		SetAsDegenerated();
	}

	return bDegeneration2D;
}

TSharedPtr<FTopologicalEdge> FTopologicalEdge::Make(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2, const FLinearBoundary& InBoundary)
{
	TSharedRef<FTopologicalEdge> Edge = FEntity::MakeShared<FTopologicalEdge>(InCurve, InVertex1, InVertex2, InBoundary);
	return ReturnIfValid(Edge, /*bCheckVertices*/ true);
}

TSharedPtr<FTopologicalEdge> FTopologicalEdge::Make(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2)
{
	TSharedRef<FTopologicalEdge> Edge = FEntity::MakeShared<FTopologicalEdge>(InCurve, InVertex1, InVertex2);
	return ReturnIfValid(Edge, /*bCheckVertices*/ true);
}

TSharedPtr<FTopologicalEdge> FTopologicalEdge::Make(const TSharedRef<FRestrictionCurve>& InCurve, const FLinearBoundary& InBoundary)
{
	TSharedRef<FTopologicalEdge> Edge = FEntity::MakeShared<FTopologicalEdge>(InCurve, InBoundary);
	return ReturnIfValid(Edge, /*bCheckVertices*/ false);
}

TSharedPtr<FTopologicalEdge> FTopologicalEdge::Make(const TSharedRef<FRestrictionCurve>& InCurve)
{
	return Make(InCurve, InCurve->GetBoundary());
}

TSharedPtr<FTopologicalEdge> FTopologicalEdge::Make(const TSharedRef<FSurface>& InSurface, const FVector2d& InCoordinateVertex1, const TSharedRef<FTopologicalVertex>& InVertex1, const FVector2d& InCoordinateVertex2, const TSharedRef<FTopologicalVertex>& InVertex2)
{
	TSharedRef<FTopologicalEdge> Edge = FEntity::MakeShared<FTopologicalEdge>(InSurface, InCoordinateVertex1, InVertex1, InCoordinateVertex2, InVertex2);
	return ReturnIfValid(Edge, /*bCheckVertices*/ false);
}


TSharedPtr<FTopologicalEdge> FTopologicalEdge::ReturnIfValid(TSharedRef<FTopologicalEdge>& InEdge, bool bCheckVertices)
{
	if (InEdge->CheckIfDegenerated())
	{
		InEdge->Delete();
		return TSharedPtr<FTopologicalEdge>();
	}

	if (bCheckVertices && !InEdge->CheckVertices())
	{
		InEdge->Delete();
		return TSharedPtr<FTopologicalEdge>();
	}

	InEdge->Finalize();
	InEdge->LinkVertex();
	return InEdge;
}

bool FTopologicalEdge::HasSameLengthAs(const FTopologicalEdge& Edge, double EdgeLengthTolerance) const
{
	double Min;
	double Max;
	if (Length() < Edge.Length())
	{
		Min = Length();
		Max = Edge.Length();
	}
	else
	{
		Min = Edge.Length();
		Max = Length();
	}

	if (Min / Max > 0.95) // 95 %
	{
		return true;
	}

	if ((Max - Min) < EdgeLengthTolerance)
	{
		return true;
	}

	return false;
};

bool FTopologicalEdge::IsTangentAtExtremitiesWith(const FTopologicalEdge& Edge) const
{
	TFunction<bool(EOrientation)> IsTangentAtExtremities = [&](EOrientation Orientation) ->  bool
	{
		FVector EdgeStartTangent;
		FVector EdgeEndTangent;
		FVector StartTangent;
		FVector EndTangent;
		GetTangentsAtExtremities(StartTangent, EndTangent, Orientation);
		Edge.GetTangentsAtExtremities(EdgeStartTangent, EdgeEndTangent, EOrientation::Front);

		double StartAngle = FVectorUtil::ComputeCosinus(EdgeStartTangent, StartTangent);
		double EndAngle = FVectorUtil::ComputeCosinus(EdgeEndTangent, EndTangent);

		if (StartAngle >= UE_DOUBLE_HALF_SQRT_3 && EndAngle >= UE_DOUBLE_HALF_SQRT_3)
		{
			return true;
		}
		else
		{
			return false;
		}
	};

	const FTopologicalVertex* ActiveEdgeVertex1 = &*Edge.GetStartVertex()->GetLinkActiveEntity();
	const FTopologicalVertex* ActiveEdgeVertex2 = &*Edge.GetEndVertex()->GetLinkActiveEntity();
	const FTopologicalVertex* ActiveVertex1 = &*GetStartVertex()->GetLinkActiveEntity();
	const FTopologicalVertex* ActiveVertex2 = &*GetEndVertex()->GetLinkActiveEntity();

	if ((ActiveVertex1 == ActiveEdgeVertex1) && (ActiveVertex2 == ActiveEdgeVertex2))
	{
		if (!IsTangentAtExtremities(EOrientation::Front))
		{
			if (ActiveVertex1 != ActiveEdgeVertex2)
			{
				return false;
			}

			// Self connected case
			return IsTangentAtExtremities(EOrientation::Back);
		}
		return true;
	}
	else if ((ActiveVertex1 == ActiveEdgeVertex2) && (ActiveVertex2 == ActiveEdgeVertex1))
	{
		return IsTangentAtExtremities(EOrientation::Back);
	}
	return false;
}

bool FTopologicalEdge::IsLinkableTo(const FTopologicalEdge& Edge, double EdgeLengthTolerance) const
{
	if (IsDeleted() || Edge.IsDeleted() ||
		IsDegenerated() || Edge.IsDegenerated())
	{
		return false;
	}

	if (!HasSameLengthAs(Edge, EdgeLengthTolerance))
	{
		return false;
	}

	return IsTangentAtExtremitiesWith(Edge);
}

void FTopologicalEdge::LinkIfCoincident(FTopologicalEdge& Twin, double EdgeLengthTolerance, double SquareJoiningTolerance)
{
	if (IsDeleted() || Twin.IsDeleted())
	{
		return;
	}

	// Degenerated twin edges are not linked
	if (IsDegenerated() || Twin.IsDegenerated())
	{
		if (HasSameLengthAs(Twin, EdgeLengthTolerance))
		{
			SetAsDegenerated();
			Twin.SetAsDegenerated();
		}
		return;
	}

	const FVector& Edge1Vertex1 = GetStartVertex()->GetBarycenter();
	const FVector& Edge1Vertex2 = GetEndVertex()->GetBarycenter();
	const FVector& Edge2Vertex1 = Twin.GetStartVertex()->GetBarycenter();
	const FVector& Edge2Vertex2 = Twin.GetEndVertex()->GetBarycenter();

	// Define the orientation
	const double SquareDistanceE1V1_E2V1 = GetStartVertex()->IsLinkedTo(Twin.GetStartVertex()) ? 0. : FVector::DistSquared(Edge1Vertex1, Edge2Vertex1);
	const double SquareDistanceE1V2_E2V2 = GetEndVertex()->IsLinkedTo(Twin.GetEndVertex()) ? 0. : FVector::DistSquared(Edge1Vertex2, Edge2Vertex2);

	const double SquareDistanceE1V1_E2V2 = GetStartVertex()->IsLinkedTo(Twin.GetEndVertex()) ? 0. : FVector::DistSquared(Edge1Vertex1, Edge2Vertex2);
	const double SquareDistanceE1V2_E2V1 = GetEndVertex()->IsLinkedTo(Twin.GetStartVertex()) ? 0. : FVector::DistSquared(Edge1Vertex2, Edge2Vertex1);

	const double SquareDistanceSameOrientation = SquareDistanceE1V1_E2V1 + SquareDistanceE1V2_E2V2;
	const double SquareDistanceReverseOrientation = SquareDistanceE1V1_E2V2 + SquareDistanceE1V2_E2V1;
	if (SquareDistanceSameOrientation < SquareDistanceReverseOrientation)
	{
		if (SquareDistanceE1V1_E2V1 < SquareJoiningTolerance)
		{
			GetStartVertex()->Link(*Twin.GetStartVertex());
		}
		else
		{
			UE_LOGF(LogCADKernelBase, Warning, "Edge %d and Edge %d are too far (%f) to be connected\n", GetId(), Twin.GetId(), sqrt(SquareDistanceE1V1_E2V1));
			return;
		}

		if (SquareDistanceE1V2_E2V2 < SquareJoiningTolerance)
		{
			GetEndVertex()->Link(*Twin.GetEndVertex());
		}
		else
		{
			UE_LOGF(LogCADKernelBase, Warning, "Edge %d and Edge %d are too far (%f) to be connected\n", GetId(), Twin.GetId(), sqrt(SquareDistanceE1V2_E2V2));
			return;
		}
	}
	else
	{
		if (SquareDistanceE1V1_E2V2 < SquareJoiningTolerance)
		{
			GetStartVertex()->Link(*Twin.GetEndVertex());
		}
		else
		{
			UE_LOGF(LogCADKernelBase, Warning, "Edge %d and Edge %d are to far (%f) to be connected\n", GetId(), Twin.GetId(), sqrt(SquareDistanceE1V1_E2V2));
			return;
		}

		if (SquareDistanceE1V2_E2V1 < SquareJoiningTolerance)
		{
			GetEndVertex()->Link(*Twin.GetStartVertex());
		}
		else
		{
			UE_LOGF(LogCADKernelBase, Warning, "Edge %d and Edge %d are to far (%f) to be connected\n", GetId(), Twin.GetId(), sqrt(SquareDistanceE1V2_E2V1));
			return;
		}
	}

	if (IsLinkableTo(Twin, EdgeLengthTolerance))
	{
		MakeLink(Twin);
	}
}

void FTopologicalEdge::Link(FTopologicalEdge& Twin)
{
	if (IsDegenerated() || Twin.IsDegenerated())
	{
		SetAsDegenerated();
		Twin.SetAsDegenerated();
		return;
	}

	if (IsDeleted() || Twin.IsDeleted())
	{
		return;
	}

	MakeLink(Twin);
}

void FTopologicalEdge::Disjoin()
{
	RemoveFromLink();
	GetStartVertex()->RemoveFromLink();
	GetEndVertex()->RemoveFromLink();
}

void FTopologicalEdge::Empty()
{
	if (StartVertex.IsValid())
	{
		StartVertex->RemoveConnectedEdge(*this);
		StartVertex->DeleteIfIsolated();
		StartVertex.Reset();
	}
	if (EndVertex.IsValid())
	{
		EndVertex->RemoveConnectedEdge(*this);
		EndVertex->DeleteIfIsolated();
		EndVertex.Reset();
	}

	if (TopologicalLink.IsValid())
	{
		TopologicalLink->RemoveEntity(*this);
	}

	Curve.Reset();
	Loop = nullptr;
	Mesh.Reset();

	TLinkable<FTopologicalEdge, FEdgeLink>::Empty();
}

FTopologicalFace* FTopologicalEdge::GetFace() const
{
	if (Loop != nullptr)
	{
		return Loop->GetFace();
	}
	return nullptr;
}

void FTopologicalEdge::ComputeCrossingPointCoordinates()
{
	if (FMath::IsNearlyEqual(Boundary.GetMin(), Boundary.GetMax(), UE_DOUBLE_SMALL_NUMBER))
	{
		// #cadkernel_check: Why could this happen? Shouldn't it be detected way earlier?
		return;
	}

	double Tolerance = GetTolerance3D();

	{
		FSurfacicPolyline Presampling;
		FSurfacicCurveSamplerOnParam Sampler(*Curve, Boundary, Tolerance * 10., Tolerance, Presampling);
		Sampler.Sample();

		Presampling.SwapCoordinates(CrossingPointUs);
	}

	ensureCADKernel(FMath::IsNearlyEqual(CrossingPointUs.Last(), Boundary.GetMax(), Curve->GetMinLinearTolerance()));
	// #cadkernel_check: To investigate - It looks like the sampler does not start and end the sample data with the boundary values???
	if (!FMath::IsNearlyEqual(CrossingPointUs.Last(), Boundary.GetMax(), UE_DOUBLE_SMALL_NUMBER))
	{
		CrossingPointUs.Add(Boundary.GetMax());
	}

	// Check sampling:
	// the main idea is to avoid very small delta U between two or more points.
	// e.g. CrossingPointUs = {0, 0.25, 0.5, 0.50001, 0.75, 0.9999, 0.99995, 1}
	// If small delta Us are identified, they are smoothed out with the next
	// e.g. CrossingPointUs = {0, 0.25, 0.5, 0.50001, 0.75, 0.9999, 0.99995, 1}
	// => CrossingPointUs = {0, 0.25, 0.416, 0.583, 0.75, 0.83, 0.92, 1}

	double DeltaUMean = Boundary.Length() / (CrossingPointUs.Num() - 1);
	double DeltaUMin = DeltaUMean * 0.03;

	int32 IndexMin = 0;
	double LocalUMin = DeltaUMin;
	for (int32 Index = 1; Index < CrossingPointUs.Num(); ++Index)
	{
		double DeltaU = CrossingPointUs[Index] - CrossingPointUs[IndexMin];
		if (DeltaU < LocalUMin)
		{
			LocalUMin += DeltaUMin;
			continue;
		}
		if ((Index - IndexMin) > 1)
		{
			if (IndexMin > 1)
			{
				IndexMin--;
			}

			double NewDelatU = DeltaU / (Index - IndexMin);
			for (int32 Andex = IndexMin + 1; Andex < Index; ++Andex)
			{
				CrossingPointUs[Andex] = CrossingPointUs[Andex - 1] + NewDelatU;
			}
		}
		IndexMin = Index;
		LocalUMin = DeltaUMin;
	}

	if (IndexMin != CrossingPointUs.Num() - 1)
	{
		IndexMin--;
		double DeltaU = CrossingPointUs[CrossingPointUs.Num() - 1] - CrossingPointUs[IndexMin];
		double NewDelatU = DeltaU / (CrossingPointUs.Num() - 1 - IndexMin);
		for (int32 Index = IndexMin + 1; Index < CrossingPointUs.Num() - 1; ++Index)
		{
			CrossingPointUs[Index] = CrossingPointUs[Index - 1] + NewDelatU;
		}
	}
}

void FTopologicalEdge::SetStartVertex(const double NewCoordinate)
{
	ensureCADKernel(Curve->GetUMax() > NewCoordinate);
	Boundary.SetMin(NewCoordinate);
	FCurvePoint OutPoint;
	Curve->EvaluatePoint(NewCoordinate, OutPoint);
	StartVertex->SetCoordinates(OutPoint.Point);
}

void FTopologicalEdge::SetEndVertex(const double NewCoordinate)
{
	ensureCADKernel(Curve->GetUMin() < NewCoordinate);
	Boundary.SetMax(NewCoordinate);
	FCurvePoint OutPoint;
	Curve->EvaluatePoint(NewCoordinate, OutPoint);
	EndVertex->SetCoordinates(OutPoint.Point);
}

void FTopologicalEdge::SetStartVertex(const double NewCoordinate, const FVector& NewPoint3D)
{
	ensureCADKernel(Curve->GetUMin() < NewCoordinate);
	Boundary.SetMin(NewCoordinate);
	StartVertex->SetCoordinates(NewPoint3D);
}

void FTopologicalEdge::SetEndVertex(const double NewCoordinate, const FVector& NewPoint3D)
{
	ensureCADKernel(Curve->GetUMax() > NewCoordinate);
	Boundary.SetMax(NewCoordinate);
	EndVertex->SetCoordinates(NewPoint3D);
}

void FTopologicalEdge::ComputeLength()
{
	Length3D = Curve->ApproximateLength(Boundary);
	Max2DTolerance = Length3D * FactorToComputeMaxTol;
}

double FTopologicalEdge::Length() const
{
	if (Length3D < 0)
	{
		const_cast<FTopologicalEdge*>(this)->ComputeLength();
	}
	return Length3D;
}

void FTopologicalEdge::GetTangentsAtExtremities(FVector& StartTangent, FVector& EndTangent, bool bForward) const
{
	ensureCADKernel(Curve->Polyline.Size());

	FDichotomyFinder Finder(Curve->Polyline.GetCoordinates());
	int32 StartIndex = Finder.Find(Boundary.Min);
	int32 EndIndex = Finder.Find(Boundary.Max);

	const TArray<FVector>& Polyline3D = Curve->Polyline.GetPoints();
	if (bForward)
	{
		StartTangent = Polyline3D[StartIndex + 1] - Polyline3D[StartIndex];
		EndTangent = Polyline3D[EndIndex] - Polyline3D[EndIndex + 1];
	}
	else
	{
		EndTangent = Polyline3D[StartIndex + 1] - Polyline3D[StartIndex];
		StartTangent = Polyline3D[EndIndex] - Polyline3D[EndIndex + 1];
	}
}


void FTopologicalEdge::Sample(const double DesiredSegmentLength, TArray<double>& OutCoordinates) const
{
	Curve->Sample(Boundary, DesiredSegmentLength, OutCoordinates);
}

int32 FTopologicalEdge::EvaluateCuttingPointNum()
{
	double Num = 0;
	for (int32 Index = 0; Index < CrossingPointUs.Num() - 1; Index++)
	{
		Num += ((CrossingPointUs[Index + 1] - CrossingPointUs[Index]) / CrossingPointDeltaUMaxs[Index]);
	}
	Num *= 1.5;
	return (int32)Num;
}

double FTopologicalEdge::TransformTwinEdgeCoordinateToLocalCoordinate(const FTopologicalEdge& TwinEdge, const double InTwinCoordinate) const
{
	if (this == &TwinEdge)
	{
		return InTwinCoordinate;
	}

	if (IsDegenerated() || TwinEdge.IsDegenerated())
	{
		// linear transform
		const bool bSameDirection = IsSameDirection(TwinEdge);
		const double Start = bSameDirection ? Boundary.GetMin() : Boundary.GetMax();
		const double End = bSameDirection ? Boundary.GetMax() : Boundary.GetMin();

		const double Distance = End - Start;
		if (FMath::IsNearlyZero(Distance))
		{
			return Start;
		}

		const double TwinStart = TwinEdge.GetBoundary().GetMin();
		const double TwinEnd = TwinEdge.GetBoundary().GetMax();
		const double TwinDistance = TwinEnd - TwinStart;
		if (FMath::IsNearlyZero(TwinDistance))
		{
			return Start;
		}

		return Start + (TwinStart - InTwinCoordinate) * Distance / TwinDistance;
	}

	FVector PointOnEdge = TwinEdge.GetCurve()->Approximate3DPoint(InTwinCoordinate);
	FVector ProjectedPoint;
	return Curve->GetCoordinateOfProjectedPoint(Boundary, PointOnEdge, ProjectedPoint);
}

double FTopologicalEdge::TransformLocalCoordinateToActiveEdgeCoordinate(const double InLocalCoordinate) const
{
	if (IsActiveEntity())
	{
		return InLocalCoordinate;
	}

	const FTopologicalEdge& ActiveEdge = *GetLinkActiveEntity();
	return ActiveEdge.TransformTwinEdgeCoordinateToLocalCoordinate(*this, InLocalCoordinate);
}

double FTopologicalEdge::TransformActiveEdgeCoordinateToLocalCoordinate(const double InActiveEdgeCoordinate) const
{
	if (IsActiveEntity())
	{
		return InActiveEdgeCoordinate;
	}

	const FTopologicalEdge& ActiveEdge = *GetLinkActiveEntity();
	return TransformTwinEdgeCoordinateToLocalCoordinate(ActiveEdge, InActiveEdgeCoordinate);
}

void FTopologicalEdge::TransformTwinEdgeCoordinatesToLocalCoordinates(const FTopologicalEdge& TwinEdge, const TArray<double>& InTwinCoordinates, TArray<double>& OutLocalCoordinates) const
{
	if (this == &TwinEdge)
	{
		OutLocalCoordinates = InTwinCoordinates;
		return;
	}

	if (IsDegenerated() || TwinEdge.IsDegenerated())
	{
		// linear transform
		const bool bSameDirection = IsSameDirection(TwinEdge);
		const double Start = bSameDirection ? Boundary.GetMin() : Boundary.GetMax();
		const double End = bSameDirection ? Boundary.GetMax() : Boundary.GetMin();

		const double Distance = End - Start;
		if (FMath::IsNearlyZero(Distance))
		{
			OutLocalCoordinates.Init(Start, InTwinCoordinates.Num());
			return;
		}

		const double TwinStart = TwinEdge.GetBoundary().GetMin();
		const double TwinEnd = TwinEdge.GetBoundary().GetMax();
		const double TwinDistance = TwinEnd - TwinStart;
		if (FMath::IsNearlyZero(TwinDistance))
		{
			OutLocalCoordinates.Init(Start, InTwinCoordinates.Num());
			return;
		}

		OutLocalCoordinates.Empty(InTwinCoordinates.Num());
		const double Factor = Distance / TwinDistance;
		for (double TwinCoordinate : InTwinCoordinates)
		{
			OutLocalCoordinates.Add(Start + (TwinCoordinate - TwinStart) * Factor);
		}
		return;
	}

	TArray<FVector> Cutting3DPoints;
	TwinEdge.ApproximatePoints(InTwinCoordinates, Cutting3DPoints);
	ProjectTwinEdgePoints(Cutting3DPoints, IsSameDirection(TwinEdge), OutLocalCoordinates);
}

void FTopologicalEdge::TransformLocalCoordinatesToActiveEdgeCoordinates(const TArray<double>& InLocalCoordinates, TArray<double>& OutActiveEdgeCoordinates) const
{
	const FTopologicalEdge& ActiveEdge = *GetLinkActiveEntity();
	return ActiveEdge.TransformTwinEdgeCoordinatesToLocalCoordinates(*this, InLocalCoordinates, OutActiveEdgeCoordinates);
}

void FTopologicalEdge::TransformActiveEdgeCoordinatesToLocalCoordinates(const TArray<double>& InActiveEdgeCoordinates, TArray<double>& OutLocalCoordinates) const
{
	const FTopologicalEdge& ActiveEdge = *GetLinkActiveEntity();
	return TransformTwinEdgeCoordinatesToLocalCoordinates(ActiveEdge, InActiveEdgeCoordinates, OutLocalCoordinates);
}

void FTopologicalEdge::SortImposedCuttingPoints()
{
	Algo::Sort(ImposedCuttingPointUs, [](const FImposedCuttingPoint& C1, const FImposedCuttingPoint& C2) { return C1.Coordinate < C2.Coordinate; });

	if (ImposedCuttingPointUs.Num() > 1)
	{
		int32 LastIndex = -1;
		int32 NewIndex = 0;
		for (int32 Index = 1; Index < ImposedCuttingPointUs.Num(); ++Index)
		{
			if (FMath::IsNearlyEqual(ImposedCuttingPointUs[NewIndex].Coordinate, ImposedCuttingPointUs[Index].Coordinate))
			{
				if (LastIndex < 0)
				{
					LastIndex = NewIndex;
				}
				if (ImposedCuttingPointUs[NewIndex].OppositNodeIndex == -1)
				{
					Swap(ImposedCuttingPointUs[NewIndex], ImposedCuttingPointUs[Index]);
				}
				else if (ImposedCuttingPointUs[Index].OppositNodeIndex == -1)
				{
				}
				else if (ImposedCuttingPointUs[NewIndex].OppositNodeIndex == ImposedCuttingPointUs[Index].OppositNodeIndex)
				{
					ImposedCuttingPointUs[NewIndex].DeltaU = FMath::Max(ImposedCuttingPointUs[NewIndex].DeltaU, ImposedCuttingPointUs[Index].DeltaU);
				}
				else if ((LastIndex >= 0) && (LastIndex != NewIndex) && (ImposedCuttingPointUs[LastIndex].OppositNodeIndex == ImposedCuttingPointUs[Index].OppositNodeIndex))
				{
					ImposedCuttingPointUs[LastIndex].DeltaU = FMath::Max(ImposedCuttingPointUs[LastIndex].DeltaU, ImposedCuttingPointUs[Index].DeltaU);
				}
				else
				{
					NewIndex++;
				}
			}
			else
			{
				LastIndex = -1;
				NewIndex++;
				if (NewIndex != Index)
				{
					Swap(ImposedCuttingPointUs[NewIndex], ImposedCuttingPointUs[Index]);
				}
			}
		}
		NewIndex++;
		ImposedCuttingPointUs.SetNum(NewIndex);
	}
}



void FTopologicalEdge::ProjectTwinEdgePointsOn2DCurve(const TSharedRef<FTopologicalEdge>& InTwinEdge, const TArray<double>& InTwinEdgePointCoords, TArray<FVector2d>& OutPoints2D)
{
	if (&InTwinEdge.Get() == this)
	{
		Curve->Approximate2DPoints(InTwinEdgePointCoords, OutPoints2D);
	}
	else
	{
		TArray<FVector> Points3D;
		InTwinEdge->ApproximatePoints(InTwinEdgePointCoords, Points3D);

		TArray<double> Coordinates;

		bool bSameDirection = IsSameDirection(*InTwinEdge);
		const double ToleranceOfProjection = Length3D * 0.1;
		Curve->ProjectTwinCurvePoints(Boundary, Points3D, bSameDirection, Coordinates, ToleranceOfProjection);
		Curve->Approximate2DPoints(Coordinates, OutPoints2D);
	}
}

void FTopologicalEdge::GenerateMeshElements(FModelMesh& MeshModel)
{
	FTopologicalEdge& ActiveEdge = *GetLinkActiveEntity();

	bool bSameDirection = IsSameDirection(ActiveEdge);

	FEdgeMesh& EdgeMesh = ActiveEdge.GetOrCreateMesh(MeshModel);

	int32 StartVertexNodeIndex = ActiveEdge.GetStartVertex()->GetOrCreateMesh(MeshModel).GetMesh();
	int32 EndVertexNodeIndex = ActiveEdge.GetEndVertex()->GetOrCreateMesh(MeshModel).GetMesh();

	TArray<double> CuttingPointCoordinates = GetCuttingPointCoordinates();
	ensureCADKernel(CuttingPointCoordinates.Num() > 1);
	CuttingPointCoordinates.RemoveAt(0);
	CuttingPointCoordinates.Pop();

	TArray<FVector>& Coordinates = EdgeMesh.GetNodeCoordinates();
	ApproximatePoints(CuttingPointCoordinates, Coordinates);

	if (!bSameDirection)
	{
		Algo::Reverse(Coordinates);
	}

	EdgeMesh.RegisterCoordinates();
	EdgeMesh.Mesh(StartVertexNodeIndex, EndVertexNodeIndex);
	MeshModel.AddMesh(EdgeMesh);
	ActiveEdge.SetMeshedMarker();
	SetMeshedMarker();
}

bool FTopologicalEdge::IsSameDirection(const FTopologicalEdge& Edge) const
{
	if (!TopologicalLink)
	{
		return true;
	}

	if (TopologicalLink != Edge.GetLink())
	{
		return true;
	}

	if (&Edge == this)
	{
		return true;
	}

	TSharedPtr<const FVertexLink> Vertex1Edge = GetStartVertex()->GetLink();
	TSharedPtr<const FVertexLink> Vertex2Edge = GetEndVertex()->GetLink();

	if (Vertex1Edge == Vertex2Edge)
	{
		if (Edge.IsDegenerated())
		{
			return true;
		}

		FVector EdgeStartTangent;
		FVector EdgeEndTangent;
		Edge.GetTangentsAtExtremities(EdgeStartTangent, EdgeEndTangent, true);

		FVector StartTangent;
		FVector EndTangent;
		GetTangentsAtExtremities(StartTangent, EndTangent, true);

		double StartAngle = FVectorUtil::ComputeCosinus(StartTangent, EdgeStartTangent);
		double EndAngle = FVectorUtil::ComputeCosinus(EndTangent, EdgeEndTangent);

		if (StartAngle >= 0 && EndAngle >= 0)
		{
			return true;
		}
		if (StartAngle <= 0 && EndAngle <= 0)
		{
			return false;
		}

		Edge.SetAsDegenerated();
		return true;
	}

	return Vertex1Edge == Edge.GetStartVertex()->GetLink();
}

FEdgeMesh& FTopologicalEdge::GetOrCreateMesh(FModelMesh& ShellMesh)
{
	if (!IsActiveEntity())
	{
		return GetLinkActiveEdge()->GetOrCreateMesh(ShellMesh);
	}

	if (!Mesh)
	{
		Mesh = FEntity::MakeShared<FEdgeMesh>(ShellMesh, *this);
	}

	return *Mesh;
}

void FTopologicalEdge::RemovePreMesh()
{
	if (IsMeshed())
	{
		return;
	}
	CuttingPointUs.Empty();
	ResetPreMeshed();
}

const FTopologicalEdge* FTopologicalEdge::GetPreMeshedTwin() const
{
	for (const FTopologicalEdge* Edge : GetLink()->GetTwinEntities())
	{
		if (!Edge->GetCuttingPoints().IsEmpty())
		{
			return Edge;
		}
	}
	return nullptr;
}

void FTopologicalEdge::AddImposedCuttingPointU(const double ImposedCuttingPointU, int32 OppositeNodeIndex, const double DeltaU)
{
	if(Boundary.Contains(ImposedCuttingPointU))
	{
		ImposedCuttingPointUs.Emplace(ImposedCuttingPointU, OppositeNodeIndex, DeltaU);
	}
}

void FTopologicalEdge::AddTwinsCuttingPoint(double Coord, double DeltaU)
{
	if (FMath::IsNearlyEqual(Coord, Boundary.GetMin(), GetTolerance2DAt(Boundary.GetMin())))
	{
		Coord = Boundary.GetMin();
	}
	else if (FMath::IsNearlyEqual(Coord, Boundary.GetMax(), GetTolerance2DAt(Boundary.GetMax())))
	{
		Coord = Boundary.GetMax();
	}

	CuttingPointUs.Emplace(Coord, ECoordinateType::ImposedCoordinate, FPairOfIndex::Undefined, DeltaU);
}

void FTopologicalEdge::TransferCuttingPointFromMeshedEdge(bool bOnlyWithOppositeNode, FAddCuttingPointFunc AddCuttingPoint)
{
	const FTopologicalEdge* PreMeshedTwin = GetPreMeshedTwin();

	if ((PreMeshedTwin == nullptr) || (PreMeshedTwin == this))
	{
		return;
	}

	TArray<double> PreMeshEdgeCuttingPointCoords = PreMeshedTwin->GetCuttingPointCoordinates();
	TArray<double> CuttingPointCoords;
	TransformTwinEdgeCoordinatesToLocalCoordinates(*PreMeshedTwin, PreMeshEdgeCuttingPointCoords, CuttingPointCoords);

	const TArray<FCuttingPoint>& PreMeshEdgeCuttingPoints = PreMeshedTwin->GetCuttingPoints();
	CuttingPointUs.Empty(PreMeshEdgeCuttingPoints.Num());
	for (int32 Index = 0; Index < PreMeshEdgeCuttingPoints.Num(); ++Index)
	{
		const FCuttingPoint& PreMeshEdgeCuttingPoint = PreMeshEdgeCuttingPoints[Index];

		if (bOnlyWithOppositeNode && PreMeshEdgeCuttingPoint.OppositNodeIndices[0] < 0)
		{
			continue;
		}

		const double PreMeshEdgeCuttingPointCoord = PreMeshEdgeCuttingPoint.Coordinate;
		const double PreMeshEdgeCuttingPointDeltaU = PreMeshEdgeCuttingPoint.IsoDeltaU;

		const double CuttingPointCoord = CuttingPointCoords[Index];
		const double EdgeTol = GetTolerance2DAt(CuttingPointCoord);
		const double PreMeshEdgeTol = PreMeshedTwin->GetTolerance2DAt(PreMeshEdgeCuttingPointCoord);

		const double CuttingPointDeltaU = PreMeshEdgeCuttingPointDeltaU * EdgeTol / PreMeshEdgeTol;

		AddCuttingPoint(CuttingPointCoord, PreMeshEdgeCuttingPoint.Type, PreMeshEdgeCuttingPoint.OppositNodeIndices, CuttingPointDeltaU);
	}
}

TArray<double> FTopologicalEdge::GetCuttingPointCoordinates() const
{
	TArray<double> CuttingPointCoordinates;
	CuttingPointCoordinates.Reserve(GetCuttingPoints().Num());
	for (const FCuttingPoint& CuttingPoint : GetCuttingPoints())
	{
		CuttingPointCoordinates.Add(CuttingPoint.Coordinate);
	}
	return MoveTemp(CuttingPointCoordinates);
}

TArray<double> FTopologicalEdge::GetPreElementLengths() const
{
	const TArray<FCuttingPoint>& CuttingPoints = GetCuttingPoints();
	const int32 CuttingPointCount = CuttingPoints.Num();

	TArray<FVector> InnerNodes;
	if (CuttingPointCount > 2)
	{
		TArray<double> Coordinates;
		Coordinates.Reserve(CuttingPointCount);
		for (int32 Index = 1; Index < CuttingPointCount - 1; ++Index)
		{
			Coordinates.Add(CuttingPoints[Index].Coordinate);
		}
		ApproximatePoints(Coordinates, InnerNodes);
	}

	const FVector& StartNode = GetStartVertex()->GetCoordinates();
	const FVector& EndNode = GetEndVertex()->GetCoordinates();
	return PolylineTools::ComputePolylineSegmentLengths(StartNode, InnerNodes, EndNode);
}

TSharedPtr<FTopologicalEdge> FTopologicalEdge::CreateEdgeByMergingEdges(const double SmallEdgeTolerance, TArray<FOrientedEdge>& Edges, const TSharedRef<FTopologicalVertex>& StartVertex, const TSharedRef<FTopologicalVertex>& EndVertex)
{
	TSharedRef<FSurface> CarrierSurface = Edges[0].Entity->GetCurve()->GetCarrierSurface();

	// check if all curves are 2D NURBS
	bool bAreNurbs = true;
	int32 NurbsMaxDegree = 0;

	TArray<TSharedPtr<FNURBSCurve>> NurbsCurves;
	NurbsCurves.Reserve(Edges.Num());

	bool bCanRemove = true;
	for (FOrientedEdge& Edge : Edges)
	{
		if (Edge.Entity->GetCurve()->Get2DCurve()->GetCurveType() != ECurve::Nurbs)
		{
			return TSharedPtr<FTopologicalEdge>();
		}

		double EdgeLength = Edge.Entity->Length();
		if (bCanRemove && EdgeLength < SmallEdgeTolerance)
		{
			NurbsCurves.Emplace(TSharedPtr<FNURBSCurve>());
			bCanRemove = false;
			continue; // the edge will be ignored 
		}
		bCanRemove = true;


		// Find the max degree of the NURBS
		TSharedPtr<FNURBSCurve> NURBS = NurbsCurves.Emplace_GetRef(StaticCastSharedRef<FNURBSCurve>(Edge.Entity->GetCurve()->Get2DCurve()));
		int32 NurbsDegree = NURBS->GetDegree();
		if (NurbsDegree > NurbsMaxDegree)
		{
			NurbsMaxDegree = NurbsDegree;
		}

		// Edge has restricted its curve ?
		FLinearBoundary EdgeBoundary = Edge.Entity->GetBoundary();
		FLinearBoundary CurveBoundary = NURBS->GetBoundary();

		double ParametricTolerance = NURBS->GetBoundary().ComputeMinimalTolerance();

		if (!FMath::IsNearlyEqual(EdgeBoundary.Min, CurveBoundary.Min, ParametricTolerance) ||
			!FMath::IsNearlyEqual(EdgeBoundary.Max, CurveBoundary.Max, ParametricTolerance))
		{
			// ToDO, check if the next edge is not the complementary of this

			// cancel
			return TSharedPtr<FTopologicalEdge>();
		}
	}

	bool bEdgeNeedToBeExtend = false;
	int32 PoleCount = 0;
	double LastCoordinate = 0;

	for (int32 Index = 0; Index < Edges.Num(); Index++)
	{
		TSharedPtr<FNURBSCurve>& NURBS = NurbsCurves[Index];
		if (!NURBS.IsValid())
		{
			bEdgeNeedToBeExtend = true;
			continue; // the edge will be ignored 
		}

		if (NURBS->GetDegree() < NurbsMaxDegree)
		{
			NURBS = BSpline::DuplicateNurbsCurveWithHigherDegree(NurbsMaxDegree, *NURBS);
			if (!NURBS.IsValid())
			{
				// cancel
				return TSharedPtr<FTopologicalEdge>();
			}
		}
		else
		{
			NURBS = FEntity::MakeShared<FNURBSCurve>(*NURBS);
		}

		if (Edges[Index].Direction == EOrientation::Back)
		{
			NURBS->Invert();
		}

		NURBS->SetStartNodalCoordinate(LastCoordinate);
		LastCoordinate = NURBS->GetBoundary().GetMax();

		PoleCount += NURBS->GetPoleCount();
	}

	if (bEdgeNeedToBeExtend)
	{
		for (int32 Index = 0; Index < Edges.Num(); Index++)
		{
			if (!NurbsCurves[Index].IsValid())
			{
				double PreviousLength = Index > 0 ? Edges[Index - 1].Entity->Length() : 0;
				double NextLength = Index < Edges.Num() - 1 ? Edges[Index + 1].Entity->Length() : 0;

				double TargetCoordinate = 0;
				EOrientation FrontOrientation = PreviousLength > NextLength ? EOrientation::Front : EOrientation::Back;
				TargetCoordinate = Edges[Index].Direction == FrontOrientation ? Edges[Index].Entity->GetBoundary().GetMax() : Edges[Index].Entity->GetBoundary().GetMin();

				if (PreviousLength > NextLength)
				{
					TargetCoordinate = Edges[Index].Direction == EOrientation::Front ? Edges[Index].Entity->GetBoundary().GetMax() : Edges[Index].Entity->GetBoundary().GetMin();
				}
				else
				{
					TargetCoordinate = Edges[Index].Direction == EOrientation::Front ? Edges[Index].Entity->GetBoundary().GetMin() : Edges[Index].Entity->GetBoundary().GetMax();
				}
				FVector2d Target = Edges[Index].Entity->Approximate2DPoint(TargetCoordinate);

				int32 NeigborIndex = PreviousLength > NextLength ? Index - 1 : Index + 1;
				if (NeigborIndex < 0 || NeigborIndex >= NurbsCurves.Num())
				{
					return TSharedPtr<FTopologicalEdge>();
				}
				NurbsCurves[NeigborIndex]->ExtendTo(FVector(Target, 0.));
			}
		}
	}

	TArray<double> NewNodalVector;
	TArray<double> NewWeights;
	TArray<FVector> NewPoles;
	NurbsMaxDegree++;
	NewNodalVector.Reserve(PoleCount + NurbsMaxDegree);
	NewPoles.Reserve(PoleCount + NurbsMaxDegree);

	bool bIsRational = false;
	for (const TSharedPtr<FNURBSCurve>& NurbsCurve : NurbsCurves)
	{
		if (!NurbsCurve.IsValid())
		{
			continue;
		}

		if (NurbsCurve->IsRational())
		{
			bIsRational = true;
			break;
		}
	}

	if (bIsRational)
	{
		NewWeights.Reserve(PoleCount + NurbsMaxDegree);
		for (const TSharedPtr<FNURBSCurve>& NurbsCurve : NurbsCurves)
		{
			if (!NurbsCurve.IsValid())
			{
				continue;
			}

			if (!NewPoles.IsEmpty())
			{
				NewPoles.Pop();
				NewWeights.Pop();
			}

			NewPoles.Append(NurbsCurve->GetPoles());
			if (NurbsCurve->IsRational())
			{
				NewWeights.Append(NurbsCurve->GetWeights());
			}
			else
			{
				for (int32 Index = 0; Index < NurbsCurve->GetPoles().Num(); ++Index)
				{
					NewWeights.Add(1.);
				}
			}
		}
	}
	else
	{
		for (const TSharedPtr<FNURBSCurve>& NurbsCurve : NurbsCurves)
		{
			if (!NurbsCurve.IsValid())
			{
				continue;
			}

			if (!NewPoles.IsEmpty())
			{
				NewPoles.Pop();
			}

			NewPoles.Append(NurbsCurve->GetPoles());
		}
	}

	for (const TSharedPtr<FNURBSCurve>& NurbsCurve : NurbsCurves)
	{
		if (!NurbsCurve.IsValid())
		{
			continue;
		}

		if (NewNodalVector.IsEmpty())
		{
			NewNodalVector.Append(NurbsCurve->GetNodalVector());
		}
		else
		{
			NewNodalVector.SetNum(NewNodalVector.Num() - 1);
			NewNodalVector.Append(NurbsCurve->GetNodalVector().GetData() + NurbsMaxDegree, NurbsCurve->GetNodalVector().Num() - NurbsMaxDegree);
		}
	}

	TSharedRef<FNURBSCurve> MergedNURBS = FEntity::MakeShared<FNURBSCurve>(NurbsMaxDegree - 1, NewNodalVector, NewPoles, NewWeights, 2);

	// Make new edge and delete the old ones ===================================================

	TSharedRef<FRestrictionCurve> RestrictionCurve = FEntity::MakeShared<FRestrictionCurve>(CarrierSurface, MergedNURBS);

	TSharedPtr<FTopologicalEdge> NewEdge = Make(RestrictionCurve, StartVertex, EndVertex);
	if (!NewEdge.IsValid())
	{
		return TSharedPtr<FTopologicalEdge>();
	}

	FTopologicalLoop* Loop = Edges[0].Entity->GetLoop();
	ensureCADKernel(Loop != nullptr);
	Loop->ReplaceEdges(Edges, NewEdge);

	for (const FOrientedEdge& OrientedEdge : Edges)
	{
		OrientedEdge.Entity->Delete();
	}

	return NewEdge;
}

void FTopologicalEdge::ReplaceEdgeVertex(bool bIsStartVertex, TSharedRef<FTopologicalVertex>& NewVertex)
{
	NewVertex->AddConnectedEdge(*this);

	TSharedPtr<FTopologicalVertex>& OldVertex = bIsStartVertex ? StartVertex : EndVertex;
	if (OldVertex->GetTwinEntityCount() > 1)
	{
		OldVertex->Link(*NewVertex);
	}

	OldVertex->RemoveConnectedEdge(*this);

	// Delete if no more connected to any edge
	OldVertex->DeleteIfIsolated();

	OldVertex = NewVertex;
}

bool FTopologicalEdge::ExtendTo(bool bIsStartExtremity, const FVector2d& NewExtremityCoordinate, TSharedRef<FTopologicalVertex>& NewVertex)
{
	if (bIsStartExtremity ? FMath::IsNearlyEqual(Boundary.Min, Curve->GetBoundary().Min) : FMath::IsNearlyEqual(Boundary.Max, Curve->GetBoundary().Max))
	{
		Curve->ExtendTo(NewExtremityCoordinate);
	}
	else
	{
		FVector ProjectedPoint;
		double UProjectedPoint = ProjectPoint(NewVertex->GetCoordinates(), ProjectedPoint);
		if (FVector::Distance(ProjectedPoint, NewVertex->GetCoordinates()) > GetTolerance3D())
		{
			return false;
		}

		if (bIsStartExtremity)
		{
			Boundary.Min = UProjectedPoint;
		}
		else
		{
			Boundary.Max = UProjectedPoint;
		}
	}

	ReplaceEdgeVertex(bIsStartExtremity, NewVertex);
	Length3D = -1.;

	return true;
}

void FTopologicalEdge::ComputeEdge2DProperties(FEdge2DProperties& EdgeCharacteristics)
{
	const TArray<FVector2d>& Polyline2D = Curve->Polyline.Get2DPoints();
	const TArray<FVector>& Polyline3D = Curve->Polyline.GetPoints();
	const TArray<double>& Parameters = Curve->Polyline.GetCoordinates();

	FDichotomyFinder Finder(Curve->Polyline.GetCoordinates());
	int32 StartIndex = Finder.Find(Boundary.Min);
	int32 EndIndex = Finder.Find(Boundary.Max);

	for (int32 Index = StartIndex; Index <= EndIndex; ++Index)
	{
		double Slope = ComputeUnorientedSlope(Polyline2D[Index], Polyline2D[Index + 1], 0);
		if (Slope > 2.)
		{
			Slope = 4. - Slope;
		}
		EdgeCharacteristics.Add(Slope, FVector::Distance(Polyline3D[Index], Polyline3D[Index + 1]));
	}
}

FVector FTopologicalEdge::GetTangentAt(const FTopologicalVertex& InVertex)
{
	if (&InVertex == StartVertex.Get())
	{
		return Curve->GetTangentAt(Boundary.GetMin());
	}
	else if (&InVertex == EndVertex.Get())
	{
		FVector Tangent = Curve->GetTangentAt(Boundary.GetMax());
		Tangent *= -1.;
		return Tangent;
	}
	else if (InVertex.GetLink() == StartVertex->GetLink())
	{
		return Curve->GetTangentAt(Boundary.GetMin());
	}
	else if (InVertex.GetLink() == EndVertex->GetLink())
	{
		FVector Tangent = Curve->GetTangentAt(Boundary.GetMax());
		Tangent *= -1.;
		return Tangent;
	}
	else
	{
		ensureCADKernel(false);
		return FVector::ZeroVector;
	}
}

FVector2d FTopologicalEdge::GetTangent2DAt(const FTopologicalVertex& InVertex)
{
	if (&InVertex == StartVertex.Get())
	{
		return Curve->GetTangent2DAt(Boundary.GetMin());
	}
	else if (&InVertex == EndVertex.Get())
	{
		FVector2d Tangent = Curve->GetTangent2DAt(Boundary.GetMax());
		Tangent *= -1.;
		return Tangent;
	}
	else if (InVertex.GetLink() == StartVertex->GetLink())
	{
		return Curve->GetTangent2DAt(Boundary.GetMin());
	}
	else if (InVertex.GetLink() == EndVertex->GetLink())
	{
		FVector2d Tangent = Curve->GetTangent2DAt(Boundary.GetMax());
		Tangent *= -1.;
		return Tangent;
	}
	else
	{
		ensureCADKernel(false);
		return FVector2d::ZeroVector;
	}
}

void FTopologicalEdge::SpawnIdent(FDatabase& Database)
{
	if (!FEntity::SetId(Database))
	{
		return;
	}

	StartVertex->SpawnIdent(Database);
	EndVertex->SpawnIdent(Database);
	Curve->SpawnIdent(Database);

	if (TopologicalLink.IsValid())
	{
		TopologicalLink->SpawnIdent(Database);
	}
	if (Mesh.IsValid())
	{
		Mesh->SpawnIdent(Database);
	}
}

FTopologicalVertex* FTopologicalEdge::SplitAt(double SplittingCoordinate, const FVector& NewVertexCoordinate, bool bKeepStartVertexConnectivity, TSharedPtr<FTopologicalEdge>& NewEdge)
{
	if (GetTwinEntityCount() > 1)
	{
		return nullptr;
	}

	TSharedRef<FTopologicalVertex> MiddelVertex = FTopologicalVertex::Make(NewVertexCoordinate);

	if (bKeepStartVertexConnectivity)
	{
		FLinearBoundary NewEdgeBoundary(SplittingCoordinate, Boundary.Max);
		NewEdge = Make(Curve.ToSharedRef(), MiddelVertex, EndVertex.ToSharedRef(), NewEdgeBoundary);
	}
	else
	{
		FLinearBoundary NewEdgeBoundary(Boundary.Min, SplittingCoordinate);
		NewEdge = Make(Curve.ToSharedRef(), StartVertex.ToSharedRef(), MiddelVertex, NewEdgeBoundary);
	}
	if (!NewEdge.IsValid())
	{
		return nullptr;
	}

	if (bKeepStartVertexConnectivity)
	{
		EndVertex->RemoveConnectedEdge(*this);
		EndVertex = MiddelVertex;
		Boundary.Max = SplittingCoordinate;
	}
	else
	{
		StartVertex->RemoveConnectedEdge(*this);
		StartVertex = MiddelVertex;
		Boundary.Min = SplittingCoordinate;
	}
	MiddelVertex->AddConnectedEdge(*this);

	ComputeLength();

	Loop->SplitEdge(*this, NewEdge, bKeepStartVertexConnectivity);
	return &MiddelVertex.Get();
}

bool FTopologicalEdge::IsSharpEdge() const
{
	double EdgeLength = Length();
	double Step = Boundary.Length() / 7;
	FSurfacicPolyline Polyline;
	Polyline.Coordinates.Reserve(5);

	double CurrentStep = Step;
	for (int32 Index = 0; Index < 5; ++Index)
	{
		Polyline.Coordinates.Add(CurrentStep);
		CurrentStep += Step;
	}

	Polyline.bWithNormals = true;
	Curve->ApproximatePolyline(Polyline);

	FTopologicalEdge* TwinEdge = GetFirstTwinEdge();
	bool bSameOrientation = IsSameDirection(*TwinEdge);
	FSurfacicPolyline TwinPolyline;
	TwinPolyline.bWithNormals = true;
	TwinPolyline.Coordinates.Reserve(5);
	TwinEdge->ProjectTwinEdgePoints(Polyline.Points3D, bSameOrientation, TwinPolyline.Coordinates);
	TwinEdge->ApproximatePolyline(TwinPolyline);

	int32 SharpPointCoount = 0;
	for (int32 Index = 0; Index < 5; ++Index)
	{
		double CosAngle = Polyline.Normals[Index] | TwinPolyline.Normals[Index];
		if (CosAngle < 0.94) // 20 deg
		{
			return true;
		}
	}
	return false;
}

void FTopologicalEdge::Offset2D(const FVector2d& OffsetDirection)
{
	Curve->Offset2D(OffsetDirection);
}

bool FTopologicalEdge::IsConnectedTo(const FTopologicalFace* Face) const
{
	for (FTopologicalEdge* TwinEdge : GetTwinEntities())
	{
		if (TwinEdge->GetFace() == Face)
		{
			return true;
		}
	}
	return false;
}

TArray<FTopologicalFace*> FTopologicalEdge::GetLinkedFaces() const
{
	TArray<FTopologicalFace*> NeighborFaces;
	NeighborFaces.Reserve(GetTwinEntities().Num());

	for (FTopologicalEdge* TwinEdge : GetTwinEntities())
	{
		FTopologicalFace* NeighborFace = TwinEdge->GetFace();
		NeighborFaces.Add(NeighborFace);
	}

	return MoveTemp(NeighborFaces);
}

}
