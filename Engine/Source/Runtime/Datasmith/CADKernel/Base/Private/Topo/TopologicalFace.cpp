// Copyright Epic Games, Inc. All Rights Reserved.

#include "Topo/TopologicalFace.h"

#include "Geo/Curves/RestrictionCurve.h"
#include "Geo/Curves/SegmentCurve.h"
#include "Geo/GeoPoint.h"
#include "Geo/Sampler/SamplerOnChord.h"
#include "Geo/Sampling/Polyline.h"
#include "Geo/Surfaces/Surface.h"
#include "Mesh/Criteria/CriteriaGrid.h"
#include "Mesh/Criteria/Criterion.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/Grid.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"

#ifdef CADKERNEL_DEV
#include "Topo/TopologyReport.h"
#endif

namespace UE::CADKernel
{

void FTopologicalFace::ComputeBoundary() const
{
	Boundary->Init();
	TArray<TArray<FVector2d>> TmpLoops;
	Get2DLoopSampling(TmpLoops);

	for (const TArray<FVector2d>& Loop : TmpLoops)
	{
		for (const FVector2d& Point : Loop)
		{
			Boundary->ExtendTo(Point);
		}
	}

	// Check with the carrier surface bounds
	CarrierSurface->ExtendBoundaryTo(Boundary);

	Boundary->WidenIfDegenerated();
	Boundary.SetReady();
}

#ifdef DEBUG_GET_BBOX
#include "Math/Aabb.h"
#endif

void FTopologicalFace::UpdateBBox(int32 IsoCount, const double ApproximationFactor, FBBoxWithNormal& BBox)
{
	const double SAG = GetCarrierSurface()->Get3DTolerance() * ApproximationFactor;

	TArray<TArray<FVector2d>> BoundaryApproximation;
	Get2DLoopSampling(BoundaryApproximation);

	FPolyline3D Polyline;
	const FSurface& Surface = GetCarrierSurface().Get();
	FIsoCurve3DSamplerOnChord Sampler(Surface, SAG, Polyline);

	IsoCount++;

	TFunction <void(const EIso)> UpdateBBoxWithIsos = [&](const EIso IsoType)
	{
		const FLinearBoundary& Bounds = GetBoundary().Get(IsoType);

		EIso Other = IsoType == EIso::IsoU ? EIso::IsoV : EIso::IsoU;

		double Coordinate = Bounds.Min;
		const double Step = (Bounds.Max - Bounds.Min) / IsoCount;

		for (int32 iIso = 1; iIso < IsoCount; iIso++)
		{
			FPolylineBBox IsoBBox;

			Coordinate += Step;

			TArray<double> Intersections;
			FindLoopIntersectionsWithIso(IsoType, Coordinate, BoundaryApproximation, Intersections);
			int32 IntersectionCount = Intersections.Num();
			if (IntersectionCount < 2)
			{
				continue;
			}

			FLinearBoundary CurveBounds(Intersections[0], Intersections.Last());

			Polyline.Empty();
			Sampler.Set(IsoType, Coordinate, CurveBounds);
			Sampler.Sample();

			if (IntersectionCount % 2 != 0)
			{
				TArray<FVector> SubPolyline;
				FLinearBoundary IntersectionBoundary(Intersections[IntersectionCount - 1], CurveBounds.GetMax());

				if (Polyline.Size() > 1)
				{
					Polyline.UpdateSubPolylineBBox(IntersectionBoundary, IsoBBox);
				}

				Intersections.Pop();
				IntersectionCount--;
			}

			if (IntersectionCount == 0)
			{
				continue;
			}

			if (IntersectionCount > 1)
			{
				for (int32 ISection = 0; ISection < IntersectionCount; ISection += 2)
				{
					TArray<FVector> SubPolyline;
					FLinearBoundary IntersectionBoundary(Intersections[ISection], Intersections[ISection + 1]);

					Polyline.UpdateSubPolylineBBox(IntersectionBoundary, IsoBBox);
				}
			}

			BBox.Update(IsoBBox, IsoType, Coordinate);
		}
	};

	UpdateBBoxWithIsos(EIso::IsoV);
	UpdateBBoxWithIsos(EIso::IsoU);

	BBox.UpdateNormal(*this);

#ifdef DEBUG_GET_BBOX
	{
		F3DDebugSession _(TEXT("BBox Face"));

		FAABB AABB(BBox.Min, BBox.Max);
		UE::CADKernel::DisplayAABB(AABB);

		for (int32 Index = 0; Index < 3; ++Index)
		{
			UE::CADKernel::DisplayPoint(BBox.MaxPoints[Index], EVisuProperty::YellowPoint);
			UE::CADKernel::DisplayPoint(BBox.MinPoints[Index], EVisuProperty::YellowPoint);
			UE::CADKernel::DisplaySegment(BBox.MaxPoints[Index], BBox.MaxPoints[Index] + BBox.MaxPointNormals[Index], 0, EVisuProperty::YellowCurve);
			UE::CADKernel::DisplaySegment(BBox.MinPoints[Index], BBox.MinPoints[Index] + BBox.MinPointNormals[Index], 0, EVisuProperty::YellowCurve);
		}
		Wait();
	}
#endif

}
#pragma optimize("",on)


void FTopologicalFace::ApplyNaturalLoops()
{
	const FSurfacicBoundary& Boundaries = CarrierSurface->GetBoundary();
	ApplyNaturalLoops(Boundaries);
}

void FTopologicalFace::ApplyNaturalLoops(const FSurfacicBoundary& Boundaries)
{
	ensureCADKernel(Loops.Num() == 0);

	TArray<TSharedPtr<FTopologicalEdge>> Edges;
	Edges.Reserve(4);
	TFunction<void(const FVector&, const FVector&)> BuildEdge = [&](const FVector& StartPoint, const FVector& EndPoint)
	{
		double Tolerance3D = CarrierSurface->Get3DTolerance();
		TSharedRef<FCurve> Curve2D = FEntity::MakeShared<FSegmentCurve>(StartPoint, EndPoint, 2);
		TSharedRef<FRestrictionCurve> Curve3D = FEntity::MakeShared<FRestrictionCurve>(CarrierSurface.ToSharedRef(), Curve2D);
		TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(Curve3D);
		if (!Edge.IsValid())
		{
			return;
		}
		Edges.Add(Edge);
	};

	FVector StartPoint;
	FVector EndPoint;

	// Build 4 bounding edges of the surface
	StartPoint.Set(Boundaries[EIso::IsoU].Min, Boundaries[EIso::IsoV].Min, 0.);
	EndPoint.Set(Boundaries[EIso::IsoU].Max, Boundaries[EIso::IsoV].Min, 0.);
	BuildEdge(StartPoint, EndPoint);

	StartPoint.Set(Boundaries[EIso::IsoU].Max, Boundaries[EIso::IsoV].Min, 0.);
	EndPoint.Set(Boundaries[EIso::IsoU].Max, Boundaries[EIso::IsoV].Max, 0.);
	BuildEdge(StartPoint, EndPoint);

	StartPoint.Set(Boundaries[EIso::IsoU].Max, Boundaries[EIso::IsoV].Max, 0.);
	EndPoint.Set(Boundaries[EIso::IsoU].Min, Boundaries[EIso::IsoV].Max, 0.);
	BuildEdge(StartPoint, EndPoint);

	StartPoint.Set(Boundaries[EIso::IsoU].Min, Boundaries[EIso::IsoV].Max, 0.);
	EndPoint.Set(Boundaries[EIso::IsoU].Min, Boundaries[EIso::IsoV].Min, 0.);
	BuildEdge(StartPoint, EndPoint);
	if (Edges.IsEmpty())
	{
		return;
	}

	TSharedPtr<FTopologicalEdge> PreviousEdge = Edges.Last();
	for (TSharedPtr<FTopologicalEdge>& Edge : Edges)
	{
		PreviousEdge->GetEndVertex()->Link(*Edge->GetStartVertex());
		PreviousEdge = Edge;
	}

	TArray<EOrientation> Orientations;
	Orientations.Init(EOrientation::Front, Edges.Num());

	const bool bIsExternalLoop = true;
	TSharedPtr<FTopologicalLoop> Loop = FTopologicalLoop::Make(Edges, Orientations, bIsExternalLoop, CarrierSurface->Get3DTolerance());
	AddLoop(Loop);
}

void FTopologicalFace::AddLoops(const TArray<TSharedPtr<FTopologicalLoop>>& InLoops, int32& DoubtfulLoopOrientationCount)
{
	for (TSharedPtr<FTopologicalLoop> Loop : InLoops)
	{
		if (Loop)
		{
			AddLoop(Loop);

			if (!Loop->Orient())
			{
				DoubtfulLoopOrientationCount++;
			}
		}
	}
}

void FTopologicalFace::RemoveLoop(const TSharedPtr<FTopologicalLoop>& Loop)
{
	const int32 Index = Loops.Find(Loop);
	if (Index != INDEX_NONE)
	{
		Loop->ResetSurface();
		Loops.RemoveAt(Index, EAllowShrinking::No);
	}

	if (Loops.Num() == 0)
	{
		Delete();
	}
}

void FTopologicalFace::Disjoin(TArray<FTopologicalEdge*>* NewBorderEdges)
{
	if(NewBorderEdges)
	{
		NewBorderEdges->Reserve(NewBorderEdges->Num() + EdgeCount());
	}

	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
		{
			FTopologicalEdge* Edge = OrientedEdge.Entity.Get();
			const TArray<FTopologicalEdge*> Twins = Edge->GetTwinEntities();
			for (FTopologicalEdge* TwinEdge : Twins)
			{
				if (NewBorderEdges && TwinEdge != Edge && TwinEdge->GetFace() != this)
				{
					NewBorderEdges->Add(TwinEdge);
				}
			}
			Edge->Disjoin();
			Edge->SetMarker1();
		}
	}
}

bool FTopologicalFace::HasSameBoundariesAs(const FTopologicalFace* OtherFace) const
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
		{
			const FTopologicalEdge& Edge = *OrientedEdge.Entity;
			if (!Edge.IsDegenerated() && !Edge.IsBorder() && !Edge.IsConnectedTo(OtherFace))
			{
				return false;
			}
		}
	}
	return true;
}

bool FTopologicalFace::IsADuplicatedFace() const
{
	if (!GetLoops().Num())
	{
		return false;
	}

	// Find in the adjacent faces of the first (surface or non manifold) edge, a face with the same loops
	
	TArray<FTopologicalFace*> AdjacentFaces;
	const TSharedPtr<FTopologicalLoop>& Loop = GetLoops()[0];
	for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
	{
		const FTopologicalEdge& Edge = *OrientedEdge.Entity;
		if (!Edge.IsDegenerated() && !Edge.IsBorder())
		{
			AdjacentFaces = Edge.GetLinkedFaces();
			break;
		}
	}

	for (FTopologicalFace* AdjacentFace : AdjacentFaces)
	{
		if (AdjacentFace == this)
		{
			continue;
		}

		if (HasSameBoundariesAs(AdjacentFace))
		{
			return true;
		}
	}

	return false;
}


bool FTopologicalFace::IsANonManifoldFace() const
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			if (Edge.Entity->GetTwinEntityCount() > 2)
			{
				return true;
			}
		}
	}
	return false;
}

void FTopologicalFace::DeleteNonmanifoldLink()
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			if (Edge.Entity->GetTwinEntityCount() > 2)
			{
				Edge.Entity->UnlinkTwinEntities();
			}
		}
	}
}

bool FTopologicalFace::IsABorderFace() const
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			if (Edge.Entity->GetTwinEntityCount() == 1)
			{
				return true;
			}
		}
	}
	return false;
}

bool FTopologicalFace::IsAFullyNonManifoldFace() const
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			if (!Edge.Entity->IsDegenerated() && Edge.Entity->GetTwinEntityCount() < 3)
			{
				return false;
			}
		}
	}
	return true;
}

const FTopologicalEdge* FTopologicalFace::GetLinkedEdge(const FTopologicalEdge& LinkedEdge) const
{
	for (FTopologicalEdge* TwinEdge : LinkedEdge.GetTwinEntities())
	{
		if (&*TwinEdge->GetFace() == this)
		{
			return TwinEdge;
		}
	}

	return nullptr;
}

#ifdef CADKERNEL_DEV
void FTopologicalFace::FillTopologyReport(FTopologyReport& Report) const
{
	Report.Add(this);

	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		for (const FOrientedEdge& Edge : Loop->GetEdges())
		{
			Report.Add(Edge.Entity.Get());
		}
	}
}
#endif

void FTopologicalFace::GetEdgeIndex(const FTopologicalEdge& Edge, int32& OutBoundaryIndex, int32& OutEdgeIndex) const
{
	OutEdgeIndex = INDEX_NONE;
	for (OutBoundaryIndex = 0; OutBoundaryIndex < Loops.Num(); ++OutBoundaryIndex)
	{
		TSharedPtr<FTopologicalLoop> Loop = Loops[OutBoundaryIndex];
		if ((OutEdgeIndex = Loop->GetEdgeIndex(Edge)) >= 0)
		{
			return;
		}
	}
	OutBoundaryIndex = INDEX_NONE;
}

const void FTopologicalFace::Get2DLoopSampling(TArray<TArray<FVector2d>>& LoopSamplings) const
{
	LoopSamplings.Empty(GetLoops().Num());

	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		TArray<FVector2d>& LoopSampling2D = LoopSamplings.Emplace_GetRef();
		Loop->Get2DSampling(LoopSampling2D);
	}
}

void FTopologicalFace::SpawnIdent(FDatabase& Database)
{
	if (!FEntity::SetId(Database))
	{
		return;
	}

	SpawnIdentOnEntities(Loops, Database);
	CarrierSurface->SpawnIdent(Database);
	if (Mesh.IsValid())
	{
		Mesh->SpawnIdent(Database);
	}
}

FFaceMesh& FTopologicalFace::GetOrCreateMesh(FModelMesh& MeshModel)
{
	if (!Mesh.IsValid())
	{
		Mesh = FEntity::MakeShared<FFaceMesh>(MeshModel, *this);
	}
	return *Mesh;
}

// Meshing parameters ==============================================================================================================================================================================================================================

void FTopologicalFace::InitDeltaUs()
{
	CrossingPointDeltaMins[EIso::IsoU].Init(DOUBLE_SMALL_NUMBER, CrossingCoordinates[EIso::IsoU].Num() - 1);
	CrossingPointDeltaMaxs[EIso::IsoU].Init(HUGE_VALUE, CrossingCoordinates[EIso::IsoU].Num() - 1);

	CrossingPointDeltaMins[EIso::IsoV].Init(DOUBLE_SMALL_NUMBER, CrossingCoordinates[EIso::IsoV].Num() - 1);
	CrossingPointDeltaMaxs[EIso::IsoV].Init(HUGE_VALUE, CrossingCoordinates[EIso::IsoV].Num() - 1);
}

bool FTopologicalFace::ComputeCriteriaGridSampling()
{
	const FSurfacicBoundary& FaceBoundaries = GetBoundary();
	CarrierSurface->Presample(FaceBoundaries, CrossingCoordinates);

	constexpr int32 MaxGrid = 1000000;
	if (CrossingCoordinates[EIso::IsoU].Num() * CrossingCoordinates[EIso::IsoV].Num() > MaxGrid)
	{
		// The sampling of the surface is huge. This is probably due to a degenerated carrier surface
		// The face is removed
		Remove();
		return false;
	}

	return true;
}

void FTopologicalFace::ApplyCriteria(const TArray<TSharedPtr<FCriterion>>& Criteria, const FCriteriaGrid& Grid)
{
	TArray<double>& DeltaUMaxArray = CrossingPointDeltaMaxs[EIso::IsoU];
	TArray<double>& DeltaUMinArray = CrossingPointDeltaMins[EIso::IsoU];
	TArray<double>& DeltaVMaxArray = CrossingPointDeltaMaxs[EIso::IsoV];
	TArray<double>& DeltaVMinArray = CrossingPointDeltaMins[EIso::IsoV];
	FSurfaceCurvature& SurfaceCurvature = GetCurvatures();

	double ElementLengthMin = DOUBLE_BIG_NUMBER;

	TFunction<void(const double, const double, const double)> ElementLength = [&](const double CoordNextMinusCoord, const double Length, const double DeltaMax)
	{
		if (CoordNextMinusCoord < DOUBLE_SMALL_NUMBER)
		{
			return;
		}
		const double ElemLength = Length * DeltaMax / CoordNextMinusCoord;
		if (ElementLengthMin > ElemLength)
		{
			ElementLengthMin = ElemLength;
		}
	};

	for (int32 IndexV = 0; IndexV < CrossingCoordinates[EIso::IsoV].Num() - 1; ++IndexV)
	{
		for (int32 IndexU = 0; IndexU < CrossingCoordinates[EIso::IsoU].Num() - 1; ++IndexU)
		{
			const FVector& Point_U0_V0 = Grid.GetPoint(IndexU, IndexV);
			const FVector& Point_U1_V1 = Grid.GetPoint(IndexU + 1, IndexV + 1);
			const FVector& Point_Um_V0 = Grid.GetIntermediateU(IndexU, IndexV);
			const FVector& Point_Um_V1 = Grid.GetIntermediateU(IndexU, IndexV + 1);
			const FVector& Point_U0_Vm = Grid.GetIntermediateV(IndexU, IndexV);
			const FVector& Point_U1_Vm = Grid.GetIntermediateV(IndexU + 1, IndexV);
			const FVector& Point_Um_Vm = Grid.GetIntermediateUV(IndexU, IndexV);

			// Evaluate Sag
			double LengthU;
			const double SagU = FCriterion::EvaluateSag(Point_U0_Vm, Point_U1_Vm, Point_Um_Vm, LengthU);
			double LengthV;
			const double SagV = FCriterion::EvaluateSag(Point_Um_V0, Point_Um_V1, Point_Um_Vm, LengthV);
			double LengthUV;
			const double SagUV = FCriterion::EvaluateSag(Point_U0_V0, Point_U1_V1, Point_Um_Vm, LengthUV);

			double& DeltaUMin = DeltaUMinArray[IndexU];
			double& DeltaUMax = DeltaUMaxArray[IndexU];
			double& DeltaVMin = DeltaVMinArray[IndexV];
			double& DeltaVMax = DeltaVMaxArray[IndexV];

			const double UNextMinusU = CrossingCoordinates[EIso::IsoU][IndexU + 1] - CrossingCoordinates[EIso::IsoU][IndexU];
			const double VNextMinusV = CrossingCoordinates[EIso::IsoV][IndexV + 1] - CrossingCoordinates[EIso::IsoV][IndexV];

			for (const TSharedPtr<FCriterion>& Criterion : Criteria)
			{
				Criterion->UpdateDelta(UNextMinusU, SagU, SagUV, SagV, LengthU, LengthUV, DeltaUMax, DeltaUMin, SurfaceCurvature[EIso::IsoU]);
				Criterion->UpdateDelta(VNextMinusV, SagV, SagUV, SagU, LengthV, LengthUV, DeltaVMax, DeltaVMin, SurfaceCurvature[EIso::IsoV]);
			}

			ElementLength(UNextMinusU, LengthU, DeltaUMax);
			ElementLength(VNextMinusV, LengthV, DeltaVMax);
		}
	}

	// Delta of the extremities are smooth to avoid big disparity 
	TFunction<void(TArray<double>&)> SmoothExtremities = [](TArray<double>& DeltaMaxArray)
	{
		if (DeltaMaxArray.Num() > 2)
		{
			DeltaMaxArray[0] = (DeltaMaxArray[0] + DeltaMaxArray[1] * 2) * AThird;
			DeltaMaxArray.Last() = (DeltaMaxArray.Last() + DeltaMaxArray[DeltaMaxArray.Num() - 2] * 2) * AThird;
		}
	};

	SmoothExtremities(DeltaUMaxArray);
	SmoothExtremities(DeltaVMaxArray);

	SetEstimatedMinimalElementLength(ElementLengthMin);
	SetApplyCriteriaMarker();
}


// Quad ==============================================================================================================================================================================================================================

double FTopologicalFace::GetQuadCriteria()
{
	if (GetQuadType() == EQuadType::Unset)
	{
		return 0;
	}
	return QuadCriteria;
}

void FTopologicalFace::ComputeQuadCriteria()
{
	if (GetQuadType() != EQuadType::Unset)
	{
		QuadCriteria = FMath::Max(Curvatures[EIso::IsoU].Max, Curvatures[EIso::IsoU].Max);
	}
}

void FTopologicalFace::ComputeSurfaceSideProperties()
{
	TFunction<double(const int32)> GetSideLength = [&](const int32 SideIndex)
	{
		TSharedPtr<FTopologicalLoop> Loop = GetLoops()[0];

		double Length = 0;
		int32 NextSideIndex = SideIndex + 1;
		if (NextSideIndex == GetStartSideIndices().Num())
		{
			NextSideIndex = 0;
		}
		int32 EndIndex = GetStartSideIndices()[NextSideIndex];
		for (int32 Index = GetStartSideIndices()[SideIndex]; Index != EndIndex;)
		{
			Length += Loop->GetEdge(Index)->Length();
			if (++Index == Loop->EdgeCount())
			{
				Index = 0;
			}
		}
		return Length;
	};

	Loops[0]->FindSurfaceCorners(SurfaceCorners, StartSideIndices);
	Loops[0]->ComputeBoundaryProperties(StartSideIndices, SideProperties);

	LoopLength = 0;
	for (int32 Index = 0; Index < SurfaceCorners.Num(); ++Index)
	{
		SideProperties[Index].Length3D = GetSideLength(Index);
		LoopLength += SideProperties[Index].Length3D;
	}
}

void FTopologicalFace::DefineSurfaceType()
{
	if (!CarrierSurface.IsValid())
	{
		return;
	}

	const double Tolerance3D = CarrierSurface->Get3DTolerance();
	const double GeometricTolerance = 20.0 * Tolerance3D;

	switch (SurfaceCorners.Num())
	{
	case 3:
		QuadType = EQuadType::Triangular;
		break;

	case 4:
		QuadType = EQuadType::Other;
		for (int32 Index = 0; Index < 4; ++Index)
		{
			// If the type is not ISO, the neighbor surface is checked, if it's quad so it's ok...
			if (SideProperties[Index].IsoType == EIso::UndefinedIso)
			{
				TSharedPtr<FTopologicalEdge> Edge = Loops[0]->GetEdge(StartSideIndices[Index]);
				int32 NeighborsNum = Edge->GetTwinEntityCount();
				// If non manifold Edge => Stop
				if (NeighborsNum != 2)
				{
					return;
				}

				{
					int32 OppositIndex = (Index + 2) % 4;
					SideProperties[Index].IsoType = SideProperties[OppositIndex].IsoType;
					if (SideProperties[Index].IsoType == EIso::UndefinedIso)
					{
						int32 AdjacentIndex = (Index + 1) % 4;
						if (SideProperties[AdjacentIndex].IsoType != EIso::UndefinedIso)
						{
							SideProperties[Index].IsoType = (SideProperties[AdjacentIndex].IsoType == EIso::IsoU) ? EIso::IsoV : EIso::IsoU;
						}
					}
				}

				FTopologicalFace* Neighbor = nullptr;
				{
					for(FTopologicalEdge* NeighborEdge : Edge->GetTwinEntities() )
					{
						if (NeighborEdge == Edge.Get())
						{
							continue;
						}
						Neighbor = Edge->GetFace();
					}
				}

				// it's not a quad surface
				if (Neighbor == nullptr || Neighbor->SurfaceCorners.Num() == 0)
				{
					return;
				}

				FTopologicalEdge* TwinEdge = Edge->GetFirstTwinEdge();
				if(!TwinEdge)
				{
					return;
				}

				int32 SideIndex = Neighbor->GetSideIndex(*TwinEdge);
				if (SideIndex < 0)
				{
					return;
				}

				const FEdge2DProperties& Property = Neighbor->GetSideProperty(SideIndex);
				if (Property.IsoType == EIso::UndefinedIso)
				{
					return;
				}

				double SideLength = SideProperties[Index].Length3D;
				double OtherSideLength = Property.Length3D;

				if (FMath::Abs(SideLength - OtherSideLength) < GeometricTolerance)
				{
					int32 OppositIndex = (Index + 2) % 4;
					if (SideProperties[OppositIndex].IsoType == EIso::UndefinedIso)
					{
						if (Index < 2)
						{
							if (SideProperties[!Index].IsoType == EIso::IsoU)
							{
								SideProperties[Index].IsoType = EIso::IsoV;
							}
							else
							{
								SideProperties[Index].IsoType = EIso::IsoU;
							}
						}
						return;
					}
					SideProperties[Index].IsoType = SideProperties[OppositIndex].IsoType;
				}
			}
		}

		if ((SideProperties[0].IsoType != EIso::UndefinedIso) && (SideProperties[1].IsoType != EIso::UndefinedIso) && (SideProperties[0].IsoType == SideProperties[2].IsoType) && (SideProperties[1].IsoType == SideProperties[3].IsoType))
		{
			QuadType = EQuadType::Quadrangular;
		}
		break;

	default:
		QuadType = EQuadType::Other;
		break;
	}
}

const TSharedPtr<FTopologicalLoop> FTopologicalFace::GetExternalLoop() const
{
	for (const TSharedPtr<FTopologicalLoop>& Loop : GetLoops())
	{
		if (Loop->IsExternal())
		{
			return Loop;
		}
	}
	return TSharedPtr<FTopologicalLoop>();
}

void FFaceSubset::SetMainShell(TMap<FTopologicalShapeEntity*, int32>& ShellToFaceCount)
{
	if (ShellToFaceCount.Num() == 0)
	{
		return;
	}

	int32 MaxFaceCount = 0;
	FTopologicalShapeEntity* CandidateShell = nullptr;

	for (TPair<FTopologicalShapeEntity*, int32>& Pair : ShellToFaceCount)
	{
		if (Pair.Value > MaxFaceCount)
		{
			MaxFaceCount = Pair.Value;
			CandidateShell = Pair.Key;
		}
	}

	if ((CandidateShell != nullptr) && ((CandidateShell->FaceCount() / 2 + 1) < MaxFaceCount))
	{
		MainShell = CandidateShell;
	}
}

void FFaceSubset::SetMainBody(TMap<FTopologicalShapeEntity*, int32>& BodyToFaceCount)
{
	if (BodyToFaceCount.Num() == 0)
	{
		return;
	}

	FTopologicalShapeEntity* CandidateBody = nullptr;
	int32 MaxFaceCount = 0;
	for (TPair<FTopologicalShapeEntity*, int32>& Pair : BodyToFaceCount)
	{
		if (Pair.Value > MaxFaceCount)
		{
			MaxFaceCount = Pair.Value;
			CandidateBody = Pair.Key;
		}
	}

	// if Faces come mainly from MainBody
	if ((CandidateBody != nullptr) && ((Faces.Num() / 2) <= MaxFaceCount))
	{
		MainBody = CandidateBody;
	}
}

void FFaceSubset::SetMainName(TMap<FString, int32>& NameToFaceCount)
{
	if (NameToFaceCount.Num() == 0)
	{
		return;
	}

	int32 MaxInstance = Faces.Num() / 3;
	for (TPair<FString, int32>& Pair : NameToFaceCount)
	{
		if (Pair.Value > MaxInstance)
		{
			MaxInstance = Pair.Value;
			MainName = Pair.Key;
		}
	}
}

void FFaceSubset::SetMainColor(TMap<uint32, int32>& ColorToFaceCount)
{
	int32 MaxInstance = 0;
	for (TPair<uint32, int32>& Pair : ColorToFaceCount)
	{
		if (Pair.Value > MaxInstance)
		{
			MaxInstance = Pair.Value;
			MainColor = Pair.Key;
		}
	}
}

void FFaceSubset::SetMainMaterial(TMap<uint32, int32>& MaterialToFaceCount)
{
	int32 MaxInstance = 0;
	for (TPair<uint32, int32>& Pair : MaterialToFaceCount)
	{
		if (Pair.Value > MaxInstance)
		{
			MaxInstance = Pair.Value;
			MainColor = Pair.Key;
		}
	}
}

}