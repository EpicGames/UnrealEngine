// Copyright Epic Games, Inc. All Rights Reserved.
#include "Topo/TopologicalFaceUtilities.h"

#include "Geo/Curves/CurveUtilities.h"
#include "Geo/GeoEnum.h"
#include "Geo/Surfaces/SurfaceUtilities.h"

#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalLoop.h"

#include "Algo/Reverse.h"
#include "Algo/BinarySearch.h"

namespace UE::CADKernel::TopologicalFaceUtilities
{
	bool IsPlanar(const UE::CADKernel::FTopologicalFace& Face)
	{
		using namespace UE::CADKernel;

		bool bIsPlanar = true;

		if (SurfaceUtilities::IsPlanar(*Face.GetCarrierSurface()))
		{
			for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
			{
				for (const FOrientedEdge& Edge : Loop->GetEdges())
				{
					if (CurveUtilities::GetDegree(*Edge.Entity->GetCurve()) == 1)
					{
						if (!Edge.Entity->GetTwinEdge())
						{
							continue;
						}
						else if (CurveUtilities::GetDegree(*Edge.Entity->GetTwinEdge()->GetCurve()) == 1)
						{
							continue;
						}
					}

					bIsPlanar = false;
					break;
				}

				if (!bIsPlanar)
				{
					return bIsPlanar;
				}
			}
		}

		return bIsPlanar;
	}
	
	TArray<FVector2d> Get2DPolyline(const UE::CADKernel::FTopologicalEdge& Edge)
	{
		using namespace UE::CADKernel;

		const FLinearBoundary& EdgeBounds = Edge.GetBoundary();
		const TArray<double>& PolyCoords = Edge.GetCurve()->GetPolyline().GetCoordinates();
		const TArray<FVector2d>& PolyPoints = Edge.GetCurve()->GetPolyline().Get2DPoints();
		const FLinearBoundary PolyBounds(PolyCoords[0] - UE_DOUBLE_SMALL_NUMBER, PolyCoords.Last() + UE_DOUBLE_SMALL_NUMBER);

		ensureCADKernel(PolyPoints.Num() > 1);

		TArray<FVector2d> PointsOut;
		PointsOut.Reserve(PolyPoints.Num());


		if (FMath::IsNearlyEqual(EdgeBounds.Min, PolyCoords[0]) && FMath::IsNearlyEqual(EdgeBounds.Max, PolyCoords.Last()))
		{
			// The entire polyline is used for the edge, just copy it in
			for (const FVector2d& Point : PolyPoints)
			{
				PointsOut.Emplace(Point.X, Point.Y);
			}
		}
		else
		{
			TFunction<void(const double&)> InsertPoint = [&](const double& Value) -> void
				{
					if (Value < PolyBounds.Min)
					{
						const double Ratio = (Value - PolyCoords[0]) / (PolyCoords[1] - PolyCoords[0]);
						const FVector2d Point = PolyPoints[0] + (PolyPoints[1] - PolyPoints[0]) * Ratio;

						PointsOut.Emplace(Point.X, Point.Y);
					}
					else if (Value > PolyBounds.Max)
					{
						const int32 LastIndex = PolyCoords.Num() - 1;
						const double Ratio = (Value - PolyCoords[LastIndex]) / (PolyCoords[LastIndex] - PolyCoords[LastIndex - 1]);
						const FVector2d Point = PolyPoints[LastIndex] + (PolyPoints[LastIndex] - PolyPoints[LastIndex - 1]) * Ratio;

						PointsOut.Emplace(Point.X, Point.Y);
					}
					else
					{
						int32 Index = Algo::LowerBound(PolyCoords, Value);

						if (FMath::IsNearlyEqual(Value, PolyCoords[Index]))
						{
							const FVector2d& Point = PolyPoints[Index];
							PointsOut.Emplace(Point.X, Point.Y);
						}
						else
						{
							const double Ratio = (Value - PolyCoords[Index - 1]) / (PolyCoords[Index] - PolyCoords[Index - 1]);
							const FVector2d Point = PolyPoints[Index - 1] + (PolyPoints[Index] - PolyPoints[Index - 1]) * Ratio;

							PointsOut.Emplace(Point.X, Point.Y);
						}
					}
				};


			InsertPoint(EdgeBounds.Min);
			int32 StartIndex = Algo::LowerBound(PolyCoords, EdgeBounds.Min);
			if (StartIndex == 0)
			{
				StartIndex++;
			}
			const int32 EndIndex = Algo::LowerBound(PolyCoords, EdgeBounds.Max);
			ensureCADKernel(EndIndex < PolyCoords.Num());
			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				PointsOut.Emplace(PolyPoints[Index].X, PolyPoints[Index].Y);
			}

			InsertPoint(EdgeBounds.Max);
		}

		return PointsOut;
	}

	TArray<FVector2d> Get2DPolyline(const UE::CADKernel::FOrientedEdge& Edge)
	{
		TArray<FVector2d> Points = Get2DPolyline(*Edge.Entity);

		if (Points.Num() < 2)
		{
			Points.Reset();
			return Points;
		}

		if (Edge.Direction == EOrientation::Back)
		{
			if (Points.Num() == 2)
			{
				FVector2d Reserve = Points[0];
				Points[0] = Points[1];
				Points[1] = Reserve;
			}
			else
			{
				Algo::Reverse(Points);
			}
		}

		return Points;
	}

	TArray<FVector2d> Get2DPolyline(const UE::CADKernel::FTopologicalLoop& Loop)
	{
		using namespace UE::CADKernel;

		const FSurfacicTolerance& IsoTolerances = Loop.GetFace()->GetCarrierSurface()->GetIsoTolerances();
		const FVector2d Tolerance2D { IsoTolerances.X, IsoTolerances.Y };

		TArray<FVector2d> VertexList;
		// #cad_debug
		FVector2d Last = FVector2d::ZeroVector;

		int32 Index = 0;
		for (const FOrientedEdge& Edge : Loop.GetEdges())
		{
			// For the time being, assuming all curves a 2D ones
			TArray<FVector2d> Polyline = Get2DPolyline(Edge);
			TArray<FVector> Polyline3D = Get3DPolyline(Edge);
			if (Polyline.Num() > 1)
			{
				// #cad_debug
				{
					if (VertexList.Num() > 0)
					{
						ensureCADKernel(Tolerance2D.ComponentwiseAllGreaterOrEqual(Last - Polyline[0]));
					}
					Last = Polyline.Last();
				}
				VertexList.Append(MoveTemp(Polyline));
				VertexList.SetNum(VertexList.Num() - 1, EAllowShrinking::No);
			}
			++Index;
		}

		if (ensureCADKernel(VertexList.Num() > 1))
		{
			ensureCADKernel(Tolerance2D.ComponentwiseAllGreaterOrEqual(Last - VertexList[0]));
		}

		return VertexList;
	}

	TArray<FVector> Get3DPolyline(const UE::CADKernel::FTopologicalEdge& Edge)
	{
		using namespace UE::CADKernel;

		const FLinearBoundary& EdgeBounds = Edge.GetBoundary();
		const TArray<double>& PolyCoords = Edge.GetCurve()->GetPolyline().GetCoordinates();
		const TArray<FVector>& PolyPoints = Edge.GetCurve()->GetPolyline().GetPoints();
		const FLinearBoundary PolyBounds(PolyCoords[0] - UE_DOUBLE_SMALL_NUMBER, PolyCoords.Last() + UE_DOUBLE_SMALL_NUMBER);

		ensureCADKernel(PolyPoints.Num() > 1);

		TArray<FVector> PointsOut;
		PointsOut.Reserve(PolyPoints.Num());

		if (FMath::IsNearlyEqual(EdgeBounds.Min, PolyCoords[0]) && FMath::IsNearlyEqual(EdgeBounds.Max, PolyCoords.Last()))
		{
			// The entire polyline is used for the edge, just copy it in
			for (const FVector& Point : PolyPoints)
			{
				PointsOut.Emplace(Point.X, Point.Y, Point.Z);
			}
		}
		else
		{
			TFunction<void(const double&)> InsertPoint = [&](const double& Value) -> void
				{
					if (Value < PolyBounds.Min)
					{
						const double Ratio = (Value - PolyCoords[0]) / (PolyCoords[1] - PolyCoords[0]);
						const FVector Point = PolyPoints[0] + (PolyPoints[1] - PolyPoints[0]) * Ratio;

						PointsOut.Emplace(Point.X, Point.Y, Point.Z);
					}
					else if (Value > PolyBounds.Max)
					{
						const int32 LastIndex = PolyCoords.Num() - 1;
						const double Ratio = (Value - PolyCoords[LastIndex]) / (PolyCoords[LastIndex] - PolyCoords[LastIndex - 1]);
						const FVector Point = PolyPoints[LastIndex] + (PolyPoints[LastIndex] - PolyPoints[LastIndex - 1]) * Ratio;

						PointsOut.Emplace(Point.X, Point.Y, Point.Z);
					}
					else
					{
						int32 Index = Algo::LowerBound(PolyCoords, Value);

						if (FMath::IsNearlyEqual(Value, PolyCoords[Index]))
						{
							const FVector& Point = PolyPoints[Index];
							PointsOut.Emplace(Point.X, Point.Y, Point.Z);
						}
						else
						{
							const double Ratio = (Value - PolyCoords[Index - 1]) / (PolyCoords[Index] - PolyCoords[Index - 1]);
							const FVector Point = PolyPoints[Index - 1] + (PolyPoints[Index] - PolyPoints[Index - 1]) * Ratio;

							PointsOut.Emplace(Point.X, Point.Y, Point.Z);
						}
					}
				};


			InsertPoint(EdgeBounds.Min);
			int32 StartIndex = Algo::LowerBound(PolyCoords, EdgeBounds.Min);
			if (StartIndex == 0)
			{
				StartIndex++;
			}
			const int32 EndIndex = Algo::LowerBound(PolyCoords, EdgeBounds.Max);
			ensureCADKernel(EndIndex < PolyCoords.Num());
			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				PointsOut.Emplace(PolyPoints[Index].X, PolyPoints[Index].Y, PolyPoints[Index].Z);
			}

			InsertPoint(EdgeBounds.Max);
		}

		printf(">>> %lf", PolyBounds.Min);

		return PointsOut;
	}

	TArray<FVector> Get3DPolyline(const UE::CADKernel::FOrientedEdge& Edge)
	{
		TArray<FVector> Points = Get3DPolyline(*Edge.Entity);

		if (Points.Num() < 2)
		{
			Points.Reset();
			return Points;
		}

		if (Edge.Direction == EOrientation::Back)
		{
			if (Points.Num() == 2)
			{
				FVector Reserve = Points[0];
				Points[0] = Points[1];
				Points[1] = Reserve;
			}
			else
			{
				Algo::Reverse(Points);
			}
		}

		return Points;
	}

	TArray<FVector> Get3DPolyline(const UE::CADKernel::FTopologicalLoop& Loop)
	{
		using namespace UE::CADKernel;

		const double Tolerance3D = Loop.GetFace()->GetCarrierSurface()->Get3DTolerance();

		TArray<FVector> VertexList;
		// #cad_debug
		FVector Last = FVector::ZeroVector;

		int32 Index = 0;
		for (const FOrientedEdge& Edge : Loop.GetEdges())
		{
			// For the time being, assuming all curves a 2D ones
			TArray<FVector> Polyline = Get3DPolyline(Edge);
			if (Polyline.Num() > 1)
			{
				// #cad_debug
				{
					if (VertexList.Num() > 0)
					{
						ensureCADKernel(Last.Equals(Polyline[0], Tolerance3D));
					}
					Last = Polyline.Last();
				}

				VertexList.Append(MoveTemp(Polyline));
				VertexList.SetNum(VertexList.Num() - 1, EAllowShrinking::No);
			}
			++Index;
		}

		if (ensureCADKernel(VertexList.Num() > 1))
		{
			ensureCADKernel(Last.Equals(VertexList[0], Tolerance3D));
		}

		return VertexList;
	}
}
