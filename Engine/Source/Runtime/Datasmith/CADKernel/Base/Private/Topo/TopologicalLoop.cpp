// Copyright Epic Games, Inc. All Rights Reserved.

#include "Topo/TopologicalLoop.h"

#include "Algo/AllOf.h"
#include "Geo/Curves/RestrictionCurve.h"
#include "Geo/GeoEnum.h"
#include "Math/SlopeUtils.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalLink.h"
#include "Topo/TopologicalFace.h"
#include "UI/Message.h"
#include "Utils/Util.h"

namespace UE::CADKernel
{

TSharedPtr<FTopologicalLoop> FTopologicalLoop::Make(const TArray<TSharedPtr<FTopologicalEdge>>& InEdges, const TArray<EOrientation>& InEdgeDirections, const bool bIsExternalLoop, double GeometricTolerance)
{
	TSharedRef<FTopologicalLoop> LoopRef = FEntity::MakeShared<FTopologicalLoop>(InEdges, InEdgeDirections, bIsExternalLoop);
	FTopologicalLoop& Loop = *LoopRef;

	const FSurface& Surface = *InEdges[0]->GetCurve()->GetCarrierSurface();
	const bool bIsSphere = Surface.GetSurfaceType() == ESurface::Sphere;
	if (bIsSphere && InEdges.Num() == 2)
	{
		// Workaround of TechSoft Bug (#jira https://techsoft3d.atlassian.net/servicedesk/customer/portal/7/SDHE-19879)
		// If the loop is composed of two coincident edges in the parametric space, linking the poles. Offset one edge of 2Pi
		const FTopologicalEdge& Edge0 = *InEdges[0];
		FTopologicalEdge& Edge1 = *InEdges[1];

		const FVector2d StartPoint = Edge0.Approximate2DPoint(Edge0.GetBoundary().GetMin());
		const FVector2d EndPoint = Edge0.Approximate2DPoint(Edge0.GetBoundary().GetMax());
		if (FMath::IsNearlyEqual(FMath::Abs(StartPoint.Y), DOUBLE_HALF_PI) && FMath::IsNearlyEqual(FMath::Abs(EndPoint.Y), DOUBLE_HALF_PI))
		{
			const double TolU = Surface.GetIsoTolerance(EIso::IsoU);
			const bool Edge0IsIso = Edge0.GetCurve()->IsIso(EIso::IsoU, TolU);
			const bool Edge1IsIso = Edge1.GetCurve()->IsIso(EIso::IsoU, TolU);
			const FVector2d Edge1EndPoint = Edge1.Approximate2DPoint(InEdgeDirections[0]== InEdgeDirections[1] ? Edge1.GetBoundary().GetMax() : Edge1.GetBoundary().GetMin());
			double Distance = FVector2d::DistSquared(Edge1EndPoint, StartPoint);
			if (Distance < FMath::Square(TolU))
			{
				const FVector2d Offset(TWO_PI, 0.);
				Edge1.Offset2D(Offset);
			}
		}
	}

	Loop.EnsureLogicalClosing(GeometricTolerance);
	Loop.RemoveDegeneratedEdges();

	if (Loop.GetEdges().IsEmpty())
	{
		return  TSharedPtr<FTopologicalLoop>();
	}

	for (FOrientedEdge& OrientedEdge : Loop.GetEdges())
	{
		OrientedEdge.Entity->SetLoop(Loop);
	}

	TArray<FVector2d> LoopSampling;
	Loop.Get2DSampling(LoopSampling);
	FAABB2D LoopBoundary;
	LoopBoundary += LoopSampling;
	Loop.Boundary.Set(LoopBoundary.GetMin(), LoopBoundary.GetMax());

	if (Algo::AllOf(Loop.GetEdges(), [](const FOrientedEdge& Edge) { return Edge.Entity->IsDegenerated(); }))
	{
		Loop.DeleteLoopEdges();
		return TSharedPtr<FTopologicalLoop>();
	}

	double Length = 0;
	for (const FOrientedEdge& Edge : Loop.GetEdges())
	{
		Length += Edge.Entity->Length();
	}
	if (Length < 10 * GeometricTolerance)
	{
		// Degenerated Loop
		Loop.DeleteLoopEdges();
		return TSharedPtr<FTopologicalLoop>();
	}

	return LoopRef;
}

FTopologicalLoop::FTopologicalLoop(const TArray<TSharedPtr<FTopologicalEdge>>& InEdges, const TArray<EOrientation>& InEdgeDirections, const bool bIsExternalLoop)
	: Face(nullptr)
	, bIsExternal(bIsExternalLoop)
{
	Edges.Reserve(InEdges.Num());
	for (int32 Index = 0; Index < InEdges.Num(); ++Index)
	{
		TSharedPtr<FTopologicalEdge> Edge = InEdges[Index];
		EOrientation Orientation = InEdgeDirections[Index];
		Edges.Emplace(Edge, Orientation);
	}
}

void FTopologicalLoop::DeleteLoopEdges()
{
	for (FOrientedEdge& Edge : Edges)
	{
		Edge.Entity->Delete();
		Edge.Entity.Reset();
	}
	Edges.Empty();
}

void FTopologicalLoop::RemoveEdge(TSharedPtr<FTopologicalEdge>& EdgeToRemove)
{
	for (int32 IEdge = 0; IEdge < Edges.Num(); IEdge++)
	{
		if (Edges[IEdge].Entity == EdgeToRemove)
		{
			EdgeToRemove->RemoveLoop();
			Edges.RemoveAt(IEdge);
			return;
		}
	}
	ensureCADKernel(false);
}

EOrientation FTopologicalLoop::GetDirection(TSharedPtr<FTopologicalEdge>& InEdge, bool bAllowLinkedEdge) const
{
	ensureCADKernel(InEdge.IsValid());

	for (const FOrientedEdge& BoundaryEdge : Edges)
	{
		if (BoundaryEdge.Entity == InEdge)
		{
			return BoundaryEdge.Direction;
		}
		else if (bAllowLinkedEdge && BoundaryEdge.Entity->IsLinkedTo(InEdge.ToSharedRef()))
		{
			return BoundaryEdge.Direction;
		}
	}

	UE_LOGF(LogCADKernelBase, Verbose, "Edge %d is not in boundary %d Edges\n", InEdge->GetId(), GetId());
	ensureCADKernel(false);
	return EOrientation::Front;
}


void FTopologicalLoop::Get2DSampling(TArray<FVector2d>& LoopSampling) const
{
	if (Edges.IsEmpty())
	{
		return;
	}

	int32 PointCount = 0;
	for (const FOrientedEdge& Edge : Edges)
	{
		PointCount += Edge.Entity->GetCurve()->GetPolylineSize();
	}

	LoopSampling.Empty(PointCount);

	for (const FOrientedEdge& Edge : Edges)
	{
		Edge.Entity->GetDiscretization2DPoints(Edge.Direction, LoopSampling);
		LoopSampling.Pop();
	}
	LoopSampling.Emplace(LoopSampling[0]);
}

bool FTopologicalLoop::Get2DSamplingWithoutDegeneratedEdges(TArray<FVector2d>& LoopSampling) const
{
	if (Edges.IsEmpty())
	{
		return false;
	}

	double LoopLength = 0;
	int32 EdgeCount = 0;
	int32 PointCount = 0;
 	for (const FOrientedEdge& Edge : Edges)
	{
		if (Edge.Entity->IsDegenerated())
		{
			continue;
		}
		EdgeCount++;
		PointCount += Edge.Entity->GetCurve()->GetPolylineSize();
		LoopLength += Edge.Entity->Length();
	}

	double LoopMeanLength = LoopLength / EdgeCount;
	double MinEdgeLength = LoopMeanLength * 0.01;
	MinEdgeLength = FMath::Max(MinEdgeLength, Face->GetCarrierSurface()->Get3DTolerance());

	LoopSampling.Empty(PointCount);

	for (const FOrientedEdge& Edge : Edges)
	{
		if (Edge.Entity->Length() < MinEdgeLength)
		{
			continue;
		}
		Edge.Entity->GetDiscretization2DPoints(Edge.Direction, LoopSampling);
		LoopSampling.Pop();
	}

	if(LoopSampling.Num() < 3)
	{
		return false;
	}

	LoopSampling.Emplace(LoopSampling[0]);
	return true;
}

/**
 * To check loop orientation, we check the orientation of the extremity points i.e. "o" points below
 *
 *                   o
 *                  / \
 *                 /   \
 *                o     o
 *                 \   /
 *                  \ /
 *                   o
 *
 * For these points, the slop is compute.
 * If the slop is between 0 and 4, the loop at the point is well oriented otherwise not
 *
 * The difficulties start when the slop is closed to 0 or 4 i.e.
 *
 *     -----o-----
 *    |           |
 *
 * In this case, the orientation of the next segment is compare to the bounding box
 *
 * The last very difficult case is a sharp case i.e.
 * The slop is closed to 0 or 8, so it could be a pick in self intersecting. 
 * We try to recompute the slop a the closed point
 *     ___________                 _______
 *    |       o---O    			  |       O  
 *    |     /         			  |     /        
 *    |    o               => 	  |    o         
 *    |    |					  |    |
 *
 * If the slop is still closed to 0 or 8, the point is "UndefinedOrientation"
 * This case was found in the shape of char '1'
 * 
 * @return false if the orientation is doubtful
 * 
 */

//#define DEBUG_ORIENT
bool FTopologicalLoop::Orient()
{
	bool bSucceed = true;

	ensureCADKernel(Edges.Num() > 0);

	TArray<FVector2d> LoopSampling;

	if (!Get2DSamplingWithoutDegeneratedEdges(LoopSampling))
	{
		// the loop is degenerated
		return false;
	}

#ifdef DEBUG_ORIENT
	bool bDisplayDebug = (this->GetFace()->GetId() == 735);
	if(bDisplayDebug)
	{
		F3DDebugSession _(*FString::Printf(TEXT("Loop before orientation")));
		DisplayOrientedPolyline(LoopSampling, EVisuProperty::BlueCurve);
	}
#endif

	LoopSampling.Pop();
	TSet<int32> ExtremityIndex;
	ExtremityIndex.Reserve(8);

	int32 PointCount = LoopSampling.Num();

	double UMin = HUGE_VAL;
	double UMax = -HUGE_VAL;
	double VMin = HUGE_VAL;
	double VMax = -HUGE_VAL;

	TFunction<void(const int32, const int32, const int32)> FindExtremity = [&](const int32 StartIndex, const int32 EndIndex, const int32 Increment)
	{
		UMin = HUGE_VAL;
		UMax = -HUGE_VAL;
		VMin = HUGE_VAL;
		VMax = -HUGE_VAL;

		int32 IndexUMin = 0;
		int32 IndexUMax = 0;
		int32 IndexVMin = 0;
		int32 IndexVMax = 0;

		for (int32 Index = StartIndex; Index != EndIndex; Index += Increment)
		{
			if (LoopSampling[Index].X > UMax)
			{
				UMax = LoopSampling[Index].X;
				IndexUMax = Index;
			}
			if (LoopSampling[Index].X < UMin)
			{
				UMin = LoopSampling[Index].X;
				IndexUMin = Index;
			}

			if (LoopSampling[Index].Y > VMax)
			{
				VMax = LoopSampling[Index].Y;
				IndexVMax = Index;
			}
			if (LoopSampling[Index].Y < VMin)
			{
				VMin = LoopSampling[Index].Y;
				IndexVMin = Index;
			}
		}
		ExtremityIndex.Add(IndexUMax);
		ExtremityIndex.Add(IndexUMin);
		ExtremityIndex.Add(IndexVMax);
		ExtremityIndex.Add(IndexVMin);
	};

	FindExtremity(0, PointCount, 1);
	FindExtremity(PointCount - 1, -1, -1);

	int32 WrongOrientationCount = 0;
	int32 GoodOrientationCount = 0;
	int32 UndefinedOrientationCount = 0;
	TFunction<void(const int32)> CompareOrientation = [&](int32 Index)
	{
		int32 NextIndex = Index + 1;
		if (NextIndex == PointCount)
		{
			NextIndex = 0;
		}
		int32 PreviousIndex = Index == 0 ? PointCount - 1 : Index - 1;

		// if the slop of the selected segments is not close to the BBox side (closed of 0 or 4), so the angle between the neighboring segments of the local extrema is not closed to 4 and allows to defined the orientation
		// Pic case: the slop is compute between previous and next segment of the extrema 
		//     if the slop is closed to 0 or 8, it could be pick in self intersecting. We try to recompute the slop a the closed point
		//          if the slop is still closed to 0 or 8, the pic is not used
		double Slope = ComputePositiveSlope(LoopSampling[Index], LoopSampling[NextIndex], LoopSampling[PreviousIndex]);

		if(Slope > 7.9 || Slope < 0.1)
		{
			double SquareLengthBefore = FVector2d::DistSquared(LoopSampling[Index], LoopSampling[PreviousIndex]);
			double SquareLengthAfter = FVector2d::DistSquared(LoopSampling[Index], LoopSampling[NextIndex]);
			if(SquareLengthBefore < SquareLengthAfter)
			{
				Index = PreviousIndex;
				PreviousIndex = Index == 0 ? PointCount - 1 : Index - 1;
			}
			else
			{
				Index = NextIndex;
				NextIndex = Index + 1;
				if (NextIndex == PointCount)
				{
					NextIndex = 0;
				}
			}

			Slope = ComputePositiveSlope(LoopSampling[Index], LoopSampling[NextIndex], LoopSampling[PreviousIndex]);

#ifdef DEBUG_ORIENT
			if (bDisplayDebug)
			{
				F3DDebugSession _(*FString::Printf(TEXT("Fix Pic Node %f"), Slope));
				DisplayPoint(LoopSampling[Index], EVisuProperty::BluePoint, Index);
				DisplaySegment(LoopSampling[PreviousIndex], LoopSampling[Index], EVisuProperty::GreenCurve);
				DisplaySegment(LoopSampling[NextIndex], LoopSampling[Index], EVisuProperty::GreenCurve);
			}
#endif
		}


		if (Slope > 7.9 || Slope < 0.1)
		{
			UndefinedOrientationCount++;
		}
		else if (Slope > 4.2)
		{
			WrongOrientationCount++;
		}
		else if (Slope < 3.8)
		{
			GoodOrientationCount++;
		}
		else
		{
			// Extrema case: the slop is compute between the next segment and the nearest BBox side 
			double ReferenceSlope = 0;
			if (FMath::IsNearlyEqual(LoopSampling[Index].X, UMin))
			{
				ReferenceSlope = 6;
			}
			else if (FMath::IsNearlyEqual(LoopSampling[Index].X, UMax))
			{
				ReferenceSlope = 2;
			}
			else if (FMath::IsNearlyEqual(LoopSampling[Index].Y, VMin))
			{
				ReferenceSlope = 0;
			}
			else if (FMath::IsNearlyEqual(LoopSampling[Index].Y, VMax))
			{
				ReferenceSlope = 4;
			}

			Slope = ComputeUnorientedSlope(LoopSampling[Index], LoopSampling[NextIndex], ReferenceSlope);
			// slop should be closed to [0, 0.2] or [3.8, 4]
			if (Slope > 3.8)
			{
				WrongOrientationCount++;
			}
			else if (Slope < 0.2)
			{
				GoodOrientationCount++;
			}
			else
			{
#ifdef DEBUG_ORIENT
				if (bDisplayDebug)
				{
					F3DDebugSession _(*FString::Printf(TEXT("Pic case")));
					{
						F3DDebugSession _(*FString::Printf(TEXT("Loop")));
						DisplayPolyline(LoopSampling, EVisuProperty::BlueCurve);
					}
					{
						F3DDebugSession _(*FString::Printf(TEXT("Next")));
						DisplaySegment(LoopSampling[NextIndex], LoopSampling[Index], EVisuProperty::YellowCurve);
					}
					{
						F3DDebugSession _(*FString::Printf(TEXT("Node %f"), Slope));
						DisplayPoint(LoopSampling[Index], EVisuProperty::RedPoint, Index);
					}
					Wait();
				}
#endif
				UndefinedOrientationCount++;
			}
		}

#ifdef DEBUG_ORIENT
		if (bDisplayDebug)
		{
			F3DDebugSession _(*FString::Printf(TEXT("Pic case")));
			{
				F3DDebugSession _(*FString::Printf(TEXT("Loop")));
				DisplayPolyline(LoopSampling, EVisuProperty::BlueCurve);
			}
			{
				F3DDebugSession _(*FString::Printf(TEXT("Next")));
				DisplaySegment(LoopSampling[NextIndex], LoopSampling[Index], EVisuProperty::YellowCurve);
			}
			{
				F3DDebugSession _(*FString::Printf(TEXT("Previous")));
				DisplaySegment(LoopSampling[PreviousIndex], LoopSampling[Index], EVisuProperty::YellowCurve);
			}
			{
				F3DDebugSession _(*FString::Printf(TEXT("Node %f"), Slope));
				DisplayPoint(LoopSampling[Index], EVisuProperty::RedPoint, Index);
			}
			Wait();
		}
#endif
	};

	for (int32 Index : ExtremityIndex)
	{
		CompareOrientation(Index);
	}

	if ((WrongOrientationCount != 0 && GoodOrientationCount != 0) || UndefinedOrientationCount > FMath::Max(WrongOrientationCount, GoodOrientationCount))
	{
#ifdef DEBUG_ORIENT
		if (bDisplayDebug)
		{
			F3DDebugSession GraphicSession(TEXT("Points of evaluation"));
			{
				F3DDebugSession G(*FString::Printf(TEXT("Loop Discretization %d"), Face->GetId()));
				DisplayPolyline(LoopSampling, EVisuProperty::BlueCurve);
				for (int32 Index = 0; Index < LoopSampling.Num(); ++Index)
				{
					DisplayPoint(LoopSampling[Index], Index);
				}
			}

			for (int32 Index : ExtremityIndex)
			{
				F3DDebugSession G(*FString::Printf(TEXT("Seg UMin Loop Discretization %d"), Face->GetId()));
				DisplayPoint(LoopSampling[Index], EVisuProperty::RedPoint, Index);
			}
			Wait();
		}
#endif

		bSucceed = false;
		UE_LOGF(LogCADKernelBase, Warning, "Loop Orientation of surface %d is doubtful\n", Face->GetId());
	}

	if ((WrongOrientationCount > GoodOrientationCount) == bIsExternal)
	{
		SwapOrientation();
	}

#ifdef DEBUG_ORIENT
	if (bDisplayDebug)
	{
		LoopSampling.Empty();
		if (Get2DSamplingWithoutDegeneratedEdges(LoopSampling))
		{
			F3DDebugSession _(*FString::Printf(TEXT("Loop oriented")));
			DisplayOrientedPolyline(LoopSampling, EVisuProperty::BlueCurve);
		}
	}
#endif

	return bSucceed;
}

void FTopologicalLoop::SwapOrientation()
{
	TArray<FOrientedEdge> TmpEdges;
	TmpEdges.Reserve(Edges.Num());
	for (int32 Index = Edges.Num() - 1; Index >= 0; Index--)
	{
		TmpEdges.Emplace(Edges[Index].Entity, GetReverseOrientation(Edges[Index].Direction));
	}
	Swap(TmpEdges, Edges);
}

void FTopologicalLoop::ReplaceEdge(TSharedPtr<FTopologicalEdge>& OldEdge, TSharedPtr<FTopologicalEdge>& NewEdge)
{
	for (int32 IEdge = 0; IEdge < (int32)Edges.Num(); IEdge++)
	{
		if (Edges[IEdge].Entity == OldEdge)
		{
			Edges[IEdge].Entity = NewEdge;
			OldEdge->RemoveLoop();
			NewEdge->SetLoop(*this);
			return;
		}
	}
	ensureCADKernel(false);
}

void FTopologicalLoop::SplitEdge(FTopologicalEdge& SplitEdge, TSharedPtr<FTopologicalEdge> NewEdge, bool bSplitEdgeIsFirst)
{
	NewEdge->SetLoop(*this);

	for (int32 IEdge = 0; IEdge < Edges.Num(); IEdge++)
	{
		if (Edges[IEdge].Entity.Get() == &SplitEdge)
		{
			EOrientation OldEdgeDirection = Edges[IEdge].Direction;
			if ((OldEdgeDirection == EOrientation::Front) == bSplitEdgeIsFirst)
			{
				IEdge++;
			}
			Edges.EmplaceAt(IEdge, NewEdge, OldEdgeDirection);
			return;
		}
	}
	ensureCADKernel(false);
}

void FTopologicalLoop::ReplaceEdge(TSharedPtr<FTopologicalEdge>& Edge, TArray<TSharedPtr<FTopologicalEdge>>& NewEdges)
{
	TArray<FOrientedEdge> TmpEdges;
	int32 NewEdgeNum = Edges.Num() + NewEdges.Num();
	TmpEdges.Reserve(NewEdgeNum);

	Edge->RemoveLoop();
	for (TSharedPtr<FTopologicalEdge>& NewEdge : NewEdges)
	{
		NewEdge->SetLoop(*this);
	}

	for (int32 IEdge = 0; IEdge < Edges.Num(); IEdge++)
	{
		if (Edges[IEdge].Entity == Edge)
		{
			EOrientation OldEdgeDirection = Edges[IEdge].Direction;
			if (OldEdgeDirection == EOrientation::Front)
			{
				Edges[IEdge].Entity = NewEdges[0];
				for (int32 INewEdge = 1; INewEdge < NewEdges.Num(); INewEdge++)
				{
					IEdge++;
					Edges.EmplaceAt(IEdge, NewEdges[INewEdge], EOrientation::Front);
				}
			}
			else
			{
				Edges[IEdge].Entity = NewEdges[0];
				for (int32 INewEdge = 1; INewEdge < NewEdges.Num(); INewEdge++)
				{
					Edges.EmplaceAt(IEdge, NewEdges[INewEdge], EOrientation::Back);
				}
			}
			return;
		}
	}
	ensureCADKernel(false);
}

void FTopologicalLoop::ReplaceEdges(TArray<FOrientedEdge>& OldEdges, TSharedPtr<FTopologicalEdge>& NewEdge)
{
	for (FOrientedEdge& Edge : OldEdges)
	{
		Edge.Entity->RemoveLoop();
	}

	NewEdge->SetLoop(*this);

	for (int32 IEdge = 0; IEdge < Edges.Num(); IEdge++)
	{
		if (Edges[IEdge] == OldEdges[0])
		{
			Edges[IEdge].Direction = EOrientation::Front;
			Edges[IEdge].Entity = NewEdge;
			IEdge++;

			int32 EdgeCount = Edges.Num();
			int32 EdgeToRemoveCount = OldEdges.Num() - 1;
			if (IEdge + EdgeToRemoveCount < EdgeCount)
			{
				for (int32 Index = 0; Index < EdgeToRemoveCount; Index++)
				{
					Edges.RemoveAt(IEdge);
				}
			}
			else
			{
				int32 EdgeToRemoveAtEnd = EdgeCount - IEdge;
				int32 EdgeToRemoveAtStart = EdgeToRemoveCount - EdgeToRemoveAtEnd;
				Edges.SetNum(IEdge);
				for (int32 Index = 0; Index < EdgeToRemoveAtStart; Index++)
				{
					Edges.RemoveAt(0);
				}
			}
			return;
		}
	}
	ensureCADKernel(false);
}

void FTopologicalLoop::FindSurfaceCorners(TArray<TSharedPtr<FTopologicalVertex>>& OutCorners, TArray<int32>& OutStartSideIndex) const
{
	TArray<double> BreakValues;
	FindBreaks(OutCorners, OutStartSideIndex, BreakValues);
}

void FTopologicalLoop::ComputeBoundaryProperties(const TArray<int32>& StartSideIndex, TArray<FEdge2DProperties>& OutSideProperties) const
{
	if (StartSideIndex.Num() == 0)
	{
		return;
	}

	OutSideProperties.Reserve(StartSideIndex.Num());

	int32 EdgeIndex = StartSideIndex[0];
	for (int32 SideIndex = 0; SideIndex < StartSideIndex.Num(); ++SideIndex)
	{
		int32 LastEdgeIndex = SideIndex + 1;
		LastEdgeIndex = (LastEdgeIndex == StartSideIndex.Num()) ? StartSideIndex[0] : StartSideIndex[LastEdgeIndex];

		FEdge2DProperties& SideProperty = OutSideProperties.Emplace_GetRef();
		do
		{
			Edges[EdgeIndex].Entity->ComputeEdge2DProperties(SideProperty);
			if (++EdgeIndex == Edges.Num())
			{
				EdgeIndex = 0;
			}
		} while (EdgeIndex != LastEdgeIndex);
		SideProperty.Finalize();
	}
}

void FTopologicalLoop::CheckEdgesOrientation()
{
	FOrientedEdge PreviousEdge = Edges.Last();
	FSurfacicCurveExtremities PreviousExtremities;
	PreviousEdge.Entity->GetExtremities(PreviousExtremities);

	FOrientedEdge OrientedEdge = Edges[0];
	FSurfacicCurveExtremities EdgeExtremities;
	OrientedEdge.Entity->GetExtremities(EdgeExtremities);


	TFunction<void()> CheckOrientation = [&]()
	{
		int32 PIndex = 0;
		int32 EIndex = 0;
		double SmallDistance = FVector2d::Distance(PreviousExtremities[0].Point2D, EdgeExtremities[0].Point2D);
		double Distance = FVector2d::Distance(PreviousExtremities[0].Point2D, EdgeExtremities[1].Point2D);
		if (Distance < SmallDistance)
		{
			SmallDistance = Distance;
			EIndex = 1;
		}
		Distance = FVector2d::Distance(PreviousExtremities[1].Point2D, EdgeExtremities[0].Point2D);
		if (Distance < SmallDistance)
		{
			SmallDistance = Distance;
			PIndex = 1;
			EIndex = 0;
		}
		Distance = FVector2d::Distance(PreviousExtremities[1].Point2D, EdgeExtremities[1].Point2D);
		if (Distance < SmallDistance)
		{
			SmallDistance = Distance;
			PIndex = 1;
			EIndex = 1;
		}

		if ((PIndex == 0 && PreviousEdge.Direction != EOrientation::Back)
			|| (PIndex == 1 && PreviousEdge.Direction != EOrientation::Front)
			|| (EIndex == 0 && OrientedEdge.Direction != EOrientation::Front)
			|| (EIndex == 1 && OrientedEdge.Direction != EOrientation::Back))
		{
			UE_LOGF(LogCADKernelBase, Verbose, "CheckEdgesOrientation failed for loop %d edge %d\n", GetId(), OrientedEdge.Entity->GetId());
		}
	};

	CheckOrientation();

	PreviousEdge = MoveTemp(OrientedEdge);
	FMemory::Memcpy(&PreviousExtremities, &EdgeExtremities, sizeof(FSurfacicCurveExtremities));

	for (int32 Index = 1; Index < Edges.Num(); ++Index)
	{
		OrientedEdge = Edges[Index];
		OrientedEdge.Entity->GetExtremities(EdgeExtremities);

		CheckOrientation();

		PreviousEdge = MoveTemp(OrientedEdge);
		FMemory::Memcpy(&PreviousExtremities, &EdgeExtremities, sizeof(FSurfacicCurveExtremities));
	}
}

void FTopologicalLoop::CheckLoopWithTwoEdgesOrientation()
{
	FOrientedEdge Edge0 = Edges[0];
	FSurfacicCurveExtremities Edge0Extremities;
	Edge0.Entity->GetExtremities(Edge0Extremities);

	FOrientedEdge Edge1 = Edges[1];
	FSurfacicCurveExtremities Edge1Extremities;
	Edge1.Entity->GetExtremities(Edge1Extremities);

	int32 IndexEdge0 = 0;
	int32 IndexEdge1 = 0;
	double SmallDistance = FVector2d::Distance(Edge0Extremities[0].Point2D, Edge1Extremities[0].Point2D);
	double Distance = FVector2d::Distance(Edge0Extremities[0].Point2D, Edge1Extremities[1].Point2D);
	if (Distance < SmallDistance)
	{
		SmallDistance = Distance;
		IndexEdge1 = 1;
	}
	Distance = FVector2d::Distance(Edge0Extremities[1].Point2D, Edge1Extremities[0].Point2D);
	if (Distance < SmallDistance)
	{
		SmallDistance = Distance;
		IndexEdge0 = 1;
		IndexEdge1 = 0;
	}
	Distance = FVector2d::Distance(Edge0Extremities[1].Point2D, Edge1Extremities[1].Point2D);
	if (Distance < SmallDistance)
	{
		SmallDistance = Distance;
		IndexEdge0 = 1;
		IndexEdge1 = 1;
	}

	if ((IndexEdge0 == IndexEdge1 && Edge0.Direction == Edge1.Direction)
		|| (IndexEdge0 != IndexEdge1 && Edge0.Direction != Edge1.Direction))
	{
		UE_LOGF(LogCADKernelBase, Verbose, "CheckEdgesOrientation failed for loop %d between edge %d and edge %d\n", GetId(), Edge0.Entity->GetId(), Edge1.Entity->GetId());
		//ensureCADKernel(false);
	}
}


void FTopologicalLoop::RemoveDegeneratedEdges()
{
#ifdef CADKERNEL_DEV
	switch (EdgeCount())
	{
	case 1:
		break;
	case 2:
		CheckLoopWithTwoEdgesOrientation();
		break;
	default:
		CheckEdgesOrientation();
		break;
	}
#endif

	FOrientedEdge DegeneratedOrientedEdge;
	FSurfacicCurveExtremities DegeneratedEdgeExtremities;

	TFunction<bool(bool, const int32)> RemoveDegeneratedEdge = [&](bool bPrevious, const int32 OtherEdgeIndex)
	{
		TSharedPtr<FTopologicalEdge>& DegeneratedEdge = DegeneratedOrientedEdge.Entity;

		FOrientedEdge NearOrientedEdge = Edges[OtherEdgeIndex];
		FSurfacicCurveExtremities NearEdgeExtremities;
		NearOrientedEdge.Entity->GetExtremities(NearEdgeExtremities);

		TSharedRef<FTopologicalVertex> DegeneratedEdgeVertex = (bPrevious == (DegeneratedOrientedEdge.Direction == EOrientation::Front)) ? DegeneratedEdge->GetStartVertex() : DegeneratedEdge->GetEndVertex();
		TSharedRef<FTopologicalVertex> OtherDegeneratedEdgeVertex = (bPrevious == (DegeneratedOrientedEdge.Direction == EOrientation::Front)) ? DegeneratedEdge->GetEndVertex() : DegeneratedEdge->GetStartVertex();
		FVector2d DegeneratedEdgeTangent = DegeneratedEdge->GetTangent2DAt(*DegeneratedEdgeVertex);

		TSharedRef<FTopologicalVertex> NearEdgeVertex = (bPrevious == (NearOrientedEdge.Direction == EOrientation::Front)) ? NearOrientedEdge.Entity->GetEndVertex() : NearOrientedEdge.Entity->GetStartVertex();
		FVector2d NearEdgeTangent = NearOrientedEdge.Entity->GetTangent2DAt(*NearEdgeVertex);

		FVector2d& NearEdgeExtremity = (bPrevious == (NearOrientedEdge.Direction == EOrientation::Front)) ? NearEdgeExtremities[1].Point2D : NearEdgeExtremities[0].Point2D;
		FVector2d& DegeneratedEdgeExtremity = (bPrevious == (DegeneratedOrientedEdge.Direction == EOrientation::Front)) ? DegeneratedEdgeExtremities[0].Point2D : DegeneratedEdgeExtremities[1].Point2D;
		FVector2d& OtherDegeneratedEdgeExtremity = (bPrevious == (DegeneratedOrientedEdge.Direction == EOrientation::Front)) ? DegeneratedEdgeExtremities[1].Point2D : DegeneratedEdgeExtremities[0].Point2D;

		double Slope = ComputeUnorientedSlope(FVector2d::ZeroVector, DegeneratedEdgeTangent, NearEdgeTangent);
		constexpr double FlatSlope = 3.9; // ~175 deg. 
		// a degenerated edge has to be really tangent with its neighbor edge to be merge with it
		// otherwise it could distort the neighbor edge.
		if (Slope > FlatSlope)
		{
			NearOrientedEdge.Entity->ExtendTo((bPrevious == (NearOrientedEdge.Direction != EOrientation::Front)), OtherDegeneratedEdgeExtremity, OtherDegeneratedEdgeVertex);
			DegeneratedEdge->Delete();
			return true;
		}
		return false;
	};

	for (int32 Index = Edges.Num() - 1; Index >= 0; --Index)
	{
		DegeneratedOrientedEdge = Edges[Index];
		if (DegeneratedOrientedEdge.Entity->IsDegenerated())
		{
			// Is this edge is tangent in 2d space with previous or next edge
			DegeneratedOrientedEdge.Entity->GetExtremities(DegeneratedEdgeExtremities);

			int32 PreviousEdgeIndex = (Index == 0) ? Edges.Num() - 1 : Index - 1;
			if (RemoveDegeneratedEdge(true, PreviousEdgeIndex))
			{
				Edges.RemoveAt(Index);
				continue;
			}

			int32 NextEdgeIndex = (Index == Edges.Num() - 1) ? 0 : Index + 1;
			if (RemoveDegeneratedEdge(false, NextEdgeIndex))
			{
				Edges.RemoveAt(Index);
			}
		}
	}
}

void FTopologicalLoop::EnsureLogicalClosing(const double Tolerance3D)
{
	const double SquareTolerance3D = FMath::Square(Tolerance3D);

	const TSharedRef<FSurface>& Surface = Edges[0].Entity->GetCurve()->GetCarrierSurface();

	FOrientedEdge PreviousEdge = Edges.Last();

	FSurfacicCurveExtremities PreviousExtremities;
	PreviousEdge.Entity->GetExtremities(PreviousExtremities);
	FSurfacicCurvePointWithTolerance PreviousExtremity;

	if (PreviousEdge.Direction == EOrientation::Front)
	{
		PreviousExtremity = PreviousExtremities[1];
	}
	else
	{
		PreviousExtremity = PreviousExtremities[0];
	}

	const FSurfacicTolerance& Tolerance2D = Surface->GetIsoTolerances();
	const FSurfacicTolerance LargeGapTolerance2D = Tolerance2D * 3.;
	const FSurfacicTolerance SmallGapTolerance2D = Tolerance2D * 0.1;
	for (int32 Index = 0; Index < Edges.Num(); ++Index)
	{
		FOrientedEdge OrientedEdge = Edges[Index];
		FSurfacicCurvePointWithTolerance EdgeExtremities[2];
		OrientedEdge.Entity->GetExtremities(EdgeExtremities);

		const int32 ExtremityIndex = OrientedEdge.Direction == EOrientation::Front ? 0 : 1;
		const double SquareGap3D = FVector::DistSquared(EdgeExtremities[ExtremityIndex].Point, PreviousExtremity.Point);

		TSharedRef<FTopologicalVertex> PreviousEdgeEndVertex = PreviousEdge.Direction == EOrientation::Front ? PreviousEdge.Entity->GetEndVertex() : PreviousEdge.Entity->GetStartVertex();
		TSharedRef<FTopologicalVertex> EdgeStartVertex = OrientedEdge.Direction == EOrientation::Front ? OrientedEdge.Entity->GetStartVertex() : OrientedEdge.Entity->GetEndVertex();

		const FVector2d Gap2D = EdgeExtremities[ExtremityIndex].Point2D - PreviousExtremity.Point2D;
		if (SquareGap3D > SquareTolerance3D)
		{
			UE_LOGF(LogCADKernelBase, Warning, "Loop %d Gap 3D : %f\n", Id, sqrt(SquareGap3D));
			const FVector PreviousTangent = PreviousEdge.Entity->GetTangentAt(*PreviousEdgeEndVertex);
			const FVector EdgeTangent = OrientedEdge.Entity->GetTangentAt(*EdgeStartVertex);

			FVector Gap = EdgeExtremities[ExtremityIndex].Point - PreviousExtremity.Point;

			double CosAngle = FVectorUtil::ComputeCosinus(Gap, EdgeTangent);
			constexpr double CosTangentLimit = 0.9; // ~25 deg : 25 deg is not big angle between the extremity curve tangent and the missing segment to close the gap.  
			if (CosAngle > CosTangentLimit) 
			{
				OrientedEdge.Entity->ExtendTo(OrientedEdge.Direction == EOrientation::Front, PreviousExtremity.Point2D, PreviousEdgeEndVertex);
			}
			else
			{
				CosAngle = FVectorUtil::ComputeCosinus(Gap, PreviousTangent);
				if (CosAngle < -CosTangentLimit) // ~25 deg
				{
					PreviousEdge.Entity->ExtendTo(PreviousEdge.Direction == EOrientation::Back, EdgeExtremities[ExtremityIndex].Point2D, EdgeStartVertex);
				}
				else
				{
					if (PreviousEdgeEndVertex->IsLinkedTo(EdgeStartVertex))
					{
						PreviousEdgeEndVertex->UnlinkTo(*EdgeStartVertex);
					}

					TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(Surface, PreviousExtremity.Point2D, PreviousEdgeEndVertex, EdgeExtremities[ExtremityIndex].Point2D, EdgeStartVertex);
					if (Edge.IsValid())
					{
						Edges.EmplaceAt(Index, Edge, EOrientation::Front);
						Edge->SetLoop(*this);
						++Index;
					}
					PreviousEdgeEndVertex = EdgeStartVertex;
				}
			}
		}
		else if (FMath::Abs(Gap2D.X) > LargeGapTolerance2D.X || FMath::Abs(Gap2D.Y) > LargeGapTolerance2D.Y)
		{
			// if the gap is not so large and the extremity of on edge is tangent to the gap segment then we extend the edge to close the gap
			UE_LOGF(LogCADKernelBase, Verbose, "Loop %d Gap 2D : [%f, %f] vs Tol2D [%f, %f]\n", Id, FMath::Abs(Gap2D.X), FMath::Abs(Gap2D.Y), Tolerance2D.X, Tolerance2D.Y);

			// If the gap in 2d is big e.g. side of a degenerated patch => build an edge
			TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(Surface, PreviousExtremity.Point2D, PreviousEdgeEndVertex, EdgeExtremities[ExtremityIndex].Point2D, EdgeStartVertex);
			if (Edge.IsValid())
			{
				Edges.EmplaceAt(Index, Edge, EOrientation::Front);
				Edge->SetLoop(*this);

				PreviousEdgeEndVertex->Link(*EdgeStartVertex);
				++Index;
			}
		}
		else if (FMath::Abs(Gap2D.X) > SmallGapTolerance2D.X || FMath::Abs(Gap2D.Y) > SmallGapTolerance2D.Y)
		{
			// if the gap is not so large and the extremity of on edge is tangent to the gap segment then we extend the edge to close the gap
			UE_LOGF(LogCADKernelBase, Warning, "Loop %d Gap 2D : [%f, %f] vs Tol2D [%f, %f]\n", Id, FMath::Abs(Gap2D.X), FMath::Abs(Gap2D.Y), Tolerance2D.X, Tolerance2D.Y);

			const FVector2d PreviousTangent = PreviousEdge.Entity->GetTangent2DAt(*PreviousEdgeEndVertex);
			const FVector2d EdgeTangent = OrientedEdge.Entity->GetTangent2DAt(*EdgeStartVertex);

			double CosAngle = FVectorUtil::ComputeCosinus(Gap2D, EdgeTangent);
			constexpr double CosFlatTangent = 0.98;  // ~10 deg: In 2D, the distortion due to the parametric space, degenerated area and else imposes to be more careful before extending a curve.

			//   ----------------------*         ------------**--------*
            //               *---------*    Or                         *
			//               |				                           |
			//               |				                           |
			//               |				                           |
			if (FMath::Abs(CosAngle) > CosFlatTangent)
			{
				OrientedEdge.Entity->ExtendTo(OrientedEdge.Direction == EOrientation::Front, PreviousExtremity.Point2D, PreviousEdgeEndVertex);
			}
			else
			{
				CosAngle = FVectorUtil::ComputeCosinus(Gap2D, PreviousTangent);
				if (FMath::Abs(CosAngle) > CosFlatTangent)
				{
					PreviousEdge.Entity->ExtendTo(PreviousEdge.Direction == EOrientation::Back, EdgeExtremities[ExtremityIndex].Point2D, EdgeStartVertex);
				}
				else
				{
					// if the gap is not so large and the extremity of on edge is tangent to the gap segment then we extend the edge to close the gap
					UE_LOGF(LogCADKernelBase, Warning, "Add degenerated edges for small 2d gap\n");
					TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(Surface, PreviousExtremity.Point2D, PreviousEdgeEndVertex, EdgeExtremities[ExtremityIndex].Point2D, EdgeStartVertex);
					if (Edge.IsValid())
					{
						Edges.EmplaceAt(Index, Edge, EOrientation::Front);
						Edge->SetLoop(*this);

						PreviousEdgeEndVertex->Link(*EdgeStartVertex);
						++Index;
					}
					else
					{
						// joins two vertices
						PreviousEdgeEndVertex->Link(*EdgeStartVertex);
					}
				}
			}
		}
		else
		{
			PreviousEdgeEndVertex->Link(*EdgeStartVertex);
		}

		if (OrientedEdge.Direction == EOrientation::Front)
		{
			PreviousExtremity = EdgeExtremities[1];
		}
		else
		{
			PreviousExtremity = EdgeExtremities[0];
		}
		PreviousEdge = OrientedEdge;
	}
}

void FTopologicalLoop::FindBreaks(TArray<TSharedPtr<FTopologicalVertex>>& OutBreaks, TArray<int32>& OutStartSideIndex, TArray<double>& OutBreakValues) const
{
	const double MinCosAngleOfBreak = -0.7;  // 135 deg
	if (Edges.Num() == 0)
	{
		return;
	}

	int32 EdgeNum = (int32)Edges.Num();
	OutBreaks.Empty(EdgeNum);
	OutBreakValues.Empty(EdgeNum);
	OutStartSideIndex.Empty(EdgeNum);

	FVector StartTangentEdge;
	FVector EndTangentPreviousEdge;
	Edges[EdgeNum - 1].Entity->GetTangentsAtExtremities(StartTangentEdge, EndTangentPreviousEdge, Edges[EdgeNum - 1].Direction == EOrientation::Front);
	bool bPreviousIsSurface = (Edges[EdgeNum - 1].Entity->GetTwinEntityCount() > 1);

	for (int32 Index = 0; Index < EdgeNum; Index++)
	{
		FVector EndTangentEdge;
		Edges[Index].Entity->GetTangentsAtExtremities(StartTangentEdge, EndTangentEdge, Edges[Index].Direction == EOrientation::Front);
		bool bIsSurface = (Edges[Index].Entity->GetTwinEntityCount() > 1);

		// if both edge are border, the rupture is not evaluate. 
		if (bIsSurface || bPreviousIsSurface)
		{
			double CosAngle = FVectorUtil::ComputeCosinus(StartTangentEdge, EndTangentPreviousEdge);

#ifdef FIND_BREAKS
			{
				FVector& Start = BoundaryEdges[Index]->GetStartVertex(BoundaryEdgeDirections[Index])->GetCoordinates();
				Open3DDebugSession(TEXT("Cos Angle " + Utils::ToString(CosAngle));
				DisplayPoint(Start, (CosAngle > MinCosAngleOfBreak) ? EVisuProperty::RedPoint : EVisuProperty::BluePoint);
				DisplaySegment(Start, Start + StartTangentEdge);
				DisplaySegment(Start, Start + EndTangentPreviousEdge);
				Close3DDebugSession();
			}
#endif

			if (CosAngle > MinCosAngleOfBreak)
			{
				OutBreaks.Add(Edges[Index].Direction == EOrientation::Front ? Edges[Index].Entity->GetStartVertex() : Edges[Index].Entity->GetEndVertex());
				OutBreakValues.Add(CosAngle);
				OutStartSideIndex.Add(Index);
			}
		}

		EndTangentPreviousEdge = EndTangentEdge;
		bPreviousIsSurface = bIsSurface;
	}
}

namespace TopologicalLoopImpl
{
void FindLoopIntersectionsWithIso(const EIso Iso, const double IsoParameter, const TArray<FVector2d>& Loop, TArray<double>& OutIntersections)
{
	OutIntersections.Empty(8);

	int32 UIndex = Iso == EIso::IsoU ? 0 : 1;
	int32 VIndex = Iso == EIso::IsoU ? 1 : 0;

	TFunction<void(const FVector2d&, const FVector2d&)> ComputeIntersection = [&](const FVector2d& Point1, const FVector2d& Point2)
	{
		if (IsoParameter > Point1[UIndex] && IsoParameter <= Point2[UIndex])
		{
			double Intersection = (IsoParameter - Point1[UIndex]) / (Point2[UIndex] - Point1[UIndex]) * (Point2[VIndex] - Point1[VIndex]) + Point1[VIndex];
			OutIntersections.Add((IsoParameter - Point1[UIndex]) / (Point2[UIndex] - Point1[UIndex]) * (Point2[VIndex] - Point1[VIndex]) + Point1[VIndex]);
		}
	};

	const FVector2d* Point1 = &Loop.Last();
	for (const FVector2d& Point2 : Loop)
	{
		if (!FMath::IsNearlyEqual((*Point1)[UIndex], Point2[UIndex]))
		{
			if ((*Point1)[UIndex] < Point2[UIndex])
			{
				ComputeIntersection(*Point1, Point2);
			}
			else
			{
				ComputeIntersection(Point2, *Point1);
			}
		}
		Point1 = &Point2;
	}

	Algo::Sort(OutIntersections);
}
};

bool FTopologicalLoop::IsInside(const FTopologicalLoop& OtherLoop) const
{
	TArray<FVector2d> Sampling2D;
	Get2DSampling(Sampling2D);

	TArray<FVector2d> OtherSampling2D;
	OtherLoop.Get2DSampling(OtherSampling2D);

#ifdef DEBUG_IS_INSIDE
	bool bDisplayDebug = true; //(GetId() == 1046569);
	if (bDisplayDebug)
	{
		{
			F3DDebugSession _(*FString::Printf(TEXT("Loop")));
			DisplayPolylineWithScale(Sampling2D, EVisuProperty::BlueCurve);
		}
		{
			F3DDebugSession _(*FString::Printf(TEXT("External Loop")));
			DisplayPolylineWithScale(OtherSampling2D, EVisuProperty::RedCurve);
		}
		Wait();
	}
#endif

	int32 InsidePoint = 0;
	int32 OutsidePoint = 0;

	TFunction<void(const FVector2d&, EIso)> CountLeftIntersection = [&](const FVector2d& Point, EIso Iso)
	{
		TArray<double> Intersections;
		TopologicalLoopImpl::FindLoopIntersectionsWithIso(Iso, Point[Iso], Sampling2D, Intersections);

		int32 IntersectionLeftCount = 0;
		const EIso OtherIso = Iso == EIso::IsoU ? EIso::IsoV : EIso::IsoU;

		for (const double& Intersection : Intersections)
		{
			if (Intersection < Point[OtherIso])
			{
				IntersectionLeftCount++;
			}
			else
			{
				break;
			}
		}

		if (IntersectionLeftCount % 2 == 0)
		{
			OutsidePoint++;
		}
		else
		{
			InsidePoint++;
		}
	};

	const int32 Step = 7;
	const int32 OtherSampling2DCount = OtherSampling2D.Num();
	const int32 Increment = FMath::Max(OtherSampling2DCount / Step, 1);
	for (int32 Index = 0; Index < OtherSampling2DCount; Index += Increment)
	{
		const FVector2d& TestPoint = OtherSampling2D[Index];
		CountLeftIntersection(TestPoint, EIso::IsoU);
		CountLeftIntersection(TestPoint, EIso::IsoV);
	}

	return InsidePoint < OutsidePoint;
}

double FTopologicalLoop::Length() const
{
	double Length = 0;
	for (const FOrientedEdge& Edge : GetEdges())
	{
		Length += Edge.Entity->Length();
	}
	return Length;
}

}