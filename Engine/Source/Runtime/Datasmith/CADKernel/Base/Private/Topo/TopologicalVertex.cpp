// Copyright Epic Games, Inc. All Rights Reserved.

#include "Topo/TopologicalVertex.h"

#include "Mesh/Structure/ModelMesh.h"
#include "Mesh/Structure/VertexMesh.h"
#include "Topo/TopologicalEdge.h"

namespace UE::CADKernel
{

void FTopologicalVertex::AddConnectedEdge(FTopologicalEdge& Edge)
{
	ConnectedEdges.Add(&Edge);
}

void FTopologicalVertex::RemoveConnectedEdge(FTopologicalEdge& Edge)
{
	if (ConnectedEdges.IsEmpty())
	{
		return;
	}

	for (int32 EdgeIndex = 0; EdgeIndex < ConnectedEdges.Num(); EdgeIndex++)
	{
		if (ConnectedEdges[EdgeIndex] == &Edge)
		{
			ConnectedEdges.RemoveAt(EdgeIndex);
			return;
		}
	}
}

bool FTopologicalVertex::IsBorderVertex() const
{
	for (FTopologicalVertex* Vertex : GetTwinEntities())
	{
		if (Vertex != nullptr && !Vertex->IsDeleted())
		{
			for (const FTopologicalEdge* Edge : Vertex->GetDirectConnectedEdges())
			{
				if (Edge != nullptr)
				{
					if (Edge->GetTwinEntityCount() == 1)
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

const FTopologicalFace* FTopologicalVertex::GetFace() const
{
	if (ConnectedEdges.IsEmpty())
	{
		return nullptr;
	}
	return ConnectedEdges[0]->GetFace();
}


void FTopologicalVertex::GetConnectedEdges(const FTopologicalVertex& OtherVertex, TArray<FTopologicalEdge*>& OutEdges) const
{
	OutEdges.Reserve(GetTwinEntityCount());

	TSharedPtr<TTopologicalLink<FTopologicalVertex>> OtherVertexLink = OtherVertex.GetLink();
	for (const FTopologicalVertex* Vertex : GetTwinEntities())
	{
		if (Vertex != nullptr)
		{
			for (FTopologicalEdge* Edge : Vertex->GetDirectConnectedEdges())
			{
				if (Edge != nullptr)
				{
					if (Edge->GetOtherVertex(*Vertex)->GetLink() == OtherVertexLink)
					{
						OutEdges.Add(Edge);
					}
				}
			}
		}
	}
}

void FTopologicalVertex::Link(FTopologicalVertex& Twin)
{
	// The active vertex is always the closest of the Barycenter
	if (TopologicalLink.IsValid() && Twin.TopologicalLink.IsValid())
	{
		if (TopologicalLink == Twin.TopologicalLink)
		{
			return;
		}
	}

	FVector Barycenter = GetBarycenter() * (double)GetTwinEntityCount()
		+ Twin.GetBarycenter() * (double)Twin.GetTwinEntityCount();

	MakeLink(Twin);

	Barycenter /= (double)GetTwinEntityCount();
	GetLink()->SetBarycenter(Barycenter);

	// Find the closest vertex of the Barycenter
	GetLink()->DefineActiveEntity();
}

void FTopologicalVertex::UnlinkTo(FTopologicalVertex& OtherVertex)
{
	TSharedPtr<FVertexLink> OldLink = GetLink();
	ResetTopologicalLink();
	OtherVertex.ResetTopologicalLink();

	for (FTopologicalVertex* Vertex : OldLink->GetTwinEntities())
	{
		if (!Vertex || Vertex == this || Vertex == &OtherVertex)
		{
			continue;
		}

		Vertex->ResetTopologicalLink();
		double Distance1 = Distance(*Vertex);
		double Distance2 = OtherVertex.Distance(*Vertex);
		if (Distance1 < Distance2)
		{
			Link(*Vertex);
		}
		else
		{
			OtherVertex.Link(*Vertex);
		}
	}
}

void FVertexLink::ComputeBarycenter()
{
	Barycenter = FVector::ZeroVector;
	for (const FTopologicalVertex* Vertex : TwinEntities)
	{
		Barycenter += Vertex->Coordinates;
	}
	Barycenter /= TwinEntities.Num();
}

void FVertexLink::DefineActiveEntity()
{
	if (TwinEntities.Num() == 0)
	{
		ActiveEntity = nullptr;
		return;
	}

	double MinDistanceSquare = HUGE_VALUE;
	FTopologicalVertex* ClosedVertex = TwinEntities.HeapTop();
	for (FTopologicalVertex* Vertex : TwinEntities)
	{
		ensureCADKernel(!Vertex->IsDeleted());

		double Square = Vertex->SquareDistance(Barycenter);
		if (Square < MinDistanceSquare)
		{
			MinDistanceSquare = Square;
			ClosedVertex = Vertex;
			if (FMath::IsNearlyZero(Square))
			{
				break;
			}
		}
	}
	ActiveEntity = ClosedVertex;
}

FVertexMesh& FTopologicalVertex::GetOrCreateMesh(FModelMesh& MeshModel)
{
	if (!IsActiveEntity())
	{
		return GetLinkActiveEntity()->GetOrCreateMesh(MeshModel);
	}

	if (!Mesh.IsValid())
	{
		Mesh = FEntity::MakeShared<FVertexMesh>(MeshModel, *this);

		Mesh->GetNodeCoordinates().Emplace(GetBarycenter());
		Mesh->RegisterCoordinates();
		MeshModel.AddMesh(*Mesh);
		SetMeshedMarker();
	}
	return *Mesh;
}

void FTopologicalVertex::SpawnIdent(FDatabase& Database)
{
	if (!FEntity::SetId(Database))
	{
		return;
	}

	if (TopologicalLink.IsValid())
	{
		TopologicalLink->SpawnIdent(Database);
	}

	if (Mesh.IsValid())
	{
		Mesh->SpawnIdent(Database);
	}
}


} // namespace UE::CADKernel
