// Copyright Epic Games, Inc. All Rights Reserved.

#include "Topo/Topomaker.h"

#include "Core/Chrono.h"
#include "Core/Session.h"
#include "Geo/Curves/RestrictionCurve.h"
#include "Topo/Body.h"
#include "Topo/FaceAnalyzer.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalLink.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLoop.h"
#include "Topo/TopologicalVertex.h"
#include "Topo/TopologicalShapeEntity.h"
#include "UI/Message.h"
#include "Utils/Util.h"

namespace UE::CADKernel
{

namespace SewOption
{
static bool IsForceJoining(ESewOption SewOptions)
{
	return (SewOptions & ESewOption::ForceJoining) == ESewOption::ForceJoining;
}

static bool IsRemoveThinFaces(ESewOption SewOptions)
{
	return (SewOptions & ESewOption::RemoveThinFaces) == ESewOption::RemoveThinFaces;
}

static bool IsRemoveDuplicatedFaces(ESewOption SewOptions)
{
	return (SewOptions & ESewOption::RemoveDuplicatedFaces) == ESewOption::RemoveDuplicatedFaces;
}
} // namespace

namespace TopomakerTools
{

/**
 * Merge Border Vertices with other vertices.
 * @param Vertices: the initial array of active vertices to process, this array is updated at the end of the process
 */
void MergeCoincidentVertices(TArray<FTopologicalVertex*>& VerticesToMerge, double Tolerance)
{
	const double SquareTolerance = FMath::Square(Tolerance);
	const double WeigthTolerance = 3 * Tolerance;

	int32 VertexNum = (int32)VerticesToMerge.Num();

	TArray<double> VerticesWeight;
	VerticesWeight.Reserve(VertexNum);

	TArray<int32> SortedVertexIndices;
	SortedVertexIndices.Reserve(VertexNum);

#ifdef DEBUG_MERGE_COINCIDENT_VERTICES
	F3DDebugSession _(TEXT("Merge Coincident Vertices"));
	for (const FTopologicalVertex* Vertex : VerticesToMerge)
	{
		F3DDebugSession A(*FString::Printf(TEXT("Vertex %d"), Vertex->GetId()));
		Display(*Vertex);
	}
#endif

	for (FTopologicalVertex* Vertex : VerticesToMerge)
	{
		VerticesWeight.Add(FVectorUtil::DiagonalAxisCoordinate(Vertex->GetCoordinates()));
	}

	for (int32 Index = 0; Index < VerticesToMerge.Num(); ++Index)
	{
		SortedVertexIndices.Add(Index);
	}
	SortedVertexIndices.Sort([&VerticesWeight](const int32& Index1, const int32& Index2) { return VerticesWeight[Index1] < VerticesWeight[Index2]; });

	for (int32 IndexI = 0; IndexI < VertexNum; ++IndexI)
	{
		FTopologicalVertex* Vertex = VerticesToMerge[SortedVertexIndices[IndexI]];
		if (Vertex->HasMarker1())
		{
			continue;
		}

		ensureCADKernel(Vertex->IsActiveEntity());

		Vertex->SetMarker1();

		double VertexWeigth = VerticesWeight[SortedVertexIndices[IndexI]];
		FVector Barycenter = Vertex->GetBarycenter();

		for (int32 IndexJ = IndexI + 1; IndexJ < VertexNum; ++IndexJ)
		{
			FTopologicalVertex* OtherVertex = VerticesToMerge[SortedVertexIndices[IndexJ]];
			if (OtherVertex->HasMarker1())
			{
				continue;
			}

			double OtherVertexWeigth = VerticesWeight[SortedVertexIndices[IndexJ]];
			if ((OtherVertexWeigth - VertexWeigth) > WeigthTolerance)
			{
				break;
			}

			double DistanceSqr = OtherVertex->GetLinkActiveEntity()->SquareDistance(Barycenter);
			if (DistanceSqr < SquareTolerance)
			{
				OtherVertex->SetMarker1();
				Vertex->Link(*OtherVertex);
				Barycenter = Vertex->GetBarycenter();
			}
		}
	}

	for (FTopologicalVertex* Vertex : VerticesToMerge)
	{
		Vertex->ResetMarker1();
	}

	TArray<FTopologicalVertex*> ActiveVertices;
	ActiveVertices.Reserve(VertexNum);

	for (FTopologicalVertex* Vertex : VerticesToMerge)
	{
		FTopologicalVertex& ActiveVertex = *Vertex->GetLinkActiveEntity();
		if (ActiveVertex.HasMarker1())
		{
			continue;
		}
		ActiveVertex.SetMarker1();
		ActiveVertices.Add(&ActiveVertex);
	}

	for (FTopologicalVertex* Vertex : ActiveVertices)
	{
		Vertex->ResetMarker1();
	}

	Swap(ActiveVertices, VerticesToMerge);
}

FTopologicalVertex* SplitAndLink(FTopologicalVertex& StartVertex, FTopologicalEdge& EdgeToLink, FTopologicalEdge& EdgeToSplit, double SquareSewTolerance, double SquareMinEdgeLength)
{
	FTopologicalVertex* VertexToLink = EdgeToLink.GetOtherVertex(StartVertex);
	FTopologicalVertex* EndVertex = EdgeToSplit.GetOtherVertex(StartVertex);

	double SquareDistanceToVertexToLink = EndVertex->SquareDistanceBetweenBarycenters(*VertexToLink);
	if (SquareDistanceToVertexToLink < SquareMinEdgeLength)
	{
		VertexToLink->Link(*EndVertex);
		EdgeToLink.Link(EdgeToSplit);
		return &VertexToLink->GetLinkActiveEntity().Get();
	}

	double SquareDistanceToStartVertex = StartVertex.SquareDistanceBetweenBarycenters(*VertexToLink);
	if (SquareDistanceToStartVertex < SquareMinEdgeLength)
	{
		VertexToLink->Link(StartVertex);
		EdgeToLink.SetAsDegenerated();
		return &VertexToLink->GetLinkActiveEntity().Get();
	}

	FVector ProjectedPoint;
	double UProjectedPoint = EdgeToSplit.ProjectPoint(VertexToLink->GetBarycenter(), ProjectedPoint);

	double SquareDistanceToProjectedPoint = FVector::DistSquared(ProjectedPoint, VertexToLink->GetBarycenter());
	if (SquareDistanceToProjectedPoint > SquareSewTolerance)
	{
		return nullptr;
	}

	double SquareDistanceToOtherPoint = FVector::DistSquared(ProjectedPoint, EndVertex->GetBarycenter());
	if (FMath::IsNearlyEqual(UProjectedPoint, EdgeToSplit.GetBoundary().GetMax(), UE_DOUBLE_SMALL_NUMBER) || SquareDistanceToOtherPoint < SquareMinEdgeLength)
	{
		// the new point is closed to the extremity, a degenerated edge will be created, so the edges are joined
		VertexToLink->Link(*EndVertex);
		EdgeToLink.Link(EdgeToSplit);
		return &VertexToLink->GetLinkActiveEntity().Get();
	}

	SquareDistanceToProjectedPoint = FVector::DistSquared(ProjectedPoint, StartVertex.GetBarycenter());
	if (FMath::IsNearlyEqual(UProjectedPoint, EdgeToSplit.GetBoundary().GetMin(), UE_DOUBLE_SMALL_NUMBER) || SquareDistanceToProjectedPoint < SquareMinEdgeLength)
	{
		VertexToLink->Link(StartVertex);
		EdgeToLink.SetAsDegenerated();
		return &VertexToLink->GetLinkActiveEntity().Get();
	}

	TArray<FTopologicalVertex*> NewTwinVertices;
	if (EdgeToSplit.GetTwinEntityCount() > 1)
	{
		TArray<FTopologicalEdge*> TwinEdges = EdgeToSplit.GetTwinEntities();
		EdgeToSplit.UnlinkTwinEntities();
		for (FTopologicalEdge* TwinEdge : TwinEdges)
		{
			if (TwinEdge != &EdgeToSplit)
			{
				FTopologicalVertex* NewVertex = SplitAndLink(StartVertex, EdgeToLink, *TwinEdge, SquareSewTolerance, SquareMinEdgeLength);
				if (NewVertex)
				{
					NewTwinVertices.Add(NewVertex);
				}
			}
		}
	}

	// JoinParallelEdges process all edges connected to startVertex (ConnectedEdges).
	// Connected edges must remain compliant i.e. all edges of ConnectedEdges must be connected to StartVertex
	// EdgeToSplit->SplitAt() must keep EdgeToSplit connected to StartVertex
	bool bKeepStartVertexConnectivity = (StartVertex.GetLink() == EdgeToSplit.GetStartVertex()->GetLink());

	TSharedPtr<FTopologicalEdge> NewEdge;
	FTopologicalVertex* NewVertex = EdgeToSplit.SplitAt(UProjectedPoint, ProjectedPoint, bKeepStartVertexConnectivity, NewEdge);
	if (!NewVertex)
	{
		return nullptr;
	}

	VertexToLink->Link(*NewVertex);
	EdgeToLink.Link(EdgeToSplit);

	if (NewTwinVertices.Num() > 0)
	{
		for (FTopologicalVertex* Vertex : NewTwinVertices)
		{
			NewVertex->Link(*Vertex);
		}
	}

	return NewVertex;
}

FTopologicalVertex* StitchParallelEdgesFrom(FTopologicalVertex* Vertex, double SewTolerance, double MinEdgeLength, bool bProhibitSewingEdgesOfSameFace)
{
	double SquareTolerance = FMath::Square(SewTolerance);
	double SquareMinEdgeLength = FMath::Square(MinEdgeLength);

	TArray<FTopologicalEdge*> ConnectedEdges;
	Vertex->GetConnectedEdges(ConnectedEdges);
	const int32 ConnectedEdgeCount = ConnectedEdges.Num();
	if (ConnectedEdgeCount == 1)
	{
		return nullptr;
	}

#ifdef DEBUG_STITCH_PARALLEL_EDGES_FROM
	{
		F3DDebugSession B(*FString::Printf(TEXT("Vertex %d"), Vertex->GetId()));
		{
			F3DDebugSession A(*FString::Printf(TEXT("Vertex %d"), Vertex->GetId()));
			Display(*Vertex);
		}
		for (const FTopologicalEdge* Edge : ConnectedEdges)
		{
			F3DDebugSession A(*FString::Printf(TEXT("Edge %d"), Edge->GetId()));
			int32 TwinCount = Edge->GetTwinEntityCount();
			EVisuProperty Property = EVisuProperty::RedCurve;
			switch (TwinCount)
			{
			case 1:
				Property = EVisuProperty::BorderEdge;
				break;
			case 2:
				Property = EVisuProperty::BlueCurve;
				break;
			default:
				break;
			}
			Display(*Edge, Property);
		}
		Wait();
	}
#endif

	for (int32 EdgeI = 0; EdgeI < ConnectedEdgeCount - 1; ++EdgeI)
	{
		FTopologicalEdge* Edge = ConnectedEdges[EdgeI];
		ensureCADKernel(Edge->GetLoop() != nullptr);

		if (Edge->IsDegenerated())
		{
			continue;
		}

		if (!Edge->IsActiveEntity())
		{
			continue;
		}
		bool bFirstEdgeBorder = Edge->IsBorder();

		for (int32 EdgeJ = EdgeI + 1; EdgeJ < ConnectedEdgeCount; ++EdgeJ)
		{
			FTopologicalEdge* SecondEdge = ConnectedEdges[EdgeJ];
			if (SecondEdge->IsDegenerated())
			{
				continue;
			}
			if (!SecondEdge->IsActiveEntity())
			{
				continue;
			}
			bool bSecondEdgeBorder = SecondEdge->IsBorder();

			// Process only if at least one edge is Border
			if (!bFirstEdgeBorder && !bSecondEdgeBorder)
			{
				continue;
			}

			if(bProhibitSewingEdgesOfSameFace)
			{
				bool bVoidSelection = false;
				for (FTopologicalEdge* FirstEdgeTwin : Edge->GetTwinEntities())
				{
					FTopologicalFace* FirstEdgeFace = FirstEdgeTwin->GetFace();
					for (FTopologicalEdge* SecondEdgeTwin : SecondEdge->GetTwinEntities())
					{
						if (FirstEdgeFace == SecondEdgeTwin->GetFace())
						{
							bVoidSelection = true;
							break;
						}
					}
				}
				if (bVoidSelection)
				{
					continue;
				}
			}

			FVector StartTangentEdge = Edge->GetTangentAt(*Vertex);
			FVector StartTangentOtherEdge = SecondEdge->GetTangentAt(*Vertex);

			double CosAngle = FVectorUtil::ComputeCosinus(StartTangentEdge, StartTangentOtherEdge);
			if (CosAngle < UE_DOUBLE_HALF_SQRT_3) // cos(30 deg)
			{
				continue;
			}

#ifdef DEBUG_STITCH_PARALLEL_EDGES_FROM
			{
				F3DDebugSession B(*FString::Printf(TEXT("StitchParallelEdges")));
				{
					F3DDebugSession A(*FString::Printf(TEXT("Edge %d"), Edge->GetId()));
					Display(*Edge);
				}
				{
					F3DDebugSession A(*FString::Printf(TEXT("SecondEdge %d"), SecondEdge->GetId()));
					Display(*SecondEdge, EVisuProperty::RedCurve);
				}
				Wait();
			}
#endif

			FTopologicalVertex& EndVertex = *Edge->GetOtherVertex(*Vertex)->GetLinkActiveEntity();
			FTopologicalVertex& SecondEdgeEndVertex = *SecondEdge->GetOtherVertex(*Vertex)->GetLinkActiveEntity();

			if (&EndVertex == &SecondEdgeEndVertex && Edge->IsLinkableTo(*SecondEdge, MinEdgeLength))
			{
				// should not happen but, just in case
				Edge->Link(*SecondEdge);
			}
			else
			{
				double EdgeLength = Edge->Length();
				double SecondEdgeLength = SecondEdge->Length();

				FTopologicalVertex* OtherVertex = nullptr;
				FTopologicalVertex* NewVertex = nullptr;
				if (EdgeLength < SecondEdgeLength)
				{
					OtherVertex = &SecondEdgeEndVertex;
					NewVertex = SplitAndLink(*Vertex, *Edge, *SecondEdge, SquareTolerance, SquareMinEdgeLength);
				}
				else
				{
					OtherVertex = &EndVertex;
					NewVertex = SplitAndLink(*Vertex, *SecondEdge, *Edge, SquareTolerance, SquareMinEdgeLength);
				}

				if (NewVertex && !Vertex->IsLinkedTo(*NewVertex) && !OtherVertex->IsLinkedTo(*NewVertex))
				{
					return NewVertex;
				}
			}
		}
	}
	return nullptr;
}

void StitchParallelEdges(TArray<FTopologicalVertex*>& VerticesToProcess, double SewTolerance, double MinEdgeLength, bool bProhibitSewingEdgesOfSameFace)
{
	double SquareTolerance = FMath::Square(SewTolerance);
	double SquareMinEdgeLength = FMath::Square(MinEdgeLength);

	for (int32 VertexI = 0; VertexI < VerticesToProcess.Num(); ++VertexI)
	{
		FTopologicalVertex* Vertex = VerticesToProcess[VertexI];

		if (!Vertex || Vertex->IsDeleted() || !Vertex->IsBorderVertex())
		{
			continue;
		}

		while (Vertex)
		{
			Vertex = StitchParallelEdgesFrom(Vertex, SewTolerance, MinEdgeLength, bProhibitSewingEdgesOfSameFace);
		}
	}
}

/**
 * First step, trivial edge merge i.e. couple of edges with same extremity active vertices
 */
void MergeCoincidentEdges(FTopologicalEdge* Edge, double MinEdgeLength)
{
	const FTopologicalVertex& StartVertex = *Edge->GetStartVertex()->GetLinkActiveEntity();
	const FTopologicalVertex& EndVertex = *Edge->GetEndVertex()->GetLinkActiveEntity();
	if (&StartVertex == &EndVertex && Edge->Length() < MinEdgeLength)
	{
		if (Edge->GetTwinEntityCount() > 1)
		{
			UE_LOGF(LogCADKernelBase, Warning, "Face %d Edge %d was self connected\n", Edge->GetFace()->GetId(), Edge->GetId());
			Edge->GetStartVertex()->UnlinkTo(*Edge->GetEndVertex());
		}
		else
		{
			Edge->SetAsDegenerated();
			UE_LOGF(LogCADKernelBase, Warning, "Face %d Edge %d is set as degenerated\n", Edge->GetFace()->GetId(), Edge->GetId());
		}
		return;
	}

	TArray<FTopologicalEdge*> ConnectedEdges;
	StartVertex.GetConnectedEdges(EndVertex, ConnectedEdges);
	const int32 ConnectedEdgeCount = ConnectedEdges.Num();
	if (ConnectedEdgeCount == 1)
	{
		return;
	}

	const bool bFirstEdgeBorder = Edge->IsBorder();

	for (FTopologicalEdge* SecondEdge : ConnectedEdges)
	{
		if (SecondEdge == Edge || !SecondEdge->IsActiveEntity() || SecondEdge->IsDegenerated())
		{
			continue;
		}

		const bool bSecondEdgeBorder = Edge->IsBorder();

		// Process only if at least one edge is Border
		if (!bFirstEdgeBorder && !bSecondEdgeBorder)
		{
			continue;
		}

		if (Edge->GetFace() != SecondEdge->GetFace() && Edge->IsLinkableTo(*SecondEdge, MinEdgeLength))
		{
			Edge->Link(*SecondEdge);
		}
	}
}

void MergeCoincidentEdges(TArray<FTopologicalEdge*>& EdgesToProcess, double MinEdgeLength)
{
	for (FTopologicalEdge* Edge : EdgesToProcess)
	{
		if (Edge->IsDegenerated() || !Edge->IsBorder())
		{
			continue;
		}
		MergeCoincidentEdges(Edge, MinEdgeLength);
	}
}

void MergeCoincidentEdges(TArray<FTopologicalVertex*>& VerticesToProcess, double MinEdgeLength)
{
	for (FTopologicalVertex* Vertex : VerticesToProcess)
	{
		if (Vertex->IsDeleted() || !Vertex->IsActiveEntity() || Vertex->HasMarker1())
		{
			continue;
		}
		Vertex->SetMarker1();

		TArray<FTopologicalEdge*> ConnectedEdges;
		Vertex->GetConnectedEdges(ConnectedEdges);
		const int32 ConnectedEdgeCount = ConnectedEdges.Num();
		if (ConnectedEdgeCount == 1)
		{
			continue;
		}

		for (int32 EdgeI = 0; EdgeI < ConnectedEdgeCount - 1; ++EdgeI)
		{
			FTopologicalEdge* Edge = ConnectedEdges[EdgeI];
			if (!Edge->IsActiveEntity() || Edge->IsDegenerated())
			{
				continue;
			}

			MergeCoincidentEdges(Edge, MinEdgeLength);
		}
	}

	for (FTopologicalVertex* Vertex : VerticesToProcess)
	{
		Vertex->ResetMarker1();
	}
}

void GetVertices(const TArray<FTopologicalEdge*>& InEdges, TArray<FTopologicalVertex*>& OutVertices)
{
	OutVertices.Reset(InEdges.Num());

	for (const FTopologicalEdge* Edge : InEdges)
	{
		if (!Edge || Edge->IsDeleted())
		{
			continue;
		}

		Edge->GetStartVertex()->GetLinkActiveEntity()->ResetMarker1();
		Edge->GetEndVertex()->GetLinkActiveEntity()->ResetMarker1();
	}

	for (const FTopologicalEdge* Edge : InEdges)
	{
		if (!Edge || Edge->IsDeleted())
		{
			continue;
		}

		if (!Edge->GetStartVertex()->GetLinkActiveEntity()->HasMarker1())
		{
			Edge->GetStartVertex()->GetLinkActiveEntity()->SetMarker1();
			OutVertices.Add(&*Edge->GetStartVertex()->GetLinkActiveEntity());
		}
		if (!Edge->GetEndVertex()->GetLinkActiveEntity()->HasMarker1())
		{
			Edge->GetEndVertex()->GetLinkActiveEntity()->SetMarker1();
			OutVertices.Add(&*Edge->GetEndVertex()->GetLinkActiveEntity());
		}
	}

	for (FTopologicalVertex* Vertex : OutVertices)
	{
		Vertex->ResetMarker1();
	}
}

} // namespace TopomakerTools

FTopomaker::FTopomaker(FSession& InSession, const FTopomakerOptions& InOptions)
	: Session(InSession)
{
	SetTolerance(InOptions);

	int32 ShellCount = 0;
	for (const TSharedPtr<FBody>& Body : Session.GetModel().GetBodies())
	{
		ShellCount += Body->GetShells().Num();
	}
	Shells.Reserve(ShellCount);

	for (const TSharedPtr<FBody>& Body : Session.GetModel().GetBodies())
	{
		Body->CompleteMetaData();
		for (const TSharedPtr<FShell>& Shell : Body->GetShells())
		{
			Shell->PropagateBodyOrientation();
			Shells.Add(Shell.Get());
		}
	}

	InitFaces();
}

FTopomaker::FTopomaker(FSession& InSession, const TArray<TSharedPtr<FShell>>& InShells, const FTopomakerOptions& InOptions)
	: Session(InSession)
{
	SetTolerance(InOptions);

	Shells.Reserve(InShells.Num());
	for (const TSharedPtr<FShell>& Shell : InShells)
	{
		Shell->PropagateBodyOrientation();
		Shells.Add(Shell.Get());
	}
	InitFaces();
}

FTopomaker::FTopomaker(FSession& InSession, const TArray<TSharedPtr<FTopologicalFace>>& InFaces, const FTopomakerOptions& InOptions)
	: Session(InSession)
{
	SetTolerance(InOptions);

	Faces.Reserve(InFaces.Num());
	for (const TSharedPtr<FTopologicalFace>& Face : InFaces)
	{
		Faces.Add(Face);
	}
}

void FTopomaker::InitFaces()
{
	int32 FaceCount = 0;
	for (FShell* Shell : Shells)
	{
		FaceCount += Shell->FaceCount();
	}
	Faces.Reserve(FaceCount);

	for (FShell* Shell : Shells)
	{
		for (const FOrientedFace& Face : Shell->GetFaces())
		{
			Faces.Add(Face.Entity);
		}
	}

	for (const TSharedPtr<FTopologicalFace>& Face : Faces)
	{
		Face->ResetMarkers();
	}
}

void FTopomaker::EmptyShells()
{
	for (FShell* Shell : Shells)
	{
		Shell->RemoveFaces();
	}
}

void FTopomaker::Sew()
{
	TArray<FTopologicalVertex*> BorderVertices;
	GetBorderVertices(BorderVertices);
	TopomakerTools::MergeCoincidentVertices(BorderVertices, SewTolerance);

	CheckSelfConnectedEdge(EdgeLengthTolerance, BorderVertices);

	// basic case: merge pair of edges connected together at both extremities i.e. pair of edges with same extremity active vertices.
	// #cadkernel_check: Commenting the line below prevent the creation of the mesh in UE-228246. Why?
	TopomakerTools::MergeCoincidentEdges(BorderVertices, EdgeLengthTolerance);

	// advance case: two partially coincident edges connected at one extremity i.e. coincident along the shortest edge
	BorderVertices.Reset();
	GetBorderVertices(BorderVertices);
	TopomakerTools::StitchParallelEdges(BorderVertices, SewTolerance, EdgeLengthTolerance, /*bProhibitSewingEdgesOfSameFace*/ false);

	const bool bForceJoining = SewOption::IsForceJoining(SewOptions);
	if (bForceJoining)
	{
		BorderVertices.Empty(BorderVertices.Num());
		GetBorderVertices(BorderVertices);
		TopomakerTools::MergeCoincidentVertices(BorderVertices, SewToleranceToForceJoin);

		CheckSelfConnectedEdge(EdgeLengthTolerance, BorderVertices);

		// re process with new edges from MergeUnconnectedSuccessiveEdges and new merged vertices (if bForceJoining)
		BorderVertices.Reset();
		GetBorderVertices(BorderVertices);
		TopomakerTools::MergeCoincidentEdges(BorderVertices, LargeEdgeLengthTolerance);

		// advance case: two partially coincident edges connected at one extremity i.e. coincident along the shortest edge
		BorderVertices.Reset();
		GetBorderVertices(BorderVertices);
		TopomakerTools::StitchParallelEdges(BorderVertices, SewToleranceToForceJoin, LargeEdgeLengthTolerance, /*bProhibitSewingEdgesOfSameFace*/ true);
	}

	if (SewOption::IsRemoveDuplicatedFaces(SewOptions))
	{
		RemoveDuplicatedFaces();
	}

	if (SewOption::IsRemoveThinFaces(SewOptions))
	{
		RemoveThinFaces();
		// advance case: two partially coincident edges connected at one extremity i.e. coincident along the shortest edge
		BorderVertices.Reset();
		GetBorderVertices(BorderVertices);

		TopomakerTools::MergeCoincidentEdges(BorderVertices, bForceJoining ? LargeEdgeLengthTolerance : EdgeLengthTolerance);
		
		TopomakerTools::StitchParallelEdges(BorderVertices, bForceJoining ? SewToleranceToForceJoin : SewTolerance, bForceJoining ? LargeEdgeLengthTolerance : EdgeLengthTolerance, /*bProhibitSewingEdgesOfSameFace*/ true);
	}

	ResetMarkersOfFaces();
}

void FTopomaker::GetVertices(TArray<FTopologicalVertex*>& Vertices)
{
	Vertices.Empty(10 * Faces.Num());

	for (TSharedPtr<FTopologicalFace> Face : Faces)
	{
		if (Face->IsDeletedOrDegenerated())
		{
			continue;
		}

		for (TSharedPtr<FTopologicalLoop> Loop : Face->GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;

				if (!Edge->GetStartVertex()->GetLinkActiveEntity()->HasMarker1())
				{
					Edge->GetStartVertex()->GetLinkActiveEntity()->SetMarker1();
					Vertices.Add(&*Edge->GetStartVertex()->GetLinkActiveEntity());
				}
				if (!Edge->GetEndVertex()->GetLinkActiveEntity()->HasMarker1())
				{
					Edge->GetEndVertex()->GetLinkActiveEntity()->SetMarker1();
					Vertices.Add(&*Edge->GetEndVertex()->GetLinkActiveEntity());
				}
			}
		}
	}
	for (FTopologicalVertex* Vertex : Vertices)
	{
		Vertex->ResetMarker1();
	}
}

void FTopomaker::GetBorderVertices(TArray<FTopologicalVertex*>& BorderVertices)
{
	TArray<FTopologicalVertex*> Vertices;
	GetVertices(Vertices);

	BorderVertices.Empty(Vertices.Num());

	for (FTopologicalVertex* Vertex : Vertices)
	{
		if (Vertex->IsBorderVertex())
		{
			BorderVertices.Add(Vertex);
		}
	}
}

void FTopomaker::MergeUnconnectedSuccessiveEdges()
{
	for (TSharedPtr<FTopologicalFace> FacePtr : Faces)
	{
		if (!FacePtr.IsValid())
		{
			continue;
		}

		FTopologicalFace* Face = FacePtr.Get();

		TArray<TArray<FOrientedEdge>> ArrayOfCandidates;
		ArrayOfCandidates.Reserve(10);

		// For each loop, find unconnected successive edges...
		for (TSharedPtr<FTopologicalLoop> Loop : Face->GetLoops())
		{
			TArray<FOrientedEdge>& Edges = Loop->GetEdges();
			int32 EdgeCount = Edges.Num();

			// Find the starting edge i.e. the next edge of the first edge that its ending vertex is connecting to 3 or more edges 
			// The algorithm start to the last edge of the loop, if it verifies the criteria then the first edge is the edges[0]
			int32 EndIndex = EdgeCount;
			{
				TSharedPtr<FTopologicalVertex> EndVertex;
				do
				{
					EndIndex--;
					EndVertex = Edges[EndIndex].Direction == EOrientation::Front ? Edges[EndIndex].Entity->GetEndVertex() : Edges[EndIndex].Entity->GetStartVertex();
				} while (EndVertex->ConnectedEdgeCount() == 2 && EndIndex > 0);
			}
			EndIndex++;

			// First step
			// For the loop, find all arrays of successive unconnected edges
			TArray<FOrientedEdge>* Candidates = &ArrayOfCandidates.Emplace_GetRef();
			Candidates->Reserve(10);
			bool bCanStop = false;
			for (int32 Index = EndIndex; bCanStop == (Index != EndIndex); ++Index)
			{
				if (Index == EdgeCount)
				{
					Index = 0;
				}
				bCanStop = true;

				FOrientedEdge& Edge = Edges[Index];
				if (Edge.Entity->GetTwinEntityCount() == 1)
				{
					TSharedPtr<FTopologicalVertex> EndVertex = Edge.Direction == EOrientation::Front ? Edge.Entity->GetEndVertex() : Edge.Entity->GetStartVertex();

					TArray<FTopologicalEdge*> ConnectedEdges;
					EndVertex->GetConnectedEdges(ConnectedEdges);

					bool bEdgeIsNotTheLast = false;
					if (ConnectedEdges.Num() == 2)
					{
						// check if the edges are tangents
						FVector StartTangentEdge = ConnectedEdges[0]->GetTangentAt(*EndVertex);
						FVector StartTangentOtherEdge = ConnectedEdges[1]->GetTangentAt(*EndVertex);

						double CosAngle = FVectorUtil::ComputeCosinus(StartTangentEdge, StartTangentOtherEdge);
						if (CosAngle < -UE_DOUBLE_HALF_SQRT_3) // Cos(30 deg)
						{
							bEdgeIsNotTheLast = true;
						}
					}

					if (bEdgeIsNotTheLast || Candidates->Num() > 0)
					{
						Candidates->Add(Edges[Index]);
					}

					if (!bEdgeIsNotTheLast && Candidates->Num() > 0)
					{
						Candidates = &ArrayOfCandidates.Emplace_GetRef();
						Candidates->Reserve(10);
					}
				}
			}
		}

		// Second step, 
		// Each array of edges are merged to generated a single edge, the loop is updated
		for (TArray<FOrientedEdge>& Candidates : ArrayOfCandidates)
		{
			if (Candidates.Num())
			{
				TSharedRef<FTopologicalVertex> StartVertex = Candidates[0].Direction == EOrientation::Front ? Candidates[0].Entity->GetStartVertex() : Candidates[0].Entity->GetEndVertex();
				TSharedRef<FTopologicalVertex> EndVertex = Candidates.Last().Direction == EOrientation::Front ? Candidates.Last().Entity->GetEndVertex() : Candidates.Last().Entity->GetStartVertex();
				FTopologicalEdge::CreateEdgeByMergingEdges(LargeEdgeLengthTolerance, Candidates, StartVertex, EndVertex);
			}
		}
	}
}

void FTopomaker::RemoveIsolatedEdges()
{
	TArray<FTopologicalEdge*> IsolatedEdges;

	TArray<FTopologicalVertex*> Vertices;
	GetVertices(Vertices);

	for (const FTopologicalVertex* Vertex : Vertices)
	{
		for (const FTopologicalVertex* TwinVertex : Vertex->GetTwinEntities())
		{
			TArray<FTopologicalEdge*> Edges = Vertex->GetDirectConnectedEdges();
			for (FTopologicalEdge* Edge : Edges)
			{
				if (Edge->GetLoop() == nullptr)
				{
					IsolatedEdges.Add(Edge);
				}
			}
		}
	}
}

void SplitVertexLinkByShell(FTopologicalVertex* InVertex)
{
	TMap<const FShell*, TArray<FTopologicalVertex*>> ShellToVertices;

	const TArray<FTopologicalVertex*>& Twins = InVertex->GetTwinEntities();
	ShellToVertices.Reserve(Twins.Num());

	for (FTopologicalVertex* TwinVertex : Twins)
	{
		const FShell* OtherShell = (const FShell*)TwinVertex->GetDirectConnectedEdges()[0]->GetFace()->GetHost();
		TArray<FTopologicalVertex*>& Vertices = ShellToVertices.FindOrAdd(OtherShell);
		Vertices.Add(TwinVertex);
	}

#ifdef DEBUG_Split_Vertex_Link_By_Shell
	{
		F3DDebugSession B(*FString::Printf(TEXT("Shells")));
		for (TPair<const FShell*, TArray<FTopologicalVertex*>>& Pair : ShellToVertices)
		{
			F3DDebugSession B(*FString::Printf(TEXT("Shell to vertex")));
			{
				F3DDebugSession B(*FString::Printf(TEXT("Shell")));
				Display(*Pair.Key);
			}
			TArray<FTopologicalVertex*>& Vertices = Pair.Value;
			for (const FTopologicalVertex* Vertex : Vertices)
			{
				{
					F3DDebugSession A(*FString::Printf(TEXT("Vertex %d"), Vertex->GetId()));
					Display(*Vertex);
					Display(*Vertex->GetDirectConnectedEdges()[0], EVisuProperty::RedPoint);
					Display(*Vertex->GetFace());
				}
			}
		}
		Wait();
	}
#endif

	InVertex->UnlinkTwinEntities();

	for (TPair<const FShell*, TArray<FTopologicalVertex*>>& Pair : ShellToVertices)
	{
		TArray<FTopologicalVertex*>& Vertices = Pair.Value;
		if (Vertices.Num() > 1)
		{
			FTopologicalVertex* FirstVertex = Vertices[0];
			for (FTopologicalVertex* TwinVertex : Vertices)
			{
				FirstVertex->Link(*TwinVertex);
			}
		}
	}
}

void FTopomaker::UnlinkNonManifoldVertex()
{
	TArray<FTopologicalVertex*> Vertices;
	GetVertices(Vertices);

	TArray<FTopologicalVertex*> VerticesToSplit;
	VerticesToSplit.Reserve(Vertices.Num());

	for (FTopologicalVertex* Vertex : Vertices)
	{
		const TArray<FTopologicalVertex*>& Twins = Vertex->GetTwinEntities();
		const FShell* Shell = (const FShell*) Twins[0]->GetFace()->GetHost();

		for (const FTopologicalVertex* TwinVertex : Twins)
		{
			const FShell* OtherShell = (const FShell*)TwinVertex->GetDirectConnectedEdges()[0]->GetFace()->GetHost();
			if (OtherShell != Shell)
			{
				TSharedPtr<FTopologicalVertex> ActiveVertex = Vertex->GetLinkActiveEntity();
				if(!ActiveVertex->IsProcessed())
				{
					VerticesToSplit.Add(ActiveVertex.Get());
					ActiveVertex->SetProcessedMarker();
				}
			}
		}
	}

	for (const FTopologicalVertex* Vertex : VerticesToSplit)
	{
		Vertex->ResetProcessedMarker();
	}

	for (FTopologicalVertex* Vertex : VerticesToSplit)
	{
		SplitVertexLinkByShell(Vertex);
	}
}

void FindBorderVertex(const TArray<FTopologicalEdge*>& Edges, TArray<FTopologicalVertex*>& OutBorderVertices)
{
	OutBorderVertices.Reserve(Edges.Num());

	TFunction<void(FTopologicalVertex&)> AddVertex = [&](FTopologicalVertex& Vertex)
	{
		if (!Vertex.GetLinkActiveEntity()->HasMarker1())
		{
			OutBorderVertices.Add(&*Vertex.GetLinkActiveEntity());
			Vertex.GetLinkActiveEntity()->SetMarker1();
		}
	};

	for (FTopologicalEdge* Edge : Edges)
	{
		if (Edge->IsBorder())
		{
			AddVertex(*Edge->GetStartVertex());
			AddVertex(*Edge->GetEndVertex());
		}
	}

	for (FTopologicalVertex* Vertex : OutBorderVertices)
	{
		Vertex->ResetMarker1();
	}

}

void FTopomaker::RemoveThinFaces()
{
	TArray<FTopologicalEdge*> BorderEdges;
	TArray<FTopologicalVertex*> BorderVertices;

	// Find thin faces
	for (TSharedPtr<FTopologicalFace> Face : Faces)
	{
		if (Face->GetLoops().Num() == 0)
		{
			Face->SetDeletedMarker();
			continue;
		}

		FFaceAnalyzer Analyer(*Face, ThinFaceWidth);
		double GapSize = 0;
		if (Analyer.IsThinFace(GapSize))
		{
			Face->Remove(&BorderEdges);
			Face->SetDeletedMarker();

			TopomakerTools::GetVertices(BorderEdges, BorderVertices);
			TopomakerTools::MergeCoincidentVertices(BorderVertices, GapSize * 1.2);
			TopomakerTools::MergeCoincidentEdges(BorderEdges, GapSize * 1.2);

			BorderEdges.Reset();
		}
	}
}

void FTopomaker::SetSelfConnectedEdgeDegenerated(TArray<FTopologicalVertex*>& Vertices)
{
	for (const FTopologicalVertex* Vertex : Vertices)
	{
		for (const FTopologicalVertex* TwinVertex : Vertex->GetTwinEntities())
		{
			for (FTopologicalEdge* Edge : TwinVertex->GetDirectConnectedEdges())
			{
				if (Edge->GetStartVertex()->IsLinkedTo(Edge->GetEndVertex()))
				{
					if (!Edge->IsDegenerated() && Edge->Length() < 2 * SewTolerance)
					{
						Edge->SetAsDegenerated();
					}
				}
			}
		}
	}
}

void FTopomaker::CheckSelfConnectedEdge(double MaxLengthOfDegeneratedEdge, TArray<FTopologicalVertex*>& OutBorderVertices)
{
	TFunction<void(FTopologicalVertex&)> AddVertexIfBorder = [&](FTopologicalVertex& Vertex)
	{
		TSharedRef<FTopologicalVertex> ActiveVertex = Vertex.GetLinkActiveEntity();
		if (ActiveVertex->IsBorderVertex())
		{
			OutBorderVertices.Add(&*ActiveVertex);
		}
	};

	for (TSharedPtr<FTopologicalFace> Face : Faces)
	{
		for (TSharedPtr<FTopologicalLoop> Loop : Face->GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				TSharedPtr<FTopologicalEdge> Edge = OrientedEdge.Entity;
				if (Edge->GetStartVertex()->IsLinkedTo(Edge->GetEndVertex()))
				{
					if (!Edge->IsDegenerated() && Edge->Length() < MaxLengthOfDegeneratedEdge)
					{
						if (Edge->GetTwinEntityCount() > 1)
						{
							UE_LOGF(LogCADKernelBase, Warning, "Face %d Edge %d was self connected\n", Face->GetId(), Edge->GetId());
							Edge->GetStartVertex()->UnlinkTo(*Edge->GetEndVertex());
						}
						else
						{
							Edge->SetAsDegenerated();
							UE_LOGF(LogCADKernelBase, Warning, "Face %d Edge %d is set as degenerated\n", Face->GetId(), Edge->GetId());
						}
					}
				}
			}
		}
	}
}

void FTopomaker::SplitIntoConnectedShells()
{
	DeleteNonmanifoldLink();

	// Processed1 : Surfaces added in CandidateSurfacesForMesh

	int32 TopologicalFaceCount = Faces.Num();
	// Is closed ?
	// Is one shell ?

	TArray<FFaceSubset> SubShells;

	int32 ProcessFaceCount = 0;

	TArray<FTopologicalFace*> Front;
	TFunction<void(const FTopologicalFace&, FFaceSubset&)> GetNeighboringFaces = [&](const FTopologicalFace& Face, FFaceSubset& Shell)
	{
		for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
		{
			for (const FOrientedEdge& OrientedEdge : Loop->GetEdges())
			{
				const TSharedPtr<FTopologicalEdge>& Edge = OrientedEdge.Entity;

				if (Edge->GetTwinEntityCount() == 1)
				{
					if (!Edge->IsDegenerated())
					{
						Shell.BorderEdgeCount++;
					}
					continue;
				}

				if (Edge->GetTwinEntityCount() > 2)
				{
					Shell.NonManifoldEdgeCount++;
				}

				for (FTopologicalEdge* NextEdge : Edge->GetTwinEntities())
				{
					FTopologicalFace* NextFace = NextEdge->GetFace();
					if ((NextFace == nullptr) || NextFace->IsNotToOrAlreadyProcess())
					{
						continue;
					}
					NextFace->SetProcessedMarker();
					Front.Add(NextFace);
				}
			}
		}
	};

	TFunction<void(FFaceSubset&)> PropagateFront = [&](FFaceSubset& Shell)
	{
		while (Front.Num())
		{
			FTopologicalFace* Face = Front.Pop();
			if (Face == nullptr)
			{
				continue;
			}

			Shell.Faces.Add(Face);
			GetNeighboringFaces(*Face, Shell);
		}
	};

	SetToProcessMarkerOfFaces();	
		
	for (const TSharedPtr<FTopologicalFace>& Face : Faces)
	{
		if (Face->IsProcessedDeletedOrDegenerated())
		{
			continue;
		}

		FFaceSubset& Shell = SubShells.Emplace_GetRef();
		Shell.Faces.Reserve(TopologicalFaceCount - ProcessFaceCount);
		Front.Empty(TopologicalFaceCount);

		Face->SetProcessedMarker();
		Front.Add(Face.Get());
		PropagateFront(Shell);
		ProcessFaceCount += Shell.Faces.Num();

		if (ProcessFaceCount == TopologicalFaceCount)
		{
			break;
		}
	}

	ResetMarkersOfFaces();

	// for each FaceSubset, find the main shell
	for (FFaceSubset& FaceSubset : SubShells)
	{
		TMap<FTopologicalShapeEntity*, int32> BodyToFaceCount;
		TMap<FTopologicalShapeEntity*, int32> ShellToFaceCount;
		TMap<uint32, int32> ColorToFaceCount;
		TMap<FString, int32> NameToFaceCount;

		for (FTopologicalFace* Face : FaceSubset.Faces)
		{
			FTopologicalShapeEntity* Shell = Face->GetHost();
			FTopologicalShapeEntity* Body = Shell->GetHost();

			ShellToFaceCount.FindOrAdd(Shell)++;
			BodyToFaceCount.FindOrAdd(Body)++;
			ColorToFaceCount.FindOrAdd(Face->GetColorId())++;
			NameToFaceCount.FindOrAdd(Face->GetName())++;
		}

		FaceSubset.SetMainShell(ShellToFaceCount);
		FaceSubset.SetMainBody(BodyToFaceCount);
		FaceSubset.SetMainName(NameToFaceCount);
		FaceSubset.SetMainColor(ColorToFaceCount);
	}

	EmptyShells();

	// for each FaceSubset, process the Shell
	for (FFaceSubset FaceSubset : SubShells)
	{
		if (FaceSubset.MainShell != nullptr)
		{
			FShell* Shell = (FShell*)FaceSubset.MainShell;
#ifdef CADKERNEL_DEBUG
			ensure(Shell->GetFaces().Num() == 0);
#endif
			Shell->Add(FaceSubset.Faces);
		}
		else
		{
			FBody* Body = (FBody*)FaceSubset.MainBody;
			if (Body == nullptr)
			{
				TSharedRef<FBody> SharedBody = FEntity::MakeShared<FBody>();
				Session.GetModel().Add(SharedBody);
				Body = &SharedBody.Get();

				Session.SpawnEntityIdent(*Body);
				Body->SetName(FaceSubset.MainName);
				Body->SetColorId(FaceSubset.MainColor);
				Body->SetHostId(Body->GetId());
			}

			TSharedRef<FShell> Shell = FEntity::MakeShared<FShell>();

			Shells.Add(&*Shell);
			Body->AddShell(Shell);
			Session.SpawnEntityIdent(*Shell);

			Shell->Add(FaceSubset.Faces);
			Shell->SetName(FaceSubset.MainName);
			Shell->SetColorId(FaceSubset.MainColor);
			Shell->SetHostId(Shell->GetId());
		}
	}

	UnlinkFromOther();
	UnlinkNonManifoldVertex();

	RemoveEmptyShells();
	Session.GetModel().RemoveEmptyBodies();
}

void FTopomaker::RemoveEmptyShells()
{
#ifdef CADKERNEL_DEV
	{
		FModel& Model = Session.GetModel();
		for (const TSharedPtr<FBody>& Body : Model.GetBodies())
		{
			ensureCADKernel(!Body->HasMarker1());
		}
	}
#endif

	TArray<FBody*> Bodies;
	Bodies.Reserve(Shells.Num());

	TArray<FShell*> NewShells;

	for (FShell* Shell : Shells)
	{
		if (Shell->FaceCount() == 0)
		{
			FBody* Body = (FBody*)Shell->GetHost();
			if (Body != nullptr && !Body->HasMarker1())
			{
				Body->SetMarker1();
				Bodies.Add(Body);
			}
			Shell->SetDeletedMarker();
		}
		else
		{
			NewShells.Add(Shell);
		}
	}
	Swap(Shells, NewShells);

	for (FBody* Body : Bodies)
	{
		Body->RemoveEmptyShell();
		Body->ResetMarkers();
	}
}

void FTopomaker::OrientShells()
{
	for (FShell* Shell : Shells)
	{
		int32 FaceSwapCount = Shell->Orient();
	}
}

void FTopomaker::RemoveDuplicatedFaces()
{
	TArray<FTopologicalFace*> NonManifoldFaces;
	NonManifoldFaces.Reserve(Faces.Num() / 10);
	for (TSharedPtr<FTopologicalFace>& FacePtr : Faces)
	{
		FTopologicalFace& Face = *FacePtr;
		if (Face.IsDeleted())
		{
			continue;
		}

		if (Face.IsANonManifoldFace())
		{
			if (Face.IsAFullyNonManifoldFace())
			{
				if (Face.IsADuplicatedFace())
				{
					Face.Delete();
				}
			}
			else
			{
				NonManifoldFaces.Add(&Face);
			}
		}
	}


	// Step 2: Process of Face with NonManifold and without border edges
	for (FTopologicalFace* Face : NonManifoldFaces)
	{
		if (Face->IsDeleted() || !Face->IsANonManifoldFace())
		{
			continue;
		}

		if (!Face->IsABorderFace())
		{
			if (Face->IsADuplicatedFace())
			{
				Face->Delete();
			}
		}
	}

	// Step 3: Process of the remaining non manifold Face
	for (FTopologicalFace* Face : NonManifoldFaces)
	{
		if (Face->IsDeleted() || !Face->IsANonManifoldFace())
		{
			continue;
		}

		if (Face->IsADuplicatedFace())
		{
			Face->Delete();
		}
	}
}

void FTopomaker::SetToProcessMarkerOfFaces()
{
	for (const TSharedPtr<FTopologicalFace>& Face : Faces)
	{
		if (!Face->IsDeletedOrDegenerated())
		{
			Face->SetToProcessMarker();
		}
	}
}

void FTopomaker::ResetMarkersOfFaces()
{
	for (const TSharedPtr<FTopologicalFace>& Face : Faces)
	{
		Face->ResetMarkers();
	}
}

void FTopomaker::DeleteNonmanifoldLink()
{
	for (const TSharedPtr<FTopologicalFace>& Face : Faces)
	{
		Face->DeleteNonmanifoldLink();
	}
}

void FTopomaker::UnlinkFromOther()
{
	TFunction<void(TArray<FTopologicalVertex*>&)> MergeCoincidents = [&Tolerance = Tolerance](TArray<FTopologicalVertex*>& Vertices)
	{
		TArray<FTopologicalVertex*> ActiveVerticesToLink;
		ActiveVerticesToLink.Reserve(Vertices.Num());
		for (FTopologicalVertex* Vertex : Vertices)
		{
			if (!Vertex->IsProcessed())
			{
				ActiveVerticesToLink.Add(&Vertex->GetLinkActiveEntity().Get());
				Vertex->SetProcessedMarker();
			}
		}
		for (FTopologicalVertex* Vertex : Vertices)
		{
			Vertex->ResetProcessedMarker();
		}

		TopomakerTools::MergeCoincidentVertices(ActiveVerticesToLink, Tolerance);
	};

	TArray<FTopologicalVertex*> VerticesToLink;
	for (FShell* Shell : Shells)
	{
		VerticesToLink.Reset();
		Shell->UnlinkFromOther(VerticesToLink);
		MergeCoincidents(VerticesToLink);
	}

	if(Shells.IsEmpty())
	{
		VerticesToLink.Reset();
		TArray<FTopologicalFace*> FacePtrs;
		for (TSharedPtr<FTopologicalFace> Face : Faces)
		{
			if (!Face->IsDeleted())
			{
				FacePtrs.Add(Face.Get());
			}
		}

		ShellTools::UnlinkFromOther(FacePtrs, VerticesToLink);
		MergeCoincidents(VerticesToLink);
	}
}

}
