// Copyright Epic Games, Inc. All Rights Reserved.

#include "Message.h"

DEFINE_LOG_CATEGORY(LogCADKernelBase);
