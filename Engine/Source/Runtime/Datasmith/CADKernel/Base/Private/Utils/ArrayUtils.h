// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "Math/Boundary.h"
#include "Utils/IndexOfCoordinateFinder.h"

namespace UE::CADKernel
{
namespace ArrayUtils
{
/**
 * Merge two array of sorted values and keep the resulting array sorted.
 * The value to insert is inserted if and only if it is not nearly equal to an existent value
 * and if it is inside the boundary of the InitialArray ([InitialArray[0], InitialArray.Last()])
 * The values of the existing array are always kept.
 */
inline void InsertInside(TArray<double>& InitialArray, const TArray<double>& ValuesToInsert, double Epsilon)
{
	ensureCADKernel(InitialArray.Num() >= 2);

	if (ValuesToInsert.Num() == 0)
	{
		return;
	}

	InitialArray.Reserve(InitialArray.Num() + ValuesToInsert.Num());

	FDichotomyFinder Finder(InitialArray);

	int32 Index = 0;
	for (double Value : ValuesToInsert)
	{
		if (Value < InitialArray[0])
		{
			continue;
		}

		if (Value > InitialArray.Last())
		{
			break;
		}

		Index = Finder.Find(Value);

		if (FMath::IsNearlyEqual(Value, InitialArray[Index], Epsilon))
		{
			continue;
		}

		if (FMath::IsNearlyEqual(Value, InitialArray[Index + 1], Epsilon))
		{
			continue;
		}

		InitialArray.EmplaceAt(Index + 1, Value);
		Finder.StartUpper++;
	}
}

/**
 * Merge two array of sorted values and keep the resulting array sorted.
 * The value to insert is inserted if and only if it is not nearly equal to an existent value except for the first and last value of ValuesToInsert if they are outside initial array boundary
 */
inline void Complete(TArray<double>& InitialArray, TArray<double>& ValuesToInsert, double Epsilon)
{
	if (ValuesToInsert.Num() == 0)
	{
		return;
	}

	if (InitialArray.Num() == 0)
	{
		Swap(InitialArray, ValuesToInsert);
		return;
	}

	InitialArray.Reserve(InitialArray.Num() + ValuesToInsert.Num());

	FDichotomyFinder Finder(InitialArray);

	int32 Index = 0;
	for (double Value : ValuesToInsert)
	{
		if (Value < InitialArray[0])
		{
			if (FMath::IsNearlyEqual(Value, InitialArray[Index], Epsilon))
			{
				InitialArray[Index] = Value;
			}
			else
			{
				InitialArray.EmplaceAt(Index++, Value);
				Finder.StartUpper++;
			}
			continue;
		}

		if (Value > InitialArray.Last())
		{
			if (FMath::IsNearlyEqual(Value, InitialArray.Last(), Epsilon))
			{
				InitialArray.Last() = Value;
			}
			else
			{
				InitialArray.Emplace(Value);;
			}
			continue;
		}

		Index = Finder.Find(Value);

		if (FMath::IsNearlyEqual(Value, InitialArray[Index], Epsilon))
		{
			continue;
		}

		if (FMath::IsNearlyEqual(Value, InitialArray[Index + 1], Epsilon))
		{
			continue;
		}

		InitialArray.EmplaceAt(Index + 1, Value);
		Finder.StartUpper++;
	}
}

inline void RemoveDuplicates(TArray<double>& ArrayToProcess, double Epsilon)
{
	if (ArrayToProcess.Num() < 2)
	{
		return;
	}

	for (int32 Index = ArrayToProcess.Num() - 1; Index > 1; --Index)
	{
		if (FMath::IsNearlyEqual(ArrayToProcess[Index], ArrayToProcess[Index - 1], Epsilon))
		{
			ArrayToProcess.RemoveAt(Index - 1);
		}
	}

	if (ArrayToProcess.Num() > 1)
	{
		if (FMath::IsNearlyEqual(ArrayToProcess[0], ArrayToProcess[1], Epsilon))
		{
			ArrayToProcess.RemoveAt(1);
		}
	}
}

inline void SubArrayWithoutBoundary(const TArray<double>& InitialArray, const FLinearBoundary& Boundary, double Tolerance, TArray<double>& OutArray)
{
	int32 InitialSize = InitialArray.Num();
	OutArray.Empty(InitialSize + 2);

	if (InitialSize > 1)
	{
		FDichotomyFinder Finder(InitialArray);
		int32 MinIndex = Finder.Find(Boundary.GetMin());
		int32 MaxIndex = Finder.Find(Boundary.GetMax());

		if (InitialArray[MinIndex] > Boundary.GetMax())
		{
			return;
		}

		if (InitialArray[MaxIndex] < Boundary.GetMin())
		{
			return;
		}

		// MinIndex >= 0, if MinIndex == 0, Boundary.GetMin() can be < InitialArray[0]
		if (InitialArray[MinIndex] < Boundary.GetMin())
		{
			MinIndex++;
		}

		// FDichotomyFinder return the low index, so MaxIndex < InitialArray.Num() - 1
		if (InitialArray[MaxIndex + 1] < Boundary.GetMax())
		{
			MaxIndex++;
		}

		if (Boundary.GetMin() + 5 * Tolerance > InitialArray[MinIndex])
		{
			MinIndex++;
		}

		if (Boundary.GetMax() - 5 * Tolerance < InitialArray[MaxIndex])
		{
			MaxIndex--;
		}

		const int32 ElementCount = MaxIndex - MinIndex + 1;
		if (ElementCount > 0)
		{
			OutArray.Append(InitialArray.GetData() + MinIndex, ElementCount);
		}
	}
	else if (InitialSize == 1)
	{
		double Value = InitialArray[0];
		if ((Boundary.GetMin() + 5 * Tolerance > Value) || (Boundary.GetMax() - 5 * Tolerance < Value))
		{
			return;
		}
		OutArray.Add(Value);
	}
}

/**
 * Find the index of the segment containing the coordinate i.e. Coordinate is in [InCoordinates[OutIndex], InCoordinates[OutIndex+1]]
 * This method is very fast if initial value of Index is near the solution.
 * This is very useful to process linked set of coordinate like loop point coordinates.
 */
inline void FindCoordinateIndex(const TArray<double>& InCoordinates, double Coordinate, int32& OutIndex)
{
	if (OutIndex >= InCoordinates.Num())
	{
		OutIndex = InCoordinates.Num() - 1;
	}

	if (OutIndex < 0)
	{
		OutIndex = 0;
	}

	while (OutIndex > 0 && Coordinate < InCoordinates[OutIndex])
	{
		OutIndex--;
	}

	for (; OutIndex + 2 < InCoordinates.Num() && Coordinate > InCoordinates[OutIndex + 1]; ++OutIndex)
	{
	}

	ensureCADKernel(InCoordinates.IsValidIndex(OutIndex));
};

};
}
