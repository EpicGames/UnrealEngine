// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "Math/Boundary.h"
#include "Utils/IndexOfCoordinateFinder.h"

namespace UE::CADKernel
{
namespace Utils
{
template<typename SRC, typename DST>
inline void TSharedPtrArrayConversion(const TArray<TSharedPtr<SRC>>& Source, TArray<TSharedPtr<DST>>& Destination)
{
	Destination.Reserve(Destination.Num() + Source.Num());
	for (const TSharedPtr<SRC>& SourceItem : Source)
	{
		Destination.Add(StaticCastSharedPtr<DST>(SourceItem));
	}
}

template<typename SRC, typename DST>
inline void TSharedPtrArrayConversionChecked(const TArray<TSharedPtr<SRC>>& Source, TArray<TSharedPtr<DST>>& Destination)
{
	Destination.Reserve(Destination.Num() + Source.Num());
	for (const TSharedPtr<SRC>& SourceItem : Source)
	{
		TSharedPtr<DST> ConvertedPtr = StaticCastSharedPtr<DST>(SourceItem);
		if (!ConvertedPtr.IsValid())
		{
			continue;
		}
		Destination.Add(ConvertedPtr);
	}

}
};
}
