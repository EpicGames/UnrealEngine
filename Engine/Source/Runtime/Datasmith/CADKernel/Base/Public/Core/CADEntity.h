// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/HaveStates.h"
#include "Core/OrientedEntity.h"
#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Serialization/Archive.h"

#ifdef CADKERNEL_DEV
#include "Toolkit/Core/InfoEntity.h"
#endif

namespace UE::CADKernel
{
enum class EEntity : uint8
{
	None = 0,

	Curve,
	Surface,

	EdgeLink,
	VertexLink,

	TopologicalEdge,
	TopologicalFace,

	TopologicalLink,
	TopologicalLoop,

	TopologicalVertex,

	Shell,
	Body,
	Model,

	MeshModel,
	Mesh,

	Group,
	Criterion,
	Property,

	EntityTypeEnd,
};

class FDatabase;
class FEntity;
class FInfoEntity;
class FSession;
class FSystem;

class CADKERNEL_API FEntity : public TSharedFromThis<FEntity>, public FHaveStates
{
	friend class FDatabase;

protected:
	static const TCHAR* TypesNames[];
	FIdent Id = 0;

public:
	virtual ~FEntity();

	virtual void Delete()
	{
		Empty();
		SetDeletedMarker();
	}

	virtual void Empty()
	{

	}

	template<typename OtherEntity, typename... InArgTypes>
	static TSharedRef<OtherEntity> MakeShared(InArgTypes&&... Args)
	{
		OtherEntity* Entity = new OtherEntity(Forward<InArgTypes>(Args)...);
		TSharedRef<OtherEntity> NewShared = MakeShareable<OtherEntity>(Entity);

#ifdef CADKERNEL_DEV
		AddEntityInDatabase(NewShared);
#endif
		return NewShared;
	}

	template<typename OtherEntity>
	static TSharedRef<OtherEntity> MakeShared(FCADKernelArchive& Archive)
	{
		OtherEntity* Entity = new OtherEntity();
		TSharedRef<OtherEntity> NewShared = MakeShareable<OtherEntity>(Entity);
		return NewShared;
	}

	/**
	 * To Serialize a TSharedPtr<FEntity>, the Id of the FEntity is saved in the archive (ArchiveId).
	 * At the deserialization of the archive, a map is build linking ArchiveId to the TSharedPtr of the deserialized entities.
	 * At the deserialization of a TSharedPtr<FEntity>, the ArchiveId of the TSharedPtr<FEntity> is get. The map return the associated TSharedPtr.
	 * If the Entity is not yet deserialized, the TSharedPtr is add to a waiting list to be set as soon as the associated Entity is deserialized
	 * See FDatabase for more details
	 */
	static void SerializeIdent(FCADKernelArchive& Ar, TSharedPtr<FEntity>& Entity, bool bSaveSelection = true);
	static void SerializeIdent(FCADKernelArchive& Ar, TWeakPtr<FEntity>& Entity, bool bSaveSelection = true);
	static void SerializeIdent(FCADKernelArchive& Ar, FEntity** Entity, bool bSaveSelection = true);

	template<typename EntityType>
	static void SerializeIdent(FCADKernelArchive& Ar, EntityType** Entity, bool bSaveSelection = true)
	{
		SerializeIdent(Ar, (FEntity**)Entity, bSaveSelection);
	}

	template<typename EntityType>
	static void SerializeIdent(FCADKernelArchive& Ar, TSharedPtr<EntityType>& Entity, bool bSaveSelection = true)
	{
		SerializeIdent(Ar, (TSharedPtr<FEntity>&) Entity, bSaveSelection);
	}

	template<typename EntityType>
	static void SerializeIdent(FCADKernelArchive& Ar, TWeakPtr<EntityType>& Entity, bool bSaveSelection = true)
	{
		SerializeIdent(Ar, (TWeakPtr<FEntity>&) Entity, bSaveSelection);
	}

	/**
	 * SerializeIdent of each TSharedPtr<FEntity> of the array
	 */
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<FEntity*>& Array, bool bSaveSelection = true);
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<TWeakPtr<FEntity>>& Array, bool bSaveSelection = true);
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<TSharedPtr<FEntity>>& Array, bool bSaveSelection = true);
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<TOrientedEntity<FEntity>>& Array);

	template<typename EntityType>
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<EntityType*>& Array, bool bSaveSelection = true)
	{
		SerializeIdents(Ar, (TArray<FEntity*>&) Array, bSaveSelection);
	}

	template<typename EntityType>
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<TWeakPtr<EntityType>>& Array, bool bSaveSelection = true)
	{
		SerializeIdents(Ar, (TArray<TWeakPtr<FEntity>>&) Array, bSaveSelection);
	}

	template<typename EntityType>
	static void SerializeIdents(FCADKernelArchive& Ar, TArray<TSharedPtr<EntityType>>& Array, bool bSaveSelection = true)
	{
		SerializeIdents(Ar, (TArray<TSharedPtr<FEntity>>&) Array, bSaveSelection);
	}

	/**
	 * Spawn Ident on each FEntity* of the array
	 */
	static void SpawnIdentOnEntities(TArray<FEntity*>& Array, FDatabase& Database)
	{
		for (FEntity* Entity : Array)
		{
			Entity->SpawnIdent(Database);
		}
	}

	/**
	 * Spawn Ident on each TSharedPtr<FEntity> of the array
	 */
	static void SpawnIdentOnEntities(TArray<TSharedPtr<FEntity>>& Array, FDatabase& Database)
	{
		for (TSharedPtr<FEntity>& Entity : Array)
		{
			Entity->SpawnIdent(Database);
		}
	}

	static void SpawnIdentOnEntities(TArray<TOrientedEntity<FEntity>>& Array, FDatabase& Database)
	{
		for (TOrientedEntity<FEntity>& Entity : Array)
		{
			Entity.Entity->SpawnIdent(Database);
		}
	}

	template<typename EntityType>
	static void SpawnIdentOnEntities(TArray<TSharedPtr<EntityType>>& Array, FDatabase& Database)
	{
		SpawnIdentOnEntities((TArray<TSharedPtr<FEntity>>&) Array, Database);
	}

	/**
	 * Reset Processed flags Recursively on each TSharedPtr<FEntity> of the array
	 */
	static void ResetMarkersRecursivelyOnEntities(const TArray<FEntity*>& Array)
	{
		for (const FEntity* Entity : Array)
		{
			Entity->ResetMarkersRecursively();
		}
	}

	static void ResetMarkersRecursivelyOnEntities(const TArray<TWeakPtr<FEntity>>& Array)
	{
		for (const TWeakPtr<FEntity>& Entity : Array)
		{
			Entity.Pin()->ResetMarkersRecursively();
		}
	}

	static void ResetMarkersRecursivelyOnEntities(const TArray<TSharedPtr<FEntity>>& Array)
	{
		for (const TSharedPtr<FEntity>& Entity : Array)
		{
			Entity->ResetMarkersRecursively();
		}
	}

	static void ResetMarkersRecursivelyOnEntities(const TArray<TOrientedEntity<FEntity>>& Array)
	{
		for (const TOrientedEntity<FEntity>& OrientedEntity : Array)
		{
			OrientedEntity.Entity->ResetMarkersRecursively();
		}
	}

	template<typename EntityType>
	static void ResetMarkersRecursivelyOnEntities(const TArray<TSharedPtr<EntityType>>& Array)
	{
		ResetMarkersRecursivelyOnEntities((const TArray<TSharedPtr<FEntity>>&) Array);
	}

	/**
	 * Serialization of a FEntity. Each class derived from FEntity has to override this method (and call the direct base class override method first)
	 * E.g.
	 * 	class FEntityXXX : public FEntityXX
	 *  {
	 *		virtual void Serialize(FCADKernelArchive& Ar) override
	 * 		{
	 *			FEntityXX::Serialize(Ar);
	 *			...
	 * 		}
	 *	}
	 */
	virtual void Serialize(FCADKernelArchive& Ar)
	{
		Ar << Id;
		FHaveStates::Serialize(Ar);
	}

	/**
	 * Deserialize the next entity in the archive.
	 */
	static TSharedPtr<FEntity> Deserialize(FCADKernelArchive& Ar);

	static const TCHAR* GetTypeName(EEntity Type);

	virtual EEntity GetEntityType() const = 0;

	bool IsTopologicalEntity() const
	{
		return (GetEntityType() >= EEntity::EdgeLink && GetEntityType() <= EEntity::Model);
	}

	bool IsTopologicalShapeEntity() const
	{
		return (GetEntityType() == EEntity::TopologicalFace) || (GetEntityType() >= EEntity::Shell && GetEntityType() <= EEntity::Model);
	}

	bool IsGeometricalEntity()
	{
		return (GetEntityType() == EEntity::Curve || GetEntityType() == EEntity::Surface);
	}

	const TCHAR* GetTypeName() const
	{
		return GetTypeName(GetEntityType());
	}

	const FIdent& GetId() const
	{
		return Id;
	}

#ifdef CADKERNEL_DEV
	void InfoEntity() const;
	virtual FInfoEntity& GetInfo(FInfoEntity& EntityInfo) const;
	static void AddEntityInDatabase(TSharedRef<FEntity> Entity);
#endif

	virtual void SpawnIdent(FDatabase& Database)
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}
	}

	virtual void ResetMarkersRecursively() const
	{
		ResetMarkers();
	}

protected:

	bool SetId(FDatabase& Database);
};
}

