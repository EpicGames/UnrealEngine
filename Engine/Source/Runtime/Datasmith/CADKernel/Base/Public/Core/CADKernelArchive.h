// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "HAL/FileManager.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace UE::CADKernel
{
class FEntity;
class FSession;
class FModel;

class FCADKernelArchive
{
public:
	TUniquePtr<FMemoryReader> MemoryReader;
	TUniquePtr<FMemoryArchive> MemoryArchive;
	TSharedPtr<FArchive> ArchivePtr;
	FArchive& Archive;
	FSession& Session;
	FModel* ArchiveModel = nullptr;

	FCADKernelArchive(FSession& InSession, FArchive& InArchive)
		: Archive(InArchive)
		, Session(InSession)
	{
	}

	/**
	 * Archive reader constructor
	 */
	FCADKernelArchive(FSession& InSession, const TArray<uint8>& InRawData)
		: MemoryArchive(MakeUnique<FMemoryReader>(InRawData, true))
		, Archive(*MemoryArchive)
		, Session(InSession)
	{
	}

	FCADKernelArchive(FSession& InSession, TArray<uint8>& InRawData)
		: MemoryArchive(MakeUnique<FMemoryWriter>(InRawData, true))
		, Archive(*MemoryArchive)
		, Session(InSession)
	{
	}

	FCADKernelArchive(FSession& InSession, FArchive* InArchive)
		: ArchivePtr(MakeShareable<FArchive>(InArchive))
		, Archive(*InArchive)
		, Session(InSession)
	{
	}

	static TSharedPtr<FCADKernelArchive> CreateArchiveWriter(FSession& InSession, const TCHAR* FileName)
	{
		FArchive* Ar = IFileManager::Get().CreateFileWriter(FileName);
		if (Ar == nullptr)
		{
			return TSharedPtr<FCADKernelArchive>();
		}
		return MakeShared<FCADKernelArchive>(InSession, Ar);
	}

	static TSharedPtr<FCADKernelArchive> CreateArchiveReader(FSession& InSession, const TCHAR* FileName)
	{
		FArchive* Ar = IFileManager::Get().CreateFileReader(FileName);
		if (!Ar)
		{
			return TSharedPtr<FCADKernelArchive>();
		}
		return MakeShared<FCADKernelArchive>(InSession, Ar);
	}

	static TSharedPtr<FCADKernelArchive> CreateArchiveWriter(FSession& InSession, TArray<uint8>& InRawData)
	{
		FArchive* Ar = new FMemoryWriter(InRawData, true);
		if (!Ar)
		{
			return TSharedPtr<FCADKernelArchive>();
		}
		return MakeShared<FCADKernelArchive>(InSession, Ar);
	}

	static TSharedPtr<FCADKernelArchive> CreateArchiveReader(FSession& InSession, const TArray<uint8>& InRawData)
	{
		FArchive* Ar = new FMemoryReader(InRawData, true);
		if (!Ar)
		{
			return TSharedPtr<FCADKernelArchive>();
		}
		return MakeShared<FCADKernelArchive>(InSession, Ar);
	}

	template<typename EntityType>
	void operator<<(EntityType& Entity)
	{
		Archive << Entity;
	}

	bool IsLoading() const
	{
		return Archive.IsLoading();
	}

	bool IsSaving() const
	{
		return Archive.IsSaving();
	}

	void Serialize(void* Value, int64 Length)
	{
		Archive.Serialize(Value, Length);
	}

	template<typename EntityType>
	void Serialize(TArray<EntityType>& EntityArray)
	{
		if (Archive.IsLoading())
		{
			int32 ElementCount = 0;
			Archive << ElementCount;
			EntityArray.SetNum(ElementCount);
		}
		else
		{
			int32 ElementCount = EntityArray.Num();
			Archive << ElementCount;
		}
		Archive.Serialize((void*)EntityArray.GetData(), EntityArray.Num() * sizeof(EntityType));
	}

	template<typename EntityType>
	void Serialize(EntityType& EntityArray)
	{
		Archive.Serialize((void*)&EntityArray, sizeof(EntityType));
	}

	void SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, FEntity** Entity);
	void SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, TWeakPtr<FEntity>& Entity);
	void SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, TSharedPtr<FEntity>& Entity);

	void AddEntityToSave(FIdent Id);
	void AddEntityFromArchive(TSharedPtr<FEntity>& Entity);

	template<typename EntityType>
	void AddEntityFromArchive(TSharedPtr<EntityType>& Entity)
	{
		AddEntityFromArchive((TSharedPtr<FEntity>&) Entity);
	}

	int64 TotalSize()
	{
		return Archive.TotalSize();
	}

	int64 Tell()
	{
		return Archive.Tell();
	}

	void Close()
	{
		Archive.Close();
	}
};
}

