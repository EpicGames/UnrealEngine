// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if 0
#include "Core/Types.h"

#ifdef CADKERNEL_DEV
#include <chrono>
#include <string>
#include <utility>
#endif

namespace UE::CADKernel
{
enum class ETimeUnit : uint8
{
	NanoSeconds = 0,
	MicroSeconds = 1,
	MilliSeconds,
	Seconds
};

#ifdef CADKERNEL_DEV
typedef std::chrono::high_resolution_clock::time_point FTimePoint;
typedef std::chrono::high_resolution_clock::duration FDuration;
#else
typedef uint64 FTimePoint;
typedef uint64 FDuration;
#endif

class CADKERNEL_API FChrono
{
	ETimeUnit DefaultUnit = ETimeUnit::NanoSeconds;
public:
	static const FTimePoint Now()
	{
#ifdef CADKERNEL_DEV
		return std::chrono::high_resolution_clock::now();
#else
		return 0;
#endif
	}

	static const FDuration Elapse(FTimePoint StartTime)
	{
#ifdef CADKERNEL_DEV
		return (std::chrono::high_resolution_clock::now() - StartTime);
#else
		return 0;
#endif
	}

	static const FDuration Init()
	{
#ifdef CADKERNEL_DEV
		return FDuration();
#else
		return 0;
#endif
	}

	template<typename Unit>
	static int64 ConvertInto(FDuration Duration)
	{
#ifdef CADKERNEL_DEV
		return std::chrono::duration_cast<Unit>(Duration).count();
#else
		return 0;
#endif
	}

	static void PrintClockElapse(EVerboseLevel Level, const TCHAR* Indent, const TCHAR* Process, FDuration Duration, ETimeUnit Unit = ETimeUnit::MicroSeconds);
};
}
#endif