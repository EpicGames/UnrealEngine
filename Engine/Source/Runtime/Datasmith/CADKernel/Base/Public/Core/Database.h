// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/Types.h"
#include "Containers/Queue.h"

namespace UE::CADKernel
{
class FCADKernelArchive;
class FModel;
class FTopologicalEntity;
class FTopologicalShapeEntity;

class CADKERNEL_API FDatabase
{
	friend class FCADKernelArchive;
	friend class FEntity;
	friend class FSession;
	template<typename, ESPMode> friend class SharedPointerInternals::TIntrusiveReferenceController;

protected:

	/**
	 * Root Geometry Entity node
	 */
	TSharedPtr<FModel> Model;

	/**
	 * Array (use as a Map<FIdent, TSharedPtr<FEntity>>) of all the entities create during the session
	 * Each Entity.Id is the index of the entity in the array
	 */
	TArray<TSharedPtr<FEntity>> DatabaseEntities;

	/**
	 * Recycling of the Id to avoid fragmented idents
	 */
	TArray<FIdent> AvailableIdents;

	/**
	 * Array (use as a Map<FIdent, TSharedPtr<FEntity>>) of all the entities of an archive
	 * Each Entity is at the index of AchiveId in the array
	 */
	TArray<TSharedPtr<FEntity>> ArchiveEntities;

	/**
	 * Array (use as a Map<FIdent, TArray<TSharedPtr<FEntity>*>>)
	 * Some Entity reference an other Entities (with a TSharedPtr).
	 * If at the deserialization of the Entity, the referenced entity is not yet created,
	 * the pointer of the TSharedPtr of the referenced entity is added in the array associated to the ArchiveId of the referenced entity.
	 *
	 * E.g.
	 * Edge
	 *   |- ArchiveId = 5
	 *   |- Vertex1 (ArchiveId = 6)
	 *
	 * IdToWaitingPointers[ArchiveId].Add(&Edge.Vertex)
	 */
	TArray<TArray<TSharedPtr<FEntity>*>> ArchiveIdToWaitingSharedPointers;
	TArray<TArray<TWeakPtr<FEntity>*>> ArchiveIdToWaitingWeakPointers;
	TArray<TArray<FEntity**>> ArchiveIdToWaitingPointers;

	/**
	 * Dedicated to SerializeSelection. Set of entity Ids needed to be saved
	 */
	TSet<FIdent> EntitiesToBeSerialized;

	/**
	 * Dedicated to SerializeSelection. Queue of entity Ids waiting to be saved
	 */
	TQueue<FIdent> NotYetSerialized;

	mutable bool bForceSpawning = false;
	bool bIsRecursiveSerialization = false;
	uint32 EntityCount;

public:

	FDatabase();

	FModel& GetModel() { return *GetModelAsShared(); }
	TSharedPtr<FModel> GetModelAsShared();

	/**
	 * Remove from database
	 */
	void RemoveEntity(FIdent idEntity);
	void RemoveEntity(FEntity& Entity);


	/**
	 * Recursive serialization of a selection. The selection and all the dependencies are serialized.
	 * All entities that need to be saved (added in EntitiesToBeSerialized set) for the completeness of the archive are added to the queue of entities to Serialized (NotYetSerialized).
	 */
	void SerializeSelection(FCADKernelArchive& Ar, const TArray<FIdent>& SelectionIds);

	/**
	 * All entities of the session are saved in the archive
	 */
	void Serialize(FCADKernelArchive& Ar);

	void Deserialize(FCADKernelArchive& Ar);

	/**
	 * 	Method to clean entity descriptions i.e. removing invalid references (like unsaved linked edges of an edge because the neighbor topological face is not saved)
	 */
	void CleanArchiveEntities();

	/**
	 * Delete all the entities of the session
	 */
	void Empty();

	/**
	 * All Entities are saved in the session map that allows to retrieve them according to their Id
	 * This process is automatically done during the entity creation (FEntity::MakeShared)
	 */
	void AddEntity(TSharedPtr<FEntity> Entity)
	{
		if (!Entity.IsValid())
		{
			return;
		}

		if (Entity->GetId() > 0)
		{
			return;
		}
		FIdent Id = CreateId();
		DatabaseEntities[Id] = Entity;
		Entity->Id = Id;
	}

	/**
	 * During the load of an FArchive:
	 * All Entities are saved in the session map (like Entity created in runtime) that allows to retrieve them according to their Id.
	 * Archive's entities are also saved in a temporary map for the load purpose (c.f. FEntity::Serialize)
	 * This process is automatically done during the entity creation (FEntity::MakeShared)
	 */
	void AddEntityFromArchive(TSharedPtr<FEntity>Entity)
	{
		FIdent ArchiveId = Entity->GetId();
		ensureCADKernel(ArchiveEntities.Num() > (int32)ArchiveId);
		// Add the entity to ArchiveEntities map.
		ArchiveEntities[ArchiveId] = Entity;

		// Check if the entity is not referenced by other entities if the archive(ArchiveIdToWaitingPointers).
		// If yes(IdToWaitingPointers[ArchiveId].Num() > 0), set all TSharedPtr with the new entity
		{
			TArray<TSharedPtr<FEntity>*>& WaitingList = ArchiveIdToWaitingSharedPointers[ArchiveId];
			if (WaitingList.Num() > 0)
			{
				for (TSharedPtr<FEntity>* SharedPtr : WaitingList)
				{
					*SharedPtr = Entity;
				}
				WaitingList.Empty();
			}
		}

		{
			TArray<TWeakPtr<FEntity>*>& WaitingList = ArchiveIdToWaitingWeakPointers[ArchiveId];
			if (WaitingList.Num() > 0)
			{
				for (TWeakPtr<FEntity>* WeakPtr : WaitingList)
				{
					*WeakPtr = Entity;
				}
				WaitingList.Empty();
			}
		}

		{
			TArray<FEntity**>& WaitingList = ArchiveIdToWaitingPointers[ArchiveId];
			if (WaitingList.Num() > 0)
			{
				for (FEntity** Ptr : WaitingList)
				{
					*Ptr = &*Entity;
				}
				WaitingList.Empty();
			}
		}

		// Reset Id to add it in the DatabaseEntities map.
		Entity->Id = 0;
		// Add the entity to DatabaseEntities map.
		AddEntity(Entity);
	}

	/**
	 * Find TSharedPtr<FEntity> associate to an Archive Id
	 * If the Entity is not yet created, add the entity in the waiting list
	 */
	void SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, TSharedPtr<FEntity>& Entity)
	{
		if (ArchiveId == 0)
		{
			Entity = TSharedPtr<FEntity>();
			return;
		}

		TSharedPtr<FEntity>& EntityPtr = ArchiveEntities[ArchiveId];
		if (EntityPtr.IsValid())
		{
			Entity = EntityPtr;
		}
		else
		{
			ArchiveIdToWaitingSharedPointers[ArchiveId].Add(&Entity);
		}
	}

	/**
	 * Find TWeakPtr<FEntity> associate to an Archive Id
	 * If the Entity is not yet created, add the entity in the waiting list
	 */
	void SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, TWeakPtr<FEntity>& Entity)
	{
		if (ArchiveId == 0)
		{
			Entity = TWeakPtr<FEntity>();
			return;
		}

		TSharedPtr<FEntity>& EntityPtr = ArchiveEntities[ArchiveId];
		if (EntityPtr.IsValid())
		{
			Entity = EntityPtr;
		}
		else
		{
			ArchiveIdToWaitingWeakPointers[ArchiveId].Add(&Entity);
		}
	}

	/**
	 * Find FEntity* associate to an Archive Id
	 * If the Entity is not yet created, add the entity in the waiting list
	 */
	void SetReferencedEntityOrAddToWaitingList(FIdent ArchiveId, FEntity** Entity)
	{
		if (ArchiveId == 0)
		{
			*Entity = nullptr;
			return;
		}

		TSharedPtr<FEntity>& EntityPtr = ArchiveEntities[ArchiveId];
		if (EntityPtr.IsValid())
		{
			*Entity = EntityPtr.Get();
		}
		else
		{
			ArchiveIdToWaitingPointers[ArchiveId].Add(Entity);
		}
	}

	uint32 SpawnEntityIdents(const TArray<TSharedPtr<FEntity>>& SelectedEntities, bool bForceSpawning = false);
	uint32 SpawnEntityIdents(const TArray<FEntity*>& SelectedEntities, bool bForceSpawning = false);
	uint32 SpawnEntityIdent(FEntity& SelectedEntity, bool bForceSpawning = false);

protected:
	/**
	 * Dedicated method for SerializeSelection
	 */
	void AddEntityToSave(FIdent EntityId)
	{
		if (bIsRecursiveSerialization && !EntitiesToBeSerialized.Find(EntityId))
		{
			EntitiesToBeSerialized.Add(EntityId);
			NotYetSerialized.Enqueue(EntityId);
		}
	}

private:
	FIdent CreateId();

	// =========================================================================================================================================================================================================
	// =========================================================================================================================================================================================================
	// =========================================================================================================================================================================================================
	//
	//
	//                                                                            NOT YET REVIEWED
	//
	//
	// =========================================================================================================================================================================================================
	// =========================================================================================================================================================================================================
	// =========================================================================================================================================================================================================

public:

	TSharedPtr<FEntity> GetEntity(FIdent id) const;
	void GetEntities(const TArray<FIdent>& ids, TArray<TSharedPtr<FEntity>>& Entities) const;
	void GetEntities(const TArray<FIdent>& ids, TArray<FEntity*>& Entities) const;
	void GetTopologicalEntities(const TArray<FIdent>& ids, TArray<FTopologicalEntity*>& Entities) const;
	void GetTopologicalShapeEntities(const TArray<FIdent>& ids, TArray<FTopologicalShapeEntity*>& Entities) const;

	void GetEntitiesOfType(EEntity Type, TArray<TSharedPtr<FEntity>>& OutEntities) const;
	void GetEntitiesOfTypes(const TSet<EEntity>& Types, TArray<TSharedPtr<FEntity>>& OutEntities) const;

	void ExpandSelection(TSharedPtr<FEntity> Entities, const TSet<EEntity>& Filter, TSet<TSharedPtr<FEntity>>& OutSelection) const;
	void ExpandSelection(const TArray<TSharedPtr<FEntity>>& Entities, const TSet<EEntity>& Filter, TSet<TSharedPtr<FEntity>>& OutSelection) const;
	void ExpandSelection(const TArray<TSharedPtr<FEntity>>& Entities, const TSet<EEntity>& Filter, TArray<TSharedPtr<FEntity>>& OutSelection) const;

	void TopologicalEntitiesSelection(const TArray<TSharedPtr<FEntity>>& Entities, TArray<TSharedPtr<FTopologicalEntity>>& OutTopoEntities, bool bFindSurfaces, bool bFindEdges, bool bFindVertices) const;
	void CadEntitiesSelection(const TArray<TSharedPtr<FEntity>>& Entities, TArray<TSharedPtr<FEntity>>& OutCadEntities) const;
	void MeshEntitiesSelection(const TArray<TSharedPtr<FEntity>>& Entities, TArray<TSharedPtr<FEntity>>& OutMeshEntities) const;

	TSharedPtr<FEntity> GetFirstEntityOfType(EEntity Type) const;
};

} // namespace UE::CADKernel

