// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"

#include "Math/MatrixH.h"

class FString;

namespace UE::CADKernel
{

class CADKERNEL_API FEntityGeom : public FEntity
{
	friend class FCoreTechBridge;

protected:
	FIdent CtKioId = 0;

public:

	FEntityGeom() = default;

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const
	{
		return TSharedPtr<FEntityGeom>();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FEntity::Serialize(Ar);
		Ar << CtKioId;
	}

	FIdent GetKioId() const
	{
		return CtKioId;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity& Info) const override;
#endif
};

}

