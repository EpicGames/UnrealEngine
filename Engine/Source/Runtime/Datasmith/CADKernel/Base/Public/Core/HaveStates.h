// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Core/CADKernelArchive.h"

enum class EHaveStates : uint16
{
	None              = 0x0000u,  // No flags.
					  
	IsApplyCriteria   = 0x0001u,  // used for FTopologicalFace, FEdge
	IsMeshed          = 0x0002u,  // used for FTopologicalFace, FEdge
	IsPreMeshed       = 0x0004u,  // used for FTopologicalFace, FEdge
					  
	ThinZone          = 0x0008u,  // used for FTopologicalFace, FEdge, FEdgeSegment  
					  
	Degenerated       = 0x0010u,  // used for FEdge, FGrid
				      
	IsDeleted         = 0x0020u,  // used for all class

	IsVirtuallyMeshed = 0x0040u,  // used for FEdge

	ThinPeak          = 0x0080u,  // used for FEdge 

	IsBackOriented    = 0x0100u,  // used for FTopologicalFace
	IsInner           = 0x0100u,  // used for FEdgeSegment, FShell, 

	FirstSideClosed   = 0x0200u,  // used for FThinZone2D
	SecondSideClosed  = 0x0400u,  // used for FThinZone2D

	HasMarker1        = 0x0800u,  // used for all class
	HasMarker2        = 0x1000u,  // used for all class
				      
	ToProcess         = 0x2000u,  // used for all class
	IsWaiting         = 0x4000u,  // used for all class
	IsProcessed       = 0x8000u,  // used for all class
				      
	AllMarkers        = 0xF800u,  // used for all class

	All = 0xFFFFu
};

ENUM_CLASS_FLAGS(EHaveStates);

namespace UE::CADKernel
{
class CADKERNEL_API FHaveStates
{
protected:
	mutable EHaveStates States;

public:

	FHaveStates()
		: States(EHaveStates::None)
	{};

	void Serialize(FCADKernelArchive& Ar)
	{
		Ar << States;
	}

	void ResetElementStatus()
	{
		States = EHaveStates::None;
	}

	bool HasMarker1() const
	{
		return ((States & EHaveStates::HasMarker1) == EHaveStates::HasMarker1);
	}

	bool HasMarker2() const
	{
		return ((States & EHaveStates::HasMarker2) == EHaveStates::HasMarker2);
	}

	//bool HasMarker3() const
	//{
	//	return ((States & EHaveStates::HasMarker3) == EHaveStates::HasMarker3);
	//}

	bool HasMarker1And2() const
	{
		constexpr EHaveStates HasMarker12 = EHaveStates::HasMarker1 | EHaveStates::HasMarker2;
		return ((States & HasMarker12) == HasMarker12);
	}

	bool HasMarker1Or2() const
	{
		constexpr EHaveStates HasMarker12 = EHaveStates::HasMarker1 | EHaveStates::HasMarker2;
		return ((States & HasMarker12) != EHaveStates::None);
	}

	void SetMarker1() const
	{
		States |= EHaveStates::HasMarker1;
	}

	void SetMarker2() const
	{
		States |= EHaveStates::HasMarker2;
	}

	void ResetMarker1() const
	{
		States &= ~EHaveStates::HasMarker1;
	}

	void ResetMarker2() const
	{
		States &= ~EHaveStates::HasMarker2;
	}

	void ResetMarkers() const
	{
		States &= ~EHaveStates::AllMarkers;
	}

	bool IsDeleted() const
	{
		return ((States & EHaveStates::IsDeleted) == EHaveStates::IsDeleted);
	}

	void SetDeletedMarker() const
	{
		States |= EHaveStates::IsDeleted;
	}

	void ResetDeleted() const
	{
		States &= ~EHaveStates::IsDeleted;
	}

	virtual bool IsDeletedOrDegenerated() const
	{
		constexpr EHaveStates DeletedOrDegenerated = EHaveStates::Degenerated | EHaveStates::IsDeleted;
		return ((States & DeletedOrDegenerated) != EHaveStates::None);
	}

	virtual bool IsDegenerated() const
	{
		return ((States & EHaveStates::Degenerated) == EHaveStates::Degenerated);
	}

	virtual void SetAsDegenerated() const
	{
		States |= EHaveStates::Degenerated;
	}

	virtual void ResetDegenerated() const
	{
		States &= ~EHaveStates::Degenerated;
	}

	void SetWaitingMarker() const
	{
		States |= EHaveStates::IsWaiting;
	}

	void ResetWaitingMarker() const
	{
		States &= ~EHaveStates::IsWaiting;
	}

	virtual bool IsWaiting() const
	{
		return ((States & EHaveStates::IsWaiting) == EHaveStates::IsWaiting);
	}

	void SetProcessedMarker() const
	{
		States |= EHaveStates::IsProcessed;
	}

	void ResetProcessedMarker() const
	{
		States &= ~EHaveStates::IsProcessed;
	}

	virtual bool IsProcessed() const
	{
		return ((States & EHaveStates::IsProcessed) == EHaveStates::IsProcessed);
	}

	virtual bool IsProcessedDeletedOrDegenerated() const
	{
		constexpr EHaveStates ProcessedDeletedOrDegenerated = EHaveStates::Degenerated | EHaveStates::IsDeleted | EHaveStates::IsProcessed;
		return ((States & ProcessedDeletedOrDegenerated) != EHaveStates::None);
	}


	void SetToProcessMarker() const
	{
		States |= EHaveStates::ToProcess;
	}

	void ResetToProcessMarker() const
	{
		States &= ~EHaveStates::ToProcess;
	}

	virtual bool IsToProcess() const
	{
		return ((States & EHaveStates::ToProcess) == EHaveStates::ToProcess);
	}

	virtual bool IsNotToProcess() const
	{
		return ((States & EHaveStates::ToProcess) != EHaveStates::ToProcess);
	}

	virtual bool IsNotToOrAlreadyProcess() const
	{
		return IsNotToProcess() || IsProcessed();
	}


};
}

