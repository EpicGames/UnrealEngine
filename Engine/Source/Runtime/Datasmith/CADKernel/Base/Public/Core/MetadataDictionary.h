// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/Types.h"
#include "Math/MathConst.h"

namespace UE::CADKernel
{

class FMetadataDictionary
{
private:
	FString Name;
	int32   HostId = 0;
	int32   PatchId = 0;
	uint32  ColorId = 0;
	uint32  MaterialId = 0;
	int32   LayerId = -1;

public:
	FMetadataDictionary() = default;

	void Serialize(FCADKernelArchive& Ar)
	{
		Ar.Archive << Name;
		Ar.Archive << HostId;
		Ar.Archive << PatchId;
		Ar.Archive << ColorId;
		Ar.Archive << MaterialId;
		Ar.Archive << LayerId;
	}

	void CompleteDictionary(const FMetadataDictionary& MetaData)
	{
		if (Name.IsEmpty())
		{
			Name = MetaData.Name;
		}
		if (HostId == 0)
		{
			HostId = MetaData.HostId;
		}
		if (PatchId == 0)
		{
			PatchId = MetaData.PatchId;
		}
		if (ColorId == 0)
		{
			ColorId = MetaData.ColorId;
		}
		if (MaterialId == 0)
		{
			MaterialId = MetaData.MaterialId;
		}
		if (LayerId == 0)
		{
			LayerId = MetaData.LayerId;
		}
	}

	void ExtractMetaData(TMap<FString, FString>& OutMetaData) const 
	{
		if (!Name.IsEmpty())
		{
			OutMetaData.Add(TEXT("Name"), Name);
		}
		if (HostId != 0)
		{
			OutMetaData.Add(TEXT("HostId"), FString::FromInt(HostId));
		}
		if (PatchId != 0)
		{
			OutMetaData.Add(TEXT("PatchId"), FString::FromInt(PatchId));
		}
		if (ColorId != 0)
		{
			OutMetaData.Add(TEXT("ColorName"), FString::Printf(TEXT("%u"), ColorId));
		}
		if (MaterialId != 0)
		{
			OutMetaData.Add(TEXT("MaterialName"), FString::Printf(TEXT("%u"), MaterialId));
		}
		if (LayerId != 0)
		{
			OutMetaData.Add(TEXT("LayerId"), FString::FromInt(LayerId));
		}
	}

	void SetHostId(const int32 InHostId)
	{
		HostId = InHostId;
	}

	const int32 GetHostId() const
	{
		return HostId;
	}

	void SetLayer(const int32 InLayerId)
	{
		LayerId = InLayerId;
	}

	int32 GetLayer() const
	{
		return LayerId;
	}

	void SetName(const FString& InName)
	{
		Name = InName;
	}

	bool HasName() const
	{
		return !Name.IsEmpty();
	}

	const TCHAR* GetName() const
	{
		return *Name;
	}

	void SetColorId(const uint32& InColorId)
	{
		ColorId = InColorId;
	}

	uint32 GetColorId() const
	{
		return ColorId;
	}

	void SetMaterialId(const uint32& InMaterialId)
	{
		MaterialId = InMaterialId;
	}

	uint32 GetMaterialId() const
	{
		return MaterialId;
	}

	void SetPatchId(int32 InPatchId)
	{
		PatchId = InPatchId;
	}

	int32 GetPatchId() const
	{
		return PatchId;
	}
};


}


