// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"

namespace UE::CADKernel
{
template <typename KeyType>
class CADKERNEL_API TOrientedEntity
{
public:
	TSharedPtr<KeyType> Entity;
	EOrientation Direction;

	TOrientedEntity(TSharedPtr<KeyType> InEntity, EOrientation InDirection)
		: Entity(InEntity)
		, Direction(InDirection)
	{
	}

	TOrientedEntity() = default;
	TOrientedEntity(TOrientedEntity&&) = default;
	TOrientedEntity(const TOrientedEntity&) = default;
	TOrientedEntity& operator=(TOrientedEntity&&) = default;
	TOrientedEntity& operator=(const TOrientedEntity&) = default;
};
}

