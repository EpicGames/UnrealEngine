// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/Database.h"
#include "Core/CADEntity.h"
#include "Core/Types.h"
#include "Math/Geometry.h"

class FArchive;

namespace UE::CADKernel
{
class FEntity;
class FModel;

class CADKERNEL_API FSession
{
	friend FEntity;
	friend FCADKernelArchive;
	template<typename, ESPMode> friend class SharedPointerInternals::TIntrusiveReferenceController;

protected:

	double GeometricTolerance;
	FDatabase Database;
	int32 LastHostId;

public:
#if defined(CADKERNEL_DEV) || defined(CADKERNEL_STDA)
	static FSession Session;
#endif

	FSession(double InGeometricTolerance)
		: GeometricTolerance(InGeometricTolerance)
	{
		IntersectionTool::SetTolerance(InGeometricTolerance);
	}

	FModel& GetModel() { return *GetModelAsShared(); }
	TSharedPtr<FModel> GetModelAsShared();

	void Serialize(FCADKernelArchive& Ar)
	{
		Ar << GeometricTolerance;
		IntersectionTool::SetTolerance(GeometricTolerance);
	}

	FDatabase& GetDatabase()
	{
		return Database;
	}

	/**
	 * Tolerance must not be modified as soon as a geometric entity has been build.
	 */
	void SetGeometricTolerance(double NewTolerance);

	double GetGeometricTolerance() const
	{
		return GeometricTolerance;
	}

	/**
	 * Save the database as a FAchive in a file
	 * Mandatory: all entity have to have a defined ID
	 * Use SpawnEntityIdent if needed
	 */
	bool SaveDatabase(const TCHAR* FilePath);

	/**
	 * Save a selection and all the dependencies as a FAchive in a file
	 * Mandatory: all entity have to have a defined ID
	 * Use SpawnEntityIdent if needed
	 */
	void SaveDatabase(const TCHAR* FileName, const TArray<TSharedPtr<FEntity>>& Entities);

	/**
	 * Save a selection and all the dependencies as a FAchive in a file
	 * Mandatory: all entity have to have a defined ID
	 * Use SpawnEntityIdent if needed
	 */
	void SaveDatabase(const TCHAR* FileName, const TArray<FEntity*>& Entities);

	/**
	 * Save a selection and all the dependencies as a FAchive in a file
	 * Mandatory: all entity have to have a defined ID
	 * Use SpawnEntityIdent if needed
	 */
	void SaveDatabase(const TCHAR* FileName, FEntity& Entity)
	{
		TArray<FEntity*> Entities;
		Entities.Emplace(&Entity);
		SaveDatabase(FileName, Entities);
	}

	/**
	 * Save a selection and all the dependencies as a FAchive in a file
	 * Mandatory: all entity have to have a defined ID
	 * Use SpawnEntityIdent if needed
	 */
	void SaveDatabase(const TCHAR* FileName, const TSharedPtr<FEntity> Entity)
	{
		TArray<TSharedPtr<FEntity>> Entities;
		Entities.Emplace(Entity);
		SaveDatabase(FileName, Entities);
	}

	/**
	 * Load and add a database in the current session database
	 * Entity ID is set for all loaded entities
	 */
	bool LoadDatabase(const TCHAR* FilePath);

	bool SaveDatabase(TArray<uint8>& Bytes);
	bool LoadDatabase(const TArray<uint8>& Bytes);

	/**
	 * Add a database defined by a RawData in the current session database
	 * Entity ID is set for all loaded entities
	 */
	void AddDatabase(const TArray<uint8>& InRawData);

	void Clear()
	{
		Database.Empty();
	}

	/**
	 * To be consistent,  all entity to save have to had an Id.
	 * This method browses all sub entities and set their Id if needed
	 * @param bForceSpawning If false, the process does not iterate through the children of entities with a defined ID
	 */
	uint32 SpawnEntityIdent(FEntity& SelectedEntity, bool bForceSpawning = false)
	{
		return Database.SpawnEntityIdent(SelectedEntity, bForceSpawning);
	}

	uint32 SpawnEntityIdents(const TArray<TSharedPtr<FEntity>>& SelectedEntities, bool bForceSpawning = false)
	{
		return Database.SpawnEntityIdents(SelectedEntities, bForceSpawning);
	}

	uint32 SpawnEntityIdents(const TArray<FEntity*>& SelectedEntities, bool bForceSpawning = false)
	{
		return Database.SpawnEntityIdents(SelectedEntities, bForceSpawning);
	}

	int32 GetLastHostId() const
	{
		return LastHostId;
	}

	int32 NewHostId()
	{
		return ++LastHostId;
	}

	/**
	 * For stitching purpose, stitching can generate new body needing a host id.
	 * To avoid duplicate, the first generated host id can be set.
	 */
	void SetFirstNewHostId(int32 StartHostId)
	{
		LastHostId = StartHostId;
	}
};

} // namespace UE::CADKernel

