// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef DEV_CADKERNEL_LIB
#include "Utils/UnrealEnvironment.h"
#else
#include "CoreMinimal.h"
#endif

#define SMALL_NUMBER_SQUARE (1.e-16)
#define DOUBLE_SMALL_NUMBER_SQUARE (1.e-16)

#define KINDA_SMALL_NUMBER_SQUARE (1.e-8)
#define DOUBLE_KINDA_SMALL_NUMBER_SQUARE (1.e-8)

#define HUGE_VALUE (1e8)
#define DOUBLE_HUGE_VALUE (1e8)

#define HUGE_VALUE_SQUARE (1e16)
#define DOUBLE_HUGE_VALUE_SQUARE (1e16)

constexpr double AThird = 1. / 3.;
constexpr double AQuarter = 0.25;
constexpr double ASixth = 1. / 6.;
constexpr double AEighth  = 0.125;

typedef uint32 FIdent;

namespace UE::CADKernel
{

struct FPairOfIndex
{
	int32 Value0;
	int32 Value1;

	FPairOfIndex(int32 InValue1, int32 InValue2)
		: Value0(InValue1)
		, Value1(InValue2)
	{
	}

	FPairOfIndex(int32 InValue1)
		: Value0(InValue1)
		, Value1(-1)
	{
	}

	int32& operator[](const int32 Index)
	{
		return Index == 0 ? Value0 : Value1;
	}

	const int32& operator[](const int32 Index) const
	{
		return Index == 0 ? Value0 : Value1;
	}

	void Add(const int32 Value)
	{
		if(Value >= 0)
		{
			if (Value0 < 0)
			{
				Value0 = Value;
			}
			else if (Value0 != Value && Value1 < 0)
			{
				Value1 = Value;
			}
		}
	}

	void Add(const FPairOfIndex Values)
	{
		Add(Values.Value0);
		Add(Values.Value1);
	}

	static const FPairOfIndex Undefined;
};

namespace Ident
{
static const FIdent Undefined = (FIdent)-1;
}

enum class EValue : uint8
{
	Entity,
	OrientedEntity,
	Point,
	Matrix,
	Integer,
	Double,
	String,
	Boolean,
	Tuple,
	List,
	Array
};

enum EVerboseLevel : uint8
{
	NoVerbose = 0,
	Spy,
	Log,
	Debug
};
} // namespace UE::CADKernel

#if CADKERNEL_DO_ENSURE
#define ensureCADKernel(InExpression) ensure(InExpression)
#else
#define ensureCADKernel(InExpression) (LIKELY(!!(InExpression)))
#endif
