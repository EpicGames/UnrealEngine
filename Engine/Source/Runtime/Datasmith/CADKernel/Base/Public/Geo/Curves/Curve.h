// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/EntityGeom.h"
#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Math/Boundary.h"
#include "UI/Message.h"
#include "Utils/Cache.h"

namespace UE::CADKernel
{
struct FCurvePoint2D;
struct FCurvePoint;
struct FNurbsCurveData;

class CADKERNEL_API FCurve : public FEntityGeom
{
	friend class FEntity;

protected:

	mutable TCache<double> GlobalLength;
	FLinearBoundary Boundary;

	int8 Dimension;

	FCurve(int8 InDimension = 3)
		: FEntityGeom()
		, Boundary()
		, Dimension(InDimension)
	{
	}

	FCurve(const FLinearBoundary& InBounds, int8 InDimension = 3)
		: FEntityGeom()
		, Boundary(InBounds)
		, Dimension(InDimension)
	{
	}

public:

	static TSharedPtr<FCurve> MakeNurbsCurve(FNurbsCurveData& InNurbsData);
	static TSharedPtr<FCurve> MakeBezierCurve(const TArray<FVector>& InPoles);
	static TSharedPtr<FCurve> MakeSplineCurve(const TArray<FVector>& InPoles);
	static TSharedPtr<FCurve> MakeSplineCurve(const TArray<FVector>& InPoles, const TArray<FVector>& Tangents);
	static TSharedPtr<FCurve> MakeSplineCurve(const TArray<FVector>& InPoles, const TArray<FVector>& ArriveTangents, const TArray<FVector>& LeaveTangents);

	// TODO
	//static TSharedPtr<FCurve> MakeConeCurve(const double InToleranceGeometric, const FMatrixH& InMatrix, double InStartRadius, double InConeAngle, const FSurfacicBoundary& InBoundary);
	//static TSharedPtr<FCurve> MakeCylinderCurve(const double InToleranceGeometric, const FMatrixH& InMatrix, const double InRadius, const FSurfacicBoundary& InBoundary);
	//static TSharedPtr<FCurve> MakePlaneCurve(const double InToleranceGeometric, const FMatrixH& InMatrix, const FSurfacicBoundary& InBoundary);
	//static TSharedPtr<FCurve> MakeSphericalCurve(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, const FSurfacicBoundary& InBoundary);
	//static TSharedPtr<FCurve> MakeTorusCurve(const double InToleranceGeometric, const FMatrixH& InMatrix, double InMajorRadius, double InMinorRadius, const FSurfacicBoundary& InBoundary);

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		// Curve's type is serialize because it is used to instantiate the correct entity on deserialization (@see Deserialize(FCADKernelArchive& Archive)) 
		if (Ar.IsSaving())
		{
			ECurve CurveType = GetCurveType();
			Ar << CurveType;
		}
		FEntityGeom::Serialize(Ar);
		Ar << Dimension;
		Ar << Boundary;
	}

	/**
	 * Specific method for curve family to instantiate the correct derived class of FCurve
	 */
	static TSharedPtr<FEntity> Deserialize(FCADKernelArchive& Archive);

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	int32 GetDimension() const
	{
		return Dimension;
	}

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Curve;
	}

	virtual ECurve GetCurveType() const = 0;

	double GetUMin() const
	{
		return Boundary.Min;
	};

	double GetUMax() const
	{
		return Boundary.Max;
	};

	const FLinearBoundary& GetBoundary() const
	{
		return Boundary;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override = 0;
	virtual void Offset(const FVector& OffsetDirection) = 0;

	virtual double GetLength(double Tolerance) const;

	/**
	 * Evaluate exact 3D point of the curve at the input Coordinate
	 * The function can only be used with 3D curve (Dimension == 3)
	 */
	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const
	{
		ensureCADKernel(false);
	}

	/**
	 * Evaluate exact 3D point of the curve at the input Coordinate
	 * The function can only be used with 3D curve (Dimension == 3)
	 */
	virtual FVector EvaluatePoint(double Coordinate) const
	{
		FCurvePoint OutPoint;
		EvaluatePoint(Coordinate, OutPoint);
		return OutPoint.Point;
	}

	/**
	 * Evaluate exact 2D point of the curve at the input Coordinate
	 * The function can only be used with 2D curve (Dimension == 2)
	 */
	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const
	{
		ensureCADKernel(false);
	}

	/**
	 * Evaluate exact 3D point of the curve at the input Coordinate
	 * The function can only be used with 3D curve (Dimension == 3)
	 */
	virtual FVector2d Evaluate2DPoint(double Coordinate) const
	{
		FCurvePoint2D OutPoint;
		Evaluate2DPoint(Coordinate, OutPoint);
		return OutPoint.Point;
	}

	/**
	 * Evaluate exact 2D point of the curve at the input Coordinate
	 * The function can only be used with 2D curve (Dimension == 2)
	 */
	virtual void Evaluate2DPoint(double Coordinate, FVector2d& OutPoint) const
	{
		ensureCADKernel(Dimension == 2);
		FCurvePoint2D CurvePoint;
		Evaluate2DPoint(Coordinate, CurvePoint);
		OutPoint = CurvePoint.Point;
	}

	/**
	 * Evaluate exact 3D points of the curve at the input Coordinates
	 * The function can only be used with 3D curve (Dimension == 3)
	 */
	virtual void EvaluatePoints(const TArray<double>& Coordinates, TArray<FCurvePoint>& OutPoints, int32 DerivativeOrder = 0) const;

	/**
	 * Evaluate exact 3D points of the curve at the input Coordinates
	 * The function can only be used with 3D curve (Dimension == 3)
	 */
	virtual void EvaluatePoints(const TArray<double>& Coordinates, TArray<FVector>& OutPoints) const;

	/**
	 * Evaluate exact 2D points of the curve at the input Coordinates
	 * The function can only be used with 2D curve (Dimension == 2)
	 */
	virtual void Evaluate2DPoints(const TArray<double>& Coordinates, TArray<FVector2d>& OutPoints) const;

	/**
	 * Evaluate exact 2D points of the curve at the input Coordinates
	 * The function can only be used with 2D curve (Dimension == 2)
	 */
	virtual void Evaluate2DPoints(const TArray<double>& Coordinates, TArray<FCurvePoint2D>& OutPoints, int32 DerivativeOrder = 0) const;

	void FindNotDerivableCoordinates(int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const
	{
		return FindNotDerivableCoordinates(Boundary, DerivativeOrder, OutNotDerivableCoordinates);
	}
	virtual void FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const;


	/**
	 * Generate a pre-sampling of the curve saved in OutCoordinates.
	 * This sampling is light enough to allow a fast computation, precise enough to compute accurately meshing criteria
	 */
	void Presample(TArray<double>& OutSampling, double Tolerance) const
	{
		Presample(Boundary, Tolerance, OutSampling);
	}

	virtual void Presample(const FLinearBoundary& InBoundary, double Tolerance, TArray<double>& OutSampling) const;

	/**
	 * Make a new curve based on the new bounds.
	 * If the new bound is nearly equal to the initial bound, no curve is made (return TSharedPtr<FCurve>())
	 */
	virtual TSharedPtr<FCurve> MakeBoundedCurve(const FLinearBoundary& InBoundary);

	/**
	 * Rebound the curve, if not possible make a new curve based on the new bounds.
	 */
	virtual TSharedPtr<FCurve> Rebound(const FLinearBoundary& InBoundary);

	/**
	 * Linear deformation of the curve along the axis [Start point, End point] so that the nearest extremity is at the desired position and the other is not modified
	 */
	virtual void ExtendTo(const FVector& DesiredPosition)
	{
		ensureCADKernel(Dimension == 3);
		ensureCADKernel(false);
	}

	/**
	 * Linear deformation of the curve along the axis [Start point, End point] so that the nearest extremity is at the desired position and the other is not modified
	 */
	virtual void ExtendTo(const FVector2d& DesiredPosition)
	{
		ensureCADKernel(Dimension == 2);
		FVector Point(DesiredPosition, 0.);
		ExtendTo(Point);
	}

protected:

	virtual double ComputeLength(const FLinearBoundary& InBoundary, double Tolerance) const;
	virtual double ComputeLength2D(const FLinearBoundary& InBoundary, double Tolerance) const;
};
} // namespace UE::CADKernel

