// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Geo/Curves/RestrictionCurve.h"

namespace UE::CADKernel
{
	class FCurve;
	class FRestrictionCurve;

	namespace CurveUtilities
	{
		int32 CADKERNEL_API GetPoleCount(const UE::CADKernel::FCurve& Curve);
		int32 CADKERNEL_API GetPoleCount(const UE::CADKernel::FRestrictionCurve& Curve);

		TArray<FVector> CADKERNEL_API GetPoles(const UE::CADKernel::FCurve& Curve);
		TArray<FVector2d> CADKERNEL_API GetPoles(const UE::CADKernel::FRestrictionCurve& Curve);
		TArray<FVector2d> CADKERNEL_API Get2DPolyline(const UE::CADKernel::FRestrictionCurve& Curve);
		TArray<FVector> CADKERNEL_API Get3DPolyline(const UE::CADKernel::FRestrictionCurve& Curve);

		int32 CADKERNEL_API GetDegree(const UE::CADKernel::FCurve& Curve);
		int32 CADKERNEL_API GetDegree(const UE::CADKernel::FRestrictionCurve& Curve);
	}
}