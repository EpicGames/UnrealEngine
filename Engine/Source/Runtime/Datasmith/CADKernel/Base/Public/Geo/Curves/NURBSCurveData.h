// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
struct FNurbsCurveData
{
	bool bIsRational;
	int32 Dimension = 0;

	int32 Degree = 0;
	TArray<double> NodalVector;

	TArray<double> Weights;
	TArray<FVector> Poles;
};
} // namespace UE::CADKernel

