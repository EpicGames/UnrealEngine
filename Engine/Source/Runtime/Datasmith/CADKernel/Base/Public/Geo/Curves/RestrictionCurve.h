// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/Types.h"
#include "Geo/Curves/SurfacicCurve.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Geo/Sampling/SurfacicPolyline.h"
#include "Geo/Surfaces/Surface.h"
#include "Math/Boundary.h"
#include "Math/MatrixH.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
class FCurve;
class FEntityGeom;

/**
 * A restriction curve is the curve carrying an edge
 *
 * It's defined by
 * - A surfacic curve defined by a 2D curve (@see Curve2D) and the carrier surface (@see CarrierSurface) of the TopologicalFace containing the edge:
 * - A linear approximation of the surfacic curve (@see Polyline) respecting the System Geometrical Tolerance (@see FKernelParameters::GeometricalTolerance)
 *   The linear approximation is:
 *      - an array of increasing coordinates (@see Polyline::Coordinates)
 *      - an array of 2D Points relating to coordinates: points of curve in the parametric space of the carrier surface (@see Polyline::Points2D)
 *      - an array of 3D Points relating to coordinates: 3D points of curve (@see Polyline::Points3D)
 *      - an array of Normal relating to coordinates: Surface's normal (@see Polyline::Normal)
 */
class CADKERNEL_API FRestrictionCurve : public FSurfacicCurve
{
	friend class FEntity;
	friend class FTopologicalEdge;

protected:

	FSurfacicPolyline Polyline;
	double MinLinearTolerance;

	FRestrictionCurve(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> InCurve2D)
		: FSurfacicCurve(InCurve2D, InCarrierSurface)
		, Polyline(InCarrierSurface, InCurve2D)
	{
		MinLinearTolerance = Boundary.ComputeMinimalTolerance();
	}

	FRestrictionCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FSurfacicCurve::Serialize(Ar);
		Polyline.Serialize(Ar);
		if (Ar.IsLoading())
		{
			MinLinearTolerance = Boundary.ComputeMinimalTolerance();
		}
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Restriction;
	}

	const FSurfacicPolyline& GetPolyline() const
	{
		return Polyline;
	}

	const TSharedRef<FCurve> Get2DCurve() const
	{
		return Curve2D.ToSharedRef();
	}

	double GetMinLinearTolerance() const
	{
		return MinLinearTolerance;
	}

	TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override
	{
		ensureCADKernel(false);
		return TSharedPtr<FEntityGeom>();
	}

	/**
	 * Fast computation of the point in the parametric space of the carrier surface.
	 * This approximation is based on FSurfacicPolyline::Polyline
	 */
	FVector2d Approximate2DPoint(double InCoordinate) const
	{
		return Polyline.Approximate2DPoint(InCoordinate);
	}

	/**
	 * Fast computation of the point in the parametric space of the carrier surface.
	 * This approximation is based on FSurfacicPolyline::Polyline
	 */
	FVector Approximate3DPoint(double InCoordinate) const
	{
		return Polyline.Approximate3DPoint(InCoordinate);
	}

	FVector GetTangentAt(double InCoordinate) const
	{
		return Polyline.GetTangentAt(InCoordinate);
	}

	FVector2d GetTangent2DAt(double InCoordinate) const
	{
		return Polyline.GetTangent2DAt(InCoordinate);
	}

	/**
	 * Fast computation of the point in the parametric space of the carrier surface.
	 * This approximation is based on FSurfacicPolyline::Polyline
	 */
	void Approximate2DPoints(const TArray<double>& InCoordinates, TArray<FVector2d>& OutPoints) const
	{
		return Polyline.Approximate2DPoints(InCoordinates, OutPoints);
	}

	/**
	 * Fast computation of the 3D point of the curve.
	 * This approximation is based on FSurfacicPolyline::Polyline
	 */
	void Approximate3DPoints(const TArray<double>& InCoordinates, TArray<FVector>& OutPoints) const
	{
		return Polyline.Approximate3DPoints(InCoordinates, OutPoints);
	}

	/**
	 * Approximation of surfacic polyline (points 2d, 3d, normals, tangents) defined by its coordinates compute with carrier surface polyline
	 */
	void ApproximatePolyline(FSurfacicPolyline& OutPolyline) const
	{
		Polyline.ApproximatePolyline(OutPolyline);
	}

	template<class PointType>
	double GetCoordinateOfProjectedPoint(const FLinearBoundary& InBoundary, const PointType& PointOnEdge, PointType& ProjectedPoint) const
	{
		return Polyline.GetCoordinateOfProjectedPoint(InBoundary, PointOnEdge, ProjectedPoint);
	}

	template<class PointType>
	void ProjectPoints(const FLinearBoundary& InBoundary, const TArray<PointType>& InPointsToProject, TArray<double>& ProjectedPointCoordinates, TArray<PointType>& ProjectedPoints) const
	{
		Polyline.ProjectPoints(InBoundary, InPointsToProject, ProjectedPointCoordinates, ProjectedPoints);
	}

	/**
	 * Project a set of points of a twin curve on the 3D polyline and return the coordinate of the projected point
	 * @param ToleranceOfProjection: Max error between the both curve to stop the search of projection of a point 
	 */
	void ProjectTwinCurvePoints(const FLinearBoundary& InBoundary, const TArray<FVector>& InPointsToProject, bool bSameOrientation, TArray<double>& OutProjectedPointCoords, double ToleranceOfProjection) const
	{
		Polyline.ProjectCoincidentalPolyline(InBoundary, InPointsToProject, bSameOrientation, OutProjectedPointCoords, ToleranceOfProjection);
	}

	void ComputeIntersectionsWithIsos(const FLinearBoundary& InBoundary, const TArray<double>& InIsoCoordinates, const EIso InTypeIso, const FSurfacicTolerance& ToleranceIso, TArray<double>& OutIntersection) const
	{
		Polyline.ComputeIntersectionsWithIsos(InBoundary, InIsoCoordinates, InTypeIso, ToleranceIso, OutIntersection);
	}

	/**
	 * A check is done to verify that:
	 * - the curve is degenerated in the parametric space of the carrier surface i.e. the 2D length of the curve is not nearly equal to zero
	 * - the curve is degenerated in 3D i.e. the 3D length of the curve is not nearly equal to zero
	 *
	 * A curve can be degenerated in 3D and not in 2D in the case of locally degenerated carrier surface.
	 */
	void CheckIfDegenerated(const FLinearBoundary& InBoundary, bool& bDegeneration2D, bool& bDegeneration3D, double& Length3D) const
	{
		if (Polyline.Coordinates.Num() <  2)
		{
			bDegeneration2D = true;
			bDegeneration3D = true;
			Length3D = 0.;
			return;
		}
		double Tolerance = GetCarrierSurface()->Get3DTolerance();
		const FSurfacicTolerance& Tolerances2D = GetCarrierSurface()->GetIsoTolerances();
		Polyline.CheckIfDegenerated(Tolerance, Tolerances2D, InBoundary, bDegeneration2D, bDegeneration3D, Length3D);
	}

	void GetExtremities(const FLinearBoundary& InBoundary, FSurfacicCurveExtremities& Extremities) const
	{
		double Tolerance = GetCarrierSurface()->Get3DTolerance();
		const FSurfacicTolerance& Tolerances2D = GetCarrierSurface()->GetIsoTolerances();
		Polyline.GetExtremities(InBoundary, Tolerance, Tolerances2D, Extremities);
	}

	double GetToleranceAt(const double InCoordinate)
	{
		FDichotomyFinder Finder(Polyline.GetCoordinates());
		const int32 Index = Finder.Find(InCoordinate);
		const double Tolerance = GetCarrierSurface()->Get3DTolerance();

		return Polyline.ComputeLinearToleranceAt(Tolerance, MinLinearTolerance, Index);
	}

	FVector2d GetExtremityTolerances(const FLinearBoundary& InBoundary)
	{
		FDichotomyFinder Finder(Polyline.GetCoordinates());
		const int32 StartIndex = Finder.Find(InBoundary.Min);
		const int32 EndIndex = Finder.Find(InBoundary.Max);

		const double Tolerance = GetCarrierSurface()->Get3DTolerance();

		FVector2d ExtremityTolerance;
		ExtremityTolerance[0] = Polyline.ComputeLinearToleranceAt(Tolerance, MinLinearTolerance, StartIndex);
		ExtremityTolerance[1] = Polyline.ComputeLinearToleranceAt(Tolerance, MinLinearTolerance, EndIndex);
		return ExtremityTolerance;
	}

	/**
	 * @return the size of the polyline i.e. the count of points.
	 */
	int32 GetPolylineSize()
	{
		return Polyline.Size();
	}

	/**
	 * Get the sub polyline bounded by the input InBoundary in the orientation of the input InOrientation and append it to the output OutPoints
	 */
	template<class PointType>
	void GetDiscretizationPoints(const FLinearBoundary& InBoundary, EOrientation Orientation, TArray<PointType>& OutPoints) const
	{
		Polyline.GetSubPolyline(InBoundary, Orientation, OutPoints);
	}

	/**
	 * Get the sub polyline bounded by the input InBoundary in the orientation of the input InOrientation and append it to the output OutPoints
	 */
	template<class PointType>
	void GetDiscretizationPoints(const FLinearBoundary& InBoundary, TArray<double>& OutCoordinates, TArray<PointType>& OutPoints) const
	{
		Polyline.GetSubPolyline(InBoundary, OutCoordinates, OutPoints);
	}

	/**
	 * Samples the sub curve limited by the boundary respecting the input Desired segment length
	 */
	void Sample(const FLinearBoundary& InBoundary, const double DesiredSegmentLength, TArray<double>& OutCoordinates) const
	{
		Polyline.Sample(InBoundary, DesiredSegmentLength, OutCoordinates);
	}

	double ApproximateLength(const FLinearBoundary& InBoundary) const
	{
		return Polyline.GetLength(InBoundary);
	}

	void ExtendTo(const FVector2d& Point) override;

	bool IsIso(EIso Iso, double ErrorTolerance = DOUBLE_SMALL_NUMBER) const
	{
		return Polyline.IsIso(Iso, ErrorTolerance);
	}

	void Offset2D(const FVector2d& OffsetDirection);

	/**
	 * must not be call
	 */
	virtual void Offset(const FVector& OffsetDirection) override
	{
		ensureCADKernel(false);
	}

};

} // namespace UE::CADKernel

