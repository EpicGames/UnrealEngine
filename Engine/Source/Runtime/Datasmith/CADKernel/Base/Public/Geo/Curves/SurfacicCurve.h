// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h"
#include "Core/Types.h"
#include "Geo/Curves/Curve.h"
#include "Geo/GeoEnum.h"
#include "Geo/Surfaces/Surface.h"
#include "Math/MatrixH.h"

namespace UE::CADKernel
{
class FCADKernelArchive;
class FDatabase;
class FEntityGeom;
class FSurfacicPolyline;
struct FCurvePoint2D;
struct FCurvePoint;
struct FLinearBoundary;

class CADKERNEL_API FSurfacicCurve : public FCurve
{
	friend class FEntity;

protected:

	TSharedPtr<FCurve> Curve2D;
	TSharedPtr<FSurface> CarrierSurface;

	FSurfacicCurve(TSharedRef<FCurve> InCurve2D, TSharedRef<FSurface> InSurface)
		: FCurve(InCurve2D->GetBoundary())
		, Curve2D(InCurve2D)
		, CarrierSurface(InSurface)
	{
		ensureCADKernel(Curve2D.IsValid());
	}

	FSurfacicCurve() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FCurve::Serialize(Ar);
		SerializeIdent(Ar, Curve2D);
		SerializeIdent(Ar, CarrierSurface);
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		Curve2D->SpawnIdent(Database);
		CarrierSurface->SpawnIdent(Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		Curve2D->ResetMarkersRecursively();
		CarrierSurface->ResetMarkersRecursively();
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual ECurve GetCurveType() const override
	{
		return ECurve::Surfacic;
	}

	const TSharedPtr<FCurve>& Get2DCurve() const
	{
		return Curve2D;
	}

	TSharedPtr<FCurve>& Get2DCurve()
	{
		return Curve2D;
	}

	const TSharedRef<FSurface> GetCarrierSurface() const
	{
		return CarrierSurface.ToSharedRef();
	}

	void Set2DCurve(TSharedPtr<FCurve>& NewCurve2D)
	{
		ensureCADKernel(NewCurve2D.IsValid());
		Curve2D = NewCurve2D;
	}

	const TSharedPtr<FSurface>& GetSurface() const
	{
		return CarrierSurface;
	}

	TSharedPtr<FSurface>& GetSurface()
	{
		return CarrierSurface;
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const override;

	/**
	 * Must not be call
	 */
	virtual void Offset(const FVector& OffsetDirection) override
	{
		ensureCADKernel(false);
	}

	virtual void EvaluatePoint(double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder = 0) const override;

	virtual void Evaluate2DPoint(double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder = 0) const override
	{
		Curve2D->Evaluate2DPoint(Coordinate, OutPoint, DerivativeOrder);
	}

	virtual void Evaluate2DPoint(double Coordinate, FVector2d& OutPoint) const override
	{
		Curve2D->Evaluate2DPoint(Coordinate, OutPoint);
	}

	virtual void EvaluatePoints(const TArray<double>& Coordinates, TArray<FCurvePoint>& OutPoints, int32 DerivativeOrder = 0) const override;

	void EvaluateSurfacicPolyline(FSurfacicPolyline& OutPolyline) const;

	virtual void Evaluate2DPoints(const TArray<double>& Coordinates, TArray<FVector2d>& OutPoints) const override
	{
		Curve2D->Evaluate2DPoints(Coordinates, OutPoints);
	}

	virtual void Evaluate2DPoints(const TArray<double>& Coordinates, TArray<FCurvePoint2D>& OutPoints, int32 DerivativeOrder = 0) const override
	{
		Curve2D->Evaluate2DPoints(Coordinates, OutPoints, DerivativeOrder);
	}

	virtual void FindNotDerivableCoordinates(const FLinearBoundary& InBoundary, int32 DerivativeOrder, TArray<double>& OutNotDerivableCoordinates) const override;
private:
	void EvaluateSurfacicPolylineWithNormalAndTangent(FSurfacicPolyline& OutPolyline) const;
};

} // namespace UE::CADKernel

