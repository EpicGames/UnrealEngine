// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

namespace UE::CADKernel
{

enum class ECurve : uint8
{
	Bezier = 0,
	BoundedCurve,
	Composite,
	Ellipse,
	Hyperbola,
	Nurbs,
	Offset,
	Parabola,
	Polyline3D,
	Polyline2D,
	Restriction,
	Segment,
	Surfacic,
	SurfacicPolyline,
	Spline,
	None,
};

#ifdef CADKERNEL_DEV
extern const TCHAR* CurvesTypesNames[];
#endif

enum class ESurface : uint8
{
	Bezier = 0,
	Blend01,
	Blend02,
	Blend03,
	Composite,
	Cone,
	Coons,
	Cylinder,
	Nurbs,
	Offset,
	Plane,
	Revolution,
	Ruled,
	Sphere,
	TabulatedCylinder,
	Torus,
	None,
};

#ifdef CADKERNEL_DEV
extern const TCHAR* SurfacesTypesNames[];
#endif

enum class ERupture : uint8
{
	Continuity = 0, // Positional rupture i.e. points of the curve that are not G0 
	Tangency,       // Tangency rupture i.e. points of the curve that are not G1 (or G0) 
	Curvature,      // Curvature rupture i.e. points of the curve that are not G2 (or G1, G0)
};

enum EIso : uint8
{
	IsoU = 0,
	IsoV = 1,
	UndefinedIso
};

#ifdef CADKERNEL_DEV
extern const TCHAR* IsoNames[];
#endif

constexpr EIso Other(const EIso Iso)
{
	return Iso == EIso::IsoU ? EIso::IsoV : EIso::IsoU;
}

enum EOrientation : uint8
{
	Back = 0,
	Front = 1
};

enum ESituation : uint8
{
	Undefined = 0,
	Inside,
	Outside
};

#ifdef CADKERNEL_DEV
extern const TCHAR* OrientationNames[];
#endif

enum ELimit : uint8
{
	Start = 0,
	End = 1
};

inline EOrientation SameOrientation(bool bIsSameOrientation)
{
	if (bIsSameOrientation)
	{
		return EOrientation::Front;
	}
	else
	{
		return EOrientation::Back;
	}
}

inline EOrientation GetReverseOrientation(EOrientation Orientation)
{
	if (Orientation == EOrientation::Front)
	{
		return EOrientation::Back;
	}
	else
	{
		return EOrientation::Front;
	}
}

inline void SwapOrientation(EOrientation& Orientation)
{
	if (Orientation == EOrientation::Front)
	{
		Orientation = EOrientation::Back;
	}
	else
	{
		Orientation = EOrientation::Front;
	}
}

} // ns
