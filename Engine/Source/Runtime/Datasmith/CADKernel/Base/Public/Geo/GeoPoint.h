// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
struct FCurvePoint2D
{
	int32 DerivativeOrder = -1;
	FVector2d Point = FVector2d::ZeroVector;
	FVector2d Gradient = FVector2d::ZeroVector;
	FVector2d Laplacian = FVector2d::ZeroVector;
};

struct FSurfacicPoint
{
	int32 DerivativeOrder = -1;
	FVector Point = FVector::ZeroVector;
	FVector GradientU = FVector::ZeroVector;
	FVector GradientV = FVector::ZeroVector;
	FVector LaplacianU = FVector::ZeroVector;
	FVector LaplacianV = FVector::ZeroVector;
	FVector LaplacianUV = FVector::ZeroVector;
};

struct FSurfacicCurvePoint
{
	bool bWithNormals;
	bool bWithTangent;

	FVector2d Point2D = FVector2d::ZeroVector;
	FVector Point = FVector::ZeroVector;
	FVector Normal = FVector::ZeroVector;
	FVector Tangent = FVector::ZeroVector;
};

struct FSurfacicCurvePointWithTolerance
{
	FVector2d Point2D = FVector2d::ZeroVector;
	FVector Point = FVector::ZeroVector;
	FSurfacicTolerance Tolerance = FVectorUtil::FarawayPoint2D;
};

typedef FSurfacicCurvePointWithTolerance FSurfacicCurveExtremities[2];

struct FCurvePoint
{
	int32 DerivativeOrder = -1;
	FVector Point = FVector::ZeroVector;
	FVector Gradient = FVector::ZeroVector;
	FVector Laplacian = FVector::ZeroVector;

	FCurvePoint() = default;

	FCurvePoint(FVector InPoint)
		: Point(InPoint)
	{
		Gradient = FVector::ZeroVector;
		Laplacian = FVector::ZeroVector;
	}

	void Init()
	{
		Point = FVector::ZeroVector;
		Gradient = FVector::ZeroVector;
		Laplacian = FVector::ZeroVector;
	}

	/**
	 * Compute the 3D surface curve point property (3D Coordinate, Gradient, Laplacian) according to
	 * its 2D curve point property and the 3D surface point property
	 */
	void Combine(const FCurvePoint2D& Point2D, const FSurfacicPoint& SurfacicPoint)
	{
		ensureCADKernel(Point2D.DerivativeOrder >= 0);
		ensureCADKernel(SurfacicPoint.DerivativeOrder >= 0);

		ensureCADKernel(Point2D.DerivativeOrder <= SurfacicPoint.DerivativeOrder);

		DerivativeOrder = Point2D.DerivativeOrder;
		Point = SurfacicPoint.Point;

		if (DerivativeOrder > 0)
		{
			Gradient = SurfacicPoint.GradientU * Point2D.Gradient.X + SurfacicPoint.GradientV * Point2D.Gradient.Y;
		}

		if (DerivativeOrder > 1)
		{
			Laplacian = SurfacicPoint.LaplacianU * FMath::Square(Point2D.Gradient.X)
				+ 2.0 * SurfacicPoint.LaplacianUV * Point2D.Gradient.X * Point2D.Gradient.Y
				+ SurfacicPoint.LaplacianV * FMath::Square(Point2D.Gradient.Y)
				+ SurfacicPoint.GradientU * Point2D.Laplacian.X
				+ SurfacicPoint.GradientV * Point2D.Laplacian.Y;
		}
	}
};

struct FCoordinateGrid
{
	TArray<double> Coordinates[2];

	FCoordinateGrid()
	{
	}

	FCoordinateGrid(const TArray<double>& InUCoordinates, const TArray<double>& InVCoordinates)
	{
		Coordinates[EIso::IsoU] = InUCoordinates;
		Coordinates[EIso::IsoV] = InVCoordinates;
	}

	void Swap(TArray<double>& InUCoordinates, TArray<double>& InVCoordinates)
	{
		::Swap(Coordinates[EIso::IsoU], InUCoordinates);
		::Swap(Coordinates[EIso::IsoV], InVCoordinates);
	}

	int32 Count() const
	{
		return Coordinates[EIso::IsoU].Num() * Coordinates[EIso::IsoV].Num();
	}

	int32 IsoCount(EIso Iso) const
	{
		return Coordinates[Iso].Num();
	}

	void SetNum(int32 UNumber, int32 VNumber)
	{
		Coordinates[0].SetNum(UNumber);
		Coordinates[1].SetNum(VNumber);
	}

	void Empty(int32 UNumber = 0, int32 VNumber = 0)
	{
		Coordinates[0].Empty(UNumber);
		Coordinates[1].Empty(VNumber);
	}

	constexpr TArray<double>& operator[](EIso Iso)
	{
		ensureCADKernel(Iso == 0 || Iso == 1);
		return Coordinates[Iso];
	}

	constexpr const TArray<double>& operator[](EIso Iso) const
	{
		ensureCADKernel(Iso == 0 || Iso == 1);
		return Coordinates[Iso];
	}
};


}