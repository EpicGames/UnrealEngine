// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "Geo/Sampling/PolylineTools.h"
#include "Geo/Surfaces/Surface.h"

#include "Serialization/Archive.h"

namespace UE::CADKernel
{

template<class PointType>
class TPolyline
{
public:

	TArray<double> Coordinates;
	TArray<PointType> Points;
	TArray<PointType> Tangent;
	TPolylineApproximator<PointType> Approximator;
	bool bWithTangent = false;


	TPolyline(const TArray<PointType>& InPoints, const TArray<double>& InCoordinates)
		: Coordinates(InCoordinates)
		, Points(InPoints)
		, Approximator(Coordinates, Points)
	{
	}

	TPolyline(const TArray<PointType>& InPoints)
		: Points(InPoints)
		, Approximator(Coordinates, Points)
	{
	}

	TPolyline()
		: Approximator(Coordinates, Points)
	{
	}

	void Serialize(FCADKernelArchive& Ar)
	{
		Ar << Points;
		Ar << Coordinates;
	}

	PointType ApproximatePoint(double InCoordinate) const
	{
		return Approximator.ApproximatePoint(InCoordinate);
	}

	void ApproximatePoints(const TArray<double>& InCoordinates, TArray<PointType>& OutPoints) const
	{
		Approximator.ApproximatePoints(InCoordinates, OutPoints);
	}

	void GetSubPolyline(const FLinearBoundary& InBoundary, const EOrientation InOrientation, TArray<PointType>& OutPoints) const
	{
		Approximator.GetSubPolyline(InBoundary, InOrientation, OutPoints);
	}

	/**
	 * The FPolylineBBox is updated by adding all the points of the polyline included in the boundary
	 */
	void UpdateSubPolylineBBox(const FLinearBoundary& InBoundary, FPolylineBBox& OutBBox) const
	{
		Approximator.UpdateSubPolylineBBox(InBoundary, OutBBox);
	}

	void Sample(const FLinearBoundary& Boundary, const double DesiredSegmentLength, TArray<double>& OutCoordinates) const
	{
		Approximator.SamplePolyline(Boundary, DesiredSegmentLength, OutCoordinates);
	}

	double GetCoordinateOfProjectedPoint(const FLinearBoundary& Boundary, const PointType& PointOnEdge, PointType& ProjectedPoint) const
	{
		return Approximator.ProjectPointToPolyline(Boundary, PointOnEdge, ProjectedPoint);
	}

	void ProjectPoints(const FLinearBoundary& InBoundary, const TArray<PointType>& InPointsToProject, TArray<double>& ProjectedPointCoordinates, TArray<PointType>& ProjectedPoints) const
	{
		Approximator.ProjectPointsToPolyline(InBoundary, InPointsToProject, ProjectedPointCoordinates, ProjectedPoints);
	}

	const PointType& GetPointAt(int32 Index) const
	{
		return Points[Index];
	}

	const TArray<PointType>& GetPoints() const
	{
		return Points;
	}

	const TArray<double>& GetCoordinates() const
	{
		return Coordinates;
	}

	TArray<double>& GetCoordinates()
	{
		return Coordinates;
	}

	void SwapCoordinates(TArray<double>& NewCoordinates)
	{
		Swap(NewCoordinates, Coordinates);
		Points.Empty(Coordinates.Num());
		if (bWithTangent)
		{
			Tangent.Empty(Coordinates.Num());
		}
	}

	void GetAt(int32 Index, double& Coordinate, PointType& Point)
	{
		ensureCADKernel(Coordinates.IsValidIndex(Index));
		Coordinate = Coordinates[Index];
		Point = Points[Index];
	}

	/**
	 * @return the size of the polyline i.e. the count of points.
	 */
	int32 Size() const
	{
		return Points.Num();
	}

	/**
	 * Reserves memory such that the polyline can contain at least Number elements.
	 *
	 * @param Number The number of elements that the polyline should be able to contain after allocation.
	 */
	void Reserve(int32 Number)
	{
		Points.Reserve(Number);
		Coordinates.Reserve(Number);
	}

	/**
	 * Empties the polyline.
	 *
	 * @param Slack (Optional) The expected usage size after empty operation. Default is 0.
	 */
	void Empty(int32 Slack = 0)
	{
		Points.Empty(Slack);
		Coordinates.Empty(Slack);
	}

	void EmplaceAt(int32 Index, TPolyline<PointType>& Polyline, int32 PointIndex)
	{
		Coordinates.EmplaceAt(Index, Polyline.Coordinates[PointIndex]);
		Points.EmplaceAt(Index, Polyline.Points[PointIndex]);
	}


	void RemoveComplementaryPoints(int32 Offset)
	{
		const int32 Count = Points.Num();

		TBitArray<> Markers(true, Count);

		for (int32 Index = Count - 1; Index >= 0; Index -= Offset)
		{
			Markers[Index] = false;
		}

		int32 CurrentIndex = 0;
		for (int32 Index = 0; Index < Count; ++Index)
		{
			if (Markers[Index])
			{
				Coordinates[CurrentIndex] = Coordinates[Index];
				Points[CurrentIndex] = Points[Index];
				++CurrentIndex;
			}
		}

		Coordinates.SetNum(CurrentIndex, EAllowShrinking::No);
		Points.SetNum(CurrentIndex, EAllowShrinking::No);
	}

	void Pop()
	{
		Coordinates.Pop(EAllowShrinking::No);
		Points.Pop(EAllowShrinking::No);
	}

	double GetLength(const FLinearBoundary& InBoundary) const
	{
		return Approximator.ComputeLengthOfSubPolyline(InBoundary);
	}
};

typedef TPolyline<FVector> FPolyline3D;

class FPolyline2D : public TPolyline<FVector2d>
{
public:

	/**
	 * Add to OutIntersection the curve coordinate of the intersection with the Iso curve
	 */
	void FindIntersectionsWithIso(const EIso Iso, double IsoParameter, TArray<double>& OutIntersections)
	{
		OutIntersections.Empty(8);

		int32 UIndex = Iso == EIso::IsoU ? 0 : 1;
		int32 VIndex = Iso == EIso::IsoU ? 1 : 0;

		TFunction<void(const int32, const int32, const int32)> ComputeIntersection = [&](const int32 Index1, const int32& Index2, const int32 MinIndex)
		{
			if (IsoParameter > Points[Index1][UIndex] && IsoParameter <= Points[Index2][UIndex])
			{
				double Intersection = (IsoParameter - Points[Index1][UIndex]) / (Points[Index2][UIndex] - Points[Index1][UIndex]);
				OutIntersections.Add(PolylineTools::LinearInterpolation(Coordinates, MinIndex, Intersection));
			}
		};

		for (int32 Index = 1; Index < Points.Num(); ++Index)
		{
			if (!FMath::IsNearlyEqual(Points[Index - 1][UIndex], Points[Index][UIndex]))
			{
				if (Points[Index - 1][UIndex] < Points[Index][UIndex])
				{
					ComputeIntersection(Index - 1, Index, Index - 1);
				}
				else
				{
					ComputeIntersection(Index, Index - 1, Index - 1);
				}
			}
		}
	}
};

} // ns UE::CADKernel

