// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Aabb.h"
#include "Math/Boundary.h"
#include "Math/Geometry.h"
#include "Math/Point.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Utils/IndexOfCoordinateFinder.h"

#include "Algo/ForEach.h"
#include "Algo/Reverse.h"

namespace UE::CADKernel
{

struct FPolylineBBox
{
	FVector Max;
	FVector Min;
	FVector MaxPoints[3];
	FVector MinPoints[3];
	double CoordinateOfMaxPoint[3] = { -HUGE_VALUE, -HUGE_VALUE, -HUGE_VALUE };
	double CoordinateOfMinPoint[3] = { HUGE_VALUE, HUGE_VALUE, HUGE_VALUE };

	FPolylineBBox()
		: Max(-HUGE_VALUE, -HUGE_VALUE, -HUGE_VALUE)
		, Min(HUGE_VALUE, HUGE_VALUE, HUGE_VALUE)
	{
	}

	void Update(const double Coordinate, const FVector& Point)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			if (Point[Index] > Max[Index])
			{
				Max[Index] = Point[Index];
				MaxPoints[Index] = Point;
				CoordinateOfMaxPoint[Index] = Coordinate;
			}

			if (Point[Index] < Min[Index])
			{
				Min[Index] = Point[Index];
				MinPoints[Index] = Point;
				CoordinateOfMinPoint[Index] = Coordinate;
			}
		}
	}

	void Update(const double Coordinate, const FVector2d& Point2D)
	{
		Update(Coordinate, FVector(Point2D, 0.));
	}
};

namespace PolylineTools
{

inline bool IsDichotomyToBePreferred(int32 InPolylineSize, int32 ResultSize)
{
	double MeanLinearIteration = InPolylineSize / ResultSize;
	double MaxDichotomyIteration = FMath::Log2((double)InPolylineSize);

	if (MeanLinearIteration > MaxDichotomyIteration)
	{
		return true;
	}
	return false;
}

template<typename PointType>
inline PointType LinearInterpolation(const TArray<PointType>& Array, const int32 Index, const double Coordinate)
{
	ensureCADKernel(Index + 1 < Array.Num());
	return PointOnSegment<PointType>(Array[Index], Array[Index + 1], Coordinate);
}

inline double SectionCoordinate(const TArray<double>& Array, const int32 Index, const double Coordinate)
{
	ensureCADKernel(Index + 1 < Array.Num());
	const double DeltaU = Array[Index + 1] - Array[Index];
	if (FMath::IsNearlyZero(DeltaU))
	{
		return 0;
	}
	return (Coordinate - Array[Index]) / DeltaU;
}

template<typename PointType>
double ComputeLength(const TArray<PointType>& Polyline)
{
	double Length = 0;
	for (int32 Index = 1; Index < Polyline.Num(); ++Index)
	{
		Length += PointType::Distance(Polyline[Index - 1], Polyline[Index]);
	}
	return Length;
}

template<typename PointType>
double ComputeSquareToleranceForProjection(const TArray<PointType>& Polyline)
{
	const double Tolerance = ComputeLength(Polyline) * 0.1;
	return Tolerance * Tolerance;
}

template<typename PointType>
TArray<double> ComputePolylineSegmentLengths(const PointType& StartPoint, const TArray<PointType>& InnerPolyline, const PointType& EndPoint)
{
	TArray<double> ElementLength;
	const int32 InnerPointCount = InnerPolyline.Num();
	if (InnerPointCount > 0)
	{
		ElementLength.Reserve(InnerPointCount + 1);
		const PointType* PrevPoint = &InnerPolyline[0];
		{
			ElementLength.Add(PointType::Distance(*PrevPoint, StartPoint));
		}

		for (int32 Index = 1; Index < InnerPointCount; ++Index)
		{
			const FVector& CurrentPoint = InnerPolyline[Index];
			ElementLength.Add(PointType::Distance(*PrevPoint, CurrentPoint));
			PrevPoint = &CurrentPoint;
		}
		{
			ElementLength.Add(PointType::Distance(*PrevPoint, EndPoint));
		}
	}
	else
	{
		ElementLength.Reserve(1);
		ElementLength.Add(PointType::Distance(StartPoint, EndPoint));
	}

	return MoveTemp(ElementLength);
}

/**
 * Progressively deforms a polyline (or a control polygon) so that its end is in the desired position
 */
template<typename PointType>
void ExtendTo(TArray<PointType>& Polyline, const PointType& DesiredEnd)
{
	double DistanceStart = PointType::DistSquared(Polyline[0], DesiredEnd);
	double DistanceEnd = PointType::DistSquared(Polyline.Last(), DesiredEnd);

	if (DistanceStart < DistanceEnd)
	{
		Algo::Reverse(Polyline);
	}

	double PolylineLength = ComputeLength(Polyline);

	PointType Delta = DesiredEnd - Polyline.Last();
	Delta /= PolylineLength;

	PolylineLength = 0;
	PolylineLength = PointType::Distance(Polyline[1], Polyline[0]);
	for (int32 Index = 1; Index < Polyline.Num() - 1; ++Index)
	{
		double LengthNextSegment = PointType::Distance(Polyline[Index], Polyline[Index + 1]);
		Polyline[Index] += Delta * PolylineLength;
		PolylineLength += LengthNextSegment;
	}
	Polyline.Last() = DesiredEnd;

	if (DistanceStart < DistanceEnd)
	{
		Algo::Reverse(Polyline);
	}
}

template<class PointType>
PointType ComputePoint(const TArray<double>& PolylineCoordinates, const TArray<PointType>& PolylinePoints, const int32 Index, const double PointCoordinate)
{
	double Delta = PolylineCoordinates[Index + 1] - PolylineCoordinates[Index];
	if (FMath::IsNearlyZero(Delta, (double)DOUBLE_SMALL_NUMBER))
	{
		return PolylinePoints[Index];
	}

	return PolylinePoints[Index] + (PolylinePoints[Index + 1] - PolylinePoints[Index]) * (PointCoordinate - PolylineCoordinates[Index]) / Delta;
};

} // ns PolylineTools

template<class PointType>
class TPolylineApproximator
{
protected:
	const TArray<double>& PolylineCoordinates;
	const TArray<PointType>& PolylinePoints;

public:
	TPolylineApproximator(const TArray<double>& InPolylineCoordinates, const TArray<PointType>& InPolylinePoints)
		: PolylineCoordinates(InPolylineCoordinates)
		, PolylinePoints(InPolylinePoints)
	{
	}

protected:

	double ComputeCurvilinearCoordinatesOfPolyline(const FLinearBoundary& InBoundary, TArray<double>& OutCurvilinearCoordinates, int32 BoundaryIndices[2]) const
	{
		GetStartEndIndex(InBoundary, BoundaryIndices);

		OutCurvilinearCoordinates.Reserve(BoundaryIndices[1] - BoundaryIndices[0] + 2);

		double LastEdgeSegmentLength;
		double EdgeLength = 0;
		double LengthOfSegment = 0;

		OutCurvilinearCoordinates.Add(0);
		if (BoundaryIndices[1] > BoundaryIndices[0])
		{
			PointType StartPoint = ComputePoint(BoundaryIndices[0], InBoundary.Min);
			PointType PointIndice0 = PolylinePoints[BoundaryIndices[0]];

			PointType EndPoint = ComputePoint(BoundaryIndices[1], InBoundary.Max);
			PointType NextPointIndice1 = PolylinePoints[BoundaryIndices[1] + 1];

			EdgeLength = PointType::Distance(StartPoint, PolylinePoints[BoundaryIndices[0] + 1]);
			double LengthOfSegments = PointType::Distance(PointIndice0, PolylinePoints[BoundaryIndices[0] + 1]);

			LastEdgeSegmentLength = PointType::Distance(EndPoint, PolylinePoints[BoundaryIndices[1]]);
			const double LastSegmentLength = PointType::Distance(NextPointIndice1, PolylinePoints[BoundaryIndices[1]]);

			OutCurvilinearCoordinates.Add(LengthOfSegments);
			for (int32 Index = BoundaryIndices[0] + 1; Index < BoundaryIndices[1]; ++Index)
			{
				const double SegLength = PointType::Distance(PolylinePoints[Index], PolylinePoints[Index + 1]);
				EdgeLength += SegLength;
				LengthOfSegments += SegLength;
				OutCurvilinearCoordinates.Add(LengthOfSegments);
			}
			EdgeLength += LastEdgeSegmentLength;
			LengthOfSegments += LastSegmentLength;
			OutCurvilinearCoordinates.Add(LengthOfSegments);
		}
		else
		{
			PointType StartPoint = ComputePoint(BoundaryIndices[0], InBoundary.Min);
			PointType EndPoint = ComputePoint(BoundaryIndices[1], InBoundary.Max);
			EdgeLength = PointType::Distance(StartPoint, EndPoint);
			const double SegLength = PointType::Distance(PolylinePoints[BoundaryIndices[0]], PolylinePoints[BoundaryIndices[0] + 1]);
			OutCurvilinearCoordinates.Add(SegLength);
		}
		return EdgeLength;
	}

	PointType ComputePoint(const int32 Index, const double PointCoordinate) const
	{
		double Delta = PolylineCoordinates[Index + 1] - PolylineCoordinates[Index];
		if (FMath::IsNearlyZero(Delta, (double)DOUBLE_SMALL_NUMBER))
		{
			return PolylinePoints[Index];
		}

		return PolylinePoints[Index] + (PolylinePoints[Index + 1] - PolylinePoints[Index]) * (PointCoordinate - PolylineCoordinates[Index]) / Delta;
	};

	/**
	 * Project a Set of points on a restricted polyline (StartIndex & EndIndex define the polyline boundary)
	 * the points are projected on all segments of the polyline, the closest are selected
	 */
	double ProjectPointToPolyline(int32 BoundaryIndices[2], const PointType& InPointToProject, PointType& OutProjectedPoint) const
	{
		double MinDistance = HUGE_VAL;
		double UForMinDistance = 0;

		double ParamU = 0.;
		int32 SegmentIndex = 0;

		for (int32 Index = BoundaryIndices[0]; Index <= BoundaryIndices[1]; ++Index)
		{
			PointType ProjectPoint = ProjectPointOnSegment<PointType>(InPointToProject, PolylinePoints[Index], PolylinePoints[Index + 1], ParamU, true);
			double SquareDistance = FMath::Square(ProjectPoint[0] - InPointToProject[0]);
			if (SquareDistance > MinDistance)
			{
				continue;
			}
			SquareDistance += FMath::Square(ProjectPoint[1] - InPointToProject[1]);
			if (SquareDistance > MinDistance)
			{
				continue;
			}
			SquareDistance += FMath::Square(ProjectPoint[2] - InPointToProject[2]);
			if (SquareDistance > MinDistance)
			{
				continue;
			}
			MinDistance = SquareDistance;
			UForMinDistance = ParamU;
			SegmentIndex = Index;
			OutProjectedPoint = ProjectPoint;
		}

		return PolylineTools::LinearInterpolation(PolylineCoordinates, SegmentIndex, UForMinDistance);
	}

public:

	void GetStartEndIndex(const FLinearBoundary& InBoundary, int32 BoundaryIndices[2]) const
	{
		FDichotomyFinder Finder(PolylineCoordinates);
		BoundaryIndices[0] = Finder.Find(InBoundary.Min);
		BoundaryIndices[1] = Finder.Find(InBoundary.Max);
	}

	/**
	 * Evaluate the point of the polyline at the input InCoordinate
	 * If the input coordinate is outside the boundary of the polyline, the coordinate of the nearest boundary is used.
	 */
	PointType ApproximatePoint(const double InCoordinate) const
	{
		FDichotomyFinder Finder(PolylineCoordinates);
		int32 Index = Finder.Find(InCoordinate);
		return ComputePoint(Index, InCoordinate);
	}

	/**
	 * Evaluate the point of the polyline at the input Coordinate
	 * If the input coordinate is outside the boundary of the polyline, the coordinate of the nearest boundary is used.
	 */
	template<class CurvePointType>
	void ApproximatePoint(double InCoordinate, CurvePointType& OutPoint, int32 InDerivativeOrder) const
	{
		FDichotomyFinder Finder(PolylineCoordinates);
		int32 Index = Finder.Find(InCoordinate);

		OutPoint.DerivativeOrder = InDerivativeOrder;

		double DeltaU = PolylineCoordinates[Index + 1] - PolylineCoordinates[Index];
		if (FMath::IsNearlyZero(DeltaU))
		{
			OutPoint.Point = PolylinePoints[Index];
			OutPoint.Gradient = PointType::ZeroVector;
			OutPoint.Laplacian = PointType::ZeroVector;
			return;
		}

		double SectionCoordinate = (InCoordinate - PolylineCoordinates[Index]) / DeltaU;

		PointType Tangent = PolylinePoints[Index + 1] - PolylinePoints[Index];
		OutPoint.Point = PolylinePoints[Index] + Tangent * SectionCoordinate;

		if (InDerivativeOrder > 0)
		{
			OutPoint.Gradient = Tangent;
			OutPoint.Laplacian = PointType::ZeroVector;
		}
	}

	template<class CurvePointType>
	void ApproximatePoints(const TArray<double>& InCoordinates, TArray<CurvePointType>& OutPoints, int32 InDerivativeOrder = 0) const
	{
		if (!InCoordinates.Num())
		{
			ensureCADKernel(false);
			return;
		}

		TFunction<void(FIndexOfCoordinateFinder&)> ComputePoints = [&](FIndexOfCoordinateFinder& Finder)
		{
			for (int32 IPoint = 0; IPoint < InCoordinates.Num(); ++IPoint)
			{
				int32 Index = Finder.Find(InCoordinates[IPoint]);

				OutPoints[IPoint].DerivativeOrder = InDerivativeOrder;

				double DeltaU = PolylineCoordinates[Index + 1] - PolylineCoordinates[Index];
				if (FMath::IsNearlyZero(DeltaU))
				{
					OutPoints[IPoint].Point = PolylinePoints[Index];
					OutPoints[IPoint].Gradient = PointType::ZeroVector;
					OutPoints[IPoint].Laplacian = PointType::ZeroVector;
					continue;
				}

				double SectionCoordinate = (InCoordinates[IPoint] - PolylineCoordinates[Index]) / DeltaU;

				PointType Tangent = PolylinePoints[Index + 1] - PolylinePoints[Index];
				OutPoints[IPoint].Point = PolylinePoints[Index] + Tangent * SectionCoordinate;

				if (InDerivativeOrder > 0)
				{
					OutPoints[IPoint].Gradient = Tangent;
					OutPoints[IPoint].Laplacian = PointType::ZeroVector;
				}
			}
		};

		FDichotomyFinder DichotomyFinder(PolylineCoordinates);

		int32 StartIndex = DichotomyFinder.Find(InCoordinates[0]);
		int32 EndIndex = DichotomyFinder.Find(InCoordinates.Last());
		bool bUseDichotomy = PolylineTools::IsDichotomyToBePreferred(EndIndex - StartIndex, InCoordinates.Num());

		OutPoints.SetNum(InCoordinates.Num(), EAllowShrinking::No);
		if (bUseDichotomy)
		{
			DichotomyFinder.StartLower = StartIndex;
			DichotomyFinder.StartUpper = EndIndex;
			ComputePoints(DichotomyFinder);
		}
		else
		{
			FLinearFinder LinearFinder(PolylineCoordinates, StartIndex);
			ComputePoints(LinearFinder);
		}
	}


	/**
	 * Evaluate the points of the polyline associated to the increasing array of input Coordinates
	 * If the input coordinate is outside the boundary of the polyline, the coordinate of the nearest boundary is used.
	 */
	void ApproximatePoints(const TArray<double>& InCoordinates, TArray<PointType>& OutPoints) const
	{
		if (!InCoordinates.Num())
		{
			return;
		}

		TFunction<void(FIndexOfCoordinateFinder&)> ComputePoints = [&](FIndexOfCoordinateFinder& Finder)
		{
			for (double Coordinate : InCoordinates)
			{
				int32 Index = Finder.Find(Coordinate);
				OutPoints.Emplace(ComputePoint(Index, Coordinate));
			}
		};

		FDichotomyFinder DichotomyFinder(PolylineCoordinates);

		int32 StartIndex = DichotomyFinder.Find(InCoordinates[0]);
		int32 EndIndex = DichotomyFinder.Find(InCoordinates.Last());
		bool bUseDichotomy = PolylineTools::IsDichotomyToBePreferred(EndIndex - StartIndex, InCoordinates.Num());

		OutPoints.Empty(InCoordinates.Num());
		if (bUseDichotomy)
		{
			DichotomyFinder.StartLower = StartIndex;
			DichotomyFinder.StartUpper = EndIndex;
			ComputePoints(DichotomyFinder);
		}
		else
		{
			FLinearFinder LinearFinder(PolylineCoordinates, StartIndex);
			ComputePoints(LinearFinder);
		}
	}

	void SamplePolyline(const FLinearBoundary& InBoundary, const double DesiredSegmentLength, TArray<double>& OutCoordinates) const
	{
		int32 BoundaryIndices[2];

		TArray<double> CurvilinearCoordinates;
		const double CurveLength = ComputeCurvilinearCoordinatesOfPolyline(InBoundary, CurvilinearCoordinates, BoundaryIndices);

		const PointType StartPoint = ComputePoint(BoundaryIndices[0], InBoundary.Min);
		double FromStartSegmentLength = PointType::Distance(PolylinePoints[BoundaryIndices[0]], StartPoint);

		if (FMath::IsNearlyZero(DesiredSegmentLength, DOUBLE_KINDA_SMALL_NUMBER) || CurveLength < 2. * DesiredSegmentLength)
		{
			OutCoordinates.SetNum(3);
			OutCoordinates[0] = InBoundary.GetMin();
			OutCoordinates[1] = InBoundary.GetMiddle();
			OutCoordinates[2] = InBoundary.GetMax();
			return;
		}

		int32 SegmentNum = FMath::IsNearlyZero(DesiredSegmentLength) ? 2 : (int32)FMath::Max(CurveLength / DesiredSegmentLength + 0.5, 1.0);

		const double SectionLength = CurveLength / (double)(SegmentNum);

		OutCoordinates.Empty();
		OutCoordinates.Reserve(SegmentNum + 1);
		OutCoordinates.Add(InBoundary.Min);

		TFunction<double(const int32, const int32, const double, const double)> ComputeSamplePointCoordinate = [&](const int32 IndexCurvilinear, const int32 IndexCoordinate, const double Length, const double Coordinate)
		{
			return Coordinate + (PolylineCoordinates[IndexCoordinate] - Coordinate) * (Length - CurvilinearCoordinates[IndexCurvilinear - 1]) / (CurvilinearCoordinates[IndexCurvilinear] - CurvilinearCoordinates[IndexCurvilinear - 1]);
		};

		double CurvilinearLength = 0.;

		double LastCoordinate = InBoundary.Min;
		for (int32 IndexCurvilinear = 1, IndexCoordinate = BoundaryIndices[0] + 1; IndexCurvilinear < CurvilinearCoordinates.Num(); ++IndexCurvilinear, ++IndexCoordinate)
		{
			while (FromStartSegmentLength + DOUBLE_SMALL_NUMBER < CurvilinearCoordinates[IndexCurvilinear])
			{
				const double Coordinate = ComputeSamplePointCoordinate(IndexCurvilinear, IndexCoordinate, FromStartSegmentLength, LastCoordinate);
				OutCoordinates.Add(Coordinate);
				CurvilinearLength += SectionLength;
				FromStartSegmentLength += SectionLength;
				if (CurvilinearLength > CurveLength || OutCoordinates.Num() == SegmentNum)
				{
					while (OutCoordinates.Last() + SMALL_NUMBER > InBoundary.GetMax())
					{
						OutCoordinates.Pop();
					}
					OutCoordinates.Add(InBoundary.GetMax());
					return;
				}
			}
			LastCoordinate = PolylineCoordinates[IndexCoordinate];
		}

		while (OutCoordinates.Last() + SMALL_NUMBER > InBoundary.GetMax())
		{
			OutCoordinates.Pop();
		}
		OutCoordinates.Add(InBoundary.GetMax());
	}

	/**
	 * Project a Set of points on a restricted polyline (StartIndex & EndIndex define the polyline boundary)
	 * the points are projected on all segments of the polyline, the closest are selected
	 */
	double ProjectPointToPolyline(const FLinearBoundary& InBoundary, const PointType& PointOnEdge, PointType& OutProjectedPoint) const
	{
		int32 BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);
		double Coordinate = ProjectPointToPolyline(BoundaryIndices, PointOnEdge, OutProjectedPoint);
		if (InBoundary.Contains(Coordinate))
		{
			return Coordinate;
		}

		if (Coordinate < InBoundary.GetMin())
		{
			Coordinate = InBoundary.GetMin();
		}
		else if (Coordinate > InBoundary.GetMax())
		{
			Coordinate = InBoundary.GetMax();
		}
		OutProjectedPoint = ApproximatePoint(Coordinate);
		return Coordinate;
	}

	/**
	 * Project a Set of points on a restricted polyline (StartIndex & EndIndex define the polyline boundary)
	 * Each points are projected on all segments of the restricted polyline, the closest are selected
	 */
	void ProjectPointsToPolyline(const FLinearBoundary& InBoundary, const TArray<PointType>& InPointsToProject, TArray<double>& OutProjectedPointCoords, TArray<PointType>& OutProjectedPoints) const
	{
		OutProjectedPointCoords.Empty(InPointsToProject.Num());
		OutProjectedPoints.Empty(InPointsToProject.Num());

		int32 BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);

		for (const PointType& Point : InPointsToProject)
		{
			PointType ProjectedPoint;
			double Coordinate = ProjectPointToPolyline(BoundaryIndices, Point, ProjectedPoint);

			OutProjectedPointCoords.Emplace(Coordinate);
			OutProjectedPoints.Emplace(ProjectedPoint);
		}
	}

	/**
	 * Project each point of a coincidental polyline on the Polyline.
	 * @param ToleranceOfProjection: Max error between the both curve to stop the search of projection
	 *    if ToleranceOfProjection < 0, it's compute with ComputeSquareToleranceForProjection
	 */
	void ProjectCoincidentalPolyline(const FLinearBoundary& InBoundary, const TArray<PointType>& InPointsToProject, bool bSameOrientation, TArray<double>& OutProjectedPointCoords, const double ToleranceOfProjection) const
	{
#ifdef DEBUG_PROJECT_COINCIDENTAL_POLYLINE
		static int32 Counter = 0;
		++Counter;
		bool bDisplay = true; // (Counter == 22);
		if (bDisplay)
		{
			F3DDebugSession Session(FString::Printf(TEXT("ProjectCoincidentalPolyline %d"), Counter));
			{
				F3DDebugSession _(TEXT("ProjectCoincidentalPolyline"));
				DisplayPolyline(InPointsToProject, EVisuProperty::BlueCurve);
			}
			{
				F3DDebugSession _(TEXT("ProjectCoincidentalPolyline"));
				DisplayPolyline(PolylinePoints, EVisuProperty::YellowCurve);
			}
		}
#endif

		const double SquareTol = ToleranceOfProjection > 0 ? FMath::Square(ToleranceOfProjection) : PolylineTools::ComputeSquareToleranceForProjection(PolylinePoints);

		int32 BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);

		int32 StartIndex = BoundaryIndices[0];
		const int32 EndIndex = BoundaryIndices[1] + 1;

		TFunction<double(const PointType&)> ProjectPointToPolyline = [&](const PointType& InPointToProject)
		{
#ifdef DEBUG_PROJECT_COINCIDENTAL_POLYLINE
			PointType ClosePoint;
#endif

			double MinDistance = HUGE_VAL;
			double UForMinDistance = 0;

			double ParamU = 0.;
			int32 SegmentIndex = 0;
			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				PointType ProjectPoint = ProjectPointOnSegment<PointType>(InPointToProject, PolylinePoints[Index], PolylinePoints[Index + 1], ParamU, true);
				double SquareDistance = FVector::DistSquared(ProjectPoint, InPointToProject);
				if (SquareDistance > MinDistance + SquareTol)
				{
					break;
				}

				if (SquareDistance < MinDistance)
				{
					MinDistance = SquareDistance;
					UForMinDistance = ParamU;
					SegmentIndex = Index;
#ifdef DEBUG_PROJECT_COINCIDENTAL_POLYLINE
					ClosePoint = ProjectPoint;
#endif
				}

			}
#ifdef DEBUG_PROJECT_COINCIDENTAL_POLYLINE
			if (bDisplay)
			{
				F3DDebugSession Session(TEXT("ProjectPoint"));
				{
					DisplayPoint(InPointToProject, EVisuProperty::BluePoint);
					DisplayPoint(ClosePoint, EVisuProperty::RedPoint);
					//Wait();
				}
			}
#endif

			StartIndex = SegmentIndex;
			return PolylineTools::LinearInterpolation(PolylineCoordinates, SegmentIndex, UForMinDistance);
		};

		if (bSameOrientation)
		{
			OutProjectedPointCoords.Empty(InPointsToProject.Num());
			for (const PointType& Point : InPointsToProject)
			{
				double Coordinate = ProjectPointToPolyline(Point);
				OutProjectedPointCoords.Emplace(Coordinate);
			}
		}
		else
		{
			OutProjectedPointCoords.SetNum(InPointsToProject.Num());
			for (int32 Index = InPointsToProject.Num() - 1, Pndex = 0; Index >= 0; --Index, ++Pndex)
			{
				OutProjectedPointCoords[Pndex] = ProjectPointToPolyline(InPointsToProject[Index]);
			}
		}
	}


	/**
	 * Append to the OutPoints array the sub polyline bounded by InBoundary according to the orientation
	 */
	void GetSubPolyline(const FLinearBoundary& InBoundary, const EOrientation Orientation, TArray<PointType>& OutPoints) const
	{
		int32 BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);

		int32 NewSize = OutPoints.Num() + BoundaryIndices[1] - BoundaryIndices[0] + 2;
		OutPoints.Reserve(NewSize);

		int32 PolylineStartIndex = BoundaryIndices[0];
		int32 PolylineEndIndex = BoundaryIndices[1];
		if (FMath::IsNearlyEqual(PolylineCoordinates[BoundaryIndices[0] + 1], InBoundary.Min, (double)DOUBLE_SMALL_NUMBER))
		{
			PolylineStartIndex++;
		}
		if (FMath::IsNearlyEqual(PolylineCoordinates[BoundaryIndices[1]], InBoundary.Max, (double)DOUBLE_SMALL_NUMBER))
		{
			PolylineEndIndex--;
		}

		if (Orientation)
		{
			OutPoints.Emplace(ComputePoint(BoundaryIndices[0], InBoundary.Min));
			if (PolylineEndIndex - PolylineStartIndex > 0)
			{
				OutPoints.Append(PolylinePoints.GetData() + PolylineStartIndex + 1, PolylineEndIndex - PolylineStartIndex);
			}
			OutPoints.Emplace(ComputePoint(BoundaryIndices[1], InBoundary.Max));
		}
		else
		{
			OutPoints.Emplace(ComputePoint(BoundaryIndices[1], InBoundary.Max));
			for (int32 Index = PolylineEndIndex; Index > PolylineStartIndex; --Index)
			{
				OutPoints.Emplace(PolylinePoints[Index]);
			}
			OutPoints.Emplace(ComputePoint(BoundaryIndices[0], InBoundary.Min));
		}
	}

	/**
	 * Get the subset of point defining the sub polyline bounded by InBoundary
	 * OutCoordinates and OutPoints are emptied before the process
	 */
	void GetSubPolyline(const FLinearBoundary& InBoundary, TArray<double>& OutCoordinates, TArray<PointType>& OutPoints) const
	{
		int32 BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);

		int32 NewSize = BoundaryIndices[1] - BoundaryIndices[0] + 2;

		OutCoordinates.Empty(NewSize);
		OutPoints.Empty(NewSize);

		if (FMath::IsNearlyEqual(PolylineCoordinates[BoundaryIndices[0] + 1], InBoundary.Min, (double)DOUBLE_SMALL_NUMBER))
		{
			BoundaryIndices[0]++;
		}

		if (BoundaryIndices[1]> 0 && FMath::IsNearlyEqual(PolylineCoordinates[BoundaryIndices[1]], InBoundary.Max, (double)DOUBLE_SMALL_NUMBER))
		{
			BoundaryIndices[1]--;
		}

		OutPoints.Emplace(ComputePoint(BoundaryIndices[0], InBoundary.Min));
		OutPoints.Append(PolylinePoints.GetData() + BoundaryIndices[0] + 1, BoundaryIndices[1] - BoundaryIndices[0]);
		OutPoints.Emplace(ComputePoint(BoundaryIndices[1], InBoundary.Max));

		OutCoordinates.Add(InBoundary.Min);
		OutCoordinates.Append(PolylineCoordinates.GetData() + BoundaryIndices[0] + 1, BoundaryIndices[1] - BoundaryIndices[0]);
		OutCoordinates.Add(InBoundary.Max);
	}

	/**
	 * Update the Curve bounding box with this subset of polyline
	 */
	void UpdateSubPolylineBBox(const FLinearBoundary& InBoundary, FPolylineBBox& OutBBox) const
	{
		int32 BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);

		if (BoundaryIndices[1] - BoundaryIndices[0] > 0)
		{
			if (FMath::IsNearlyEqual(PolylineCoordinates[BoundaryIndices[0] + 1], InBoundary.Min, (double)DOUBLE_SMALL_NUMBER))
			{
				BoundaryIndices[0]++;
			}

			if (FMath::IsNearlyEqual(PolylineCoordinates[BoundaryIndices[1]], InBoundary.Max, (double)DOUBLE_SMALL_NUMBER))
			{
				BoundaryIndices[1]--;
			}
		}

		OutBBox.Update(InBoundary.Min, ComputePoint(BoundaryIndices[0], InBoundary.Min));
		OutBBox.Update(InBoundary.Max, ComputePoint(BoundaryIndices[1], InBoundary.Max));
		for (int32 Index = BoundaryIndices[0] + 1; Index <= BoundaryIndices[1]; ++Index)
		{
			OutBBox.Update(PolylineCoordinates[Index], PolylinePoints[Index]);
		}
	}

	double ComputeLength() const
	{
		return PolylineTools::ComputeLength(PolylinePoints);
	}

	double ComputeLengthOfSubPolyline(const int BoundaryIndex[2], const FLinearBoundary& InBoundary) const
	{
		double Length = 0;
		if (BoundaryIndex[1] > BoundaryIndex[0])
		{
			PointType StartPoint = ComputePoint(BoundaryIndex[0], InBoundary.Min);
			PointType EndPoint = ComputePoint(BoundaryIndex[1], InBoundary.Max);
			Length = PointType::Distance(StartPoint, PolylinePoints[BoundaryIndex[0] + 1]);
			Length += PointType::Distance(EndPoint, PolylinePoints[BoundaryIndex[1]]);

			if (BoundaryIndex[1] > BoundaryIndex[0] + 1)
			{
				for (int32 Index = BoundaryIndex[0] + 1; Index < BoundaryIndex[1]; ++Index)
				{
					Length += PointType::Distance(PolylinePoints[Index], PolylinePoints[Index + 1]);
				}
			}
		}
		else
		{
			PointType StartPoint = ComputePoint(BoundaryIndex[0], InBoundary.Min);
			PointType EndPoint = ComputePoint(BoundaryIndex[1], InBoundary.Max);
			Length = PointType::Distance(StartPoint, EndPoint);
		}
		return Length;
	}

	double ComputeLengthOfSubPolyline(const FLinearBoundary& InBoundary) const
	{
		int BoundaryIndices[2];
		GetStartEndIndex(InBoundary, BoundaryIndices);
		return ComputeLengthOfSubPolyline(BoundaryIndices, InBoundary);
	}

	template<int Dimension = 3>
	void ComputeBoundingBox(const int BoundaryIndex[2], const FLinearBoundary& InBoundary, TAABB<PointType, Dimension>& Aabb)
	{
		Aabb.Empty();
		if (BoundaryIndex[1] > BoundaryIndex[0])
		{
			for (int32 Index = BoundaryIndex[0] + 1; Index <= BoundaryIndex[1]; ++Index)
			{
				Aabb += PolylinePoints[Index];
			}
		}

		PointType StartPoint = ComputePoint(BoundaryIndex[0], InBoundary.Min);
		Aabb += StartPoint;
		PointType EndPoint = ComputePoint(BoundaryIndex[1], InBoundary.Max);
		Aabb += EndPoint;
	}
};

} // namespace UE::CADKernel

