// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "Core/CADKernelArchive.h"
#include "Geo/Curves/Curve.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Geo/Surfaces/Surface.h"
#include "Math/Boundary.h"
#include "Math/MatrixH.h"
#include "Math/Point.h"
#include "Utils/IndexOfCoordinateFinder.h"

#include "Serialization/Archive.h"
#include "Algo/AllOf.h"

namespace UE::CADKernel
{

class FCurve;
class FEntityGeom;
class FInfoEntity;
class FSurface;

class CADKERNEL_API FSurfacicPolyline
{

public:

	TArray<double> Coordinates;
	TArray<FVector2d> Points2D;
	TArray<FVector> Points3D;
	TArray<FVector3f> Normals;
	TArray<FVector> Tangents;

	FSurfacicBoundary BoundingBox;

	bool bWithNormals;
	bool bWithTangent;

	FSurfacicPolyline(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> InCurve2D);

	FSurfacicPolyline(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> InCurve2D, const double Tolerance);

	FSurfacicPolyline(TSharedRef<FSurface> InCarrierSurface, TSharedRef<FCurve> InCurve2D, const double ChordTolerance, const double ParamTolerance, bool bInWithNormals/* = false*/, bool bWithTangent/* = false*/);

	FSurfacicPolyline(bool bInWithNormals = false, bool bInWithTangent = false)
		: bWithNormals(bInWithNormals)
		, bWithTangent(bInWithTangent)
	{
	}

	void Serialize(FCADKernelArchive& Ar)
	{
		Ar.Serialize(Points3D);
		Ar.Serialize(Points2D);
		Ar.Serialize(Normals);
		Ar.Serialize(Coordinates);
		Ar.Serialize(BoundingBox);
		Ar << bWithNormals;
		Ar << bWithTangent;
	}

	FInfoEntity& GetInfo(FInfoEntity&) const;

	TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH&) const;

	void CheckIfDegenerated(const double Tolerance3D, const FSurfacicTolerance& Tolerances2D, const FLinearBoundary& Boudary, bool& bDegeneration2D, bool& bDegeneration3D, double& Length3D) const;

	void GetExtremities(const FLinearBoundary& InBoundary, const double Tolerance3D, const FSurfacicTolerance& Tolerances2D, FSurfacicCurveExtremities& Extremities) const;

	FVector Approximate3DPoint(double InCoordinate) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		return Approximator3D.ApproximatePoint(InCoordinate);
	}

	void Approximate3DPoints(const TArray<double>& InCoordinates, TArray<FVector>& OutPoints) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		Approximator3D.ApproximatePoints(InCoordinates, OutPoints);
	}

	FVector2d Approximate2DPoint(double InCoordinate) const
	{
		TPolylineApproximator<FVector2d> Approximator(Coordinates, Points2D);
		return Approximator.ApproximatePoint(InCoordinate);
	}

	FVector GetTangentAt(double InCoordinate) const
	{
		FDichotomyFinder Finder(Coordinates);
		int32 Index = Finder.Find(InCoordinate);
		return Points3D[Index + 1] - Points3D[Index];
	}

	FVector2d GetTangent2DAt(double InCoordinate) const
	{
		FDichotomyFinder Finder(Coordinates);
		int32 Index = Finder.Find(InCoordinate);
		return Points2D[Index + 1] - Points2D[Index];
	}

	FSurfacicTolerance ComputeTolerance(const double Tolerance3D, const FSurfacicTolerance& MinToleranceIso, const int32 Index) const
	{
		double Distance3D = FVector::Distance(Points3D[Index], Points3D[Index + 1]);
		if (FMath::IsNearlyZero(Distance3D, (double)DOUBLE_SMALL_NUMBER))
		{
			return FVectorUtil::FarawayPoint2D;
		}
		else
		{
			FSurfacicTolerance Tolerance2D = Points2D[Index] - Points2D[Index + 1];
			return FVector2D::Max(Tolerance2D.GetAbs() * Tolerance3D / Distance3D, MinToleranceIso);
		}
	};

	double ComputeLinearToleranceAt(const double Tolerance3D, const double MinLinearTolerance, const int32 Index) const
	{
		double Distance3D = FVector::Distance(Points3D[Index], Points3D[Index + 1]);
		if (FMath::IsNearlyZero(Distance3D, (double)DOUBLE_SMALL_NUMBER))
		{
			return (Coordinates.Last() - Coordinates[0]) / 10.;
		}
		else
		{
			double LinearDistance = Coordinates[Index + 1] - Coordinates[Index];
			return FMath::Max(LinearDistance / Distance3D * Tolerance3D, MinLinearTolerance);
		}
	};

	void Approximate2DPoints(const TArray<double>& InCoordinates, TArray<FVector2d>& OutPoints) const
	{
		TPolylineApproximator<FVector2d> Approximator(Coordinates, Points2D);
		Approximator.ApproximatePoints(InCoordinates, OutPoints);
	}

	void ApproximatePolyline(FSurfacicPolyline& OutPolyline) const
	{
		if (OutPolyline.Coordinates.IsEmpty())
		{
			return;
		}

		TFunction<void(FIndexOfCoordinateFinder&)> ComputePoints = [&](FIndexOfCoordinateFinder& Finder)
		{
			int32 CoordinateCount = OutPolyline.Coordinates.Num();

			TArray<int32> SegmentIndexes;
			SegmentIndexes.Reserve(CoordinateCount);

			TArray<double> SegmentCoordinates;
			SegmentCoordinates.Reserve(CoordinateCount);


			//for (int32 Index = 0; Index < CoordinateCount; ++Index)
			for (const double Coordinate : OutPolyline.Coordinates)
			{
				const int32& Index = SegmentIndexes.Emplace_GetRef(Finder.Find(Coordinate));
				SegmentCoordinates.Emplace(PolylineTools::SectionCoordinate(Coordinates, Index, Coordinate));
			}

			OutPolyline.Points2D.Reserve(CoordinateCount);
			for (int32 Index = 0; Index < CoordinateCount; ++Index)
			{
				int32 SegmentIndex = SegmentIndexes[Index];
				double SegmentCoordinate = SegmentCoordinates[Index];
				OutPolyline.Points2D.Emplace(PolylineTools::LinearInterpolation(Points2D, SegmentIndex, SegmentCoordinate));
			}

			OutPolyline.Points3D.Reserve(CoordinateCount);
			for (int32 Index = 0; Index < CoordinateCount; ++Index)
			{
				int32 SegmentIndex = SegmentIndexes[Index];
				double SegmentCoordinate = SegmentCoordinates[Index];
				OutPolyline.Points3D.Emplace(PolylineTools::LinearInterpolation(Points3D, SegmentIndex, SegmentCoordinate));
			}

			if (bWithNormals)
			{
				for (int32 Index = 0; Index < CoordinateCount; ++Index)
				{
					int32 SegmentIndex = SegmentIndexes[Index];
					double SegmentCoordinate = SegmentCoordinates[Index];
					OutPolyline.Normals.Emplace(PolylineTools::LinearInterpolation(Normals, SegmentIndex, SegmentCoordinate));
				}
			}

			if (bWithTangent)
			{
				for (int32 Index = 0; Index < CoordinateCount; ++Index)
				{
					int32 SegmentIndex = SegmentIndexes[Index];
					double SegmentCoordinate = SegmentCoordinates[Index];
					OutPolyline.Tangents.Emplace(PolylineTools::LinearInterpolation(Tangents, SegmentIndex, SegmentCoordinate));
				}
			}
		};

		FDichotomyFinder DichotomyFinder(Coordinates);
		int32 StartIndex = 0;
		int32 EndIndex;

		bool bUseDichotomy = false;
		StartIndex = DichotomyFinder.Find(OutPolyline.Coordinates[0]);
		EndIndex = DichotomyFinder.Find(OutPolyline.Coordinates.Last());
		bUseDichotomy = PolylineTools::IsDichotomyToBePreferred(EndIndex - StartIndex, Coordinates.Num());


		if (bUseDichotomy)
		{
			DichotomyFinder.StartLower = StartIndex;
			DichotomyFinder.StartUpper = EndIndex;
			ComputePoints(DichotomyFinder);
		}
		else
		{
			FLinearFinder LinearFinder(Coordinates, StartIndex);
			ComputePoints(LinearFinder);
		}
	}


	void Sample(const FLinearBoundary& Boundary, const double DesiredSegmentLength, TArray<double>& OutCoordinates) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		Approximator3D.SamplePolyline(Boundary, DesiredSegmentLength, OutCoordinates);
	}

	double GetCoordinateOfProjectedPoint(const FLinearBoundary& Boundary, const FVector& PointOnEdge, FVector& ProjectedPoint) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		return Approximator3D.ProjectPointToPolyline(Boundary, PointOnEdge, ProjectedPoint);
	}

	double GetCoordinateOfProjectedPoint(const FLinearBoundary& Boundary, const FVector2d& PointOnEdge, FVector2d& ProjectedPoint) const
	{
		TPolylineApproximator<FVector2d> Approximator2D(Coordinates, Points2D);
		return Approximator2D.ProjectPointToPolyline(Boundary, PointOnEdge, ProjectedPoint);
	}

	void ProjectPoints(const FLinearBoundary& InBoundary, const TArray<FVector>& InPointsToProject, TArray<double>& ProjectedPointCoordinates, TArray<FVector>& ProjectedPoints) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		Approximator3D.ProjectPointsToPolyline(InBoundary, InPointsToProject, ProjectedPointCoordinates, ProjectedPoints);
	}

	void ProjectPoints(const FLinearBoundary& InBoundary, const TArray<FVector2d>& InPointsToProject, TArray<double>& ProjectedPointCoordinates, TArray<FVector2d>& ProjectedPoints) const
	{
		TPolylineApproximator<FVector2d> Approximator(Coordinates, Points2D);
		Approximator.ProjectPointsToPolyline(InBoundary, InPointsToProject, ProjectedPointCoordinates, ProjectedPoints);
	}

	/**
	 * Project each point of a coincidental polyline on the Polyline.
	 * @param ToleranceOfProjection: Max error between the both curve to stop the search of projection
	 */
	void ProjectCoincidentalPolyline(const FLinearBoundary& InBoundary, const TArray<FVector>& InPointsToProject, bool bSameOrientation, TArray<double>& OutProjectedPointCoordinates, double ToleranceOfProjection) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		Approximator3D.ProjectCoincidentalPolyline(InBoundary, InPointsToProject, bSameOrientation, OutProjectedPointCoordinates, ToleranceOfProjection);
	}

	/**
	 * The main idea of this algorithm is to process starting for the beginning of the curve to the end of the curve.
	 */
	void ComputeIntersectionsWithIsos(const FLinearBoundary& InBoundary, const TArray<double>& InIsoCoordinates, const EIso InTypeIso, const FSurfacicTolerance& ToleranceIso, TArray<double>& OutIntersection) const;

	const TArray<double>& GetCoordinates() const
	{
		return Coordinates;
	}

	const TArray<FVector2d>& Get2DPoints() const
	{
		return Points2D;
	}

	const FVector& GetPointAt(int32 Index) const
	{
		return Points3D[Index];
	}

	const TArray<FVector>& GetPoints() const
	{
		return Points3D;
	}

	const TArray<FVector3f>& GetNormals() const
	{
		return Normals;
	}

	const TArray<FVector>& GetTangents() const
	{
		return Tangents;
	}

	void SwapCoordinates(TArray<double>& NewCoordinates)
	{
		Swap(NewCoordinates, Coordinates);
		Points2D.Empty(Coordinates.Num());
		Points3D.Empty(Coordinates.Num());
		if (bWithNormals)
		{
			Normals.Empty(Coordinates.Num());
		}
		if (bWithTangent)
		{
			Tangents.Empty(Coordinates.Num());
		}
	}

	/**
	 * @return the size of the polyline i.e. the count of points.
	 */
	int32 Size() const
	{
		return Points2D.Num();
	}

	/**
	 * Get the sub 2d polyline bounded by the input InBoundary in the orientation of the input InOrientation and append it to the output OutPoints
	 */
	void GetSubPolyline(const FLinearBoundary& InBoundary, const EOrientation InOrientation, TArray<FVector2d>& OutPoints) const
	{
		TPolylineApproximator<FVector2d> Approximator(Coordinates, Points2D);
		Approximator.GetSubPolyline(InBoundary, InOrientation, OutPoints);
	}

	/**
	 * Get the sub polyline bounded by the input InBoundary in the orientation of the input InOrientation and append it to the output OutPoints
	 */
	void GetSubPolyline(const FLinearBoundary& InBoundary, TArray<double>& OutCoordinates, TArray<FVector2d>& OutPoints) const
	{
		TPolylineApproximator<FVector2d> Approximator(Coordinates, Points2D);
		Approximator.GetSubPolyline(InBoundary, OutCoordinates, OutPoints);
	}

	/**
	 * Get the sub polyline bounded by the input InBoundary in the orientation of the input InOrientation and append it to the output OutPoints
	 */
	void GetSubPolyline(const FLinearBoundary& InBoundary, const EOrientation InOrientation, TArray<FVector>& OutPoints) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		Approximator3D.GetSubPolyline(InBoundary, InOrientation, OutPoints);
	}

	/**
	 * Get the sub polyline bounded by the input InBoundary in the orientation of the input InOrientation and append it to the output OutPoints
	 */
	void GetSubPolyline(const FLinearBoundary& InBoundary, TArray<double>& OutCoordinates, TArray<FVector>& OutPoints) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		Approximator3D.GetSubPolyline(InBoundary, OutCoordinates, OutPoints);
	}

	/**
	 * Reserves memory such that the polyline can contain at least Number elements.
	 *
	 * @param Number The number of elements that the polyline should be able to contain after allocation.
	 */
	void Reserve(int32 Number)
	{
		Points3D.Reserve(Number);
		Points2D.Reserve(Number);
		Coordinates.Reserve(Number);
		if (bWithNormals)
		{
			Normals.Reserve(Number);
		}
	}

	/**
	 * Empties the polyline.
	 *
	 * @param Slack (Optional) The expected usage size after empty operation. Default is 0.
	 */
	void Empty(int32 Slack = 0)
	{
		Points3D.Empty(Slack);
		Points2D.Empty(Slack);
		Normals.Empty(Slack);
		Coordinates.Empty(Slack);
	}

	void EmplaceAt(int32 Index, FSurfacicPolyline& Polyline, int32 PointIndex)
	{
		Coordinates.EmplaceAt(Index, Polyline.Coordinates[PointIndex]);

		Points2D.EmplaceAt(Index, Polyline.Points2D[PointIndex]);
		Points3D.EmplaceAt(Index, Polyline.Points3D[PointIndex]);
		if (bWithNormals)
		{
			Normals.EmplaceAt(Index, Polyline.Normals[PointIndex]);
		}
		if (bWithTangent)
		{
			Tangents.EmplaceAt(Index, Polyline.Tangents[PointIndex]);
		}
	}

	void RemoveComplementaryPoints(int32 Offset)
	{
		const int32 Count = Points2D.Num();

		TBitArray<> Markers(true, Count);

		for (int32 Index = Count - 1; Index >= 0; Index -= Offset)
		{
			Markers[Index] = false;
		}

		int32 CurrentIndex = 0;
		for (int32 Index = 0; Index < Count; ++Index)
		{
			if (Markers[Index])
			{
				Coordinates[CurrentIndex] = Coordinates[Index];
				Points2D[CurrentIndex] = Points2D[Index];
				Points3D[CurrentIndex] = Points3D[Index];
				if (bWithNormals)
				{
					Normals[CurrentIndex] = Normals[Index];
				}
				if (bWithTangent)
				{
					Tangents[CurrentIndex] = Tangents[Index];
				}
				++CurrentIndex;
			}
		}

		Coordinates.SetNum(CurrentIndex, EAllowShrinking::No);
		Points2D.SetNum(CurrentIndex, EAllowShrinking::No);
		Points3D.SetNum(CurrentIndex, EAllowShrinking::No);
		if (bWithNormals)
		{
			Normals.SetNum(CurrentIndex, EAllowShrinking::No);
		}
		if (bWithTangent)
		{
			Tangents.SetNum(CurrentIndex, EAllowShrinking::No);
		}
	}

	void Pop()
	{
		Coordinates.Pop(EAllowShrinking::No);
		Points2D.Pop(EAllowShrinking::No);
		Points3D.Pop(EAllowShrinking::No);
		if (bWithNormals)
		{
			Normals.Pop(EAllowShrinking::No);
		}
		if (bWithTangent)
		{
			Tangents.Pop(EAllowShrinking::No);
		}
	}

	void Reverse()
	{
		Algo::Reverse(Coordinates);
		Algo::Reverse(Points2D);
		Algo::Reverse(Points3D);
		if (bWithNormals)
		{
			Algo::Reverse(Normals);
		}
		if (bWithTangent)
		{
			Algo::Reverse(Tangents);
		}
	}

	double GetLength(const FLinearBoundary& InBoundary) const
	{
		TPolylineApproximator<FVector> Approximator3D(Coordinates, Points3D);
		return Approximator3D.ComputeLengthOfSubPolyline(InBoundary);
	}

	double Get2DLength(const FLinearBoundary& InBoundary) const
	{
		TPolylineApproximator<FVector2d> Approximator(Coordinates, Points2D);
		return Approximator.ComputeLengthOfSubPolyline(InBoundary);
	}

	bool IsIso(EIso Iso, double ErrorTolerance = DOUBLE_SMALL_NUMBER) const
	{
		FVector2d StartPoint = Points2D[0];
		return Algo::AllOf(Points2D, [&](const FVector2d& Point) { 
			return FMath::IsNearlyEqual(Point[Iso], StartPoint[Iso], ErrorTolerance);
			});
	}

};

} // ns UE::CADKernel
