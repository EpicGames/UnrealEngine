// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Point.h"

namespace UE::CADKernel
{

struct FNurbsSurfaceHomogeneousData
{
	bool bSwapUV;
	int32 PoleUCount;
	int32 PoleVCount;

	int32 UDegree;
	int32 VDegree;

	TArray<double> UNodalVector;
	TArray<double> VNodalVector;

	bool bIsRational;
	TArray<double> HomogeneousPoles;
};

struct FNurbsSurfaceData
{
	bool bSwapUV;
	int32 PoleUCount;
	int32 PoleVCount;

	int32 UDegree;
	int32 VDegree;

	TArray<double> UNodalVector;
	TArray<double> VNodalVector;

	// if Weights.Num() == 0  => bIsRational = false
	TArray<double> Weights;
	TArray<FVector> Poles;
};

}