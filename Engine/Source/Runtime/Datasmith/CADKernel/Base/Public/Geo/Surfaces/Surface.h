// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/EntityGeom.h"
#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Math/Boundary.h"
#include "Math/MatrixH.h"
#include "Math/Point.h"
#include "Utils/Cache.h"

namespace UE::CADKernel
{
class FGrid;
class FInfoEntity;
class FSurfacicPolyline;
struct FCurvePoint2D;
struct FCurvePoint;
struct FSurfacicSampling;
struct FNurbsSurfaceData;
struct FNurbsSurfaceHomogeneousData;

class CADKERNEL_API FSurface : public FEntityGeom
{
	friend FEntity;

protected:

	FSurfacicBoundary Boundary;
	double Tolerance3D;

	/**
	 * Tolerance along Iso U/V is very costly to compute and not accurate.
	 * A first approximation is based on the surface boundary along U and along V
	 * Indeed, defining a Tolerance2D has no sense has the boundary length along an Iso could be very very huge compare to the boundary length along the other Iso like [[0, 1000] [0, 1]]
	 * The tolerance along an iso is the length of the boundary along this iso divided by 100 000: if the curve length in 3d is 10m, the tolerance is 0.01mm
	 * In the used, a local tolerance has to be estimated
	 */
	TCache<FSurfacicTolerance> MinToleranceIso;

private:

	virtual void InitBoundary()
	{
		Boundary.Set();
	}

protected:

	FSurface() = default;

	FSurface(double InToleranceGeometric)
		: FEntityGeom()
		, Tolerance3D(InToleranceGeometric)
	{

	}

	FSurface(double InToleranceGeometric, const FSurfacicBoundary& InBoundary)
		: FEntityGeom()
		, Boundary(InBoundary)
		, Tolerance3D(InToleranceGeometric)
	{
		Boundary.IsValid();
	}

	FSurface(double InToleranceGeometric, double UMin, double UMax, double VMin, double VMax)
		: FEntityGeom()
		, Tolerance3D(InToleranceGeometric)
	{
		Boundary.Set(UMin, UMax, VMin, VMax);
	}

	void ComputeDefaultMinToleranceIso()
	{
		MinToleranceIso.Set(Boundary[EIso::IsoU].ComputeMinimalTolerance(), Boundary[EIso::IsoV].ComputeMinimalTolerance());
	}

public:

	static TSharedPtr<FSurface> MakeBezierSurface(const double InToleranceGeometric, int32 InUDegre, int32 InVDegre, const TArray<FVector>& InPoles);
	static TSharedPtr<FSurface> MakeConeSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InStartRadius, double InConeAngle, const FSurfacicBoundary& InBoundary);
	static TSharedPtr<FSurface> MakeCylinderSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const double InRadius, const FSurfacicBoundary& InBoundary);
	static TSharedPtr<FSurface> MakeNurbsSurface(const double InToleranceGeometric, const FNurbsSurfaceData& NurbsData);
	static TSharedPtr<FSurface> MakeNurbsSurface(const double InToleranceGeometric, const FNurbsSurfaceHomogeneousData& NurbsData);
	static TSharedPtr<FSurface> MakePlaneSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, const FSurfacicBoundary& InBoundary);
	static TSharedPtr<FSurface> MakeSphericalSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InRadius, const FSurfacicBoundary& InBoundary);
	static TSharedPtr<FSurface> MakeTorusSurface(const double InToleranceGeometric, const FMatrixH& InMatrix, double InMajorRadius, double InMinorRadius, const FSurfacicBoundary& InBoundary);

	virtual void ValidateUVPoints(TArray<FVector2d>& UVPoints) const {}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		// Surface's type is serialize because it is used to instantiate the correct entity on de-serialization (@see Deserialize(FCADKernelArchive& Archive))
		if (Ar.IsSaving())
		{
			ESurface SurfaceType = GetSurfaceType();
			Ar << SurfaceType;
		}
		FEntityGeom::Serialize(Ar);

		Ar << Tolerance3D;
		Ar << MinToleranceIso;
		Ar << Boundary;
	}

	/**
	 * Specific method for surface family to instantiate the correct derived class of FCurve
	 */
	static TSharedPtr<FSurface> Deserialize(FCADKernelArchive& Archive);

	virtual ~FSurface() = default;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Surface;
	}

	virtual ESurface GetSurfaceType() const = 0;

	const FSurfacicBoundary& GetBoundary() const
	{
		ensureCADKernel(Boundary.IsValid());
		return Boundary;
	};

	void TrimBoundaryTo(const FSurfacicBoundary NewLimit)
	{
		Boundary.TrimAt(NewLimit);
	}

	void ExtendBoundaryTo(const FSurfacicBoundary MaxLimit)
	{
		Boundary.ExtendTo(MaxLimit);
	}

	virtual TSharedPtr<FEntityGeom> ApplyMatrix(const FMatrixH& InMatrix) const = 0;

	/**
	 * Tolerance along Iso U/V is very costly to compute and not accurate.
	 * A first approximation is based on the surface parametric space length along U and along V
	 * Indeed, defining a Tolerance2D has no sense has the boundary length along an Iso could be very very huge compare to the boundary length along the other Iso like [[0, 1000] [0, 1]]
	 * @see FBoundary::ComputeMinimalTolerance
	 * In the used, a local tolerance has to be estimated
	 */
	const FSurfacicTolerance& GetIsoTolerances() const
	{
		ensureCADKernel(MinToleranceIso.IsValid());
		return *MinToleranceIso;
	}

	/**
	 * Return the minimum tolerance in the parametric space of the surface along the specified axis
	 * With Tolerance3D = FSysteme.GeometricalTolerance
	 * @see FBoundary::ComputeMinimalTolerance
	 */
	double GetIsoTolerance(EIso Iso) const
	{
		ensureCADKernel(MinToleranceIso.IsValid());
		return (*MinToleranceIso)[Iso];
	}

	double Get3DTolerance() const
	{
		return Tolerance3D;
	}


	virtual void EvaluatePoint(const FVector2d& InSurfacicCoordinate, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder = 0) const = 0;

	virtual void EvaluatePoints(const TArray<FVector2d>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder = 0) const;
	virtual void EvaluatePoints(const TArray<FCurvePoint2D>& InSurfacicCoordinates, TArray<FSurfacicPoint>& OutPoint3D, int32 InDerivativeOrder = 0) const;
	virtual void EvaluatePoints(const TArray<FCurvePoint2D>& InSurfacicCoordinates, TArray<FCurvePoint>& OutPoint3D, int32 InDerivativeOrder = 0) const;
	virtual void EvaluatePoints(FSurfacicPolyline& Polyline) const;
	virtual void EvaluatePoints(const TArray<FCurvePoint2D>& Points2D, FSurfacicPolyline& Polyline) const;

	/**
	 * X = Rho cos(Alpha)
	 * Y = Rho sin(Alpha)
	 * Z = Z
	 *
	 * YZ => cylindrical projection
	 * XY => Plan projection along rotation axis
	 *
	 * For Normal(u,v) parallel Rotation Axis -> use Plan projection
	 * For Normal(u,v) perpendicular to Rotation Axis -> use cylindrical projection
	 */
	virtual FVector2d EvaluatePointInCylindricalSpace(const FVector2d& InSurfacicCoordinate) const { return FVector2d::ZeroVector; }
	virtual void EvaluatePointGridInCylindricalSpace(const FCoordinateGrid& Coordinates, TArray<FVector2d>&) const { ; }

	virtual void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const;
	void EvaluateGrid(FGrid& Grid) const;

	virtual void EvaluateNormals(const TArray<FVector2d>& Points2D, TArray<FVector3f>& Normals) const;

	virtual void EvaluatePointAndNormal(const FVector2d& InPoint2D, FVector& Point, FVector3f& Normal) const
	{
		int32 DerivativeOrder = 1;
		FSurfacicPoint SurfacicPoint;
		EvaluatePoint(InPoint2D, SurfacicPoint, DerivativeOrder);

		Point = SurfacicPoint.Point;

		FVector NormalD = SurfacicPoint.GradientU ^ SurfacicPoint.GradientV;
		Normal = (FVector3f)NormalD.GetSafeNormal();
	}

	virtual FVector EvaluateNormal(const FVector2d& InPoint2D) const
	{
		int32 DerivativeOrder = 1;
		FSurfacicPoint Point3D;
		EvaluatePoint(InPoint2D, Point3D, DerivativeOrder);

		FVector Normal = Point3D.GradientU ^ Point3D.GradientV;
		Normal.Normalize();
		return Normal;
	}

	/**
	 * Divide the parametric space in the desired number of regular subdivisions and compute the associated PointGrid
	 */
	void Sample(const FSurfacicBoundary& Bounds, int32 NumberOfSubdivisions[2], FSurfacicSampling& OutPointSampling) const;

	/**
	 * Divide the parametric space in the desired number of regular subdivisions and compute the associated CoordinateGrid
	 */
	void Sample(const FSurfacicBoundary& Boundary, int32 NumberOfSubdivisions[2], FCoordinateGrid& OutCoordinateSampling) const;

	/**
	 * Generate a pre-sampling of the surface saved in OutCoordinates.
	 * This sampling is light enough to allow a fast computation of the grid, precise enough to compute accurately meshing criteria
	 */
	virtual void Presample(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& OutCoordinates);

	virtual void LinesNotDerivables(const FSurfacicBoundary& Bounds, int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const
	{
		OutNotDerivableCoordinates.Empty();
	}

	void LinesNotDerivables(int32 InDerivativeOrder, FCoordinateGrid& OutNotDerivableCoordinates) const
	{
		LinesNotDerivables(Boundary, InDerivativeOrder, OutNotDerivableCoordinates);
	}

	/**
	 * A surface is closed along an iso axis if it's connected to itself e.g. the edge carried by the iso curve U = UMin is linked to the edge carried by the iso curve U = UMax
	 * A complete sphere, torus are example of surface closed along U and V
	 * A cylinder with circle section or a cone are example of surface closed along an iso
	 */
	virtual void IsSurfaceClosed(bool& bOutClosedAlongU, bool& bOutClosedAlongV) const
	{
		bOutClosedAlongU = false;
		bOutClosedAlongV = false;
	}

	void PresampleIsoCircle(const FSurfacicBoundary& InBoundaries, FCoordinateGrid& Coordinates, EIso Iso)
	{
		constexpr double SixOverPi = 6 / DOUBLE_PI;
		double Delta = InBoundaries.Length(Iso);
		int32 SampleCount = /*FMath::Max(3, */ (int32)(Delta * SixOverPi + 1)/*)*/;
		Delta /= SampleCount;

		Coordinates[Iso].Empty(SampleCount + 1);
		double Sample = InBoundaries[Iso].GetMin();
		for (int32 Index = 0; Index <= SampleCount; ++Index)
		{
			Coordinates[Iso].Add(Sample);
			Sample += Delta;
		}
	}

	/**
	 * This function return the scale of the input Axis.
	 * This function is useful to estimate tolerance when scales are defined in the Matrix
	 * @param InAxis a vetor of length 1
	 * @param InMatrix
	 * @param InOrigin = InMatrix*FVector::ZeroVector
	 */
	static double ComputeScaleAlongAxis(const FVector& InAxis, const FMatrixH& InMatrix, const FVector& InOrigin)
	{
		FVector Point = InMatrix.Multiply(InAxis);
		double Length = FVector::Distance(InOrigin, Point);
		return Length;
	};


};
} // namespace UE::CADKernel

