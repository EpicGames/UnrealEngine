// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace UE::CADKernel
{
	class FSurface;

	namespace SurfaceUtilities
	{
		TArray<TArray<FVector3f>> CADKERNEL_API GetPoles(const UE::CADKernel::FSurface& Surface);
		bool CADKERNEL_API IsPlanar(const UE::CADKernel::FSurface& Surface);
	}
}