// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Point.h"


enum EAABBBoundary : uint32
{
	XMax = 0x00000000u,
	YMax = 0x00000000u,
	ZMax = 0x00000000u,
	XMin = 0x00000001u,
	YMin = 0x00000002u,
	ZMin = 0x00000004u,
};

ENUM_CLASS_FLAGS(EAABBBoundary);

namespace UE::CADKernel
{
template<class PointType, int Dimension = 3>
class TAABB
{

protected:
	PointType MinCorner;
	PointType MaxCorner;

public:
	TAABB()
	{
	}

	TAABB(const PointType& InMinCorner, const PointType& InMaxCorner)
		: MinCorner(InMinCorner)
		, MaxCorner(InMaxCorner)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, TAABB& AABB)
	{
		Ar << AABB.MinCorner;
		Ar << AABB.MaxCorner;
		return Ar;
	}

	bool IsValid() const
	{
		for (int32 Axis = 0; Axis < Dimension; Axis++)
		{
			if (MinCorner[Axis] > MaxCorner[Axis])
			{
				return false;
			}
		}
		return true;
	}

	void Empty()
	{
	}

	bool Contains(const PointType& Point) const
	{
		for (int32 Axis = 0; Axis < Dimension; Axis++)
		{
			if ((Point[Axis] < MinCorner[Axis]) || (Point[Axis] > MaxCorner[Axis]))
			{
				return false;
			}
		}
		return true;
	}

	void SetMinSize(double MinSize)
	{
		for (int32 Axis = 0; Axis < Dimension; Axis++)
		{
			double AxisSize = GetSize(Axis);
			if (AxisSize < MinSize)
			{
				double Offset = (MinSize - AxisSize) / 2;
				MinCorner[Axis] -= Offset;
				MaxCorner[Axis] += Offset;
			}
		}
	}

	double GetMaxSize() const
	{
		double MaxSideSize = 0;
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			double Size = GetSize(Index);
			if (Size > MaxSideSize)
			{
				MaxSideSize = Size;
			}
		}
		return MaxSideSize;
	}

	double GetSize(int32 Axis) const
	{
		return MaxCorner[Axis] - MinCorner[Axis];
	}

	double DiagonalLength() const
	{
		return PointType::Distance(MaxCorner, MinCorner);
	}

	PointType Diagonal() const
	{
		return MaxCorner - MinCorner;
	}

	bool Contains(const TAABB& Aabb) const
	{
		return IsValid() && Aabb.IsValid() && Contains(Aabb.MinCorner) && Contains(Aabb.MaxCorner);
	}

	const PointType& GetMin() const
	{
		return MinCorner;
	}

	const PointType& GetMax() const
	{
		return MaxCorner;
	}

	TAABB& operator+= (const double* Point)
	{
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			if (Point[Index] < MinCorner[Index])
			{
				MinCorner[Index] = Point[Index];
			}
			if (Point[Index] > MaxCorner[Index])
			{
				MaxCorner[Index] = Point[Index];
			}
		}
		return *this;
	}

	TAABB& operator+= (const PointType& Point)
	{
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			if (Point[Index] < MinCorner[Index])
			{
				MinCorner[Index] = Point[Index];
			}
			if (Point[Index] > MaxCorner[Index])
			{
				MaxCorner[Index] = Point[Index];
			}
		}
		return *this;
	}


	TAABB& operator+= (const TArray<PointType>& Points)
	{
		for (const PointType& Point : Points)
		{
			*this += Point;
		}
		return *this;
	}


	void Offset(double Offset)
	{
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			MinCorner[Index] -= Offset;
			MaxCorner[Index] += Offset;
		}
	}

	TAABB& operator+= (const TAABB& aabb)
	{
		*this += aabb.MinCorner;
		*this += aabb.MaxCorner;
		return *this;
	}

	TAABB operator+ (const PointType& Point) const
	{
		TAABB Other = *this;
		Other += Point;
		return Other;
	}


	TAABB operator+ (const TAABB& Aabb) const
	{
		TAABB Other = *this;
		Other += Aabb;
		return Other;
	}
};

template<>
inline TAABB<FVector2d, 2>::TAABB()
	: MinCorner(FVectorUtil::FarawayPoint2D)
	, MaxCorner(-FVectorUtil::FarawayPoint2D)
{
}

template<>
inline TAABB<FVector, 3>::TAABB()
	: MinCorner(FVectorUtil::FarawayPoint3D)
	, MaxCorner(-FVectorUtil::FarawayPoint3D)
{
}

template<>
inline void TAABB<FVector2d, 2>::Empty()
{
	MinCorner = FVectorUtil::FarawayPoint2D;
	MaxCorner = -FVectorUtil::FarawayPoint2D;
}

template<>
inline void TAABB<FVector, 3>::Empty()
{
	MinCorner = FVectorUtil::FarawayPoint3D;
	MaxCorner = -FVectorUtil::FarawayPoint3D;
}


class FAABB : public TAABB<FVector>
{

public:
	FAABB()
		: TAABB<FVector>()
	{
	}

	FAABB(const FVector& InMinCorner, const FVector& InMaxCorner)
		: TAABB<FVector>(InMinCorner, InMaxCorner)
	{
	}

	FVector GetCorner(int32 Corner) const
	{
		return FVector(
			Corner & EAABBBoundary::XMin ? MinCorner[0] : MaxCorner[0],
			Corner & EAABBBoundary::YMin ? MinCorner[1] : MaxCorner[1],
			Corner & EAABBBoundary::ZMin ? MinCorner[2] : MaxCorner[2]
		);
	}
};

class CADKERNEL_API FAABB2D : public TAABB<FVector2d, 2>
{
public:
	FAABB2D()
		: TAABB<FVector2d, 2>()
	{
	}

	FAABB2D(const FVector& InMinCorner, const FVector& InMaxCorner)
		: TAABB<FVector2d, 2>(FVector2d(InMinCorner.X, InMinCorner.Y), FVector2d(InMaxCorner.X, InMaxCorner.Y))
	{
	}

	FVector2d GetCorner(int32 CornerIndex) const
	{
		return FVector2d(
			CornerIndex & EAABBBoundary::XMin ? MinCorner[0] : MaxCorner[0],
			CornerIndex & EAABBBoundary::YMin ? MinCorner[1] : MaxCorner[1]
		);
	}

};

} // namespace UE::CADKernel

