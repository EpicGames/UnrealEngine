// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/Boundary.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
class FGrid;
class FNURBSCurve;
class FNURBSSurface;
class FNURBSSurface;
class FSurfacicBoundary;
struct FCoordinateGrid;
struct FCurvePoint2D;
struct FCurvePoint;
struct FLinearBoundary;
struct FSurfacicPoint;
struct FSurfacicSampling;

namespace BSpline
{
/**
 * Compute the values of Bernstein polynomial
 */
CADKERNEL_API void Bernstein(int32 Degree, double InCoordinateU, double* BernsteinValuesAtU, double* BernsteinGradientsAtU, double* BernsteinLaplaciansAtU);

CADKERNEL_API void FindNotDerivableParameters(const FNURBSCurve&, int32 InDerivativeOrder, const FLinearBoundary& Boundary, TArray<double>& OutNotDerivableParameters);
CADKERNEL_API void FindNotDerivableParameters(const FNURBSSurface&, int32 InDerivativeOrder, const FSurfacicBoundary& Boundary, FCoordinateGrid& OutNotDerivableParameters);

CADKERNEL_API void Evaluate2DPoint(const FNURBSCurve&, double Coordinate, FCurvePoint2D& OutPoint, int32 DerivativeOrder);
CADKERNEL_API void EvaluatePoint(const FNURBSCurve&, double Coordinate, FCurvePoint& OutPoint, int32 DerivativeOrder);

CADKERNEL_API void EvaluatePoint(const FNURBSSurface&, const FVector2d& InPoint2D, FSurfacicPoint& OutPoint3D, int32 InDerivativeOrder);
CADKERNEL_API void EvaluatePointGrid(const FNURBSSurface&, const FCoordinateGrid& Coords, FSurfacicSampling& OutPoints, bool bComputeNormals);

CADKERNEL_API TSharedPtr<FNURBSCurve> DuplicateNurbsCurveWithHigherDegree(int32 degre, const FNURBSCurve& InCurve);
}
}

