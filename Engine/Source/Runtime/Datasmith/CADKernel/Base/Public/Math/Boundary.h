// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Math/MathConst.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
/**
 * MINIMAL_UNIT_LINEAR_TOLERANCE allows to define the minimal tolerance value of a parametric space
 * @see FLinearBoundary::ComputeMinimalTolerance
 */
#define MINIMAL_UNIT_LINEAR_TOLERANCE 1e-5

struct CADKERNEL_API FLinearBoundary
{

	/** A default boundary (0., 1.)*/
	static const FLinearBoundary DefaultBoundary;

	double Min;
	double Max;

	FLinearBoundary()
	{
		Min = 0.;
		Max = 1.;
	}

	FLinearBoundary(const FLinearBoundary& Boundary)
		: Min(Boundary.Min)
		, Max(Boundary.Max)
	{
	}

	FLinearBoundary(const double UMin, const double UMax)
	{
		Set(UMin, UMax);
	}

	FLinearBoundary(const FLinearBoundary& Boundary, const double OffsetTolerance)
		: Min(Boundary.Min - OffsetTolerance)
		, Max(Boundary.Max + OffsetTolerance)
	{
	}

	FLinearBoundary(const double UMin, const double UMax, const double OffsetTolerance)
	{
		Set(UMin, UMax);
		Offset(OffsetTolerance);
	}

	friend FArchive& operator<<(FArchive& Ar, FLinearBoundary& Boundary)
	{
		Ar.Serialize(&Boundary, sizeof(FLinearBoundary));
		return Ar;
	}

	constexpr double GetMin() const
	{
		return Min;
	}

	constexpr double GetMax() const
	{
		return Max;
	}

	constexpr double GetAt(const double Coordinate) const
	{
		return Min + (Max - Min) * Coordinate;
	}

	constexpr double GetMiddle() const
	{
		return (Min + Max) * 0.5;
	}

	double Size() const { return Max - Min; }

	void SetMin(const double Coordinates)
	{
		FMath::GetMinMax(Coordinates, Max, Min, Max);
	}

	void SetMax(const double Coordinates)
	{
		FMath::GetMinMax(Min, Coordinates, Min, Max);
	}

	void Set(const double InUMin = 0., const double InUMax = 1.)
	{
		FMath::GetMinMax(InUMin, InUMax, Min, Max);
	}

	/**
	 * Set the boundary with the min and max of the array
	 */
	void Set(const TArray<double>& Coordinates)
	{
		Init();
		for (const double& Coordinate : Coordinates)
		{
			ExtendTo(Coordinate);
		}
	}


	bool IsValid() const
	{
		return Min <= Max;
	}

	bool Contains(const double Coordinate) const
	{
		return RealCompare(Coordinate, Min) >= 0 && RealCompare(Coordinate, Max) <= 0;
	}

	double Length() const
	{
		return GetMax() - GetMin();
	}

	/**
	 * Return true if the parametric domain is to small
	 */
	bool IsDegenerated() const
	{
		double DeltaU = (Max - Min);
		return (DeltaU < DOUBLE_SMALL_NUMBER);
	}

	/**
	 * Compute the minimal tolerance of the parametric domain i.e.
	 * ToleranceMin = Boundary.Length() * MINIMAL_UNIT_LINEAR_TOLERANCE
	 * e.g. for a curve of 1m with a parametric space define between [0, 1], the parametric tolerance is 0.01
	 * This is a minimal value that has to be replace with a more accurate value when its possible
	 */
	double ComputeMinimalTolerance() const
	{
		return Length() * MINIMAL_UNIT_LINEAR_TOLERANCE;
	}

	/**
	 * If a coordinate is outside the bounds, set the coordinate at the closed limit
	 */
	void MoveInsideIfNot(double& Coordinate, const double Tolerance = DOUBLE_SMALL_NUMBER) const
	{
		if (Coordinate <= Min)
		{
			Coordinate = Min + Tolerance;
		}
		else if (Coordinate >= Max)
		{
			Coordinate = Max - Tolerance;
		}
	}

	void Offset(const double Tolerance = DOUBLE_SMALL_NUMBER)
	{
		Min -= Tolerance;
		Max += Tolerance;
	}

	/**
	 * Uses to initiate a boundary computation with ExtendTo
	 */
	void Init()
	{
		Min = HUGE_VALUE;
		Max = -HUGE_VALUE;
	}

	void ExtendTo(double MinCoordinate, double MaxCoordinate)
	{
		FMath::GetMinMax(MinCoordinate, MaxCoordinate);
		Min = FMath::Min(Min, MinCoordinate);
		Max = FMath::Max(Max, MaxCoordinate);
	}

	void TrimAt(const FLinearBoundary& MaxBound)
	{
		if (Max < MaxBound.Min || Min > MaxBound.Max)
		{
			*this = MaxBound;
			return;
		}

		Min = FMath::Max(Min, MaxBound.Min);
		Max = FMath::Min(Max, MaxBound.Max);
	}

	void ExtendTo(const FLinearBoundary& MaxBound)
	{
		Min = FMath::Min(Min, MaxBound.Min);
		Max = FMath::Max(Max, MaxBound.Max);
	}

	void ExtendTo(const double Coordinate)
	{
		if (Coordinate < Min)
		{
			Min = Coordinate;
		}

		if (Coordinate > Max)
		{
			Max = Coordinate;
		}
	}

	void RestrictTo(const FLinearBoundary& MaxBound)
	{
		if (MaxBound.Min > Min)
		{
			Min = MaxBound.Min;
		}
		if (MaxBound.Max < Max)
		{
			Max = MaxBound.Max;
		}
	}

	/**
	 * If the boundary width is near or equal to zero, it's widened by +/- DOUBLE_SMALL_NUMBER
	 */
	void WidenIfDegenerated()
	{
		if (FMath::IsNearlyEqual(Min, Max))
		{
			Min -= DOUBLE_SMALL_NUMBER;
			Max += DOUBLE_SMALL_NUMBER;
		}
	}

	FLinearBoundary& operator=(const FLinearBoundary& InBounds)
	{
		Min = InBounds.Min;
		Max = InBounds.Max;
		return *this;
	}

};

class CADKERNEL_API FSurfacicBoundary
{
private:
	FLinearBoundary UVBoundaries[2];

public:
	/** A default boundary (0., 1., 0., 1.)*/
	static const FSurfacicBoundary DefaultBoundary;

	FSurfacicBoundary() = default;

	FSurfacicBoundary(const double InUMin, const double InUMax, const double InVMin, const double InVMax)
	{
		UVBoundaries[EIso::IsoU].Set(InUMin, InUMax);
		UVBoundaries[EIso::IsoV].Set(InVMin, InVMax);
	}

	FSurfacicBoundary(const double InUMin, const double InUMax, const double InVMin, const double InVMax, const double OffsetTolerance)
	{
		UVBoundaries[EIso::IsoU].Set(InUMin, InUMax);
		UVBoundaries[EIso::IsoV].Set(InVMin, InVMax);
		Offset(OffsetTolerance);
	}

	FSurfacicBoundary(const FVector2d& Point1, const FVector2d& Point2)
	{
		Set(Point1, Point2);
	}

	FSurfacicBoundary(const FVector2d& Point1, const FVector2d& Point2, const double OffsetTolerance)
	{
		Set(Point1, Point2);
		Offset(OffsetTolerance);
	}

	void Set(const FVector2d& Point1, const FVector2d& Point2)
	{
		UVBoundaries[EIso::IsoU].Set(Point1.X, Point2.X);
		UVBoundaries[EIso::IsoV].Set(Point1.Y, Point2.Y);
	}

	friend FArchive& operator<<(FArchive& Ar, FSurfacicBoundary& Boundary)
	{
		Ar << Boundary[EIso::IsoU];
		Ar << Boundary[EIso::IsoV];
		return Ar;
	}

	void Set(const FLinearBoundary& BoundU, const FLinearBoundary& BoundV)
	{
		UVBoundaries[EIso::IsoU] = BoundU;
		UVBoundaries[EIso::IsoV] = BoundV;
	}

	void Set(const double InUMin, const double InUMax, const double InVMin, const double InVMax)
	{
		UVBoundaries[EIso::IsoU].Set(InUMin, InUMax);
		UVBoundaries[EIso::IsoV].Set(InVMin, InVMax);
	}

	void Set()
	{
		UVBoundaries[EIso::IsoU].Set();
		UVBoundaries[EIso::IsoV].Set();
	}

	/**
	 * Set the boundary with the min and max of this array
	 */
	void Set(const TArray<FVector2d>& Points)
	{
		Init();
		for (const FVector2d& Point : Points)
		{
			ExtendTo(Point);
		}
	}

	const FLinearBoundary& Get(EIso Type) const
	{
		return UVBoundaries[Type];
	}

	bool IsValid() const
	{
		return UVBoundaries[EIso::IsoU].IsValid() && UVBoundaries[EIso::IsoV].IsValid();
	}

	/**
	 * Return true if the parametric domain is to small
	 */
	bool IsDegenerated() const
	{
		return UVBoundaries[EIso::IsoU].IsDegenerated() || UVBoundaries[EIso::IsoV].IsDegenerated();
	}

	ESituation IsInside(const FSurfacicBoundary& OtherBoundary, const FSurfacicTolerance& Tolerance2D)
	{
		int32 Inside = 0;
		int32 Outside = 0;
		TFunction<void(double, double, double)> CheckInside = [&](double LeftSide, double RigthSide, double Tolerance)
		{
			if (LeftSide + Tolerance < RigthSide)
			{
				Inside++;
			}
			else if (RigthSide + Tolerance < LeftSide)
			{
				Outside++;
			}
		};

		CheckInside(OtherBoundary[EIso::IsoU].GetMin(), UVBoundaries[EIso::IsoU].GetMin(), Tolerance2D[EIso::IsoU]);
		CheckInside(OtherBoundary[EIso::IsoV].GetMin(), UVBoundaries[EIso::IsoV].GetMin(), Tolerance2D[EIso::IsoV]);
		CheckInside(UVBoundaries[EIso::IsoU].GetMax(), OtherBoundary[EIso::IsoU].GetMax(), Tolerance2D[EIso::IsoU]);
		CheckInside(UVBoundaries[EIso::IsoV].GetMax(), OtherBoundary[EIso::IsoV].GetMax(), Tolerance2D[EIso::IsoV]);

		if (Inside > 2)
		{
			return ESituation::Inside;
		}
		if (Outside > 2)
		{
			return ESituation::Outside;
		}
		return ESituation::Undefined;
	}

	/**
	 * Uses to initiate a boundary computation with ExtendTo
	 */
	void Init()
	{
		UVBoundaries[EIso::IsoU].Init();
		UVBoundaries[EIso::IsoV].Init();
	}

	void TrimAt(const FSurfacicBoundary& MaxLimit)
	{
		UVBoundaries[EIso::IsoU].TrimAt(MaxLimit[EIso::IsoU]);
		UVBoundaries[EIso::IsoV].TrimAt(MaxLimit[EIso::IsoV]);
	}

	void ExtendTo(const FSurfacicBoundary& MaxLimit)
	{
		UVBoundaries[EIso::IsoU].ExtendTo(MaxLimit[EIso::IsoU]);
		UVBoundaries[EIso::IsoV].ExtendTo(MaxLimit[EIso::IsoV]);
	}

	void ExtendTo(const FVector2d& Point)
	{
		UVBoundaries[EIso::IsoU].ExtendTo(Point.X);
		UVBoundaries[EIso::IsoV].ExtendTo(Point.Y);
	}

	void ExtendTo(const FVector& Point)
	{
		UVBoundaries[EIso::IsoU].ExtendTo(Point.X);
		UVBoundaries[EIso::IsoV].ExtendTo(Point.Y);
	}

	void RestrictTo(const FSurfacicBoundary& MaxBound)
	{
		UVBoundaries[EIso::IsoU].RestrictTo(MaxBound.UVBoundaries[EIso::IsoU]);
		UVBoundaries[EIso::IsoV].RestrictTo(MaxBound.UVBoundaries[EIso::IsoV]);
	}

	/**
	 * If Along each axis, the bound width is near equal to zero, it's widened by +/- DOUBLE_SMALL_NUMBER
	 */
	void WidenIfDegenerated()
	{
		UVBoundaries[EIso::IsoU].WidenIfDegenerated();
		UVBoundaries[EIso::IsoV].WidenIfDegenerated();
	}

	/**
	 * If a point is outside the bounds, set the coordinate to insert the point inside the bounds
	 */
	void MoveInsideIfNot(FVector& Point, const double Tolerance = DOUBLE_SMALL_NUMBER) const
	{
		UVBoundaries[EIso::IsoU].MoveInsideIfNot(Point.X, Tolerance);
		UVBoundaries[EIso::IsoV].MoveInsideIfNot(Point.Y, Tolerance);
	}

	/**
	 * If a point is outside the bounds, set the coordinate to insert the point inside the bounds
	 */
	void MoveInsideIfNot(FVector2d& Point, const double Tolerance = DOUBLE_SMALL_NUMBER) const
	{
		UVBoundaries[EIso::IsoU].MoveInsideIfNot(Point.X, Tolerance);
		UVBoundaries[EIso::IsoV].MoveInsideIfNot(Point.Y, Tolerance);
	}

	double Length(const EIso& Iso) const
	{
		return UVBoundaries[Iso].Length();
	}

	constexpr const FLinearBoundary& operator[](const EIso& Iso) const
	{
		return UVBoundaries[Iso];
	}

	constexpr FLinearBoundary& operator[](const EIso& Iso)
	{
		return UVBoundaries[Iso];
	}

	void Offset(const double Tolerance = DOUBLE_SMALL_NUMBER)
	{
		UVBoundaries[EIso::IsoU].Offset(Tolerance);
		UVBoundaries[EIso::IsoV].Offset(Tolerance);
	}

};
}

