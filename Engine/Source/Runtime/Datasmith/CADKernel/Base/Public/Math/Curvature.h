// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"

namespace UE::CADKernel
{
struct FIsoCurvature
{
	double Min = HUGE_VALUE;
	double Max = 0;
};

struct FSurfaceCurvature
{
	FIsoCurvature Curvatures[2];

	constexpr const FIsoCurvature& operator[](const EIso& Iso) const
	{
		return Curvatures[Iso];
	}

	constexpr FIsoCurvature& operator[](const EIso& Iso)
	{
		return Curvatures[Iso];
	}
};

}
