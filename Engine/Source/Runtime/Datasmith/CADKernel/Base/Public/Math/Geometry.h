// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Math/Aabb.h"
#include "Math/MathConst.h"
#include "Math/MatrixH.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
enum EPolygonSide : uint8
{
	Side01 = 0,
	Side12,
	Side20,
	Side23,
	Side30,
};

namespace IntersectionTool
{
CADKERNEL_API void SetTolerance(const double Tolerance);
}


/**
 * https://en.wikipedia.org/wiki/Circumscribed_circle#Cartesian_coordinates_2
 * With A = (0, 0)
 */
CADKERNEL_API inline FVector2d ComputeCircumCircleCenter(const FVector2d& InPoint0, const FVector2d& InPoint1, const FVector2d& InPoint2)
{
	FVector2d Segment_P0_P1 = InPoint1 - InPoint0;
	FVector2d Segment_P0_P2 = InPoint2 - InPoint0;

	// D = 2(BuCv - BvCu)
	double D = 2. * Segment_P0_P1 ^ Segment_P0_P2;
	if (FMath::IsNearlyZero(D, SMALL_NUMBER_SQUARE))
	{
		return FVector2d::ZeroVector;
	}

	// CenterU  = 1/D * (Cv.|B|.|B| - By.|C|.|C|) = 1/D * CBv ^ SquareNorms 
	// CenterV  = 1/D * (Bu.|B|.|B| - Cu.|C|.|C|) = -1/D * SquareNorms ^ CBu
	double SquareNormB = Segment_P0_P1.SquaredLength();
	double SquareNormC = Segment_P0_P2.SquaredLength();
	double CenterU = (SquareNormB * Segment_P0_P2.Y - SquareNormC * Segment_P0_P1.Y) / D;
	double CenterV = (SquareNormC * Segment_P0_P1.X - SquareNormB * Segment_P0_P2.X) / D;

	return FVector2d(CenterU, CenterV) + InPoint0;
}

CADKERNEL_API inline FVector ComputeCircumCircleCenter(const FVector& Point0, const FVector& Point1, const FVector& Point2)
{
	FVector Test = Point1 - Point0;
	FVector AxisZ = (Point1 - Point0) ^ (Point2 - Point0);
	AxisZ.Normalize();

	FVector AxisX = Point1 - Point0;
	AxisX.Normalize();
	FVector AxisY = AxisZ ^ AxisX;

	FMatrixH Matrix(Point0, AxisX, AxisY, AxisZ);
	FMatrixH MatrixInverse = Matrix;
	MatrixInverse.Inverse();

	FVector2d D2Point1 = FVectorUtil::FromVector(MatrixInverse * Point1);
	FVector2d D2Point2 = FVectorUtil::FromVector(MatrixInverse * Point2);

	double D = 2. * D2Point1 ^ D2Point2;
	if (FMath::IsNearlyZero(D, SMALL_NUMBER_SQUARE))
	{
		return FVector::ZeroVector;
	}

	double SquareNormB = D2Point1.SquaredLength();
	double SquareNormC = D2Point2.SquaredLength();

	double CenterU = (SquareNormB * D2Point2.Y - SquareNormC * D2Point1.Y) / D;
	double CenterV = (SquareNormC * D2Point1.X - SquareNormB * D2Point2.X) / D;

	FVector Center2D(CenterU, CenterV, 0);
	return Matrix * Center2D;
}


CADKERNEL_API inline FVector2d ComputeCircumCircleCenterAndSquareRadius(const FVector2d& InPoint0, const FVector2d& InPoint1, const FVector2d& InPoint2, double& OutSquareRadius)
{
	FVector2d Segment_P0_P1 = InPoint1 - InPoint0;
	FVector2d Segment_P0_P2 = InPoint2 - InPoint0;

	double D = 2. * Segment_P0_P1 ^ Segment_P0_P2;
	if (FMath::IsNearlyZero(D, SMALL_NUMBER_SQUARE))
	{
		OutSquareRadius = 0;
		return FVector2d::ZeroVector;
	}

	double SquareNormB = Segment_P0_P1.SquaredLength();
	double SquareNormC = Segment_P0_P2.SquaredLength();
	double CenterU = (SquareNormB * Segment_P0_P2.Y - SquareNormC * Segment_P0_P1.Y) / D;
	double CenterV = (SquareNormC * Segment_P0_P1.X - SquareNormB * Segment_P0_P2.X) / D;

	FVector2d Center(CenterU, CenterV);
	OutSquareRadius = Center.SquaredLength();

	return Center + InPoint0;
}

/**
 * @param OutCoordinate: the coordinate of the projected point in the segment AB (coodinate of A = 0 and of B = 1)
 * @return Projected point
 */
template<class PointType>
inline PointType ProjectPointOnSegment(const PointType& Point, const PointType& InSegmentA, const PointType& InSegmentB, double& OutCoordinate, bool bRestrictCoodinateToInside = true)
{
	PointType Segment = InSegmentB - InSegmentA;

	double SquaredLength = Segment | Segment;

	if (SquaredLength <= 0.0)
	{
		OutCoordinate = 0.0;
		return InSegmentA;
	}
	else
	{
		PointType APoint = Point - InSegmentA;
		OutCoordinate = (APoint | Segment) / SquaredLength;

		if (bRestrictCoodinateToInside)
		{
			if (OutCoordinate < 0.0)
			{
				OutCoordinate = 0.0;
				return InSegmentA;
			}

			if (OutCoordinate > 1.0)
			{
				OutCoordinate = 1.0;
				return InSegmentB;
			}
		}

		PointType ProjectedPoint = Segment * OutCoordinate;
		ProjectedPoint += InSegmentA;
		return ProjectedPoint;
	}
}

inline FVector ProjectPointOnPlane(const FVector& Point, const FVector& Origin, const FVector& InNormal, double& OutDistance)
{
	FVector Normal = InNormal;
	ensureCADKernel(!FMath::IsNearlyZero(Normal.Length()));
	Normal.Normalize();

	FVector OP = Point - Origin;
	OutDistance = OP | Normal;

	return Point - (Normal * OutDistance);
}

/**
 * @return InSegmentA + (InSegmentB - InSegmentA) | InCoordinate
 */
template<class PointType>
inline PointType PointOnSegment(const PointType& InSegmentA, const PointType& InSegmentB, double InCoordinate)
{
	PointType Segment = InSegmentB - InSegmentA;
	return InSegmentA + Segment * InCoordinate;
}

/**
 * @return the distance between the point and the segment. If the projection of the point on the segment
 * is not inside it, return the distance of the point to nearest the segment extremity
 */
template<class PointType>
inline double DistanceOfPointToSegment(const PointType& Point, const PointType& SegmentPoint1, const PointType& SegmentPoint2)
{
	double Coordinate;
	return PointType::Distance(ProjectPointOnSegment<PointType>(Point, SegmentPoint1, SegmentPoint2, Coordinate, /*bRestrictCoodinateToInside*/ true), Point);
}

/**
 * @return the distance between the point and the segment. If the projection of the point on the segment
 * is not inside it, return the distance of the point to nearest the segment extremity
 */
template<class PointType>
inline double SquareDistanceOfPointToSegment(const PointType& Point, const PointType& SegmentPoint1, const PointType& SegmentPoint2)
{
	double Coordinate;
	return FVector2d::DistSquared(ProjectPointOnSegment<PointType>(Point, SegmentPoint1, SegmentPoint2, Coordinate, /*bRestrictCoodinateToInside*/ true), Point);
}

/**
 * @return the distance between the point and the line i.e. the distance between the point and its projection on the line
 */
template<class PointType>
inline double DistanceOfPointToLine(const PointType& Point, const PointType& LinePoint1, const PointType& LineDirection)
{
	double Coordinate;
	return ProjectPointOnSegment<PointType>(Point, LinePoint1, LinePoint1 + LineDirection, Coordinate, /*bRestrictCoodinateToInside*/ false).Distance(Point);
}

CADKERNEL_API double ComputeCurvature(const FVector& Gradient, const FVector& Laplacian);
CADKERNEL_API double ComputeCurvature(const FVector& normal, const FVector& Gradient, const FVector& Laplacian);

/**
 * @return Coordinate of the projected point in the segment AB (coordinate of A = 0 and of B = 1)
 */
template<class PointType>
inline double CoordinateOfProjectedPointOnSegment(const PointType& Point, const PointType& InSegmentA, const PointType& InSegmentB, bool bRestrictCoodinateToInside = true)
{
	PointType Segment = InSegmentB - InSegmentA;

	double SquaredLength = Segment | Segment;

	if (SquaredLength <= 0.0)
	{
		return 0.0;
	}
	else
	{
		PointType APoint = Point - InSegmentA;
		double Coordinate = (APoint | Segment) / SquaredLength;

		if (bRestrictCoodinateToInside)
		{
			if (Coordinate < 0.0)
			{
				return 0.0;
			}

			if (Coordinate > 1.0)
			{
				return 1.0;
			}
		}

		return Coordinate;
	}
}

CADKERNEL_API void FindLoopIntersectionsWithIso(const EIso Iso, const double IsoParameter, const TArray<TArray<FVector2d>>& Loops, TArray<double>& OutIntersections);

template<class PointType>
struct CADKERNEL_API TSegment
{
	const PointType& Point0;
	const PointType& Point1;
	const PointType Dummy = PointType::ZeroVector;

	TSegment(const PointType& InPoint0, const PointType& InPoint1)
		: Point0(InPoint0)
		, Point1(InPoint1)
	{
	}

	constexpr const PointType& operator[](int32 Index) const
	{
		ensureCADKernel(Index < 2);
		switch (Index)
		{
		case 0:
			return Point0;
		case 1:
			return Point1;
		default:
			return Dummy;
		}
	}

	double SquaredLength() const
	{
		return FVector2d::DistSquared(Point0, Point1);
	}

	PointType GetVector() const
	{
		return Point1 - Point0;
	}
};

using FSegment2D = TSegment<FVector2d>;
using FSegment3D = TSegment<FVector>;

template<class PointType>
struct CADKERNEL_API TTriangle
{
	const PointType& Point0;
	const PointType& Point1;
	const PointType& Point2;

	TTriangle(const PointType& InPoint0, const PointType& InPoint1, const PointType& InPoint2)
		: Point0(InPoint0)
		, Point1(InPoint1)
		, Point2(InPoint2)
	{
	}

	constexpr const PointType& operator[](int32 Index) const
	{
		ensureCADKernel(Index < 3);
		switch (Index)
		{
		case 0:
			return Point0;
		case 1:
			return Point1;
		case 2:
			return Point2;
		default:
			return PointType::ZeroVector;
		}
	}

	virtual inline PointType ProjectPoint(const PointType& InPoint, FVector2d& OutCoordinate)
	{
		PointType Segment01 = Point1 - Point0;
		PointType Segment02 = Point2 - Point0;
		double SquareLength01 = Segment01.SquaredLength();
		double SquareLength02 = Segment02.SquaredLength();
		double Seg01Seg02 = Segment01 | Segment02;
		double Det = SquareLength01 * SquareLength02 - FMath::Square(Seg01Seg02);

		int32 SideIndex;
		// If the 3 points are aligned
		if (FMath::IsNearlyZero(Det))
		{
			double MaxLength = SquareLength01;
			SideIndex = Side01;
			if (SquareLength02 > MaxLength)
			{
				MaxLength = SquareLength02;
				SideIndex = Side20;
			}

			PointType Segment12 = Point2 - Point1;
			if (Segment12.SquaredLength() > MaxLength)
			{
				SideIndex = Side12;
			}
		}
		else
		{
			// Resolve
			PointType Segment1Point = InPoint - Point0;
			double Segment1PointSegment01 = Segment1Point | Segment01;
			double Segment1PointSegment02 = Segment1Point | Segment02;

			OutCoordinate.X = ((Segment1PointSegment01 * SquareLength02) - (Segment1PointSegment02 * Seg01Seg02)) / Det;
			OutCoordinate.Y = ((Segment1PointSegment02 * SquareLength01) - (Segment1PointSegment02 * Seg01Seg02)) / Det;

			// tester la solution pour choisir parmi 4 possibilites
			if (OutCoordinate.X < 0.0)
			{
				// the project point is on the segment 02
				SideIndex = Side20;
			}
			else if (OutCoordinate.Y < 0.0)
			{
				// the project point is on the segment 01
				SideIndex = Side01;
			}
			else if ((OutCoordinate.X + OutCoordinate.Y) > 1.0)
			{
				// the project point is on the segment 12
				SideIndex = Side12;
			}
			else {
				// the project point is inside the Segment
				Segment01 = Segment01 * OutCoordinate.X;
				Segment02 = Segment02 * OutCoordinate.Y;
				PointType ProjectedPoint = Segment01 + Segment02;
				ProjectedPoint = ProjectedPoint + Point0;
				return ProjectedPoint;
			}
		}

		// projects the point on the nearest side
		PointType ProjectedPoint;
		double SegmentCoordinate;
		switch (SideIndex)
		{
		case Side01:
			ProjectedPoint = ProjectPointOnSegment<PointType>(InPoint, Point0, Point1, SegmentCoordinate);
			OutCoordinate.X = SegmentCoordinate;
			OutCoordinate.Y = 0.0;
			break;
		case Side20:
			ProjectedPoint = ProjectPointOnSegment<PointType>(InPoint, Point0, Point2, SegmentCoordinate);
			OutCoordinate.X = 0.0;
			OutCoordinate.Y = SegmentCoordinate;
			break;
		case Side12:
			ProjectedPoint = ProjectPointOnSegment<PointType>(InPoint, Point1, Point2, SegmentCoordinate);
			OutCoordinate.X = 1.0 - SegmentCoordinate;
			OutCoordinate.Y = SegmentCoordinate;
			break;
		}
		return ProjectedPoint;
	}

	virtual PointType CircumCircleCenter() const
	{
		return ComputeCircumCircleCenter(Point0, Point1, Point2);
	}
};

struct CADKERNEL_API FTriangle : public TTriangle<FVector>
{
	FTriangle(const FVector& InPoint0, const FVector& InPoint1, const FVector& InPoint2)
		: TTriangle<FVector>(InPoint0, InPoint1, InPoint2)
	{
	}

	virtual FVector ComputeNormal() const
	{
		FVector Normal = (Point1 - Point0) ^ (Point2 - Point0);
		Normal.Normalize();
		return Normal;
	}
};

struct CADKERNEL_API FTriangle2D : public TTriangle<FVector2d>
{
	FTriangle2D(const FVector2d& InPoint0, const FVector2d& InPoint1, const FVector2d& InPoint2)
		: TTriangle<FVector2d>(InPoint0, InPoint1, InPoint2)
	{
	}

	FVector2d CircumCircleCenterWithSquareRadius(double& SquareRadius) const
	{
		return ComputeCircumCircleCenterAndSquareRadius(this->Point0, this->Point1, this->Point2, SquareRadius);
	}
};

/**
 * The segments must intersect because no check is done
 */
inline FVector2d FindIntersectionOfSegments2D(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD, double& OutABIntersectionCoordinate)
{
	const FVector2d AB = SegmentAB[1] - SegmentAB[0];
	const FVector2d DC = SegmentCD[0] - SegmentCD[1];
	const FVector2d AC = SegmentCD[0] - SegmentAB[0];

	double ParallelCoef = DC ^ AB;
	if (FMath::IsNearlyZero(ParallelCoef))
	{
		const double SquareAB = AB | AB;
		double CCoordinate = (AB | AC) / SquareAB;

		const FVector2d AD = SegmentCD[1] - SegmentAB[0];
		double DCoordinate = (AB | AD) / SquareAB;

		if (CCoordinate >= -DOUBLE_KINDA_SMALL_NUMBER && CCoordinate <= 1 + DOUBLE_KINDA_SMALL_NUMBER)
		{
			if (DCoordinate >= -DOUBLE_KINDA_SMALL_NUMBER && DCoordinate <= 1 + DOUBLE_KINDA_SMALL_NUMBER)
			{
				OutABIntersectionCoordinate = (DCoordinate + CCoordinate) * 0.5;
				return (SegmentCD[0] + SegmentCD[1]) * 0.5;
			}

			CCoordinate = FMath::Clamp(CCoordinate, 0., 1.);
			OutABIntersectionCoordinate = CCoordinate;
			return SegmentCD[0];
		}
		else if (DCoordinate >= -DOUBLE_KINDA_SMALL_NUMBER && DCoordinate <= 1 + DOUBLE_KINDA_SMALL_NUMBER)
		{
			DCoordinate = FMath::Clamp(DCoordinate, 0., 1.);
			OutABIntersectionCoordinate = DCoordinate;
			return SegmentCD[1];
		}
		else
		{
			OutABIntersectionCoordinate = 0.5;
			return (SegmentAB[0] + SegmentAB[1]) * 0.5;
		}
	}

	OutABIntersectionCoordinate = (DC ^ AC) / ParallelCoef;
	OutABIntersectionCoordinate = FMath::Clamp(OutABIntersectionCoordinate, 0., 1.);

	return SegmentAB[0] + AB * OutABIntersectionCoordinate;
}

/**
 * The segments must intersect because no check is done
 */
inline FVector2d FindIntersectionOfSegments2D(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD)
{
	double ABIntersectionCoordinate;
	return FindIntersectionOfSegments2D(SegmentAB, SegmentCD, ABIntersectionCoordinate);
}

/**
 * @return false if the lines are parallel
 */
inline bool FindIntersectionOfLines2D(const FSegment2D& LineAB, const FSegment2D& LineCD, FVector2d& OutIntersectionPoint)
{
	constexpr const double Min = -DOUBLE_SMALL_NUMBER;
	constexpr const double Max = 1. + DOUBLE_SMALL_NUMBER;

	const FVector2d AB = LineAB[1] - LineAB[0];
	const FVector2d DC = LineCD[0] - LineCD[1];
	const FVector2d AC = LineCD[0] - LineAB[0];

	double ParallelCoef = DC ^ AB;
	if (FMath::IsNearlyZero(ParallelCoef))
	{
		return false;
	}

	double OutABIntersectionCoordinate = (DC ^ AC) / ParallelCoef;
	OutIntersectionPoint = LineAB[0] + AB * OutABIntersectionCoordinate;
	return true;
}

/**
 * Similar as FastDoIntersect but check intersection if both segment are carried by the same line.
 * This method is 50% slower than FastIntersectSegments2D even if the segments tested are never carried by the same line
 */
CADKERNEL_API bool DoIntersect(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD);
CADKERNEL_API bool DoIntersectInside(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD);

inline bool AreParallel(const FSegment2D& SegmentAB, const FSegment2D& SegmentCD)
{
	const FVector2d AB = SegmentAB.GetVector().GetSafeNormal();
	const FVector2d CD = SegmentCD.GetVector().GetSafeNormal();
	const double ParallelCoef = AB ^ CD;
	return (FMath::IsNearlyZero(ParallelCoef, DOUBLE_KINDA_SMALL_NUMBER));
};

} // namespace UE::CADKernel

