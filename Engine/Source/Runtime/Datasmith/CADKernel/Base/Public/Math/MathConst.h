// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

namespace UE::CADKernel
{
UE_DEPRECATED(5.6, "Use FMath::GetMinMax instead.");
/**
 * Sort input A & B values into OutMin & OutMax
 */
template<typename ValueType> const void GetMinMax(const ValueType& ValueA, const ValueType& ValueB, ValueType& OutMin, ValueType& OutMax)
{
	FMath::GetMinMax(ValueA, ValueB, OutMin, OutMax);
}

UE_DEPRECATED(5.6, "Use FMath::GetMinMax instead.");
/**
 * Sort input values to be Min and Max
 */
template<typename ValueType> const void GetMinMax(ValueType& Min, ValueType& Max)
{
	FMath::GetMinMax(Min, Max);
}

/** Checks if value is within a range, exclusive on MinValue and MaxValue) */
template<typename ValueType> bool IsWithinExclusive(const ValueType& TestValue, const ValueType& MinValue, const ValueType& MaxValue)
{
	return ((TestValue > MinValue) && (TestValue < MaxValue));
}

inline int32 RealCompare(const double Value1, const double Value2, const double Tolerance = DOUBLE_SMALL_NUMBER)
{
	double Difference = Value1 - Value2;
	if (Difference < -Tolerance)
	{
		return -1;
	}
	if (Difference > Tolerance)
	{
		return 1;
	}
	return 0;
}

template< class T >
static T Cubic(const T A)
{
	return A * A * A;
}

template< class T >
uint8 ToUInt8(T Value)
{
	return FMath::Clamp((uint8)(Value / 255.), (uint8)0, (uint8)255);
}

/**
 * Wraps a periodic value into a prime period defined by its StartOfPeriod value and its EndOfPeriod value
 * Mandatory EndOfPeriod - StartOfPeriod = PeriodSize
 *
 * Fast FMath::Fmod or FMath::Floor for the special case of Slope functions since most of the time the input value doesn't need to be change
 * @see SlopeUtils.h
 */
inline double WrapTo(double Slope, const double StartOfPeriod, const double EndOfPeriod, const double PeriodLength)
{
	if (FMath::Abs(Slope) > DOUBLE_BIG_NUMBER)
	{
		return 0;
	}

	while (Slope < StartOfPeriod)
	{
		Slope += PeriodLength;
	}
	while (Slope >= EndOfPeriod)
	{
		Slope -= PeriodLength;
	}
	return Slope;
}



} // namespace UE::CADKernel	
