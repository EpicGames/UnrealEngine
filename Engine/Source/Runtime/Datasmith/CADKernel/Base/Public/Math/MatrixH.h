// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/MathConst.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
/**
* Should be unified with the math utilities implemented by the Geometry team.
*/
class CADKERNEL_API FMatrixH
{
private:
	double Matrix[16];

public:

	static const FMatrixH Identity;

	FMatrixH()
	{
		SetIdentity();
	}

	FMatrixH(const double* const InMatrix16)
	{
		for (int32 Index = 0; Index < 16; Index++)
		{
			Matrix[Index] = InMatrix16[Index];
		}
	}

	FMatrixH(const double InMatrix44[][4])
	{
		for (int32 Row = 0; Row < 4; Row++)
		{
			for (int32 Column = 0; Column < 4; Column++)
			{
				Matrix[4 * Row + Column] = InMatrix44[Row][Column];
			}
		}
	}

	FMatrixH(const FVector& Origin, const FVector& Ox, const FVector& Oy, const FVector& Oz)
	{
		BuildChangeOfCoordinateSystemMatrix(Ox, Oy, Oz, Origin);
	}

	friend FArchive& operator<<(FArchive& Ar, FMatrixH& InMatrix)
	{
		Ar.Serialize(InMatrix.Matrix, 16 * sizeof(double));
		return Ar;
	}

	void SetIdentity()
	{
		for (int32 Row = 0; Row < 4; Row++)
		{
			for (int32 Column = 0; Column < 4; Column++)
			{
				Get(Row, Column) = Row == Column ? 1. : 0.;
			}
		}
	}

	void FromAxisOrigin(const FVector& Axis, const FVector& Origin);

	/**
	 * @param Xaxis the vector X of the old coordinate system in the new coordinate system
	 * @param Yaxis the vector Y of the old coordinate system in the new coordinate system
	 * @param Zaxis the vector Z of the old coordinate system in the new coordinate system
	 * @param Origin Origin of the old coordinate system in the new coordinate system
	 * @return the transform matrix
	 */
	void BuildChangeOfCoordinateSystemMatrix(const FVector& Xaxis, const FVector& Yaxis, const FVector& Zaxis, const FVector& Origin);

	FVector Multiply(const FVector& InPoint) const
	{
		return FVector(
			InPoint.X * Get(0, 0) + InPoint.Y * Get(0, 1) + InPoint.Z * Get(0, 2) + Get(0, 3),
			InPoint.X * Get(1, 0) + InPoint.Y * Get(1, 1) + InPoint.Z * Get(1, 2) + Get(1, 3),
			InPoint.X * Get(2, 0) + InPoint.Y * Get(2, 1) + InPoint.Z * Get(2, 2) + Get(2, 3)
		);
	}

	FVector Multiply(const FVector2d& Point2D) const
	{
		return Multiply(FVector(Point2D, 0.));
	}

	FVector2d Multiply2D(const FVector2d& Point2D) const
	{
		const FVector Point = Multiply(FVector(Point2D, 0.));
		return FVector2d(Point.X, Point.Y);
	}

	FVector MultiplyVector(const FVector& InVector) const
	{
		return FVector(
			InVector.X * Get(0, 0) + InVector.Y * Get(0, 1) + InVector.Z * Get(0, 2),
			InVector.X * Get(1, 0) + InVector.Y * Get(1, 1) + InVector.Z * Get(1, 2),
			InVector.X * Get(2, 0) + InVector.Y * Get(2, 1) + InVector.Z * Get(2, 2)
		);
	}

	FVector MultiplyVector(const FVector2d& Point2D) const
	{
		return MultiplyVector(FVector(Point2D, 0.));
	}

	FVector2d MultiplyVector2D(const FVector2d& Point2D) const
	{
		const FVector Point = MultiplyVector(FVector(Point2D, 0.));
		return FVector2d(Point.X, Point.Y);
	}

	FVector3f MultiplyVector(const FVector3f& InVector) const
	{
		return FVector3f(
			(float)(InVector.X * Get(0, 0) + InVector.Y * Get(0, 1) + InVector.Z * Get(0, 2)),
			(float)(InVector.X * Get(1, 0) + InVector.Y * Get(1, 1) + InVector.Z * Get(1, 2)),
			(float)(InVector.X * Get(2, 0) + InVector.Y * Get(2, 1) + InVector.Z * Get(2, 2))
		);
	}

	static FMatrixH MakeRotationMatrix(double InAngle, const FVector InAxe);

	static FMatrixH MakeTranslationMatrix(const FVector& InPoint);

	static FMatrixH MakeScaleMatrix(FVector& Scale);
	static FMatrixH MakeScaleMatrix(double ScaleX, double ScaleY, double ScaleZ);

	/**
	 * Apply the rotation centered in origin to PointToRotate
	 */
	FVector PointRotation(const FVector& PointToRotate, const FVector& Origin) const
	{
		FVector Result = Origin;
		for (int32 Index = 0; Index < 3; Index++)
		{
			for (int32 Jndex = 0; Jndex < 3; Jndex++)
			{
				Result[Index] += Get(Index, Jndex) * (PointToRotate[Jndex] - Origin[Jndex]);
			}
		}
		return Result;
	}

	FVector2d PointRotation(const FVector2d& PointToRotate, const FVector2d& Origin) const
	{
		FVector2d Result = Origin;
		for (int32 Index = 0; Index < 2; Index++)
		{
			for (int32 Jndex = 0; Jndex < 2; Jndex++)
			{
				Result[Index] += Get(Index, Jndex) * (PointToRotate[Jndex] - Origin[Jndex]);
			}
		}
		return Result;
	}

	void Inverse();

	FMatrixH GetInverse() const
	{
		FMatrixH NewMatrix = *this;
		NewMatrix.Inverse();
		return NewMatrix;
	}

	void Transpose()
	{
		FMatrixH Tmp = *this;
		for (int32 Index = 0; Index < 4; Index++)
		{
			for (int32 Jndex = 0; Jndex < 4; Jndex++)
			{
				Get(Index, Jndex) = Tmp.Get(Jndex, Index);
			}
		}
	}

	double& Get(int32 Row, int32 Column)
	{
		return Matrix[Row * 4 + Column];
	}

	double Get(int32 Row, int32 Column) const
	{
		return Matrix[Row * 4 + Column];
	}

	double& operator()(int32 Row, int32 Column)
	{
		return Matrix[Row * 4 + Column];
	}

	double operator()(int32 Row, int32 Column) const
	{
		return Matrix[Row * 4 + Column];
	}

	double& operator[](int32 Index)
	{
		return Matrix[Index];
	}

	FMatrixH operator*(const FMatrixH& InMatrix) const
	{
		FMatrixH Result;

		for (int32 Index = 0; Index < 4; Index++)
		{
			for (int32 Jndex = 0; Jndex < 4; Jndex++)
			{
				Result.Get(Index, Jndex) = 0;
				for (int32 Kndex = 0; Kndex < 4; Kndex++)
				{
					Result.Get(Index, Jndex) = Result.Get(Index, Jndex) + (Get(Index, Kndex) * InMatrix.Get(Kndex, Jndex));
				}
			}
		}
		return Result;
	}

	void operator*=(const FMatrixH& InMatrix)
	{
		FMatrixH Result = *this * InMatrix;
		Move(*this, Result);
	}

	FVector operator*(const FVector& Point) const
	{
		return Multiply(Point);
	}

	FMatrixH operator+(const FMatrixH& InMatrix) const
	{
		FMatrixH Result;
		for (int32 i = 0; i < 16; i++)
		{
			Result.Matrix[i] = Matrix[i] + InMatrix.Matrix[i];
		}
		return Result;
	}

	void GetMatrixDouble(double* OutMatrix) const
	{
		memcpy(OutMatrix, Matrix, 16 * sizeof(double));
	}

	FVector Column(int32 Index) const
	{
		return FVector(Get(0, Index), Get(1, Index), Get(2, Index));
	}

	FVector Row(int32 Index) const
	{
		return FVector(Get(Index, 0), Get(Index, 1), Get(Index, 2));
	}

	bool IsId() const
	{
		for (int32 Row = 0; Row < 4; Row++)
		{
			for (int32 Column = 0; Column < 4; Column++)
			{
				if (Row == Column)
				{
					if (!FMath::IsNearlyEqual(Get(Row, Column), 1.)) return false;
				}
				else
				{
					if (!FMath::IsNearlyZero(Get(Row, Column))) return false;
				}
			}
		}
		return true;
	}
};

CADKERNEL_API void InverseMatrixN(double* Matrix, int32 n);

CADKERNEL_API void MatrixProduct(int32 ARowNum, int32 AColumnNum, int32 ResultRank, const double* InMatrixA, const double* InMatrixB, double* OutMatrix);

CADKERNEL_API void TransposeMatrix(int32 RowNum, int32 ColumnNum, const double* InMatrix, double* OutMatrix);

} // namespace UE::CADKernel

