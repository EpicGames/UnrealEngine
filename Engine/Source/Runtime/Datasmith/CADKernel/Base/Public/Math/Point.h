// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "Math/MathConst.h"

namespace UE::CADKernel
{

typedef FVector2d FSurfacicTolerance;

class FVectorUtil
{
public:
	static CADKERNEL_API const FVector FarawayPoint3D;
	static CADKERNEL_API const FVector2d FarawayPoint2D;

	static FVector2d FromVector(const FVector& Src)
	{
		return FVector2d(Src.X, Src.Y);
	}

	template<typename VectorType>
	static double ComputeCosinus(const VectorType& Vec1, const VectorType& Vec2)
	{
		VectorType ThisNormalized(Vec1);
		VectorType OtherNormalized(Vec2);

		ThisNormalized.Normalize();
		OtherNormalized.Normalize();

		const double Cosinus = ThisNormalized | OtherNormalized;

		return FMath::Max(-1.0, FMath::Min(Cosinus, 1.0));
	}

	template<typename VectorType>
	static double ComputeAngle(const VectorType& Vec1, const VectorType& Vec2)
	{
		VectorType ThisNormalized(Vec1);
		VectorType OtherNormalized(Vec2);

		ThisNormalized.Normalize();
		OtherNormalized.Normalize();

		return FMath::Acos(ThisNormalized | OtherNormalized);
	}

	/**
	 * Return the projection of the point on the diagonal axis (of vector (1,1,1))
	 * i.e. return X + Y + Z
	 */
	static double DiagonalAxisCoordinate(const FVector& Vec)
	{
		return Vec.X + Vec.Y + Vec.Z;
	}

	static double DiagonalAxisCoordinate(const FVector2d& Vec)
	{
		return Vec.X + Vec.Y;
	}
};

}

