// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/Types.h"
#include "Math/Point.h"
#include "Algo/AllOf.h"

namespace UE::CADKernel
{

/**
 * "Slope" is a fast angle approximation.
 *
 * This file propose all useful methods to use slope instead of angle.
 *
 * The method "ComputeSlope(const FVector2d& StartPoint, const FVector2d& EndPoint) is the main method.
 * It compute the slope between the input segment defined by two points and [0, u) axis.
 * The return value is a real in the interval [0, 8] for an angle in the interval [0, 2Pi]
 *
 * Warning, it's only an approximation... The conversion is not linear but the error is small near the integer value of slope (0, 1, 2, 3, ...8)
 *
 * To compute an angle value between two segments, the call of acos (and asin for an oriented angle) is necessary while with this approximation, only a division is useful.
 *
 * This approximation is very good when only comparison of angles is needed and more faster than acos and/or asin i.e. Slope approximation need only a division and few addition and test
 *
 * [0 - 2Pi] is divide into 8 angular sector i.e. [0, Pi/4] = [0,1], [Pi/4, Pi/2] = [1,2], ...
 *
 * The value of the slope for an angle in [0, Pi/4] = tan(angle)
 * @return a slope between [0, 8] i.e. an equivalent angle between [0, 2Pi]
 *
 * Angle (Degree) to Slop
 *		  0   = 0
 *        1   ~ 0.0175
 *        2   ~ 0.035
 *        5   ~ 0.0875
 *		 10   ~ 0.176
 *		 15.  ~ 0.268
 *		 20   ~ 0.364
 *		 25   ~ 0.466
 *       30   ~ 0.577
 *		 45   = 1
 *		 60   ~ 1.423  == 2 - Slope(30)
 *		 90   = 2
 *		120   ~ 2.577  == 2 + Slope(30)
 *		135   = 3
 *		180   = 4
 *		360   = 8
 */

namespace Slope
{

constexpr double NullSlope = 0.;

/**
 * RightSlope i.e. Right angle i.e Pi / 2
 */
constexpr double RightSlope = 2.;
constexpr double HalfPiSlope = 2.;
constexpr double NinetySlope = 2.;

/**
 * ThreeRightSlope i.e. 3Pi / 2
 */
constexpr double ThreeRightSlope = 6.;

/**
 * MinusRightSlope i.e. -Pi / 2
 */
constexpr double MinusRightSlope = -2.;

/**
 * PiSlope i.e. Pi angle
 */
constexpr double PiSlope = 4.;

/**
 * PiSlope i.e. Pi angle
 */
constexpr double TwoPiSlope = 8.;

/**
 * ThirdPiSlope i.e. Pi/3 angle (60 deg)
 */
constexpr double ThirdPiSlope = 1.422649730810374235490851219498;
constexpr double SixtySlope   = 1.422649730810374235490851219498;

/**
 * ThirdPiSlope i.e. Pi/4 angle (45 deg)
 */
constexpr double QuaterPiSlope  = 1;
constexpr double FortyFiveSlope = 0.57735026918962576450914878050196;

/**
 * ThirdPiSlope i.e. Pi/6 angle (30 deg)
 */
constexpr double SixthPiSlope = 0.57735026918962576450914878050196;
constexpr double ThirtySlope  = 0.57735026918962576450914878050196;

/**
 * ThreeQuaterPiSlope i.e. 3Pi/4 angle (135 deg)
 */
constexpr double ThreeQuaterPiSlope = 3;
                              
constexpr double OneDegree        = 0.01745506492821758576512889521973;
constexpr double TwoDegree        = 0.03492076949174773050040262577373;
constexpr double FiveDegree       = 0.08748866352592400522201866943496;
constexpr double TenDegree        = 0.17632698070846497347109038686862;
constexpr double FifteenDegree    = 0.26794919243112270647255365849413;
constexpr double TwentyDegree     = 0.36397023426620236135104788277683;
constexpr double TwentyFiveDegree = 0.46630765815499859283000619479956;

constexpr double Epsilon = 0.001;

}

typedef TFunction<double(const FVector2d&, const FVector2d&, double)> SlopeMethod;

/**
 * Transform a positive slope into an oriented slope [-4, 4] i.e. an equivalent angle between [-Pi, Pi]
 * @return a slope between [-4, 4]
 */
inline double TransformIntoOrientedSlope(double Slope)
{
	return WrapTo(Slope, -Slope::PiSlope, Slope::PiSlope, Slope::TwoPiSlope);
}

inline double TransformIntoClockwiseSlope(double Slope)
{
	return Slope::TwoPiSlope - Slope;
}

/**
 * Transform a positive slope into an unoriented slope [0, 4] i.e. an equivalent angle between [0, Pi]
 * @return a slope between [0, 4]
 */
inline double TransformIntoUnorientedSlope(double Slope)
{
	return FMath::Abs(WrapTo(Slope, -Slope::PiSlope, Slope::PiSlope, Slope::TwoPiSlope));
}

/**
 * Transform a slope into a positive slope [0, *] i.e. an equivalent angle between [0, 2.Pi]
 * @return a slope between [0, 8]
 */
inline double TransformIntoPositiveSlope(double Slope)
{
	return WrapTo(Slope, Slope::NullSlope, Slope::TwoPiSlope, Slope::TwoPiSlope);
}

/**
 * return a slope between [0, 2] relative to reference Axis i.e.
 * ComputeUnorientedSlope => 0.5 return 0.5
 * ComputeUnorientedSlope => 2.3 return 1.7
 */
inline double TransformIntoSlopeRelativeToReferenceAxis(double Slope)
{
	Slope = TransformIntoUnorientedSlope(Slope);

	if (Slope > Slope::RightSlope)
	{
		Slope = Slope::PiSlope - Slope;
	}

	return Slope;
}


/**
 * Swap a slope i.e Slope + PiSlope i.e. angle + Pi
 * @return a slope between [0, 8]
 */
inline double SwapSlopeOrientation(double Slope)
{
	const double SwapedSlope = Slope < Slope::PiSlope ? Slope + Slope::PiSlope : Slope - Slope::PiSlope;
	return TransformIntoPositiveSlope(SwapedSlope);
}

inline double ComputeSlope(const FVector2d& StartPoint, const FVector2d& EndPoint)
{
	double DeltaU = EndPoint.X - StartPoint.X;
	double DeltaV = EndPoint.Y - StartPoint.Y;
	double Delta;

	if (FMath::Abs(DeltaU) < DOUBLE_SMALL_NUMBER && FMath::Abs(DeltaV) < DOUBLE_SMALL_NUMBER)
	{
		return 0;
	}

	if (DeltaU > DOUBLE_SMALL_NUMBER)
	{
		if (DeltaV > DOUBLE_SMALL_NUMBER)
		{
			if (DeltaU > DeltaV)
			{
				Delta = DeltaV / DeltaU;
			}
			else
			{
				Delta = 2. - DeltaU / DeltaV;
			}
		}
		else
		{
			if (DeltaU > -DeltaV)
			{
				Delta = 8. + DeltaV / DeltaU;
			}
			else if (fabs(DeltaV) > DOUBLE_SMALL_NUMBER)
			{
				Delta = 6. - DeltaU / DeltaV; // deltaU/deltaV <0
			}
			else
			{
				Delta = 8.;
			}
		}
	}
	else if (-DeltaU > DOUBLE_SMALL_NUMBER)
	{
		if (DeltaV > DOUBLE_SMALL_NUMBER)
		{
			if (-DeltaU > DeltaV)
			{
				Delta = 4. + DeltaV / DeltaU;
			}
			else
			{
				Delta = 2. - DeltaU / DeltaV;
			}
		}
		else
		{
			if (-DeltaU > -DeltaV)
			{
				Delta = 4. + DeltaV / DeltaU;
			}
			else if (fabs(DeltaV) > DOUBLE_SMALL_NUMBER)
			{
				Delta = 6. - DeltaU / DeltaV;
			}
			else
			{
				Delta = 4.;
			}
		}
	}
	else
	{
		if (DeltaV > 0)
		{
			Delta = 2.;
		}
		else
		{
			Delta = 6.;
		}
	}

	return Delta;
}

/**
 * Compute the slope of a segment according to a reference slope
 * return the slope
 */
inline double ComputeSlope(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceSlope)
{
	const double Slope = ComputeSlope(StartPoint, EndPoint);
	return Slope - ReferenceSlope;
}

/**
 * Compute the slope between the segments [StartPoint, EndPoint1] and [StartPoint, EndPoint2]
 * @return a slope i.e. an equivalent angle
 */
inline double ComputeSlope(const FVector2d& StartPoint, const FVector2d& EndPoint1, const FVector2d& EndPoint2)
{
	const double ReferenceSlope = ComputeSlope(StartPoint, EndPoint1);
	const double Slope = ComputeSlope(StartPoint, EndPoint2);
	return Slope - ReferenceSlope;
}

/**
 * Compute the oriented slope of a segment according to a reference slope
 * This method is used to compute an approximation of the angle between two segments in 2D.
 * return a slope between [0, 8] i.e. an equivalent angle between [0, 2Pi]
 */
inline double ComputePositiveSlope(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceSlope)
{
	double Slope = ComputeSlope(StartPoint, EndPoint, ReferenceSlope);
	return TransformIntoPositiveSlope(Slope);
}

/**
 * Compute the positive slope between the segments [StartPoint, EndPoint1] and [StartPoint, EndPoint2]
 * @return a slope between [0, 8] i.e. an equivalent angle between [0, 2Pi]
 */
inline double ComputePositiveSlope(const FVector2d& StartPoint, const FVector2d& EndPoint1, const FVector2d& EndPoint2)
{
	double Slope = ComputeSlope(StartPoint, EndPoint1, EndPoint2);
	return TransformIntoPositiveSlope(Slope);
}

inline double ClockwiseSlope(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceSlope)
{
	return TransformIntoClockwiseSlope(ComputePositiveSlope(StartPoint, EndPoint, ReferenceSlope));
}

inline double CounterClockwiseSlope(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceSlope)
{
	return ComputePositiveSlope(StartPoint, EndPoint, ReferenceSlope);
}

/**
 * Compute the oriented slope of a segment according to a reference slope
 * @return a slope between [-4, 4] i.e. an equivalent angle between [-Pi, Pi]
 */
inline double ComputeOrientedSlope(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceSlope)
{
	return TransformIntoOrientedSlope(ComputeSlope(StartPoint, EndPoint, ReferenceSlope));
}

/**
 * Compute the positive slope between the segments [StartPoint, EndPoint1] and [StartPoint, EndPoint2]
 * @return a slope between [-4, 4] i.e. an equivalent angle between [-Pi, Pi]
 */
inline double ComputeOrientedSlope(const FVector2d& StartPoint, const FVector2d& EndPoint1, const FVector2d& EndPoint2)
{
	return TransformIntoOrientedSlope(ComputeSlope(StartPoint, EndPoint1, EndPoint2));
}

/**
 * return a slope between [0, 4] i.e. an angle between [0, Pi]
 */
inline double ComputeUnorientedSlope(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceSlope)
{
	const double Slope = ComputeSlope(StartPoint, EndPoint, ReferenceSlope);
	return TransformIntoUnorientedSlope(Slope);
}

/**
 * return a slope between [0, 1] relative to the nearest axis between horizontal or vertical axis i.e.
 * ComputeUnorientedSlope => 0.5 return 0.5
 * ComputeUnorientedSlope => 2.3 return 0.3
 * ComputeUnorientedSlope => 3.6 return 0.4
 */
inline double ComputeSlopeRelativeToNearestAxis(const FVector2d& StartPoint, const FVector2d& EndPoint)
{
	double Slope = TransformIntoUnorientedSlope(ComputeSlope(StartPoint, EndPoint));
	if (Slope > Slope::RightSlope)
	{
		Slope = Slope::PiSlope - Slope;
	}

	// if slope close to 2 means segment close to IsoU, otherwise segment close to IsoV
	// Wants a slope between 0 and 1 to manage either IsoU and IsoV
	// Close to 0 means close to IsoU or IsoV
	if (Slope > Slope::QuaterPiSlope)
	{
		Slope = Slope::RightSlope - Slope;
	}

	return Slope;
}

/**
 * return a slope between [0, 2] relative to reference Axis i.e.
 * ComputeUnorientedSlope => 0.5 return 0.5
 * ComputeUnorientedSlope => 2.3 return 1.7
 */
inline double ComputeSlopeRelativeToReferenceAxis(const FVector2d& StartPoint, const FVector2d& EndPoint, double ReferenceAxisSlope)
{
	const double Slope = ComputeSlope(StartPoint, EndPoint, ReferenceAxisSlope);
	return TransformIntoSlopeRelativeToReferenceAxis(Slope);
}

/**
 * return a slope between [0, 4] i.e. an angle between [0, Pi]
 */
inline double ComputeUnorientedSlope(const FVector2d& StartPoint, const FVector2d& EndPoint1, const FVector2d& EndPoint2)
{
	return TransformIntoUnorientedSlope(ComputeSlope(StartPoint, EndPoint1, EndPoint2));
}

/**
 *                         P1
 *          inside        /
 *                       /   inside
 *                      /
 *    A -------------- B --------------- C
 *                      \
 *           Outside     \  Outside
 *                        \
 *                         P2
 *
 * Return true if the segment BP is inside the sector defined the half-lines [BA) and [BC) in the counterclockwise.
 * Return false if ABP angle or PBC angle is too flat (smaller than FlatAngle)
 */
inline bool IsPointPInsideSectorABC(const FVector2d& PointA, const FVector2d& PointB, const FVector2d& PointC, const FVector2d& PointP, const double FlatAngle)
{
	double SlopWithNextBoundary = ComputeSlope(PointB, PointC);
	double BoundaryDeltaSlope = ComputePositiveSlope(PointB, PointA, SlopWithNextBoundary);
	double SegmentSlope = ComputePositiveSlope(PointB, PointP, SlopWithNextBoundary);
	if (SegmentSlope < FlatAngle || SegmentSlope + FlatAngle > BoundaryDeltaSlope)
	{
		return false;
	}
	return true;
}

/**
 *                         P1
 *          inside        /
 *                       /   inside
 *                      /
 *    A -------------- B --------------- C
 *                      \
 *           Outside     \  Outside
 *                        \
 *                         P2
 *
 * Return true if all of the segment BPi is inside the sector defined the half-lines [BA) and [BC) in the counterclockwise.
 */
inline bool ArePointsInsideSectorABC(const FVector2d& PointA, const FVector2d& PointB, const FVector2d& PointC, const TArray<const FVector2d*>& Points, const double FlatAngle = -DOUBLE_SMALL_NUMBER)
{
	double SlopWithNextBoundary = ComputeSlope(PointB, PointC);
	double BoundaryDeltaSlope = ComputePositiveSlope(PointB, PointA, SlopWithNextBoundary);

	return Algo::AllOf(Points, [&](const FVector2d* PointP) 
	{
		double DeltaU = PointB.X - PointP->X;
		double DeltaV = PointB.Y - PointP->Y;

		if (FMath::Abs(DeltaU) < DOUBLE_SMALL_NUMBER && FMath::Abs(DeltaV) < DOUBLE_SMALL_NUMBER)
		{
			return true;
		}

		double SegmentSlope = ComputePositiveSlope(PointB, *PointP, SlopWithNextBoundary);
		if (SegmentSlope < FlatAngle || SegmentSlope + FlatAngle > BoundaryDeltaSlope)
		{
			return false;
		}
		return true;
	});
}

inline FVector2d SlopeToVector(const double Slope)
{
	int32 SlopeStep = (int32)(Slope);
	FVector2d Vector = FVector2d::ZeroVector;
	switch (SlopeStep)
	{
	case 0:
		// Delta = DeltaV / DeltaU;
		Vector[0] = 1.;
		Vector[1] = Slope;
		break;
	case 1:
		// 2 - DeltaU / DeltaV;
		Vector[0] = 2. - Slope;
		Vector[1] = 1.;
		break;
	case 2:
		// 2 - DeltaU / DeltaV;
		Vector[0] = 2. - Slope;
		Vector[1] = 1.;
		break;
	case 3:
		// 4 + DeltaV / DeltaU;
		Vector[0] = -1.;
		Vector[1] = 4. - Slope;
		break;
	case 4:
		// 4 + DeltaV / DeltaU;
		Vector[0] = -1.;
		Vector[1] = 4. - Slope;
		break;
	case 5:
		// 6 - DeltaU / DeltaV;
		Vector[0] = Slope - 6.;
		Vector[1] = -1.;
		break;
	case 6:
		// 6 - DeltaU / DeltaV // deltaU/deltaV <0
		Vector[0] = Slope - 6.;
		Vector[1] = -1.;
		break;
	case 7:
		// 8 + DeltaV / DeltaU; // deltaU/deltaV <0
		Vector[0] = 1.;
		Vector[1] = Slope - 8.;
		break;
	default:
		break;
	}

	return Vector;
}

} // namespace UE::CADKernel	
