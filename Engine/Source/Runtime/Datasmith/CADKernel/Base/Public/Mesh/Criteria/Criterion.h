// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/Types.h"
#include "Geo/GeoPoint.h"
#include "Math/Point.h"
#include "Mesh/Criteria/CriterionType.h"

namespace UE::CADKernel
{
extern const TCHAR* CriterionTypeNames[];

// Defined for Python purpose
extern const char* CriterionTypeConstNames[];
extern const char* CriterionTypeConstDescHelp[];

class FTopologicalEdge;
struct FCurvePoint;
struct FIsoCurvature;

class CADKERNEL_API FCriterion : public FEntity
{
	friend class FEntity;

protected:

	FCriterion() = default;

public:

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		// Criterion's type is serialize because it is used to instantiate the correct entity on deserialization (@see Deserialize(FArchive& Archive)) 
		if (Ar.IsSaving())
		{
			ECriterion CriterionType = GetCriterionType();
			Ar << CriterionType;
		}
		FEntity::Serialize(Ar);
	}

	/**
	 * Specific method for curve family to instantiate the correct derived class of FCurve
	 */
	static TSharedPtr<FCriterion> Deserialize(FCADKernelArchive& Archive);

	static TSharedPtr<FCriterion> CreateCriterion(ECriterion type, double value = 0.);

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Criterion;
	}

	/**
	 * Sag & Angle criterion.pdf
	 * https://docs.google.com/presentation/d/1bUnrRFWCW3sDn9ngb9ftfQS-2JxNJaUZlh783hZMMEw/edit?usp=sharing
	 */
	virtual void ApplyOnEdgeParameters(FTopologicalEdge& Edge, const TArray<double>& Coordinates, const TArray<FCurvePoint>& Points) const;

	virtual ECriterion GetCriterionType() const = 0;

	FString GetCriterionName()
	{
		return CriterionTypeNames[(int32)GetCriterionType()];
	}

	virtual double Value() const = 0;

	static double EvaluateSag(const FVector& PointPoint, const FVector& PointNext, const FVector& PointMiddle, double& Length)
	{
		FVector ChordVec = PointNext - PointPoint;
		FVector MiddleVec = PointMiddle - PointPoint;

		double Sag = 0.0;
		double NormSqrVec = ChordVec | ChordVec;
		Length = sqrt(NormSqrVec);

		if (NormSqrVec > DOUBLE_SMALL_NUMBER)
		{
			FVector VecSag = ChordVec ^ MiddleVec;
			double NormSqrSag = VecSag | VecSag;
			Sag = NormSqrSag / NormSqrVec;

			if (Sag < SMALL_NUMBER_SQUARE)
			{
				Sag = 0.;
			}
			else
			{
				Sag = sqrt(Sag);
			}
		}
		return Sag;
	}

	virtual bool IsAppliedBetweenBreaks() const
	{
		return false;
	}

	static FString GetCriterionName(ECriterion CriterionType)
	{
		return CriterionTypeNames[(int32)CriterionType];
	}


	static double DefaultValue(ECriterion type);

	/**
	 * According to the criterion, either DeltaUMin or DeltaUMax is set e.g MinSizeCrtierion sets DeltaUMin when MaxSizeCriterion sets DeltaUMax.
	 * The implicit rule is that DeltaUMin is predominant over DeltaUMax i.e.: 
	 *	    UpdateWithUMaxValue: 
	 *			DeltaUMax = Max(DeltaUMin, Min(DeltaUMax, NewDeltaUMax))
	 *
	 *		 UpdateWithUMinValue: 
	 *			DeltaUMin = Max(DeltaUMin, NewDeltaUMin)
	 *			DeltaUMax = Max(DeltaUMax, NewDeltaUMin)
	 */
	virtual void UpdateDelta(double InDeltaU, double InUSag, double InDiagonalSag, double InVSag, double ChordLength, double DiagonalLength, double& OutDeltaUMax, double& OutDeltaUMin, FIsoCurvature& SurfaceCurvature) const
	{
		// the component according to U of the sag along the diagonal = (diagonal sag - V Sag) * U Length / diag length
		const double DiagonalSagU = FMath::Abs(InDiagonalSag - InVSag) * ChordLength / DiagonalLength;
		InUSag = FMath::Max(InUSag, DiagonalSagU);

		if (InUSag > DOUBLE_SMALL_NUMBER)
		{
			const double CriterionDeltaUMax = ComputeDeltaU(ChordLength, InDeltaU, InUSag);
			UpdateWithUMaxValue(CriterionDeltaUMax, OutDeltaUMax, OutDeltaUMin);
		}
	}

protected:

	virtual double ComputeDeltaU(double ChordLength, double DeltaU, double Sag) const
	{
		ensureCADKernel(false);
		return 0.0;
	};

	/**
	 * DeltaUMax = Max(DeltaUMin, Min(DeltaUMax, NewDeltaUMax))
	 */
	void UpdateWithUMaxValue(double NewMaxValue, double& OutDeltaUMax, const double& OutDeltaUMin) const 
	{
		if (NewMaxValue < OutDeltaUMax)
		{
			if (NewMaxValue < OutDeltaUMin)
			{
				OutDeltaUMax = OutDeltaUMin;
			}
			else
			{
				OutDeltaUMax = NewMaxValue;
			}
		}
	}

	/**
	 * DeltaUMin = Max(DeltaUMin, NewDeltaUMin)
	 * DeltaUMin cannot be smaller than DeltaUMax, so:
	 *    DeltaUMax = Max(DeltaUMax, NewDeltaUMin)
	 */
	void UpdateWithUMinValue(double NewMinValue, double& OutDeltaUMax, double& OutDeltaUMin) const
	{
		if (OutDeltaUMin < NewMinValue)
		{
			OutDeltaUMin = NewMinValue;
		}
		if (OutDeltaUMax < NewMinValue)
		{
			OutDeltaUMax = NewMinValue;
		}
	}
};

} // namespace UE::CADKernel
