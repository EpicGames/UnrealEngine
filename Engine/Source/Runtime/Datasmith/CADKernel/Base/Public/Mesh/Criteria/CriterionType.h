// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

namespace UE::CADKernel
{
enum class ECriterion : uint8
{
	Size = 0,
	MaxSize = 1,
	MinSize = 2,
	Angle = 3,
	Sag = 4,
	CADCurvature = 5,
	None = 255
};
} // namespace UE::CADKernel
