// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

namespace UE::CADKernel
{
/**
 * 3 parametric spaces are defined :
 *  -EGridSpace::Default2D the natural parametric space of the carrier surface i.e. for a Bezier surface the boundary of the space is [[0, 1], [0, 1]] and for a spherical surface [[0, 2Pi], [-Pi / 2, Pi / 2]]
 *  -EGridSpace::UniformScaled.The average curvilinear length of the iso curves is calculated in each axis.The parametric space is then scaled in each axis such that
 *            the average curvilinear length between two points P(A, V) and P(B, V) = B - A.For a spherical surface [[0, 2Pi.R / 2], [-Pi.R / 2, Pi.R / 2]] with R the radius of the sphere
 * -EGridSpace::Scaled.The UnifomeScaled space is rescaled in its most disturbe axis e.g. if V is the most disturbe axis, whatever V, length of[P(A, V), P(B, V)] = (B - A).
 *            This parametric space is closed to a good flattening of the surface.
 */
enum EGridSpace : uint8
{
	Default2D = 0,  // do not modified enum value as it is used for a static array
	Scaled = 1,
	UniformScaled = 2,
	EndGridSpace
};

enum EMeshingState : uint8
{
	NotMeshed = 0,
	PartiallyMeshed,
	FullyMeshed
};

enum ECoordinateType : uint8
{
	VertexCoordinate,
	ImposedCoordinate, // imposted cutting point by thin zone meshing process
	IsoUCoordinate,
	IsoVCoordinate,
	IsoUVCoordinate,
	OtherCoordinate
};
namespace GridSpace
{

inline FString ToString(EGridSpace Space)
{
	switch (Space)
	{
	case Default2D:
		return FString("Default2D");
	case Scaled:
		return FString("Scaled");
	case UniformScaled:
		return FString("UniformScaled");
	default:
		return FString();
	}
}

} // namespace GridSpace

} // namespace UE::CADKernel

