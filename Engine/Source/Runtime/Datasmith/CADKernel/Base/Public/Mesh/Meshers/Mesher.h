// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/Types.h"

namespace UE::CADKernel
{
class FModelMesh;
class FTopologicalShapeEntity;

class CADKERNEL_API FMesher
{
protected:

	double GeometricTolerance;
	bool bThinZoneMeshing = false;

	FModelMesh& MeshModel;

public:

	FMesher(FModelMesh& InMeshModel, double GeometricTolerance, bool bActivateThinZoneMeshing);

	void MeshEntities(TArray<FTopologicalShapeEntity*>& InEntities);

	void MeshEntity(FTopologicalShapeEntity& InEntity)
	{
		TArray<FTopologicalShapeEntity*> Entities;
		Entities.Add(&InEntity);
		MeshEntities(Entities);
	}
};

} // namespace UE::CADKernel

