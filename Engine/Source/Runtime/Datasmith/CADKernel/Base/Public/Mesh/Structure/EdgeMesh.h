// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Mesh/Structure/Mesh.h"

namespace UE::CADKernel
{
/**
 * Define the mesh of an edge. This mesh is polygon
 */
class CADKERNEL_API FEdgeMesh : public FMesh
{
public:

	/**
	 * Index of the vertices of the mesh of the TopologicalEdge in the MeshModel vertex array i.e.
	 * EdgeVerticesIndex = {5,10,11,12,13,14,3} defines Edge(Vertices[5],Vertices[10]), Edge(Vertices[10],Vertices[11]) etc
	 * of the mesh (Edge is side of mesh's triangle). "Vertices" is the mesh model vertices array
	 */
	TArray<int32> EdgeVerticesIndex;

	FEdgeMesh(FModelMesh& Model, FTopologicalEntity& TopologicalEntity)
		: FMesh(Model, TopologicalEntity)
	{
	}

	void Mesh(int32 StartVertexNodeIndex, int32 EndVertexNodeIndex)
	{
		EdgeVerticesIndex.Reserve(NodeCoordinates.Num() + 2);
		EdgeVerticesIndex.Add(StartVertexNodeIndex);
		for (int32 Index = StartNodeId; Index < NodeCoordinates.Num() + StartNodeId; ++Index)
		{
			EdgeVerticesIndex.Add(Index);
		}
		EdgeVerticesIndex.Add(EndVertexNodeIndex);
	}

	int32 GetNodeCount() const
	{
		return EdgeVerticesIndex.Num();
	}

	TArray<double> GetElementLengths() const;
};
}

