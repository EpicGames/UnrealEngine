// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "Math/Geometry.h"
#include "Core/HaveStates.h"
#include "Math/Boundary.h"
#include "Math/Point.h"
#include "Math/SlopeUtils.h"

namespace UE::CADKernel
{
class FTopologicalLoop;
class FTopologicalEdge;
class FGrid;
class FThinZone;
class FThinZone2DFinder;
class FTopologicalVertex;

class FEdgeSegment : public FHaveStates
{
private:
	FTopologicalEdge* Edge;
	double Coordinates[2];
	FVector2d USSPoints[2]; // in Uniform Scaled space

	FEdgeSegment* NextSegment;
	FEdgeSegment* PreviousSegment;

	FEdgeSegment* ClosedSegment;

	FSurfacicBoundary Boundary;
	double AxisMin;

	double SquareDistanceToClosedSegment;
	double Length;

	FIdent ChainIndex;

	FIdent Id;
	static FIdent LastId;

public:
	FEdgeSegment()
		: Edge(nullptr)
		, NextSegment(nullptr)
		, PreviousSegment(nullptr)
		, ClosedSegment(nullptr)
		, AxisMin(0.)
		, SquareDistanceToClosedSegment(HUGE_VALUE)
		, Length(-1.)
		, ChainIndex(Ident::Undefined)
		, Id(0)
	{
	};

	FEdgeSegment(const FEdgeSegment& Segment) = default;

	virtual ~FEdgeSegment() = default;

	void SetBoundarySegment(bool bInIsInnerLoop, FTopologicalEdge* InEdge, double InStartU, double InEndU, const FVector2d& InStartPoint, const FVector2d& InEndPoint)
	{
		if (bInIsInnerLoop)
		{
			SetInner();
		}

		Edge = InEdge;
		Coordinates[ELimit::Start] = InStartU;
		Coordinates[ELimit::End] = InEndU;
		USSPoints[ELimit::Start] = InStartPoint;
		USSPoints[ELimit::End] = InEndPoint;
		NextSegment = nullptr;
		PreviousSegment = nullptr;
		ClosedSegment = nullptr;

		SquareDistanceToClosedSegment = HUGE_VAL;
		Length = FVector2d::Distance(USSPoints[ELimit::Start], USSPoints[ELimit::End]);

		Id = ++LastId;
		ChainIndex = Ident::Undefined;

		Boundary.Set(USSPoints[ELimit::Start], USSPoints[ELimit::End]);

		AxisMin = Boundary[EIso::IsoU].Min + Boundary[EIso::IsoV].Min;
	};

	void UpdateReferences(TMap<int32, FEdgeSegment*>& Map)
	{
		TFunction<void(FEdgeSegment*&)> UpdateReference = [&](FEdgeSegment*& Reference)
		{
			if (Reference)
			{
				FEdgeSegment** NewReference = Map.Find(Reference->GetId());
				if (NewReference)
				{
					Reference = *NewReference;
				}
				else
				{
					Reference = nullptr;
				}
			}
		};

		UpdateReference(NextSegment);
		UpdateReference(PreviousSegment);
		UpdateReference(ClosedSegment);
	}

	double GetAxeMin() const
	{
		return AxisMin;
	}

	FIdent GetChainIndex() const
	{
		return ChainIndex;
	}

	void SetChainIndex(FIdent index)
	{
		ChainIndex = index;
	}

	bool IsInner() const
	{
		return ((States & EHaveStates::IsInner) == EHaveStates::IsInner);
	}

	void SetInner()
	{
		States |= EHaveStates::IsInner;
	}

	FIdent GetId() const
	{
		return Id;
	}

	FTopologicalEdge* GetEdge() const
	{
		return Edge;
	}

	double GetLength() const
	{
		return Length;
	}

	/**
	 * @return Center of the segment in uniform scaled space
	 */
	FVector2d GetCenter() const
	{
		return (USSPoints[ELimit::Start] + USSPoints[ELimit::End]) * 0.5;
	}

	/**
	 * @return Point a parameter U of the segment in uniform scaled space
	 */
	FVector2d ComputeEdgePoint(double EdgeParamU) const
	{
		double SegmentParamS = (EdgeParamU - Coordinates[ELimit::Start]) / (Coordinates[ELimit::End] - Coordinates[ELimit::Start]);
		return USSPoints[ELimit::Start] + (USSPoints[ELimit::End] - USSPoints[ELimit::Start]) * SegmentParamS;
	}

	/**
	 * @return the extremity Point of the segment in uniform scaled space
	 */
	const FVector2d& GetExtemity(const ELimit Limit) const
	{
		return USSPoints[Limit];
	}

	double GetCoordinate(const ELimit Limit) const
	{
		return Coordinates[Limit];
	}

	bool IsForward() const
	{
		return (Coordinates[ELimit::End] - Coordinates[ELimit::Start]) >= 0;
	}

	/**
	 * Compute the slope of the input Segment according to this.
	 */
	double ComputeUnorientedSlopeOf(const FEdgeSegment* Segment) const 
	{
		double ReferenceSlope = ComputeSlope(USSPoints[ELimit::Start], USSPoints[ELimit::End]);
		return ComputeUnorientedSlope(Segment->USSPoints[ELimit::Start], Segment->USSPoints[ELimit::End], ReferenceSlope);
	}

	/**
	 * Compute the slope of the input Segment according to this.
	 */
	double ComputeOrientedSlopeOf(const FEdgeSegment* Segment) const
	{
		double ReferenceSlope = ComputeSlope(USSPoints[ELimit::Start], USSPoints[ELimit::End]);
		return ComputeOrientedSlope(Segment->USSPoints[ELimit::Start], Segment->USSPoints[ELimit::End], ReferenceSlope);
	}

	/**
	 * Compute the slope of the Segment defined by the two input points according to this.
	 */
	double ComputeUnorientedSlopeOf(const FVector2d& Middle, const FVector2d& Projection) const 
	{
		double ReferenceSlope = ComputeSlope(USSPoints[ELimit::Start], USSPoints[ELimit::End]);
		return ComputeUnorientedSlope(Projection, Middle, ReferenceSlope);
	}

	/**
	 * Compute the slope of the Segment defined by the two input points according to this.
	 */
	double ComputeOrientedSlopeOf(const FVector2d& Middle, const FVector2d& Projection) const
	{
		double ReferenceSlope = ComputeSlope(USSPoints[ELimit::Start], USSPoints[ELimit::End]);
		return ComputeOrientedSlope(Projection, Middle, ReferenceSlope);
	}

	FEdgeSegment* GetNext() const
	{
		return NextSegment;
	}

	FEdgeSegment* GetPrevious() const
	{
		return PreviousSegment;
	}

	FEdgeSegment* GetCloseSegment() const
	{
		return ClosedSegment;
	}

	void ResetCloseData()
	{
		if (ClosedSegment->ClosedSegment == this)
		{
			ClosedSegment->ClosedSegment = nullptr;
		}
		ClosedSegment = nullptr;
		SquareDistanceToClosedSegment = HUGE_VAL;
	}

	void SetCloseSegment(FEdgeSegment* InSegmentA, double InDistance)
	{
		ClosedSegment = InSegmentA;
		SquareDistanceToClosedSegment = InDistance;

		if (InDistance < InSegmentA->SquareDistanceToClosedSegment)
		{
			InSegmentA->ClosedSegment = this;
			InSegmentA->SquareDistanceToClosedSegment = InDistance;
		}
	}

	double GetCloseSquareDistance() const
	{
		return SquareDistanceToClosedSegment;
	}

	void SetNext(FEdgeSegment* Segment)
	{
		NextSegment = Segment;
		Segment->SetPrevious(this);
	}

	double ComputeEdgeCoordinate(const double SegmentU) const
	{
		return Coordinates[ELimit::Start] + (Coordinates[ELimit::End] - Coordinates[ELimit::Start]) * SegmentU;
	}

	/** Compute the delta U corresponding to a delta length in the 2d space */
	double ComputeDeltaU(const double DeltaLength) const
	{
		return FMath::Abs(Coordinates[ELimit::End] - Coordinates[ELimit::Start]) * DeltaLength / Length;
	}

	FVector2d ProjectPoint(const FVector2d& PointToProject, double& SegmentU) const
	{
		return ProjectPointOnSegment<FVector2d>(PointToProject, USSPoints[ELimit::Start], USSPoints[ELimit::End], SegmentU, true);
	}

private:
	void SetPrevious(FEdgeSegment* Segment)
	{
		PreviousSegment = Segment;
	}
};
}

