// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Mesh/Structure/Mesh.h"

namespace UE::CADKernel
{
class FModelMesh;
class FTopologicalEntity;
class FTopologicalFace;

/**
 * Define the mesh of a topological face. This mesh is an array of triangle
 */
class CADKERNEL_API FFaceMesh : public FMesh
{
public:

	/** Index of the 3 vertices of each triangle in the local VertexIndices i.e. Triangle(a,b,c) use VertexIndices[a], VertexIndices[b], VertexIndices[c]*/
	TArray<int32> TrianglesVerticesIndex;

	/** Index of the surface mesh vertices in the MeshModel vertex set*/
	TArray<int32> VerticesGlobalIndex;

	/** Normals of the surface mesh vertices*/
	TArray<FVector3f> Normals;

	/** UV coordinates of the surface mesh vertices*/
	TArray<FVector2f> UVMap;

public:
	FFaceMesh(FModelMesh& InMeshModel, FTopologicalEntity& InTopologicalEntity)
		: FMesh(InMeshModel, InTopologicalEntity)
	{
	}

	void Init(int32 TriangleNum, int32 VertexNum)
	{
		TrianglesVerticesIndex.Reserve(TriangleNum * 3);

		VerticesGlobalIndex.Reserve(VertexNum);
		Normals.Reserve(VertexNum);
		UVMap.Reserve(VertexNum);
	}

	void AddTriangle(int32 IndexA, int32 IndexB, int32 IndexC)
	{
		if (VerticesGlobalIndex[IndexA] == VerticesGlobalIndex[IndexB] || VerticesGlobalIndex[IndexA] == VerticesGlobalIndex[IndexC] || VerticesGlobalIndex[IndexB] == VerticesGlobalIndex[IndexC])
		{
			return;
		}

		TrianglesVerticesIndex.Add(IndexA);
		TrianglesVerticesIndex.Add(IndexB);
		TrianglesVerticesIndex.Add(IndexC);
	}

	void InverseOrientation();

	void GetNodeIdToCoordinates(TMap<int32, const FVector*>& NodeIdToCoordinates) const;

};
}

