// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h"
#include "Core/EntityGeom.h"
#include "Core/Types.h"
#include "Math/Point.h"

namespace UE::CADKernel
{
class FModelMesh;
class FNode;
class FTopologicalEntity;

class CADKERNEL_API FMesh : public FEntityGeom
{
protected:
	FModelMesh& ModelMesh;
	FTopologicalEntity& TopologicalEntity;

	int32 StartNodeId;
	int32 LastNodeIndex;

	TArray<FVector> NodeCoordinates;
	int32 MeshModelIndex;

public:

	FMesh(FModelMesh& InMeshModel, FTopologicalEntity& InTopologicalEntity)
		: FEntityGeom()
		, ModelMesh(InMeshModel)
		, TopologicalEntity(InTopologicalEntity)
		, StartNodeId(0)
		, LastNodeIndex(0)
		, MeshModelIndex(0)
	{
	}

	virtual ~FMesh() = default;

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Mesh;
	}

	TArray<FVector>& GetNodeCoordinates()
	{
		return NodeCoordinates;
	}

	const TArray<FVector>& GetNodeCoordinates() const
	{
		return NodeCoordinates;
	}

	int32 RegisterCoordinates();

	const int32 GetStartVertexId() const
	{
		return StartNodeId;
	}

	const int32 GetLastVertexIndex() const
	{
		return LastNodeIndex;
	}

	const int32 GetIndexInMeshModel() const
	{
		return MeshModelIndex;
	}

	FModelMesh& GetMeshModel()
	{
		return ModelMesh;
	}

	const FModelMesh& GetMeshModel() const
	{
		return ModelMesh;
	}

	const FTopologicalEntity& GetGeometricEntity() const
	{
		return TopologicalEntity;
	}

	FTopologicalEntity& GetGeometricEntity()
	{
		return TopologicalEntity;
	}
};
}

