// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h"
#include "Core/EntityGeom.h"
#include "Core/Session.h"
#include "Core/Types.h"
#include "Mesh/Structure/Mesh.h"

namespace UE::CADKernel
{
class FCriterion;
class FDatabase;
class FEdgeMesh;
class FFaceMesh;
class FMesh;
class FSession;
class FVertexMesh;

class CADKERNEL_API FModelMesh : public FEntityGeom
{
	friend FEntity;

	TArray<TSharedPtr<FCriterion>> Criteria;

	TArray<TArray<FVector>*> GlobalPointCloud;
	FIdent LastIdUsed;

	TArray<FVertexMesh*> VertexMeshes;
	TArray<FEdgeMesh*> EdgeMeshes;
	TArray<FFaceMesh*> FaceMeshes;

	bool QuadAnalyse = false;
	double MinSize = DOUBLE_SMALL_NUMBER;
	double MaxSize = HUGE_VALUE;
	double MaxAngle = DOUBLE_PI;
	double Sag = HUGE_VALUE;

	FModelMesh()
		: FEntityGeom()
		, LastIdUsed(0)
	{
	}

public:

	const TArray<TArray<FVector>*>& GetGlobalPointCloud() const { return GlobalPointCloud; };

	int32 GetFaceCount() const
	{
		return FaceMeshes.Num();
	}

	int32 GetVertexCount() const
	{
		return LastIdUsed;
	}

	int32 GetTriangleCount() const;

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities((TArray<FEntity*>&) VertexMeshes, Database);
		SpawnIdentOnEntities((TArray<FEntity*>&) EdgeMeshes, Database);
		SpawnIdentOnEntities((TArray<FEntity*>&) FaceMeshes, Database);
	};

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities((TArray<FEntity*>&) VertexMeshes);
		ResetMarkersRecursivelyOnEntities((TArray<FEntity*>&) EdgeMeshes);
		ResetMarkersRecursivelyOnEntities((TArray<FEntity*>&) FaceMeshes);
	};


	virtual EEntity GetEntityType() const override
	{
		return EEntity::MeshModel;
	}

	const TArray<TSharedPtr<FCriterion>>& GetCriteria() const
	{
		return Criteria;
	}

	void AddCriterion(TSharedPtr<FCriterion>& Criterion);

	double GetGeometricTolerance() const
	{
		return MinSize * 0.5; // MinSize is twice GeometricTolerance
	}

	double GetMinSize() const
	{
		return MinSize;
	}

	double GetMaxSize() const
	{
		return MaxSize;
	}

	double GetAngleCriteria()
	{
		return MaxAngle;
	}

	double GetSag() const
	{
		return Sag;
	}

	void AddMesh(FVertexMesh& Mesh)
	{
		VertexMeshes.Add(&Mesh);
	}

	void AddMesh(FEdgeMesh& Mesh)
	{
		EdgeMeshes.Add(&Mesh);
	}

	void AddMesh(FFaceMesh& Mesh)
	{
		FaceMeshes.Add(&Mesh);
	}

	void RegisterCoordinates(TArray<FVector>& Coordinates, int32& OutStartVertexId, int32& OutIndex)
	{
		OutIndex = (int32)GlobalPointCloud.Num();
		OutStartVertexId = LastIdUsed;

		GlobalPointCloud.Add(&Coordinates);
		LastIdUsed += (int32)Coordinates.Num();
	}

	//const int32 GetIndexOfVertexFromId(const int32 Id) const;
	//const int32 GetIndexOfEdgeFromId(const int32 Id) const;
	//const int32 GetIndexOfSurfaceFromId(const int32 Id) const;

	const FVertexMesh* GetMeshOfVertexNodeId(const int32 Id) const;

	void GetNodeCoordinates(TArray<FVector>& NodeCoordinates) const;
	void GetNodeCoordinates(TArray<FVector3f>& NodeCoordinates) const;

	const TArray<FMesh*>& GetMeshes() const;

	const TArray<FFaceMesh*>& GetFaceMeshes() const
	{
		return FaceMeshes;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif
};

}

