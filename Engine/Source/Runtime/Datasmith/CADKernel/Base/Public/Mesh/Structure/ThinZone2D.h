// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Chrono.h"
#include "Core/HaveStates.h"
#include "Core/Types.h"
#include "Math/Point.h"

#include "Mesh/MeshEnum.h"
#include "Mesh/Structure/EdgeSegment.h"

namespace UE::CADKernel
{
	enum class EThinZone2DType : uint8
	{
		Undefined = 0,
		Global,			// a Surface globally thin
		PeakStart,		// an extremity of a Surface that is fine
		PeakEnd,		// an extremity of a Surface that is fine
		Butterfly,		// the outer loop that is like a bow tie (Butterfly)
		BetweenLoops,   // a bow tie between two different loops
		TooSmall        // -> to delete
	};

	enum class ESide : uint8
	{
		First = 0,
		Second,
		None
	};

	class FModelMesh;
	class FThinZone2D;
	class FTopologicalFace;

	using FReserveContainerFunc = TFunction<void(int32)>;
	using FAddMeshNodeFunc = TFunction<void(const int32, const FVector2d&, const double, const FEdgeSegment&, const FPairOfIndex&)>;

	class FThinZoneSide : public FHaveStates
	{
		friend FThinZone2D;

	private:
		TArray<FEdgeSegment> Segments;
		FThinZoneSide& FrontSide;

		TArray<FTopologicalEdge*> Edges;

		double SideLength;
		double MediumThickness;
		double MaxThickness;

	public:

		FThinZoneSide(FThinZoneSide* InFrontSide, const TArray<FEdgeSegment*>& InSegments);

		virtual ~FThinZoneSide() = default;

		void Empty()
		{
			Segments.Empty();
		}

		const FEdgeSegment& GetFirst() const
		{
			return Segments[0];
		}

		const FEdgeSegment& GetLast() const
		{
			return Segments.Last();
		}

		/** 
		 * Use Marker1 flag to selected edges once 
		 * Edge Marker1 has to be reset after.
		 */
		void GetEdges(TArray<FTopologicalEdge*>& OutEdges) const;

		const TArray<FTopologicalEdge*>& GetEdges() const
		{
			return Edges;
		}

		TArray<FTopologicalEdge*>& GetEdges()
		{
			return Edges;
		}

		void AddToEdge();

		void CleanMesh();

		const TArray<FEdgeSegment>& GetSegments() const
		{
			return Segments;
		}

		TArray<FEdgeSegment>& GetSegments()
		{
			return Segments;
		}

		EMeshingState GetMeshingState() const;

		double Length() const
		{
			return SideLength;
		}

		double GetThickness() const
		{
			return MediumThickness;
		}

		double GetMaxThickness() const
		{
			return MaxThickness;
		}

		bool IsInner() const
		{
			if (Segments.Num() == 0)
			{
				return true;
			}
			return Segments[0].IsInner();
		}

		bool IsClosed() const
		{
			if (Segments.Num() && (Segments[0].GetPrevious() == &Segments.Last()))
			{
				return true;
			}
			return false;
		}

		FThinZoneSide& GetFrontThinZoneSide()
		{
			return FrontSide;
		}

		void CheckEdgesZoneSide();
		void SetEdgesZoneSide(ESide Side);

		int32 GetImposedPointCount();

		void GetExistingMeshNodes(const FTopologicalFace& Face, FModelMesh& MeshModel, FReserveContainerFunc& Reserve, FAddMeshNodeFunc& AddMeshNode, const bool bWithTolerance) const;

	private:
		void ComputeThicknessAndLength();

	};

	class FThinZone2D : public FHaveStates
	{
	private:

		FThinZoneSide SideA;
		FThinZoneSide SideB;

		EThinZone2DType Category;

		double Thickness;
		double MaxThickness;

		bool bIsSwap = false;

	public:

		/**
		 * FThinZone2D::FThinZoneSides are maded with a copy of TArray<FEdgeSegment*> into TArray<FEdgeSegment> to break the link with TFactory<FEdgeSegment> of FThinZone2DFinder
		 * So FThinZone2D can be transfered into the FTopologicalFace 
		 */
		FThinZone2D(const TArray<FEdgeSegment*>& InFirstSideSegments, const TArray<FEdgeSegment*>& InSecondSideSegments)
			: SideA(&SideB, InFirstSideSegments)
			, SideB(&SideA, InSecondSideSegments)
			, Category(EThinZone2DType::Undefined)
		{
			Finalize();
 		}

		virtual ~FThinZone2D() = default;

		void Empty()
		{
			SideA.Empty();
			SideB.Empty();
			Thickness = -1;
		}

		double GetThickness() const
		{
			return Thickness;
		};

		double GetMaxThickness() const
		{
			return MaxThickness;
		};

		const FThinZoneSide& GetSide(ESide Side) const
		{
			return ((Side == ESide::Second) == bIsSwap) ? SideA : SideB;
		}

		FThinZoneSide& GetSide(ESide Side)
		{
			return const_cast<FThinZoneSide&> (static_cast<const FThinZone2D*>(this)->GetSide(Side));
		}

		FThinZoneSide& GetFirstSide()
		{
			return bIsSwap ? SideB : SideA;
		}

		FThinZoneSide& GetSecondSide()
		{
			return bIsSwap ? SideA : SideB;
		}

		const FThinZoneSide& GetFirstSide() const
		{
			return bIsSwap ? SideB : SideA;
		}

		const FThinZoneSide& GetSecondSide() const 
		{
			return bIsSwap ? SideA : SideB;
		}

		EThinZone2DType GetCategory() const
		{
			return Category;
		}

		static void SetPeakEdgesMarker(const TArray<const FTopologicalEdge*>&);

		double Length() const
		{
			return SideA.Length() + SideB.Length();
		}

		double GetMaxSideLength() const
		{
			return FMath::Max(SideA.Length(), SideB.Length());
		}

		void SetCategory(EThinZone2DType InType)
		{
			Category = InType;
		}

		/**
		 * Use Marker 3 flag to count edge once
		 * Edge Marker 3 has to be reset after.
		 */
		void GetEdges(TArray<FTopologicalEdge*>& OutSideAEdges, TArray<FTopologicalEdge*>& OutSideBEdges) const
		{
			SideA.GetEdges(OutSideAEdges);
			SideB.GetEdges(OutSideBEdges);
		}

		/**
		 * Use Marker 1 and 2 flags
		 * they have to be reset after.
		 */
		void CheckEdgesZoneSide()
		{
			SideA.CheckEdgesZoneSide();
			SideB.CheckEdgesZoneSide();
		}

		/**
		 * For each edge of the zone, set Marker1 flag if edge is in first zone, and Marker2 if edge is in second zone
		 */
		void SetEdgesZoneSide()
		{
			SideA.SetEdgesZoneSide(ESide::First);
			SideB.SetEdgesZoneSide(ESide::Second);
		}

		void AddToEdge();

		void Swap()
		{
			bIsSwap = !bIsSwap;
			switch (Category)
			{
			case EThinZone2DType::PeakStart:
			{
				Category = EThinZone2DType::PeakStart;
				break;
			}
			case EThinZone2DType::PeakEnd:
			{
				Category = EThinZone2DType::PeakStart;
				break;
			}
			default:
				break;
			};
		}

#ifdef CADKERNEL_DEV
		void Display(const FString& Title, EVisuProperty VisuProperty) const;
#endif

	private:
		void Finalize();
	};

}
