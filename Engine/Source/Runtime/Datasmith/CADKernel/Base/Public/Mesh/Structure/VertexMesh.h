// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Mesh/Structure/Mesh.h"

namespace UE::CADKernel
{
class FModelMesh;

class CADKERNEL_API FVertexMesh : public FMesh
{
protected:

public:
	FVertexMesh(FModelMesh& Model, FTopologicalEntity& TopologicalEntity)
		: FMesh(Model, TopologicalEntity)
	{
	}

	int32 GetMesh() const
	{
		return StartNodeId;
	}
};
}

