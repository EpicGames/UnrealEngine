// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h"
#include "Core/MetadataDictionary.h"
#include "Geo/GeoEnum.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEntity.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalShapeEntity.h"

namespace UE::CADKernel
{

class FCADKernelArchive;
class FDatabase;
class FModel;
class FShell;
class FTopologicalFace;

class CADKERNEL_API FBody : public FTopologicalShapeEntity
{
	friend FEntity;

private:
	TArray<TSharedPtr<FShell>> Shells;

	FBody() = default;

	FBody(const TArray<TSharedPtr<FShell>>& InShells)
	{
		for (TSharedPtr<FShell> Shell : InShells)
		{
			if (Shell.IsValid())
			{
				AddShell(Shell.ToSharedRef());
			}
		}
	}

public:

	virtual ~FBody() override
	{
		FBody::Empty();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalShapeEntity::Serialize(Ar);
		SerializeIdents(Ar, Shells);
	}


	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities(Shells, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities(Shells);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Body;
	}

	void AddShell(TSharedRef<FShell> Shell);

	void RemoveEmptyShell();

	virtual void Remove(const FTopologicalShapeEntity* ShellToRemove) override;

	virtual void Empty() override
	{
		for (TSharedPtr<FShell>& Shell : Shells)
		{
			Shell->ResetHost();
		}
		Shells.Empty();
	}

	const TArray<TSharedPtr<FShell>>& GetShells() const
	{
		return Shells;
	}

	int32 ShellCount() const
	{
		return Shells.Num();
	}

	bool IsEmpty() const
	{
		return Shells.IsEmpty();
	}

	virtual int32 FaceCount() const override
	{
		int32 FaceCount = 0;
		for (const TSharedPtr<FShell>& Shell : Shells)
		{
			FaceCount += Shell->FaceCount();
		}
		return FaceCount;
	}

	virtual void GetFaces(TArray<FTopologicalFace*>& Faces) override
	{
		for (const TSharedPtr<FShell>& Shell : Shells)
		{
			Shell->GetFaces(Faces);
		}
	}

	virtual void CompleteMetaData() override
	{
		CompleteMetaDataWithHostMetaData();
		for (TSharedPtr<FShell>& Shell : Shells)
		{
			Shell->CompleteMetaData();
		}
	}

	virtual void PropagateBodyOrientation() override
	{
		for (const TSharedPtr<FShell>& Shell : Shells)
		{
			Shell->PropagateBodyOrientation();
		}
	}

	void Orient()
	{
		for (const TSharedPtr<FShell>& Shell : Shells)
		{
			Shell->Orient();
		}
	}

#ifdef CADKERNEL_DEV
	virtual void FillTopologyReport(FTopologyReport& Report) const override;
#endif

};

}

