// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/EntityGeom.h"
#include "Topo/TopologicalEntity.h"
#include "Topo/TopologicalLink.h"

namespace UE::CADKernel
{

class FModelMesh;

template<typename EntityType, typename LinkType>
class CADKERNEL_API TLinkable : public FTopologicalEntity
{
protected:
	mutable TSharedPtr<LinkType> TopologicalLink;

public:
	TLinkable() = default;

	virtual ~TLinkable() override
	{
		TLinkable::Empty();
	}

	void Finalize()
	{
		ensureCADKernel(!TopologicalLink.IsValid());
		TopologicalLink = FEntity::MakeShared<LinkType>((EntityType&)(*this));
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalEntity::Serialize(Ar);
		SerializeIdent(Ar, TopologicalLink);
	}

	virtual void Empty() override
	{
		TopologicalLink.Reset();
	}

	const TSharedRef<const EntityType> GetLinkActiveEntity() const
	{
		ensureCADKernel(TopologicalLink.IsValid());
		EntityType* ActiveEntity = TopologicalLink->GetActiveEntity();
		return StaticCastSharedRef<EntityType>(ActiveEntity->AsShared());
	}

	TSharedRef<EntityType> GetLinkActiveEntity()
	{
		ensureCADKernel(TopologicalLink.IsValid());
		EntityType* ActiveEntity = TopologicalLink->GetActiveEntity();
		return StaticCastSharedRef<EntityType>(ActiveEntity->AsShared());
	}

	bool IsActiveEntity() const
	{
		ensureCADKernel(TopologicalLink.IsValid());

		if (TopologicalLink->GetTwinEntityNum() == 1)
		{
			return true;
		}

		return (TopologicalLink->GetActiveEntity() == this);
	}

	void Activate()
	{
		ensureCADKernel(TopologicalLink.IsValid());
		TopologicalLink->ActivateEntity(*this);
	}

	virtual TSharedPtr<LinkType> GetLink() const
	{
		ensureCADKernel(TopologicalLink.IsValid());
		return TopologicalLink;
	}

	virtual TSharedPtr<LinkType> GetLink()
	{
		ensureCADKernel(TopologicalLink.IsValid());
		return TopologicalLink;
	}

	void ResetTopologicalLink()
	{
		TopologicalLink.Reset();
		TopologicalLink = FEntity::MakeShared<LinkType>((EntityType&)(*this));
	}

	bool IsLinkedTo(const TSharedRef<EntityType>& Entity) const
	{
		if (this == &*Entity)
		{
			return true;
		}
		return (Entity->TopologicalLink == TopologicalLink);
	}

	bool IsLinkedTo(const EntityType& Entity) const
	{
		if (this == &Entity)
		{
			return true;
		}
		return (Entity.TopologicalLink == TopologicalLink);
	}

	int32 GetTwinEntityCount() const
	{
		ensureCADKernel(TopologicalLink.IsValid());
		return TopologicalLink->GetTwinEntityNum();
	}

	bool HasTwin() const
	{
		return GetTwinEntityCount() != 1;
	}

	const TArray<EntityType*>& GetTwinEntities() const
	{
		ensureCADKernel(TopologicalLink.IsValid());
		return TopologicalLink->GetTwinEntities();
	}

	virtual void RemoveFromLink()
	{
		ensureCADKernel(TopologicalLink.IsValid());
		TopologicalLink->RemoveEntity((EntityType&)*this);
		ResetTopologicalLink();
	}

	/**
	 * Unlink all twin entities
	 */
	void UnlinkTwinEntities()
	{
		TArray<EntityType*> Twins = GetTwinEntities();
		for (EntityType* Entity : Twins)
		{
			Entity->ResetTopologicalLink();
		}
	}

	const bool IsThinZone() const
	{
		return ((States & EHaveStates::ThinZone) == EHaveStates::ThinZone);
	}

	virtual void SetThinZoneMarker()
	{
		States |= EHaveStates::ThinZone;
	}

	virtual void ResetThinZone()
	{
		States &= ~EHaveStates::ThinZone;
	}

	virtual void ResetMarkersRecursively() const override
	{
		TopologicalLink->ResetMarkersRecursively();
	}


protected:

	void MakeLink(EntityType& Twin)
	{
		TSharedPtr<LinkType> Link1 = TopologicalLink;
		TSharedPtr<LinkType> Link2 = Twin.TopologicalLink;

		ensureCADKernel(Link1.IsValid() && Link2.IsValid());

		if (Link1 == Link2)
		{
			return;
		}

		if (Link2->GetTwinEntityNum() > Link1->GetTwinEntityNum())
		{
			Swap(Link1, Link2);
		}

		Link1->AddEntities(Link2->GetTwinEntities());
		for (EntityType* Entity : Link2->GetTwinEntities())
		{
			Entity->SetTopologicalLink(Link1);
		}
		Link2->Delete();
	}

protected:
	void SetTopologicalLink(TSharedPtr<LinkType> Link)
	{
		TopologicalLink = Link;
	}
};

} // namespace UE::CADKernel

