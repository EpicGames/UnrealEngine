// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/EntityGeom.h"
#include "Topo/TopologicalShapeEntity.h"
#include "CoreTypes.h"

namespace UE::CADKernel
{

class FBody;
class FDatabase;
class FGroup;
class FTopologicalEdge;
class FTopologicalEntity;
class FTopologicalFace;
class FTopologicalVertex;

class CADKERNEL_API FModel : public FTopologicalShapeEntity
{
	friend FEntity;

protected:

	TArray<TSharedPtr<FBody>> Bodies;

	FModel()
	{
		Bodies.Reserve(100);
	}

public:

	virtual ~FModel() override
	{
		FModel::Empty();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalShapeEntity::Serialize(Ar);
		SerializeIdents(Ar, (TArray<TSharedPtr<FEntity>>&) Bodies);

		if (Ar.IsLoading())
		{
			//ensureCADKernel(Archive.ArchiveModel == nullptr);
			if (Ar.ArchiveModel == nullptr)
			{
				Ar.ArchiveModel = this;
			}
		}
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities(Bodies, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities(Bodies);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Model;
	}

	void AddEntity(TSharedRef<FTopologicalEntity> InEntity);

	void Append(TArray<TSharedPtr<FBody>>& InNewBody)
	{
		Bodies.Append(InNewBody);
	}

	void Add(const TSharedPtr<FBody>& InBody)
	{
		Bodies.Add(InBody);
	}

	virtual void Empty() override
	{
		Bodies.Empty();
		FTopologicalShapeEntity::Empty();
	}

	virtual void Remove(const FTopologicalShapeEntity* InBody) override
	{
		if (!InBody)
		{
			return;
		}

		int32 Index = Bodies.IndexOfByPredicate([&](const TSharedPtr<FBody>& Body) { return (InBody == (FTopologicalShapeEntity*) Body.Get()); });
		if (Index != INDEX_NONE)
		{
			Bodies.RemoveAt(Index);
		}
	}

	void RemoveEmptyBodies();

	bool Contains(TSharedPtr<FTopologicalEntity> InEntity);

	/**
	 * Copy the body and face arrays of other model
	 */
	void Copy(const TSharedPtr<FModel>& OtherModel)
	{
		Bodies.Append(OtherModel->Bodies);
	}

	/**
	 * Copy the body and face arrays of other model
	 */
	void Copy(const FModel& OtherModel)
	{
		Bodies.Append(OtherModel.Bodies);
	}

	/**
	 * Copy the body and face arrays of other model
	 * Empty other model arrays
	 */
	void Merge(FModel& OtherModel)
	{
		Copy(OtherModel);
		OtherModel.Bodies.Empty();
	}

	int32 EntityCount() const
	{
		return Bodies.Num();
	}

	int32 BodyCount() const
	{
		return Bodies.Num();
	}

	bool IsEmpty() const
	{
		return Bodies.IsEmpty();
	}

	virtual void GetFaces(TArray<FTopologicalFace*>& OutFaces) override;

	virtual int32 FaceCount() const override;

	const TArray<TSharedPtr<FBody>>& GetBodies() const
	{
		return Bodies;
	}

	virtual void PropagateBodyOrientation() override;

	virtual void CompleteMetaData() override;

	// Topo functions

	/**
	 * Check topology of each body
	 */
	void CheckTopology();

#ifdef CADKERNEL_DEV
	virtual void FillTopologyReport(FTopologyReport& Report) const override;
#endif

	void Orient();
};

} // namespace UE::CADKernel
