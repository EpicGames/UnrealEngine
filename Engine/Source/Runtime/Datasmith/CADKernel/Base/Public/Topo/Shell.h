// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/HaveStates.h"
#include "Core/MetadataDictionary.h"
#include "Core/OrientedEntity.h"
#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalShapeEntity.h"

namespace UE::CADKernel
{

class FBody;
class FCADKernelArchive;
class FDatabase;
class FTopologicalFace;
class FTopologyReport;
struct FFaceSubset;

namespace ShellTools
{
void UnlinkFromOther(TArray<FTopologicalFace*>& Faces, TArray<FTopologicalVertex*>& VerticesToLink);
}

class CADKERNEL_API FOrientedFace : public TOrientedEntity<FTopologicalFace>
{
public:
	FOrientedFace(TSharedPtr<FTopologicalFace>& InEntity, EOrientation InOrientation)
		: TOrientedEntity(InEntity, InOrientation)
	{
	}

	FOrientedFace()
		: TOrientedEntity()
	{
	}
};

class CADKERNEL_API FShell : public FTopologicalShapeEntity
{
	friend class FEntity;

private:
	TArray<FOrientedFace> TopologicalFaces;

	FShell() = default;

	FShell(const TArray<FOrientedFace> InTopologicalFaces, bool bIsInnerShell = false)
		: FTopologicalShapeEntity()
		, TopologicalFaces(InTopologicalFaces)
	{
		if (bIsInnerShell)
		{
			SetInner();
		}
	}

	FShell(const TArray<TSharedPtr<FTopologicalFace>>& InTopologicalFaces, bool bIsInnerShell = true);

	FShell(const TArray<TSharedPtr<FTopologicalFace>>& InTopologicalFaces, const TArray<EOrientation>& InOrientations, bool bIsInnerShell = true);


public:

	virtual ~FShell() override
	{
		FShell::Empty();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalShapeEntity::Serialize(Ar);
		SerializeIdents(Ar, (TArray<TOrientedEntity<FEntity>>&) TopologicalFaces);
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities((TArray<TOrientedEntity<FEntity>>&) TopologicalFaces, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities((TArray<TOrientedEntity<FEntity>>&) TopologicalFaces);
	}

	void RemoveFaces();

	void RemoveDeletedOrDegeneratedFaces();

	virtual void Empty() override;

	void Add(TSharedRef<FTopologicalFace> InTopologicalFace, EOrientation InOrientation);
	void Add(TArray<FTopologicalFace*>& Faces);

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::Shell;
	}

	virtual int32 FaceCount() const override
	{
		return TopologicalFaces.Num();
	}

	void ReplaceFaces(TArray<FOrientedFace>& NewFaces)
	{
		Swap(TopologicalFaces, NewFaces);
		NewFaces.Reset();
	}

	const TArray<FOrientedFace>& GetFaces() const
	{
		return TopologicalFaces;
	}

	TArray<FOrientedFace>& GetFaces()
	{
		return TopologicalFaces;
	}

	virtual void GetFaces(TArray<FTopologicalFace*>& OutFaces) override;

	virtual void Merge(TSharedPtr<FShell>& Shell);

	virtual void PropagateBodyOrientation() override;

	virtual void CompleteMetaData() override;

	/**
	 * Update each FOrientedFace::Direction according to FOrientedFace::Entity->IsBackOriented() flag
	 */
	virtual void UpdateShellOrientation();

	void CheckTopology(TArray<FFaceSubset>& Subshells);

	void UnlinkFromOther(TArray<FTopologicalVertex*>& OutVerticesToLink)
	{
		TArray<FTopologicalFace*> Faces;
		GetFaces(Faces);
		ShellTools::UnlinkFromOther(Faces, OutVerticesToLink);
	}

#ifdef CADKERNEL_DEV
	virtual void FillTopologyReport(FTopologyReport& Report) const override;
#endif

	/**
	 * @return true if the shell has at least one border edge
	 */
	bool IsOpenShell();

	bool IsInner() const
	{
		return ((States & EHaveStates::IsInner) == EHaveStates::IsInner);
	}

	bool IsOuter() const
	{
		return ((States & EHaveStates::IsInner) != EHaveStates::IsInner);
	}

	void SetInner()
	{
		States |= EHaveStates::IsInner;
	}

	void SetOuter()
	{
		States &= ~EHaveStates::IsInner;
	}

	/**
	 * Orient each connected sub-shell: each connected face will have the same orientation 
	 * Each connected subset of faces will be oriented:
	 * - towards the outside if it's a closed subset (without border edge) 
	 * - according to its main/average orientation if it's an open shell 
	 * @return SwapFaceCount for report purpose
	 */
	int32 Orient();

	virtual void Remove(const FTopologicalShapeEntity*) override;

};

}

