// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Geo/Curves/Curve.h"
#include "Geo/Curves/RestrictionCurve.h"
#include "Geo/Sampling/SurfacicPolyline.h"
#include "Geo/GeoEnum.h"
#include "Math/Boundary.h"
#include "Mesh/MeshEnum.h"
#include "Topo/Linkable.h"
#include "Topo/TopologicalVertex.h"
#include "Utils/Cache.h"

namespace UE::CADKernel
{

typedef TTopologicalLink<FTopologicalEdge> FEdgeLink;

class FOrientedEdge;
class FThinZone2D;
class FThinZoneSide;

/**
 * Cutting point used for thin zone purpose
 */
struct CADKERNEL_API FImposedCuttingPoint
{
	/**
	 * coordinate of the edge's mesh nodes
	 */
	const double Coordinate = 0;
	const int32 OppositNodeIndex = -1;
	double DeltaU = 0;

	FImposedCuttingPoint()
	{
	}

	FImposedCuttingPoint(const double InCoordinate, const int32 NodeIndex1, const double InDeltaU = 0.)
		: Coordinate(InCoordinate)
		, OppositNodeIndex(NodeIndex1)
		, DeltaU(InDeltaU)
	{
	};
};

template<typename FCuttingPointType>
void GetCuttingPointCoordinates(const TArray<FCuttingPointType>& CuttingPoints, TArray<double>& CuttingPointCoordinates);

using FAddCuttingPointFunc = TFunction<void(const double, const ECoordinateType, const FPairOfIndex, const double)>;

struct FEdge2DProperties;
struct FCuttingPoint;

class FModelMesh;
class FEdgeMesh;
class FSurface;
class FThinZone;
class FTopologicalLoop;
class FTopologicalVertex;

class CADKERNEL_API FTopologicalEdge : public TLinkable<FTopologicalEdge, FEdgeLink>
{
	friend class FEntity;
	friend class FTopologicalLoop;
	friend class FTopologicalFace;

private:
	const double FactorToComputeMaxTol = 0.1;

protected:

	TSharedPtr<FTopologicalVertex> StartVertex;
	TSharedPtr<FTopologicalVertex> EndVertex;

	/**
	 * The edge is oriented in the curve orientation i.e. StartCoordinate < EndCoordinate
	 */
	FLinearBoundary Boundary;

	TSharedPtr<FRestrictionCurve> Curve;
	mutable double Length3D = -1.;

	// To avoid huge tolerance in case of degenerated edge, the max tol is defined as Length3D / 10.
	mutable double Max2DTolerance = -1;

	FTopologicalLoop* Loop = nullptr;

	TSharedPtr<FEdgeMesh> Mesh;

	/**
	 * Final U coordinates of the edge's mesh nodes
	 */
	TArray<FCuttingPoint> CuttingPointUs;

	/**
	 * U coordinates of the edge's mesh nodes for thin zone purpose
	 */
	TArray<FImposedCuttingPoint> ImposedCuttingPointUs;

	TArray<FThinZoneSide*> ThinZoneSides;
	TArray<FLinearBoundary> ThinZoneBounds;

	/**
	 * Temporary discretization of the edge used to compute the mesh of the edge
	 */
	TArray<double> CrossingPointUs;

	/**
	 * Min delta U at the crossing points to respect meshing criteria
	 */
	TArray<double> CrossingPointDeltaUMins;

	/**
	 * Max delta U at the crossing points to respect meshing criteria
	 */
	TArray<double> CrossingPointDeltaUMaxs;

private:

	/**
	 * MANDATORY
	 * Constructors of FEdge cannot be used directly, use FEdge::Make to create an FEdge object.
	 */
	FTopologicalEdge(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2, const FLinearBoundary& InBoundary);
	FTopologicalEdge(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2);
	FTopologicalEdge(const TSharedRef<FRestrictionCurve>& InCurve, const FLinearBoundary& InBoundary);
	FTopologicalEdge(const TSharedRef<FSurface>& InSurface, const FVector2d& InCoordinateVertex1, const TSharedRef<FTopologicalVertex>& InVertex1, const FVector2d& InCoordinateVertex2, const TSharedRef<FTopologicalVertex>& InVertex2);

	FTopologicalEdge() = default;

	void SetLoop(FTopologicalLoop& NewBoundary)
	{
		Loop = &NewBoundary;
	}

	void RemoveLoop()
	{
		Loop = nullptr;
	}

	/**
	 * Used by FEdge::Make
	 * Sort vertex coordinate to ensure that start coordinate < end coordinate
	 * Swap vertices if needed i.e. to ensure that 3D point at start (and end) coordinate is near to start (and end) vertex 3d coordinates
	 * @return false if one vertex is too far to the associated curve 3d point
	 */
	bool CheckVertices();

public:

	static TSharedPtr<FTopologicalEdge> Make(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2, const FLinearBoundary& InBoundary);
	static TSharedPtr<FTopologicalEdge> Make(const TSharedRef<FRestrictionCurve>& InCurve, const TSharedRef<FTopologicalVertex>& InVertex1, const TSharedRef<FTopologicalVertex>& InVertex2);
	static TSharedPtr<FTopologicalEdge> Make(const TSharedRef<FRestrictionCurve>& InCurve, const FLinearBoundary& InBoundary);
	static TSharedPtr<FTopologicalEdge> Make(const TSharedRef<FRestrictionCurve>& InCurve);

	/**
	 * Build an edge to connect two vertices carried by a 2d segment
	 */
	static TSharedPtr<FTopologicalEdge> Make(const TSharedRef<FSurface>& InSurface, const FVector2d& InCoordinateVertex1, const TSharedRef<FTopologicalVertex>& InVertex1, const FVector2d& InCoordinateVertex2, const TSharedRef<FTopologicalVertex>& InVertex2);

	/**
	 * To check the build edge before returning it or return TSharedPtr<FTopologicalEdge>() if the edge is not valid
	 */
	static TSharedPtr<FTopologicalEdge> ReturnIfValid(TSharedRef<FTopologicalEdge>& InEdge, bool bCheckVertices);

	virtual ~FTopologicalEdge() override
	{
		FTopologicalEdge::Empty();
	}

	virtual void Empty() override;

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		TLinkable<FTopologicalEdge, FEdgeLink>::Serialize(Ar);
		SerializeIdent(Ar, StartVertex);
		SerializeIdent(Ar, EndVertex);
		SerializeIdent(Ar, Curve);
		Ar << Boundary;
		SerializeIdent(Ar, &Loop);
		Ar << Length3D;
		Max2DTolerance = Length3D * FactorToComputeMaxTol;
	}

	virtual void SpawnIdent(FDatabase& Database) override;

	virtual void ResetMarkersRecursively() const override
	{
		TLinkable<FTopologicalEdge, FEdgeLink>::ResetMarkersRecursively();
		StartVertex->ResetMarkersRecursively();
		EndVertex->ResetMarkersRecursively();

		Curve->ResetMarkersRecursively();;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	double GetTolerance3D() const
	{
		return GetCurve()->GetCarrierSurface()->Get3DTolerance();
	}

	double GetTolerance2DAt(double Coordinate) const
	{
		return FMath::Min(Max2DTolerance, GetCurve()->GetToleranceAt(Coordinate));
	}

	virtual EEntity GetEntityType() const override
	{
		return EEntity::TopologicalEdge;
	}

	// ======   Topological Function   ======

	void LinkVertex();

	/**
	 * Checks if the carrier curve is degenerated i.e. the 2d length of the curve is nearly zero
	 * If the 3d length is nearly zero, the edge is flag as degenerated
	 */
	bool CheckIfDegenerated() const;

	/**
	 * It can be linked to the edge if :
	 *  - they have the same length (5% or +/- EdgeLengthTolerance)
	 *  - they are ~tangent at their extremities i.e @see IsTangentAtExtremitiesWith
	 */
	bool IsLinkableTo(const FTopologicalEdge& Edge, double EdgeLengthTolerance) const;

	/**
	 * Link two edges.
	 * Two edges can be linked if :
	 *  - Their extremities coincide following SquareJoiningTolerance's criteria,
	 *  - They are linkable (@see IsLinkableTo)
	 * 
	 * This step must be done when the loop is finalize because in some case edges can be delete, split, extend... to avoid problems
	 */
	void LinkIfCoincident(FTopologicalEdge& OtherEdge, double EdgeLengthTolerance, double SquareJoiningTolerance);

	/**
	 * Link with the other edge.
	 * No check is performed except check if degenerated ot deleted.
	 * If checks are needed, use LinkIfCoincident
	 */
	void Link(FTopologicalEdge& OtherEdge);

	/**
	 * Remove the edge and the extremity vertices of the linked entity
	 * vs UnlinkTwinEntities delete only the edge link
	 */
	void Disjoin();

	/**
	 * Remove the edge and the extremity vertices of the linked entity
	 * vs UnlinkTwinEntities delete only the edge link
	 */
	void Unlink()
	{
		Disjoin();
	}


	TSharedRef<const FTopologicalEdge> GetLinkActiveEdge() const
	{
		return StaticCastSharedRef<const FTopologicalEdge>(GetLinkActiveEntity());
	}

	TSharedRef<FTopologicalEdge> GetLinkActiveEdge()
	{
		return StaticCastSharedRef<FTopologicalEdge>(GetLinkActiveEntity());
	}

	FTopologicalEdge* GetFirstTwinEdge() const
	{
		if (!TopologicalLink)
		{
			return nullptr;
		}

		if (TopologicalLink->GetTwinEntities().Num() < 2)
		{
			return nullptr;
		}

		FTopologicalEdge* FirstTwinEdge = (TopologicalLink->GetTwinEntities()[0] == this) ? TopologicalLink->GetTwinEntities()[1] : TopologicalLink->GetTwinEntities()[0];
		return FirstTwinEdge;
	}

	FTopologicalEdge* GetTwinEdge() const
	{
		const TArray<FTopologicalEdge*>& TwinEdges = GetTwinEntities();
		if (TwinEdges.Num() > 1)
		{
			return TwinEdges[0] == this ? TwinEdges[1] : TwinEdges[0];
		}
		return nullptr;
	}

	/**
	 * @return true if the twin edge is in the same direction as this
	 */
	bool IsSameDirection(const FTopologicalEdge& Edge) const;

	/**
	 * @return true if it is ~tangent with the edge at their extremities i.e Cos(tangents) > cos(30 deg)
	 */
	bool IsTangentAtExtremitiesWith(const FTopologicalEdge & Edge) const;

	/**
	 * @return true if the edge is self connected at its extremities
	 */
	bool IsClosed() const
	{
		return StartVertex->GetLinkActiveEntity() == EndVertex->GetLinkActiveEntity();
	}

	/**
	 * This function is used to :
	 *   - sew exactly two edges of the same loop
	 *   - prepare to disconnect two edges connected by the same vertex
	 *
	 * OldVertex = bStartVertex ? StartVertex : EndVertex
	 * OldVertex can be an extremity of other edges
	 * OldVertex can be link to other vertices (TopologicalLink can be valid or not)
	 *
	 * NewVertex can be an extremity of other edges
	 * NewVertex can be linked to other vertices (TopologicalLink can be valid or not)
	 * NewVertex can be not linked to OldVertex
	 *
	 * After the process:
	 * OldVertex is no more connected to this edge but stay connected to its other linked edges
	 * If OldVertex is isolated (not connected to any edge), OldVertex is deleted
	 * otherwise OldVertex is linked to NewVertex
	 */
	void ReplaceEdgeVertex(bool bIsStartVertex, TSharedRef<FTopologicalVertex>& NewVertex);

	/**
	 * @return the containing boundary
	 */
	const FTopologicalLoop* GetLoop() const
	{
		return Loop;
	}

	/**
	 * @return the containing boundary
	 */
	FTopologicalLoop* GetLoop()
	{
		return Loop;
	}

	/**
	 * @return the carrier topological face
	 */
	FTopologicalFace* GetFace() const;

	// ======   Vertex Functions (Get, Set, ...)   ======

	const TSharedRef<FTopologicalVertex> GetStartVertex(EOrientation Forward) const
	{
		return (Forward == EOrientation::Front ? StartVertex.ToSharedRef() : EndVertex.ToSharedRef());
	}

	const TSharedRef<FTopologicalVertex> GetEndVertex(EOrientation Forward) const
	{
		return (Forward == EOrientation::Front ? EndVertex.ToSharedRef() : StartVertex.ToSharedRef());
	}

	const TSharedRef<FTopologicalVertex> GetStartVertex(bool Forward) const
	{
		return (Forward ? StartVertex.ToSharedRef() : EndVertex.ToSharedRef());
	}

	const TSharedRef<FTopologicalVertex> GetEndVertex(bool Forward) const
	{
		return (Forward ? EndVertex.ToSharedRef() : StartVertex.ToSharedRef());
	}

	const TSharedRef<FTopologicalVertex> GetStartVertex() const
	{
		return StartVertex.ToSharedRef();
	}

	const TSharedRef< FTopologicalVertex> GetEndVertex() const
	{
		return EndVertex.ToSharedRef();
	}

	TSharedRef<FTopologicalVertex> GetStartVertex()
	{
		return StartVertex.ToSharedRef();
	}

	TSharedRef<FTopologicalVertex> GetEndVertex()
	{
		return EndVertex.ToSharedRef();
	}

	TSharedPtr<FTopologicalVertex> GetOtherVertex(const TSharedRef<FTopologicalVertex>& Vertex)
	{
		return (Vertex->GetLink() == StartVertex->GetLink() ? EndVertex : (Vertex->GetLink() == EndVertex->GetLink() ? StartVertex : TSharedPtr<FTopologicalVertex>()));
	}

	FTopologicalVertex* GetOtherVertex(FTopologicalVertex& Vertex)
	{
		return (Vertex.GetLink() == StartVertex->GetLink() ? EndVertex.Get() : (Vertex.GetLink() == EndVertex->GetLink() ? StartVertex.Get() : nullptr));
	}

	const FTopologicalVertex* GetOtherVertex(const FTopologicalVertex& Vertex) const
	{
		return (Vertex.GetLink() == StartVertex->GetLink() ? EndVertex.Get() : (Vertex.GetLink() == EndVertex->GetLink() ? StartVertex.Get() : nullptr));
	}

	const TSharedPtr<FTopologicalVertex> GetOtherVertex(const TSharedRef<FTopologicalVertex>& Vertex) const
	{
		return (Vertex->GetLink() == StartVertex->GetLink() ? EndVertex : (Vertex->GetLink() == EndVertex->GetLink() ? StartVertex : TSharedPtr<FTopologicalVertex>()));
	}

	void SetStartVertex(const double NewCoordinate);
	void SetEndVertex(const double NewCoordinate);

	void SetStartVertex(const double NewCoordinate, const FVector& NewPoint3D);
	void SetEndVertex(const double NewCoordinate, const FVector& NewPoint3D);

	// ======   Boundary Function   ======

	const FLinearBoundary& GetBoundary() const
	{
		return Boundary;
	}

	double GetStartCurvilinearCoordinates() const
	{
		return Boundary.GetMin();
	}

	double GetEndCurvilinearCoordinates() const
	{
		return Boundary.GetMax();
	}

	/**
	 * @return the 3d coordinate of the start vertex (the barycenter of the twin vertices)
	 */
	FVector GetStartBarycenter()
	{
		return StartVertex->GetBarycenter();
	}

	/**
	 * @return the 3d coordinate of the end vertex (the barycenter of the twin vertices)
	 */
	FVector GetEndBarycenter()
	{
		return EndVertex->GetBarycenter();
	}

	/**
	 * @return the 3d coordinate of the start vertex (prefer GetStartBarycenter)
	 */
	FVector GetStartCoordinate()
	{
		return StartVertex->GetCoordinates();
	}

	/**
	 * @return the 3d coordinate of the end vertex (prefer GetEndBarycenter)
	 */
	FVector GetEndCoordinate()
	{
		return EndVertex->GetCoordinates();
	}

	void GetTangentsAtExtremities(FVector& StartTangent, FVector& EndTangent, bool bForward) const;
	void GetTangentsAtExtremities(FVector& StartTangent, FVector& EndTangent, EOrientation Orientation) const
	{
		GetTangentsAtExtremities(StartTangent, EndTangent, Orientation == EOrientation::Front);
	}

	// ======   Meshing Function   ======

	const FEdgeMesh* GetMesh() const
	{
		if (GetLinkActiveEntity() != AsShared())
		{
			return GetLinkActiveEdge()->GetMesh();
		}
		if (Mesh.IsValid())
		{
			return Mesh.Get();
		}
		return nullptr;
	}

	FEdgeMesh& GetOrCreateMesh(FModelMesh& MeshModel);

	/**
	 * If the mesh of the edge is not built, Empty the CuttingPoints 
	 * This allows to recompute a new discretization of the mesh based among other things on a new imposed cutting points (mesh of thin zone process)
	 */
	void RemovePreMesh();

	const FTopologicalEdge* GetPreMeshedTwin() const;
	FTopologicalEdge* GetPreMeshedTwin()
	{
		return const_cast<FTopologicalEdge*> (static_cast<const FTopologicalEdge*>(this)->GetPreMeshedTwin());
	}

	/**
	 * Generate a sampling of the curve.
	 * This sampling is used by apply meshing criteria function to defined the optimal mesh of the edge.
	 * This sampling is saved in CrossingPointUs TArray.
	 */
	void ComputeCrossingPointCoordinates();

	int32 EvaluateCuttingPointNum();

	void InitDeltaUs()
	{
		int32 Size = CrossingPointUs.Num();
		ensureCADKernel(Size >= 2);
		CrossingPointUs.SetNum(Size);
		CrossingPointDeltaUMins.Init(DOUBLE_SMALL_NUMBER, Size - 1);
		CrossingPointDeltaUMaxs.Init(2.0 * (GetEndCurvilinearCoordinates() - GetStartCurvilinearCoordinates()), Size - 1);
	}

	const TArray<double>& GetCrossingPointUs() const
	{
		return CrossingPointUs;
	}

	TArray<double>& GetCrossingPointUs()
	{
		return CrossingPointUs;
	}

	TArray<double>& GetDeltaUMins()
	{
		return CrossingPointDeltaUMins;
	}

	TArray<double>& GetDeltaUMaxs()
	{
		return CrossingPointDeltaUMaxs;
	}

	const TArray<double>& GetDeltaUMaxs() const
	{
		return CrossingPointDeltaUMaxs;
	}

	double GetDeltaUFor(double Coordinate, int32& Index) const 
	{
		for (; Index < CrossingPointUs.Num() - 1; ++Index)
		{
			if (Coordinate < CrossingPointUs[Index + 1])
			{
				if (Coordinate > CrossingPointUs[Index] - DOUBLE_SMALL_NUMBER)
				{
					break;
				}
				Index = 0;
				if(Coordinate < CrossingPointUs[1])
				{
					break;
				}
			}
		}
		return CrossingPointDeltaUMaxs[Index];
	}

	TArray<FCuttingPoint>& GetCuttingPoints()
	{
		return CuttingPointUs;
	};

	const TArray<FCuttingPoint>& GetCuttingPoints() const
	{
		return CuttingPointUs;
	}

	TArray<double> GetCuttingPointCoordinates() const;

	void TransferCuttingPointFromMeshedEdge(bool bOnlyWithOppositeNode, FAddCuttingPointFunc AddCuttingPoint);

	/**
	 * Compute the lengths of each pre-elements of the edge i.e the elements based of the cutting points of the edges. 
	 */
	TArray<double> GetPreElementLengths() const;

	// For thin zone purpose
	void SortImposedCuttingPoints();

	const TArray<FImposedCuttingPoint>& GetImposedCuttingPoints() const
	{
		return ImposedCuttingPointUs;
	}

	void AddThinZone(FThinZoneSide* InThinZoneSide, const FLinearBoundary& InThinZoneBounds)
	{
		if(InThinZoneSide)
		{
			ThinZoneSides.AddUnique(InThinZoneSide);
			ThinZoneBounds.Add(InThinZoneBounds);
		}
	}

	int32 GetThinZoneCount() const
	{
		return ThinZoneSides.Num();
	}

	const TArray<FThinZoneSide*>& GetThinZoneSides() const
	{
		return ThinZoneSides;
	}

	const TArray<FLinearBoundary>& GetThinZoneBounds() const
	{
		return ThinZoneBounds;
	}

	void AddImposedCuttingPointU(const double ImposedCuttingPointU, const int32 OppositeNodeIndex, const double DeltaU);
	void AddTwinsCuttingPoint(const double Coord, const double DeltaU);

	void GenerateMeshElements(FModelMesh& MeshModel);

	// ======   Curve Functions   ======

	TSharedRef<FRestrictionCurve> GetCurve() const
	{
		return Curve.ToSharedRef();
	}

	TSharedRef<FRestrictionCurve> GetCurve()
	{
		return Curve.ToSharedRef();
	}

	void ComputeLength();
	double Length() const;

	/**
	 * Samples the curve with segments of a desired length
	 */
	void Sample(const double DesiredSegmentLength, TArray<double>& OutCoordinates) const;

	/**
	 * Exact evaluation of point on the 3D curve
	 * According to derivativeOrder Gradient of the point (DerivativeOrder = 1) and Laplacian (DerivativeOrder = 1) can also be return
	 */
	void EvaluatePoint(double InCoordinate, int32 Derivative, FCurvePoint& Point) const
	{
		Curve->EvaluatePoint(InCoordinate, Point, Derivative);
	}

	/**
	 * Exact evaluation of points on the 3D curve
	 * According to derivativeOrder Gradient of the point (DerivativeOrder = 1) and Laplacian (DerivativeOrder = 1) can also be return
	 */
	void EvaluatePoints(const TArray<double>& InCoordinates, int32 DerivativeOrder, TArray<FCurvePoint>& OutPoints) const
	{
		Curve->EvaluatePoints(InCoordinates, OutPoints, DerivativeOrder);
	}

	/**
	 * Approximation of 3D points compute with carrier surface 3D polyline
	 */
	void ApproximatePoints(const TArray<double>& InCoordinates, TArray<FVector>& OutPoints) const
	{
		Curve->Approximate3DPoints(InCoordinates, OutPoints);
	}

	/**
	 * Approximation of 2D point defined by its coordinate compute with carrier surface 2D polyline
	 */
	FVector2d Approximate2DPoint(const double InCoordinate) const
	{
		return Curve->Approximate2DPoint(InCoordinate);
	}

	/**
	 * Approximation of 2D points defined by its coordinates compute with carrier surface 2D polyline
	 */
	void Approximate2DPoints(const TArray<double>& InCoordinates, TArray<FVector2d>& OutPoints) const
	{
		Curve->Approximate2DPoints(InCoordinates, OutPoints);
	}

	/**
	 * Approximation of surfacic polyline (points 2d, 3d, normals, tangents) defined by its coordinates compute with carrier surface polyline
	 */
	void ApproximatePolyline(FSurfacicPolyline& Polyline) const
	{
		Curve->ApproximatePolyline(Polyline);
	}

	FVector GetTangentAt(const double InCoordinate) const
	{
		return Curve->GetTangentAt(InCoordinate);
	}

	FVector2d GetTangent2DAt(const double InCoordinate) const
	{
		return Curve->GetTangent2DAt(InCoordinate);
	}

	/**
	 * Return the tangent at the input vertex
	 */
	FVector GetTangentAt(const FTopologicalVertex& InVertex);
	FVector2d GetTangent2DAt(const FTopologicalVertex& InVertex);

	/**
	 * Project Point (2D or 3D) on the polyline (2D or 3D) and return the coordinate of the projected point
	 */
	template<class PointType>
	double ProjectPoint(const PointType& InPointToProject, PointType& OutProjectedPoint) const
	{
		return Curve->GetCoordinateOfProjectedPoint(Boundary, InPointToProject, OutProjectedPoint);
	}

	/**
	 * Project a set of points on the 3D polyline and return the coordinate of the projected point
	 */
	void ProjectPoints(const TArray<FVector>& InPointsToProject, TArray<double>& OutProjectedPointCoords, TArray<FVector>& OutProjectedPoints) const
	{
		Curve->ProjectPoints(Boundary, InPointsToProject, OutProjectedPointCoords, OutProjectedPoints);
	}

	/**
	 * Project a set of points of a twin edge on the 3D polyline and return the coordinate of the projected point
	 */
	void ProjectTwinEdgePoints(const TArray<FVector>& InPointsToProject, bool bSameOrientation, TArray<double>& OutProjectedPointCoords) const
	{
		const double ToleranceOfProjection = Length3D * 0.1;
		Curve->ProjectTwinCurvePoints(Boundary, InPointsToProject, bSameOrientation, OutProjectedPointCoords, ToleranceOfProjection);
	}

	/**
	 * Compute 2D points of the edge coincident the points of the twin edge defined by their coordinates
	 */
	void ProjectTwinEdgePointsOn2DCurve(const TSharedRef<FTopologicalEdge>& InTwinEdge, const TArray<double>& InTwinEdgePointCoords, TArray<FVector2d>& OutPoints2D);

	void ComputeIntersectionsWithIsos(const TArray<double>& InIsoCoordinates, const EIso InTypeIso, const FSurfacicTolerance& ToleranceIso, TArray<double>& OutIntersection) const
	{
		Curve->ComputeIntersectionsWithIsos(Boundary, InIsoCoordinates, InTypeIso, ToleranceIso, OutIntersection);
	}

	/**
	 * Get the discretization points of the edge and add them to the outpoints TArray
	 */
	template<class PointType>
	void GetDiscretization2DPoints(EOrientation Orientation, TArray<PointType>& OutPoints) const
	{
		Curve->GetDiscretizationPoints(Boundary, Orientation, OutPoints);
	}

	double TransformLocalCoordinateToActiveEdgeCoordinate(const double LocalCoordinate) const;
	double TransformActiveEdgeCoordinateToLocalCoordinate(const double ActiveEdgeCoordinate) const;
	double TransformTwinEdgeCoordinateToLocalCoordinate(const FTopologicalEdge& TwinEdge, const double InTwinCoordinate) const;

	void TransformTwinEdgeCoordinatesToLocalCoordinates(const FTopologicalEdge& TwinEdge, const TArray<double>& InActiveEdgeCoordinate, TArray<double>& OutLocalCoordinate) const;
	void TransformActiveEdgeCoordinatesToLocalCoordinates(const TArray<double>& InActiveEdgeCoordinate, TArray<double>& OutLocalCoordinate) const;
	void TransformLocalCoordinatesToActiveEdgeCoordinates(const TArray<double>& InLocalCoordinate, TArray<double>& OutActiveEdgeCoordinate) const;

	/**
	 * Compute the edge 2D properties i.e. the mean and standard deviation of the slop of the edge in the parametric space of the carrier surface
	 */
	void ComputeEdge2DProperties(FEdge2DProperties& SlopeCharacteristics);

	void GetExtremities(FSurfacicCurveExtremities& Extremities) const
	{
		Curve->GetExtremities(Boundary, Extremities);
	}

	void Offset2D(const FVector2d& OffsetDirection);

	// ======   Geometrical Functions   ======

	/**
	 * Split the edge at input coordinate. The initial edge will be used for the first edge, the second edge will be return.
	 * @param SplittingCoordinate
	 * @param NewVertexCoordinate
	 * @param bKeepStartVertexConnectivity: if true the new edge is connected to endVertex, otherwise the new edge is connected to start vertex
	 * @param OutNewEdge, the second edge
	 */
	FTopologicalVertex* SplitAt(double SplittingCoordinate, const FVector& NewVertexCoordinate, bool bKeepStartVertexConnectivity, TSharedPtr<FTopologicalEdge>& OutNewEdge);

	/**
	 * Extend the Edge to the NewVertex.
	 * NewVertex become the new extremity of the edge. The old vertex is unconnected of the edge.
	 */
	bool ExtendTo(bool bStartExtremity, const FVector2d& NewExtremityCoordinate, TSharedRef<FTopologicalVertex>& NewVertex);

	bool IsSharpEdge() const;

	/** @return true if they have the same length +/- 5 % or +/- EdgeLengthTolerance */
	bool HasSameLengthAs(const FTopologicalEdge& Edge, double EdgeLengthTolerance) const;

	// ======   State Functions   ======

	/**
	 * Important note: A Degenerated Edge is used to close 2D boundary in case of degenerated surface to ensureCADKernel a closed boundary
	 * Specific process is done for the mesh of this kind of surface
	 */
	virtual bool IsDegenerated() const override
	{
		return FHaveStates::IsDegenerated();
	}

	/**
	 * An edge is a thin peak means that this edge is a small edge at the extremity of a peak thin zone
	 * So this edge must not be meshed (except at its extremities)
	 *
	 *                         ThinSide 0
	 *           #-------------------------------------#
	 *          /
	 *         /  <- Thin peak edge
	 *        /
	 *       #-----------------------------------------#
	 *                         ThinSide 1
	 *
	 */
	bool IsThinPeak() const
	{
		return ((States & EHaveStates::ThinPeak) == EHaveStates::ThinPeak);
	}

	virtual void SetThinPeakMarker() const
	{
		States |= EHaveStates::ThinPeak;
	}

	virtual void ResetThinPeakMarker() const
	{
		States &= ~EHaveStates::ThinPeak;
	}

	bool IsVirtuallyMeshed() const
	{
		return ((States & EHaveStates::IsVirtuallyMeshed) == EHaveStates::IsVirtuallyMeshed);
	}

	virtual void SetVirtuallyMeshedMarker() const
	{
		States |= EHaveStates::IsVirtuallyMeshed;
	}

	virtual void ResetVirtuallyMeshedMarker() const
	{
		States &= ~EHaveStates::IsVirtuallyMeshed;
	}

	/**
	 * @return true if the edge is adjacent to only one surface (its carrier surface)
	 */
	bool IsBorder() const
	{
		return GetTwinEntityCount() == 1;
	}

	/**
	 * @return true if the edge is adjacent to only two surfaces
	 */
	bool IsSurfacic() const
	{
		return GetTwinEntityCount() == 2;
	}

	bool IsConnectedTo(const FTopologicalFace* Face) const;

	TArray<FTopologicalFace*> GetLinkedFaces() const;

	/**
	 * Merge successive edges of a face in a single edge.
	 * Edges must not be topological linked to other faces i.e. edges must be border edges
	 * The merged edges are deleted
	 * @return TSharedPtr<FTopologicalEdge>() if failed
	 */
	static TSharedPtr<FTopologicalEdge> CreateEdgeByMergingEdges(const double SmallEdgeTolerance, TArray<FOrientedEdge>& Edges, const TSharedRef<FTopologicalVertex>& StartVertex, const TSharedRef<FTopologicalVertex>& EndVertex);
};

struct CADKERNEL_API FEdge2DProperties
{
	double StandardDeviation = 0;
	double MediumSlope = 0;
	double Length3D = 0;
	EIso IsoType = EIso::UndefinedIso;
	bool bIsMesh = false;
	double MeshedLength = 0;

	void Add(double InSlope, double InLength)
	{
		double Temp = InSlope * InLength;
		MediumSlope += Temp;
		Temp *= InSlope;
		StandardDeviation += Temp;
		Length3D += InLength;
	}

	// Finalize has been done on each Property
	void Add2(FEdge2DProperties& Property)
	{
		StandardDeviation = (FMath::Square(StandardDeviation) + FMath::Square(MediumSlope)) * Length3D + (FMath::Square(Property.StandardDeviation) + FMath::Square(Property.MediumSlope)) * Property.Length3D;
		MediumSlope = MediumSlope * Length3D + Property.MediumSlope * Property.Length3D;
		Length3D += Property.Length3D;

		MediumSlope /= Length3D;
		StandardDeviation /= Length3D;
		StandardDeviation -= FMath::Square(MediumSlope);
		StandardDeviation = sqrt(StandardDeviation);
	}

	// Finalize has not been done on each Property
	void Add(FEdge2DProperties& Property)
	{
		StandardDeviation += Property.StandardDeviation;
		MediumSlope += Property.MediumSlope;
		Length3D += Property.Length3D;
	}

	void Finalize()
	{
		MediumSlope /= Length3D;
		StandardDeviation /= Length3D;
		StandardDeviation -= FMath::Square(MediumSlope);
		if (StandardDeviation < 0)
		{
			StandardDeviation = 0;
		}
		else
		{
			StandardDeviation = sqrt(StandardDeviation);
		}
		IsoType = EIso::UndefinedIso;

		if (MediumSlope < 0.2)
		{
			if (StandardDeviation < 0.1)
			{
				IsoType = EIso::IsoU;
			}
		}
		else if (MediumSlope > 1.8)
		{
			if (StandardDeviation < 0.1)
			{
				IsoType = EIso::IsoV;
			}
		}
	}

};

/**
 * Cutting point used for meshing purpose
 */
struct CADKERNEL_API FCuttingPoint
{
	/**
	 * coordinate of the edge's mesh nodes
	 */
	double Coordinate;
	ECoordinateType Type;
	FPairOfIndex OppositNodeIndices;
	double IsoDeltaU = 0;

	FCuttingPoint()
		: Coordinate(0)
		, Type(ECoordinateType::OtherCoordinate)
		, OppositNodeIndices(FPairOfIndex::Undefined)
		, IsoDeltaU(HUGE_VAL)
	{
	}

	FCuttingPoint(double InCoordinate, ECoordinateType InType)
		: Coordinate(InCoordinate)
		, Type(InType)
		, OppositNodeIndices(FPairOfIndex::Undefined)
		, IsoDeltaU(HUGE_VAL)
	{
	}

	FCuttingPoint(double InCoordinate, ECoordinateType InType, FPairOfIndex InOppositNodeIndices, double DeltaU)
		: Coordinate(InCoordinate)
		, Type(InType)
		, OppositNodeIndices(InOppositNodeIndices)
		, IsoDeltaU(DeltaU)
	{
	}

	FCuttingPoint(double InCoordinate, ECoordinateType InType, int32 InOppositeNodeId, double DeltaU)
		: Coordinate(InCoordinate)
		, Type(InType)
		, OppositNodeIndices(InOppositeNodeId)
		, IsoDeltaU(DeltaU)
	{
	}
};

struct CADKERNEL_API FCuttingGrid
{
	TArray<FCuttingPoint> Coordinates[2];

	constexpr TArray<FCuttingPoint>& operator[](EIso Iso)
	{
		ensureCADKernel(Iso == 0 || Iso == 1);
		return Coordinates[Iso];
	}

	constexpr const TArray<FCuttingPoint>& operator[](EIso Iso) const
	{
		ensureCADKernel(Iso == 0 || Iso == 1);
		return Coordinates[Iso];
	}
};


template<typename FCuttingPointType>
void GetCuttingPointCoordinates(const TArray<FCuttingPointType>& CuttingPoints, TArray<double>& CuttingPointCoordinates)
{
	CuttingPointCoordinates.Empty(CuttingPoints.Num());
	for (const FCuttingPointType& CuttingPoint : CuttingPoints)
	{
		CuttingPointCoordinates.Add(CuttingPoint.Coordinate);
	}
};


} // namespace UE::CADKernel
