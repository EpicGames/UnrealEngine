// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/HaveStates.h"
#include "Core/Types.h"

namespace UE::CADKernel
{
class FModelMesh;
class FTopologicalFace;
class FTopologyReport;

class CADKERNEL_API FTopologicalEntity : public FEntity
{
	friend class FCoreTechBridge;

protected:
	FIdent CtKioId = 0;

public:

	FIdent GetKioId() const
	{
		return CtKioId;
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FEntity::Serialize(Ar);
		Ar << CtKioId;
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	const bool IsApplyCriteria() const
	{
		return ((States & EHaveStates::IsApplyCriteria) == EHaveStates::IsApplyCriteria);
	}

	virtual void SetApplyCriteriaMarker() const
	{
		States |= EHaveStates::IsApplyCriteria;
	}

	virtual void ResetApplyCriteria()
	{
		States &= ~EHaveStates::IsApplyCriteria;
	}

	bool IsNotMeshable() const
	{
		constexpr EHaveStates MeshedOrDeletedOrDegenerated = EHaveStates::IsMeshed | EHaveStates::Degenerated | EHaveStates::IsDeleted;
		return ((States & MeshedOrDeletedOrDegenerated) != EHaveStates::None);
	}

	bool IsMeshable() const
	{
		constexpr EHaveStates MeshedOrDeletedOrDegenerated = EHaveStates::IsMeshed | EHaveStates::Degenerated | EHaveStates::IsDeleted;
		return ((States & MeshedOrDeletedOrDegenerated) == EHaveStates::None);
	}

	/** 
	 * An edge is premesh if the cutting points of the mesh have been defined but the mesh is not build 
	 * This is used during thin zone meshing i.e. due to the process, an edge can be premeshed but not finalized
	 */
	bool IsPreMeshed() const
	{
		constexpr EHaveStates MeshedOrPreMeshed = EHaveStates::IsMeshed | EHaveStates::IsPreMeshed;
		return ((States & MeshedOrPreMeshed) != EHaveStates::None);
	}

	bool IsMeshed() const
	{
		return ((States & EHaveStates::IsMeshed) == EHaveStates::IsMeshed);
	}

	virtual void SetPreMeshedMarker()
	{
		States |= EHaveStates::IsPreMeshed;
	}

	virtual void SetMeshedMarker()
	{
		constexpr EHaveStates MeshedOrPreMeshed = EHaveStates::IsMeshed | EHaveStates::IsPreMeshed;
		States |= MeshedOrPreMeshed;
	}

	virtual void ResetPreMeshed()
	{
		States &= ~EHaveStates::IsPreMeshed;
	}
};

} // namespace UE::CADKernel

