// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h"
#include "Core/HaveStates.h"
#include "Core/Types.h"
#include "Geo/GeoEnum.h"
#include "Geo/GeoPoint.h"
#include "Geo/Sampling/PolylineTools.h"
#include "Geo/Surfaces/Surface.h"
#include "Math/Boundary.h"
#include "Math/Curvature.h"
#include "Math/Point.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalEntity.h"
#include "Topo/TopologicalLoop.h"
#include "Topo/TopologicalShapeEntity.h"

#include "Mesh/Structure/EdgeSegment.h"
#include "Mesh/Structure/ThinZone2D.h"

namespace UE::CADKernel
{

class FCADKernelArchive;
class FCriteriaGrid;
class FCriterion;
class FCurve;
class FDatabase;
class FFaceMesh;
class FModelMesh;
class FTopologicalVertex;
struct FSurfacicSampling;

enum class EStatut : uint8
{
	Interior = 0,
	Exterior,
	Border
};

enum class EQuadType : uint8
{
	Unset = 0,
	Quadrangular,
	Triangular,
	Other
};

struct FBBoxWithNormal;

class CADKERNEL_API FTopologicalFace : public FTopologicalShapeEntity
{
	friend class FEntity;

protected:

	TSharedPtr<FSurface> CarrierSurface;
	TArray<TSharedPtr<FTopologicalLoop>> Loops;

	mutable TCache<FSurfacicBoundary> Boundary;

	TSharedPtr<FFaceMesh> Mesh;

	/**
	 * Final U&V coordinates of the surface's mesh grid
	 */
	FCoordinateGrid MeshCuttingCoordinates;

	/**
	 * Temporary discretization of the surface used to compute the mesh of the edge
	 */
	FCoordinateGrid CrossingCoordinates;

	/**
	 * Min delta U at the crossing u coordinate to respect meshing criteria
	 */
	FCoordinateGrid CrossingPointDeltaMins;

	/**
	 * Max delta U at the crossing u coordinate to respect meshing criteria
	 */
	FCoordinateGrid CrossingPointDeltaMaxs;

	double EstimatedMinimalElementLength = DOUBLE_BIG_NUMBER;

	/**
	 * Build a non-trimmed trimmed surface
	 * This constructor has to be completed with one of the three "AddBoundaries" methods to be finalized.
	 */
	FTopologicalFace(const TSharedPtr<FSurface>& InCarrierSurface)
		: FTopologicalShapeEntity()
		, CarrierSurface(InCarrierSurface)
		, Mesh(TSharedPtr<FFaceMesh>())
	{
		ResetElementStatus();
	}

	FTopologicalFace() = default;

	/**
	 * Compute the bounds of the topological surface according to the Loops
	 */
	void ComputeBoundary() const;

public:

	virtual ~FTopologicalFace() override
	{
		FTopologicalFace::Empty();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalShapeEntity::Serialize(Ar);
		SerializeIdent(Ar, CarrierSurface);
		SerializeIdents(Ar, (TArray<TSharedPtr<FEntity>>&) Loops);
	}

	virtual void SpawnIdent(FDatabase& Database) override;

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities(Loops);
		CarrierSurface->ResetMarkersRecursively();
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::TopologicalFace;
	}

	const FSurfacicTolerance& GetIsoTolerances() const
	{
		return CarrierSurface->GetIsoTolerances();
	}

	double GetIsoTolerance(EIso Iso) const
	{
		return CarrierSurface->GetIsoTolerance(Iso);
	}

	const FSurfacicBoundary& GetBoundary() const
	{
		if (!Boundary.IsValid())
		{
			ComputeBoundary();
		}
		return Boundary;
	};

	virtual int32 FaceCount() const override
	{
		return 1;
	}

	virtual void GetFaces(TArray<FTopologicalFace*>& OutFaces) override
	{
		if (!HasMarker1())
		{
			OutFaces.Emplace(this);
			SetMarker1();
		}
	}

	virtual void CompleteMetaData() override
	{
		CompleteMetaDataWithHostMetaData();
	}

	virtual void PropagateBodyOrientation() override
	{
	}

	// ======   Loop Functions   ======

	void RemoveLoop(const TSharedPtr<FTopologicalLoop>& Loop);
	void AddLoop(const TSharedPtr<FTopologicalLoop>& Loop)
	{
		if (Loop.IsValid())
		{
			Loop->SetSurface(this);
			Loops.Add(Loop);
		}
	}

	/**
	 * Trimmed the face with an outer boundary (first boundary of the array) and inners boundaries
	 */
	void AddLoops(const TArray<TSharedPtr<FTopologicalLoop>>& Loops, int32& DoubtfulLoopOrientationCount);

	/**
	 * Trimmed the face with curves i.e. Edges will be build from the cures to make boundaries.
	 */
	void AddLoops(const TArray<TSharedPtr<FCurve>>& Restrictions);

	/**
	 * Trimmed the face with its natural limit curves (Iso UMin,  ...). This function is called to trim untrimmed topological face.
	 * This function should not be called if the topological face already has a loop.
	 */
	void ApplyNaturalLoops();

	/**
	 * Trimmed the face with the boundary limit curves (Iso UMin,  ...). This function is called to trim untrimmed topological face.
	 * This function should not be called if the topological face already has a loop.
	 */
	void ApplyNaturalLoops(const FSurfacicBoundary& Boundaries);

	int32 LoopCount() const
	{
		return Loops.Num();
	}

	const TArray<TSharedPtr<FTopologicalLoop>>& GetLoops() const
	{
		return Loops;
	}

	const TSharedPtr<FTopologicalLoop> GetExternalLoop() const;

	/**
	 * Get a sampling of each loop of the face
	 * @param OutLoopSamplings an array of 2d points
	 */
	const void Get2DLoopSampling(TArray<TArray<FVector2d>>& OutLoopSamplings) const;

	// ======   Loop edge Functions   ======

	/**
	 * @return the twin edge of linked edge belonging this topological face
	 */
	const FTopologicalEdge* GetLinkedEdge(const FTopologicalEdge& LinkedEdge) const;

	/**
	 * Finds the boundary containing the twin edge of Edge
	 * @param Edge
	 * @param OutBoundaryIndex: the index of the boundary containing the twin edge
	 * @param OutEdgeIndex: the index in the boundary of the twin edge
	 */
	void GetEdgeIndex(const FTopologicalEdge& Edge, int32& OutBoundaryIndex, int32& OutEdgeIndex) const;

	/**
	 * Count the edges.
	 */
	void EdgeCount(int32& EdgeCount) const
	{
		for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
		{
			EdgeCount += Loop->GetEdges().Num();
		}
	}

	/**
	 * Add active Edge that has not marker 1 in the edge array.
	 * Marker 1 has to be reset at the end.
	 */
	void GetActiveEdges(TArray<TSharedPtr<FTopologicalEdge>>& OutEdges) const
	{
		for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
		{
			Loop->GetActiveEdges(OutEdges);
		}
	}

	int32 EdgeCount() const
	{
		int32 EdgeNum = 0;
		for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
		{
			EdgeNum += Loop->EdgeCount();
		}
		return EdgeNum;
	}

	// ======   Carrier Surface Functions   ======

	TSharedRef<FSurface> GetCarrierSurface() const
	{
		return CarrierSurface.ToSharedRef();
	}

	// ======   Point evaluation Functions   ======

	void EvaluatePointGrid(const FCoordinateGrid& Coordinates, FSurfacicSampling& OutPoints, bool bComputeNormals = false) const
	{
		CarrierSurface->EvaluatePointGrid(Coordinates, OutPoints, bComputeNormals);
	}

	void EvaluateGrid(FGrid& Grid) const;

	/**
	 * Update the bounding box with an approximation of the surface based of n iso curves 
	 * @param ApproximationFactor: the factor apply to the geometric tolerance to defined the SAG error of the ISO
	 */
	void UpdateBBox(int32 IsoCount, const double ApproximationFactor, FBBoxWithNormal& BBox);

	// ======   Topo Functions   ======

	/** 
	 * Delete a face i.e. disjoin the face to all its linked elements before setting it as deleted
	 */
	void Remove(TArray<FTopologicalEdge*>* NewBorderEdges = nullptr)
	{
		Disjoin(NewBorderEdges);
		Delete();
	}

	virtual void Empty() override
	{
#ifndef DONT_DELETE_FACE_FOR_DEBUG
		CarrierSurface.Reset();
		for (TSharedPtr<FTopologicalLoop>& Loop : Loops)
		{
			for (FOrientedEdge& Edge : Loop->GetEdges())
			{
				Edge.Entity->Delete();
			}

			Loop->Empty();
		}
		Loops.Empty();
#endif

		Mesh.Reset();
		MeshCuttingCoordinates.Empty();
		CrossingCoordinates.Empty();
		CrossingPointDeltaMins.Empty();
		CrossingPointDeltaMaxs.Empty();

		SurfaceCorners.Empty();
		StartSideIndices.Empty();
		SideProperties.Empty();

		FTopologicalShapeEntity::Empty();
	}

	/** Has at least one non manifold edge */
	bool IsANonManifoldFace() const;

	/** Has at least one border edge */
	bool IsABorderFace() const;

	/** All its edges are non manifold */
	bool IsAFullyNonManifoldFace() const;

	/** one adjacent face shares the same loops */
	bool IsADuplicatedFace() const;

	/**
	 * Checks if the face and the other face have the same boundaries i.e. each non degenerated edge is linked to an edge of the other face
	 */
	bool HasSameBoundariesAs(const FTopologicalFace* OtherFace) const;

	/**
	 * Disconnects the face of its neighbors i.e. remove topological edge and vertex link with its neighbors
	 * @param OutNewBorderEdges the neighbors edges
	 */
	void Disjoin(TArray<FTopologicalEdge*>* NewBorderEdges = nullptr);

	/**
	 * Delete only nonmanifold edge link
	 * 
	 */
	void DeleteNonmanifoldLink();


#ifdef CADKERNEL_DEV
	virtual void FillTopologyReport(FTopologyReport& Report) const override;
#endif

	// ======   Meshing Function   ======

	FFaceMesh& GetOrCreateMesh(FModelMesh& ModelMesh);

	const bool HasTesselation() const
	{
		return Mesh.IsValid();
	}

	FFaceMesh* GetMesh() const
	{
		if (Mesh.IsValid())
		{
			return Mesh.Get();
		}
		return nullptr;
	}

	void InitDeltaUs();

	const TArray<double>& GetCuttingCoordinatesAlongIso(EIso Iso) const
	{
		return MeshCuttingCoordinates[Iso];
	}

	TArray<double>& GetCuttingCoordinatesAlongIso(EIso Iso)
	{
		return MeshCuttingCoordinates[Iso];
	}

	const FCoordinateGrid& GetCuttingPointCoordinates() const
	{
		return MeshCuttingCoordinates;
	}

	FCoordinateGrid& GetCuttingPointCoordinates()
	{
		return MeshCuttingCoordinates;
	}

	const FCoordinateGrid& GetCrossingPointCoordinates() const
	{
		return CrossingCoordinates;
	}

	const TArray<double>& GetCrossingPointCoordinates(EIso Iso) const
	{
		return CrossingCoordinates[Iso];
	}

	TArray<double>& GetCrossingPointCoordinates(EIso Iso)
	{
		return CrossingCoordinates[Iso];
	}

	/**
	 * Generate a pre-sampling of the surface saved in CrossingCoordinate.
	 * This sampling is light enough to allow a fast computation of the grid, precise enough to compute accurately meshing criteria
	 * @return false if the face is considered as degenerate and so removed
	 */
	bool ComputeCriteriaGridSampling();
		
	/**
	 * Compute CrossingPointDeltaMins and CrossingPointDeltaMaxs respecting the meshing criteria
	 */
	void ApplyCriteria(const TArray<TSharedPtr<FCriterion>>& Criteria, const FCriteriaGrid& Grid);

	const TArray<double>& GetCrossingPointDeltaMins(EIso Iso) const
	{
		return CrossingPointDeltaMins[Iso];
	}

	const TArray<double>& GetCrossingPointDeltaMaxs(EIso Iso) const
	{
		return CrossingPointDeltaMaxs[Iso];
	}

	// ======   State, Type Functions   ======

	const EQuadType GetQuadType() const
	{
		return QuadType;
	}

	const bool HasThinZone() const
	{
		return ((States & EHaveStates::ThinZone) == EHaveStates::ThinZone);
	}

	void SetHasThinZoneMarker()
	{
		States |= EHaveStates::ThinZone;
	}

	void ResetHasThinSurface()
	{
		States &= ~EHaveStates::ThinZone;
	}

	bool IsBackOriented() const
	{
		return ((States & EHaveStates::IsBackOriented) == EHaveStates::IsBackOriented);
	}

	void SwapOrientation() const
	{
		if (IsBackOriented())
		{
			ResetBackOriented();
		}
		else
		{
			SetBackOriented();
		}
	}

	void SetBackOriented() const
	{
		States |= EHaveStates::IsBackOriented;
	}

	void ResetBackOriented() const
	{
		States &= ~EHaveStates::IsBackOriented;
	}

	virtual void Remove(const FTopologicalShapeEntity*) override
	{
	}

	// ======================================================================================================================================================================================================================
	// Thin zone properties ===============================================================================================================================================================================
	// ======================================================================================================================================================================================================================
private:
	TArray<FThinZone2D> ThinZones;

public:
	void MoveThinZones(TArray<FThinZone2D>& InThinZones)
	{
		ThinZones = MoveTemp(InThinZones);
	}

	const TArray<FThinZone2D>& GetThinZones() const
	{
		return ThinZones;
	}

	TArray<FThinZone2D>& GetThinZones()
	{
		return ThinZones;
	}

	// ======================================================================================================================================================================================================================
	// Quad properties for meshing scheduling ===============================================================================================================================================================================
	// ======================================================================================================================================================================================================================
private:
	TArray<TSharedPtr<FTopologicalVertex>> SurfaceCorners;
	TArray<int32> StartSideIndices;
	TArray<FEdge2DProperties> SideProperties;
	int32 NumOfMeshedSide = 0;
	double LoopLength = -1.;
	double LengthOfMeshedSide = 0;
	double QuadCriteria = 0;
	FSurfaceCurvature Curvatures;
	EQuadType QuadType = EQuadType::Unset;

public:
	void ComputeQuadCriteria();
	double GetQuadCriteria();

	const FSurfaceCurvature& GetCurvatures() const
	{
		return Curvatures;
	}

	FSurfaceCurvature& GetCurvatures()
	{
		return Curvatures;
	}

	const FIsoCurvature& GetCurvature(EIso Iso) const
	{
		return Curvatures[Iso];
	}

	void ComputeSurfaceSideProperties();

	/**
	 * Defines if the surface is either EQuadType::QUAD, either EQuadType::TRIANGULAR or EQuadType::OTHER
	 */
	void DefineSurfaceType();

	const TArray<FEdge2DProperties>& GetSideProperties() const
	{
		return SideProperties;
	}

	FEdge2DProperties& GetSideProperty(int32 Index)
	{
		return SideProperties[Index];
	}

	const FEdge2DProperties& GetSideProperty(int32 Index) const
	{
		return SideProperties[Index];
	}

	int32& MeshedSideNum()
	{
		return NumOfMeshedSide;
	}

	const int32& MeshedSideNum() const
	{
		return NumOfMeshedSide;
	}

	void AddMeshedLength(double Length)
	{
		LengthOfMeshedSide += Length;
	}

	double MeshedSideRatio() const
	{
		return LengthOfMeshedSide / LoopLength;
	}

	int32 GetStartEdgeIndexOfSide(int32 Index) const
	{
		return StartSideIndices[Index];
	}

	const TArray<int32>& GetStartSideIndices() const
	{
		return StartSideIndices;
	}

	int32 GetSideIndex(FTopologicalEdge& Edge) const
	{
		int32 EdgeIndex = Loops[0]->GetEdgeIndex(Edge);
		if (EdgeIndex < 0)
		{
			return -1;
		}
		return GetSideIndex(EdgeIndex);
	}

	int32 GetSideIndex(int32 EdgeIndex) const
	{
		if (StartSideIndices.Num() == 0)
		{
			return -1;
		}

		if (StartSideIndices[0] > EdgeIndex)
		{
			return (int32)StartSideIndices.Num() - 1;
		}
		else
		{
			for (int32 SideIndex = 0; SideIndex < StartSideIndices.Num() - 1; ++SideIndex)
			{
				if (StartSideIndices[SideIndex] <= EdgeIndex && EdgeIndex < StartSideIndices[SideIndex + 1])
				{
					return SideIndex;
				}
			}
			return (int32)StartSideIndices.Num() - 1;
		}
	}

	void SetEstimatedMinimalElementLength(double Value)
	{
		if(Value < EstimatedMinimalElementLength)
		{
			EstimatedMinimalElementLength = Value;
		}
	}

	/**
	 * @return the minimal size of the mesh elements according to meshing criteria
	 */
	double GetEstimatedMinimalElementLength() const
	{
		return EstimatedMinimalElementLength;
	}

};

struct FFaceSubset
{
	TArray<FTopologicalFace*> Faces;
	int32 BorderEdgeCount = 0;
	int32 NonManifoldEdgeCount = 0;
	FTopologicalShapeEntity* MainShell;
	FTopologicalShapeEntity* MainBody;
	FString MainName;
	uint32 MainColor;
	uint32 MainMaterial;

	void SetMainShell(TMap<FTopologicalShapeEntity*, int32>& ShellToFaceCount);
	void SetMainBody(TMap<FTopologicalShapeEntity*, int32>& BodyToFaceCount);
	void SetMainName(TMap<FString, int32>& NameToFaceCount);
	void SetMainColor(TMap<uint32, int32>& ColorToFaceCount);
	void SetMainMaterial(TMap<uint32, int32>& MaterialToFaceCount);
};

struct FBBoxWithNormal
{
	FVector Max;
	FVector Min;
	FVector MaxPoints[3];
	FVector MinPoints[3];
	FVector2d MaxCoordinates[3];
	FVector2d MinCoordinates[3];
	FVector MaxPointNormals[3];
	FVector MinPointNormals[3];
	bool MaxNormalNeedUpdate[3] = {true, true, true};
	bool MinNormalNeedUpdate[3] = {true, true, true};

	FBBoxWithNormal()
		: Max(-HUGE_VALUE, -HUGE_VALUE, -HUGE_VALUE)
		, Min(HUGE_VALUE, HUGE_VALUE, HUGE_VALUE)
	{
	}

	/**
	 * BBox Length is the sum of the length of the 3 sides
	 */
	double Length()
	{
		double XLength = Max.X - Min.X;
		double YLength = Max.Y - Min.Y;
		double ZLength = Max.Z - Min.Z;
		return XLength + YLength + ZLength;
	}

	/**
	 * Return false if the process failed
	 */
	bool CheckOrientation(bool bHasWrongOrientation)
	{
		int32 GoodOrientation = 0;
		int32 WrongOrientation = 0;

		const FVector BBoxNormals[] = { FVector(1., 0., 0.) , FVector(0., 1., 0.) , FVector(0., 0., 1.) };

		for (int32 Index = 0; Index < 3; ++Index)
		{
			double DotProduct = MaxPointNormals[Index] | BBoxNormals[Index];
			if (DotProduct > DOUBLE_KINDA_SMALL_NUMBER)
			{
				GoodOrientation++;
			}
			else if(DotProduct < DOUBLE_KINDA_SMALL_NUMBER)
			{
				WrongOrientation++;
			}

			DotProduct = MinPointNormals[Index] | BBoxNormals[Index];
			if (DotProduct < DOUBLE_KINDA_SMALL_NUMBER)
			{
				GoodOrientation++;
			}
			else if (DotProduct > DOUBLE_KINDA_SMALL_NUMBER)
			{
				WrongOrientation++;
			}
		}

		bHasWrongOrientation = (GoodOrientation < WrongOrientation);
		return ((GoodOrientation >= 3) || (WrongOrientation >= 3)) && (GoodOrientation != WrongOrientation);
	}

	void Update(const FPolylineBBox& PolylineBBox, const EIso IsoType, double IsoCoordinate)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			if (PolylineBBox.Max[Index] > Max[Index])
			{
				Max[Index] = PolylineBBox.Max[Index];
				MaxPoints[Index] = PolylineBBox.MaxPoints[Index];
				MaxCoordinates[Index] = (IsoType == IsoU) ? FVector2d(IsoCoordinate, PolylineBBox.CoordinateOfMaxPoint[Index]) : FVector2d(PolylineBBox.CoordinateOfMaxPoint[Index], IsoCoordinate);
				MaxNormalNeedUpdate[Index] = true;
			}

			if (PolylineBBox.Min[Index] < Min[Index])
			{
				Min[Index] = PolylineBBox.Min[Index];
				MinPoints[Index] = PolylineBBox.MinPoints[Index];
				MinCoordinates[Index] = (IsoType == IsoU) ? FVector2d(IsoCoordinate, PolylineBBox.CoordinateOfMinPoint[Index]) : FVector2d(PolylineBBox.CoordinateOfMinPoint[Index], IsoCoordinate);
				MinNormalNeedUpdate[Index] = true;
			}
		}
	}

	void Update(const FVector& Point, const FVector2d InPoint2D)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			if (Point[Index] > Max[Index])
			{
				Max[Index] = Point[Index];
				MaxPoints[Index] = Point;
				MaxCoordinates[Index] = InPoint2D;
				MaxNormalNeedUpdate[Index] = true;
			}

			if (Point[Index] < Min[Index])
			{
				Min[Index] = Point[Index];
				MinPoints[Index] = Point;
				MinCoordinates[Index] = InPoint2D;
				MinNormalNeedUpdate[Index] = true;
			}
		}
	}

	void UpdateNormal(const FTopologicalFace& Face)
	{
		const FSurface& Surface = *Face.GetCarrierSurface();
		bool bSwapOrientation = Face.IsBackOriented();

		for (int32 Index = 0; Index < 3; ++Index)
		{
			if (MaxNormalNeedUpdate[Index])
			{
				MaxPointNormals[Index] = Surface.EvaluateNormal(MaxCoordinates[Index]);
				if (bSwapOrientation)
				{
					MaxPointNormals[Index] *= -1.;
				}
				MaxNormalNeedUpdate[Index] = false;
			}

			if (MinNormalNeedUpdate[Index])
			{
				MinPointNormals[Index] = Surface.EvaluateNormal(MinCoordinates[Index]);
				if (bSwapOrientation)
				{
					MinPointNormals[Index] *= -1.;
				}
				MinNormalNeedUpdate[Index] = false;
			}
		}
	}
};


} // namespace UE::CADKernel
