// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace UE::CADKernel
{
	class FOrientedEdge;
	class FTopologicalEdge;
	class FTopologicalFace;
	class FTopologicalLoop;

	namespace TopologicalFaceUtilities
	{
		bool CADKERNEL_API IsPlanar(const UE::CADKernel::FTopologicalFace& Face);
		TArray<FVector2d> CADKERNEL_API Get2DPolyline(const UE::CADKernel::FTopologicalEdge& Edge);
		TArray<FVector2d> CADKERNEL_API Get2DPolyline(const UE::CADKernel::FOrientedEdge& Edge);
		TArray<FVector2d> CADKERNEL_API Get2DPolyline(const UE::CADKernel::FTopologicalLoop& Loop);
		TArray<FVector> CADKERNEL_API Get3DPolyline(const UE::CADKernel::FTopologicalEdge& Edge);
		TArray<FVector> CADKERNEL_API Get3DPolyline(const UE::CADKernel::FOrientedEdge& Edge);
		TArray<FVector> CADKERNEL_API Get3DPolyline(const UE::CADKernel::FTopologicalLoop& Loop);
	}
}