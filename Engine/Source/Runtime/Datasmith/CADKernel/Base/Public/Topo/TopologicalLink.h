// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADEntity.h" 
#include "Math/Point.h"
#include "Topo/TopologicalEntity.h"
#include "UI/Message.h"

namespace UE::CADKernel
{

class FTopologicalEdge;
class FTopologicalVertex;

template<typename EntityType>
class CADKERNEL_API TTopologicalLink : public FTopologicalEntity
{
	friend FEntity;

protected:
	friend EntityType;
	EntityType* ActiveEntity;

	TArray<EntityType*> TwinEntities;

	TTopologicalLink()
		: ActiveEntity(nullptr)
	{
	}

	TTopologicalLink(EntityType& Entity)
		: ActiveEntity(&Entity)
	{
		TwinEntities.Add(&Entity);
	}

public:

	virtual ~TTopologicalLink() override
	{
		TTopologicalLink::Empty();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
#ifdef CADKERNEL_DEV
		if (Ar.IsSaving())
		{
			ensureCADKernel(ActiveEntity != nullptr);
			ensureCADKernel(!ActiveEntity->IsDeleted());
		}
#endif
		FEntity::Serialize(Ar);
		SerializeIdent(Ar, &ActiveEntity, false);
		SerializeIdents(Ar, TwinEntities, false);
	}

	virtual void Empty() override
	{
		TwinEntities.Empty();
		ActiveEntity = nullptr;
	}

	const EntityType* GetActiveEntity() const
	{
		ensureCADKernel(ActiveEntity);
		return ActiveEntity;
	}

	EntityType* GetActiveEntity()
	{
		ensureCADKernel(ActiveEntity);
		return ActiveEntity;
	}

	int32 GetTwinEntityNum() const
	{
		return TwinEntities.Num();
	}

	const TArray<EntityType*>& GetTwinEntities() const
	{
		return TwinEntities;
	}

	void ActivateEntity(const EntityType& NewActiveEntity)
	{
		TFunction<bool()> CheckEntityIsATwin = [&]() {
			for (EntityType* Entity : TwinEntities)
			{
				if (Entity == &NewActiveEntity)
				{
					return true;
				}
			}
			UE_LOGF(LogCADKernelBase, Warning, "FTopologicalLink::ActivateEntity, the topological entity is not found in the twins entities");
			return false;
		};

		ensureCADKernel(CheckEntityIsATwin());
		ActiveEntity = &NewActiveEntity;
	}

	void RemoveEntity(TSharedPtr<EntityType>& Entity)
	{
		EntityType* EntityPtr = *Entity;
		RemoveEntity(*EntityPtr);
	}

	void RemoveEntity(EntityType& Entity)
	{
		TwinEntities.Remove(&Entity);
		if (&Entity == ActiveEntity && TwinEntities.Num() > 0)
		{
			ActiveEntity = TwinEntities.HeapTop();
		}

		if (TwinEntities.Num() == 0)
		{
			ActiveEntity = nullptr;
			Delete();
		}
	}

	//void UnlinkTwinEntities()
	//{
	//	for (EntityType* Entity : TwinsEntities)
	//	{
	//		Entity->ResetTopologicalLink();
	//	}
	//	TwinsEntities.Empty();
	//}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity& Info) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::EdgeLink;
	}

	void AddEntity(EntityType* Entity)
	{
		TwinEntities.Add(Entity);
	}

	void AddEntity(EntityType& Entity)
	{
		TwinEntities.Add(&Entity);
	}

	template <typename LinkableType>
	void AddEntity(const LinkableType* Entity)
	{
		TwinEntities.Add((EntityType*)Entity);
	}

	template <typename ArrayType>
	void AddEntities(const ArrayType& Entities)
	{
		TwinEntities.Insert(Entities, TwinEntities.Num());
	}

	/**
	 * @return true if the Twin entity count link is modified
	 */
	virtual bool CleanLink()
	{
		TArray<EntityType*> NewTwinsEntities;
		NewTwinsEntities.Reserve(TwinEntities.Num());
		for (EntityType* Entity : TwinEntities)
		{
			if (Entity)
			{
				NewTwinsEntities.Add(Entity);
			}
		}

		if (NewTwinsEntities.Num() != TwinEntities.Num())
		{
			Swap(NewTwinsEntities, TwinEntities);
			if (TwinEntities.Num())
			{
				ActiveEntity = TwinEntities.HeapTop();
				return true;
			}
		}
		return false;
	}

	void ResetMarkersRecursively() const
	{
		for (EntityType* Entity : TwinEntities)
		{
			if (Entity)
			{
				Entity->ResetMarkers();
			}
		}
	}
};

} // namespace UE::CADKernel
