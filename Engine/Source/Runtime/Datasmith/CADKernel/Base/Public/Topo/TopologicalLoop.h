// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/OrientedEntity.h"
#include "Geo/GeoEnum.h"
#include "Math/Boundary.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalEntity.h"
#include "Topo/TopologicalVertex.h"

namespace UE::CADKernel
{

class FDatabase;
class FTopologicalFace;
class FTopologicalVertex;

class CADKERNEL_API FOrientedEdge : public TOrientedEntity<FTopologicalEdge>
{
public:
	FOrientedEdge(TSharedPtr<FTopologicalEdge>& InEntity, EOrientation InDirection)
		: TOrientedEntity(InEntity, InDirection)
	{
	}

	FOrientedEdge()
		: TOrientedEntity()
	{
	}

	bool operator==(const FOrientedEdge& Edge) const
	{
		return Entity == Edge.Entity;
	}

	TSharedRef<FTopologicalVertex> GetStartVertex()
	{
		return Direction == EOrientation::Front ? Entity->GetStartVertex() : Entity->GetEndVertex();
	}

	TSharedRef<FTopologicalVertex> GetEndVertex()
	{
		return Direction == EOrientation::Front ? Entity->GetEndVertex() : Entity->GetStartVertex();
	}

	double GetStartCoordinate()
	{
		return Direction == EOrientation::Front ? Entity->GetBoundary().GetMin() : Entity->GetBoundary().GetMax();
	}

	double GetEndCoordinate()
	{
		return Direction == EOrientation::Front ? Entity->GetBoundary().GetMax() : Entity->GetBoundary().GetMin();
	}
};

class CADKERNEL_API FTopologicalLoop : public FTopologicalEntity
{
	friend class FEntity;
	friend class FTopologicalFace;
	friend class FTopologicalFace;
	friend class FTopologicalEdge;

public:
	FSurfacicBoundary Boundary;

protected:
	TArray<FOrientedEdge> Edges;

	FTopologicalFace* Face;
	bool bIsExternal;

	FTopologicalLoop(const TArray<TSharedPtr<FTopologicalEdge>>& Edges, const TArray<EOrientation>& EdgeDirections, const bool bIsEternalLoop);

	FTopologicalLoop() = default;

private:

	void SetSurface(FTopologicalFace* HostedFace)
	{
		Face = HostedFace;
	}

	void ResetSurface()
	{
		Face = nullptr;
	}

public:

	virtual ~FTopologicalLoop() override
	{
		FTopologicalLoop::Empty();
	}

	virtual void Empty() override
	{
		for (FOrientedEdge& Edge : Edges)
		{
			Edge.Entity->Empty();
		}
		Edges.Empty();
		FTopologicalEntity::Empty();
	}

	static TSharedPtr<FTopologicalLoop> Make(const TArray<TSharedPtr<FTopologicalEdge>>& EdgeList, const TArray<EOrientation>& EdgeDirections, const bool bIsExternalLoop, const double GeometricTolerance);

	void DeleteLoopEdges();

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalEntity::Serialize(Ar);
		SerializeIdents(Ar, (TArray<TOrientedEntity<FEntity>>&) Edges);
		SerializeIdent(Ar, &Face);
		Ar << bIsExternal;
	}

	virtual void SpawnIdent(FDatabase& Database) override
	{
		if (!FEntity::SetId(Database))
		{
			return;
		}

		SpawnIdentOnEntities((TArray<TOrientedEntity<FEntity>>&) Edges, Database);
	}

	virtual void ResetMarkersRecursively() const override
	{
		ResetMarkers();
		ResetMarkersRecursivelyOnEntities((TArray<TOrientedEntity<FEntity>>&) Edges);
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::TopologicalLoop;
	}

	double Length() const;
	
	const int32 EdgeCount() const
	{
		return Edges.Num();
	}

	const TArray<FOrientedEdge>& GetEdges() const
	{
		return Edges;
	}

	TArray<FOrientedEdge>& GetEdges()
	{
		return Edges;
	}

	const FOrientedEdge* GetOrientedEdge(const FTopologicalEdge* InEdge) const 
	{
		for (const FOrientedEdge& Edge : Edges)
		{
			if (Edge.Entity.Get() == InEdge)
			{
				return &Edge;
			}
		}
		return nullptr;
	}

	/**
	 * Add active Edge that has not marker 1 in the edge array.
	 * Marker 1 has to be reset at the end.
	 */
	void GetActiveEdges(TArray<TSharedPtr<FTopologicalEdge>>& OutEdges) const
	{
		for (const FOrientedEdge& Edge : Edges)
		{
			TSharedPtr<FTopologicalEdge> ActiveEdge = Edge.Entity->GetLinkActiveEdge();
			if (!ActiveEdge->HasMarker1())
			{
				ActiveEdge->SetMarker1();
				OutEdges.Emplace(ActiveEdge);
			}
		}
	}

	FTopologicalFace* GetFace() const
	{
		return Face;
	}

	bool IsExternal() const 
	{
		return bIsExternal;
	}

	void SetExternal()
	{
		bIsExternal = true;
	}

	void SetInternal()
	{
		bIsExternal = false;
	}

	/*
	 * @return false if the orientation is doubtful
	 */
	bool Orient();
	void SwapOrientation();

	void ReplaceEdge(TSharedPtr<FTopologicalEdge>& OldEdge, TSharedPtr<FTopologicalEdge>& NewEdge);
	void ReplaceEdge(TSharedPtr<FTopologicalEdge>& Edge, TArray<TSharedPtr<FTopologicalEdge>>& NewEdges);
	void ReplaceEdges(TArray<FOrientedEdge>& Candidates, TSharedPtr<FTopologicalEdge>& NewEdge);

	/**
	 * The Edge is split in two edges : Edge + NewEdge
	 * @param bNewEdgeIsFirst == true => StartVertex Connected to Edge, EndVertexConnected to NewEdge
	 * According to the direction of Edge, if bNewEdgeIsFirst == true, NewEdge is added in the loop after (EOrientation::Front) or before (EOrientation::Back)
	 */
	void SplitEdge(FTopologicalEdge& Edge, TSharedPtr<FTopologicalEdge> NewEdge, bool bNewEdgeIsFirst);

	void RemoveEdge(TSharedPtr<FTopologicalEdge>& Edge);
	//void ReplaceEdgesWithMergedEdge(TArray<TSharedPtr<FTopologicalEdge>>& OldEdges, TSharedPtr<FTopologicalVertex>& MiddleVertex, TSharedPtr<FTopologicalEdge>& NewEdge);

	EOrientation GetDirection(TSharedPtr<FTopologicalEdge>& Edge, bool bAllowLinkedEdge = false) const;

	EOrientation GetDirection(int32 Index) const
	{
		return Edges[Index].Direction;
	}

	const TSharedPtr<FTopologicalEdge>& GetEdge(int32 Index) const
	{
		return Edges[Index].Entity;
	}

	int32 GetEdgeIndex(const FTopologicalEdge& Edge) const
	{
		for (int32 Index = 0; Index < Edges.Num(); ++Index)
		{
			if (&Edge == Edges[Index].Entity.Get())
			{
				return Index;
			}
		}
		return -1;
	}

	void Get2DSampling(TArray<FVector2d>& LoopSampling) const;

	/**
	 * The idea is to remove degenerated edges of the loop i.e. where the surface is degenerated
	 * - so where projections on these area are hazardous
	 * - so where 2d curve computation based on hazardous projection is hazardous...
	 * - so where the sampling could be in self-intersecting
	 * @return false if the loop is degenerated
	 */
	bool Get2DSamplingWithoutDegeneratedEdges(TArray<FVector2d>& LoopSampling) const;

	void FindSurfaceCorners(TArray<TSharedPtr<FTopologicalVertex>>& OutCorners, TArray<int32>& OutStartSideIndex) const;
	void FindBreaks(TArray<TSharedPtr<FTopologicalVertex>>& Ruptures, TArray<int32>& OutStartSideIndex, TArray<double>& RuptureValues) const;

	void ComputeBoundaryProperties(const TArray<int32>& StartSideIndex, TArray<FEdge2DProperties>& OutSideProperties) const;

	void EnsureLogicalClosing(const double GeometricTolerance);

	void CheckEdgesOrientation();
	void CheckLoopWithTwoEdgesOrientation();
	void RemoveDegeneratedEdges();

	bool IsInside(const FTopologicalLoop& Other) const;

};

}
