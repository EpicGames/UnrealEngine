// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/MetadataDictionary.h"
#include "Topo/TopologicalEntity.h"

namespace UE::CADKernel
{
class FModelMesh;
class FTopologicalFace;
class FTopologyReport;

class CADKERNEL_API FTopologicalShapeEntity : public FTopologicalEntity
{
	friend class FBody;
	friend class FModel;
	friend class FShell;

private:
	FTopologicalShapeEntity* HostedBy = nullptr;
	FMetadataDictionary Dictionary;

public:

	virtual ~FTopologicalShapeEntity() override 
	{
		FTopologicalShapeEntity::Empty();
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		FTopologicalEntity::Serialize(Ar);
		Dictionary.Serialize(Ar);
		SerializeIdent(Ar, &HostedBy);
	}

	virtual void Empty() override 
	{
		HostedBy = nullptr;
		FTopologicalEntity::Empty();
	}

	const FMetadataDictionary& GetMetaDataDictionary() const
	{
		return Dictionary;
	}

	void ExtractMetaData(TMap<FString, FString>& OutMetaData) const
	{
		Dictionary.ExtractMetaData(OutMetaData);
	}

	virtual void CompleteMetaData() = 0;
	void CompleteMetaDataWithHostMetaData();

	virtual int32 FaceCount() const = 0;
	virtual void GetFaces(TArray<FTopologicalFace*>& OutFaces) = 0;

	/**
	 * Each face of model is set by its orientation. This allow to make oriented mesh and to keep the face orientation in topological function.
	 * Marker2 of propagate face is set. It must be reset after the process
	 */
	virtual void PropagateBodyOrientation() = 0;

#ifdef CADKERNEL_DEV
	virtual void FillTopologyReport(FTopologyReport& Report) const = 0;
#endif

	FTopologicalShapeEntity* GetHost()
	{
		return HostedBy;
	}

	const FTopologicalShapeEntity* GetHost() const
	{
		return HostedBy;
	}

	void SetHost(FTopologicalShapeEntity* Body)
	{
		if (HostedBy)
		{
			HostedBy->Remove(this);
		}
		HostedBy = Body;
	}

	virtual void Remove(const FTopologicalShapeEntity*) = 0;

	void SetHostId(const int32 InHostId)
	{
		Dictionary.SetHostId(InHostId);
	}

	int32 GetHostId() const
	{
		return Dictionary.GetHostId();
	}

	void SetLayer(const int32 InLayerId)
	{
		Dictionary.SetLayer(InLayerId);
	}

	void SetName(const FString& InName)
	{
		Dictionary.SetName(InName);
	}

	bool HasName() const
	{
		return Dictionary.HasName();
	}

	const TCHAR* GetName() const
	{
		return Dictionary.GetName();
	}

	void SetColorId(const uint32& InColorId)
	{
		Dictionary.SetColorId(InColorId);
	}

	uint32 GetColorId() const
	{
		return Dictionary.GetColorId();
	}

	void SetMaterialId(const uint32& InMaterialId)
	{
		Dictionary.SetMaterialId(InMaterialId);
	}

	uint32 GetMaterialId() const
	{
		return Dictionary.GetMaterialId();
	}

	void SetDisplayData(const uint32& InColorId, const uint32& InMaterialId)
	{
		if (InColorId)
		{
			Dictionary.SetColorId(InColorId);
		}
		if(InMaterialId)
		{
			Dictionary.SetMaterialId(InMaterialId);
		}
	}

	void SetDisplayData(const FTopologicalShapeEntity& DisplayData)
	{
		SetDisplayData(DisplayData.GetColorId(), DisplayData.GetMaterialId());
	}

	void SetPatchId(int32 InPatchId)
	{
		Dictionary.SetPatchId(InPatchId);
	}

	int32 GetPatchId() const
	{
		return Dictionary.GetPatchId();
	}

#ifdef CADKERNEL_DEV
	virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

protected:
	/**
	 * Mandatory: The host must have remove this from his array
	 */
	void ResetHost()
	{
		HostedBy = nullptr;
	}

};

} // namespace UE::CADKernel

