// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/HaveStates.h"
#include "Core/Types.h"
#include "Math/Point.h"
#include "Topo/Linkable.h"
#include "Topo/TopologicalLink.h"

namespace UE::CADKernel
{
class FDatabase;
class FModelMesh;
class FTopologicalEdge;
class FVertexMesh;
class FTopologicalVertex;

/**
 * TTopologicalLink overload dedicated to FVertex to manage the barycenter of twin vertices
 */
class FVertexLink : public TTopologicalLink<FTopologicalVertex>
{
	friend class FTopologicalVertex;

protected:
	FVector Barycenter;

	void SetBarycenter(const FVector& Point)
	{
		Barycenter = Point;
	}

public:
	FVertexLink()
		: Barycenter(FVector::ZeroVector)
	{
	}

	FVertexLink(FTopologicalVertex& Entity)
		: TTopologicalLink<FTopologicalVertex>(Entity)
		, Barycenter(FVector::ZeroVector)
	{
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		TTopologicalLink<FTopologicalVertex>::Serialize(Ar);
		Ar << Barycenter;
	}

#ifdef CADKERNEL_DEV
	CADKERNEL_API virtual FInfoEntity& GetInfo(FInfoEntity& Info) const override;
#endif

	const FVector& GetBarycenter() const
	{
		return Barycenter;
	}

	virtual bool CleanLink() override
	{
		if (TTopologicalLink::CleanLink())
		{
			ComputeBarycenter();
			DefineActiveEntity();
			return true;
		}
		return false;
	}

	virtual EEntity GetEntityType() const override
	{
		return EEntity::VertexLink;
	}

	CADKERNEL_API void ComputeBarycenter();
	CADKERNEL_API void DefineActiveEntity();
};

class FTopologicalVertex : public TLinkable<FTopologicalVertex, FVertexLink>
{
	friend class FEntity;
	friend class FVertexLink;

protected:

	TArray<FTopologicalEdge*> ConnectedEdges;
	FVector Coordinates;
	TSharedPtr<FVertexMesh> Mesh;

	FTopologicalVertex(const FVector& InCoordinates)
		: Coordinates(InCoordinates)
	{
	}

	FTopologicalVertex() = default;

public:

	virtual ~FTopologicalVertex() override
	{
		FTopologicalVertex::Empty();
	}

	static TSharedRef<FTopologicalVertex> Make(const FVector& InCoordinate)
	{
		TSharedRef<FTopologicalVertex> Vertex = FEntity::MakeShared<FTopologicalVertex>(InCoordinate);
		Vertex->Finalize();
		return Vertex;
	}

	virtual void Serialize(FCADKernelArchive& Ar) override
	{
		if (Ar.IsSaving())
		{
			ensureCADKernel(ConnectedEdges.Num());
		}

		TLinkable<FTopologicalVertex, FVertexLink>::Serialize(Ar);
		Ar.Serialize(Coordinates);
		SerializeIdents(Ar, ConnectedEdges);
	}

	CADKERNEL_API virtual void SpawnIdent(FDatabase& Database) override;

#ifdef CADKERNEL_DEV
	CADKERNEL_API virtual FInfoEntity& GetInfo(FInfoEntity&) const override;
#endif

	virtual EEntity GetEntityType() const override
	{
		return EEntity::TopologicalVertex;
	}

	/**
	 * @return the 3d coordinate of the barycenter of the twin vertices
	 */
	inline const FVector& GetBarycenter() const
	{
		if (TopologicalLink.IsValid() && TopologicalLink->GetTwinEntityNum() > 1)
		{
			return TopologicalLink->GetBarycenter();
		}
		return Coordinates;
	}

	/**
	 * @return the 3d coordinates of the vertex (prefere GetBarycenter())
	 */
	const FVector& GetCoordinates() const
	{
		return Coordinates;
	}

	void SetCoordinates(const FVector& NewCoordinates)
	{
		if (GetLink()->GetTwinEntityNum() > 1)
		{
			// Update barycenter
			FVector BaryCenter = GetLink()->GetBarycenter() * (double)GetLink()->GetTwinEntityNum();
			BaryCenter -= Coordinates;
			BaryCenter += NewCoordinates;
			BaryCenter /= (double)GetLink()->GetTwinEntityNum();
			GetLink()->SetBarycenter(BaryCenter);
		}
		else
		{
			GetLink()->SetBarycenter(NewCoordinates);
		}
		Coordinates = NewCoordinates;
	}

	double Distance(const FTopologicalVertex& OtherVertex) const
	{
		return FVector::Distance(Coordinates, OtherVertex.Coordinates);
	}

	double SquareDistance(const FTopologicalVertex& OtherVertex) const
	{
		return FVector::DistSquared(Coordinates, OtherVertex.Coordinates);
	}

	double SquareDistanceBetweenBarycenters(const FTopologicalVertex& OtherVertex) const
	{
		return FVector::DistSquared(GetLink()->GetBarycenter(), OtherVertex.GetLink()->GetBarycenter());
	}

	double SquareDistance(const FVector& Point) const
	{
		return FVector::DistSquared(Coordinates, Point);
	}

	CADKERNEL_API FVertexMesh& GetOrCreateMesh(FModelMesh& MeshModel);

	const FVertexMesh* GetMesh() const
	{
		if (!IsActiveEntity())
		{
			return GetLinkActiveEntity()->GetMesh();
		}
		if (Mesh.IsValid())
		{
			return Mesh.Get();
		}
		return nullptr;
	}

	CADKERNEL_API void Link(FTopologicalVertex& InEntity);

	CADKERNEL_API void UnlinkTo(FTopologicalVertex& Entity);

	virtual void RemoveFromLink() override
	{
		if (TopologicalLink.IsValid())
		{
			TopologicalLink->RemoveEntity(*this);
			TopologicalLink->ComputeBarycenter();
			ResetTopologicalLink();
		}
	}

	void DeleteIfIsolated()
	{
		if (ConnectedEdges.Num() == 0)
		{
			if (TopologicalLink.IsValid())
			{
				TopologicalLink->RemoveEntity(*this);
				if (!TopologicalLink->IsDeleted())
				{
					TopologicalLink->ComputeBarycenter();
				}
				TopologicalLink.Reset();
			}
			Delete();
		}
	}

	virtual void Empty() override
	{
		ConnectedEdges.Empty();
		Mesh.Reset();
		TLinkable<FTopologicalVertex, FVertexLink>::Empty();
	}

	CADKERNEL_API bool IsBorderVertex() const;

	CADKERNEL_API void AddConnectedEdge(FTopologicalEdge& Edge);
	CADKERNEL_API void RemoveConnectedEdge(FTopologicalEdge& Edge);

	/**
	 * Mandatory: to browse all the connected edges, you have to browse the connected edges of all the twin vertices
	 * for (TWeakPtr<FTopologicalVertex> TwinVertex : Vertex->GetTwinsEntities())
	 * {
	 *    for (TWeakPtr<FTopologicalEdge> ConnectedEdge : TwinVertex.Pin()->GetDirectConnectedEdges())
	 *    {
	 *       ...
	 *    }
	 *  }
	 */
	const TArray<FTopologicalEdge*>& GetDirectConnectedEdges() const
	{
		return ConnectedEdges;
	}

	CADKERNEL_API const FTopologicalFace* GetFace() const;

	void GetConnectedEdges(TArray<FTopologicalEdge*>& OutConnectedEdges) const
	{
		if (!TopologicalLink.IsValid())
		{
			OutConnectedEdges = ConnectedEdges;
		}
		else
		{
			OutConnectedEdges.Reserve(100);
			for (const FTopologicalVertex* Vertex : GetLink()->GetTwinEntities())
			{
				OutConnectedEdges.Append(Vertex->ConnectedEdges);
			}
		}
	}

	const int32 ConnectedEdgeCount()
	{
		if (!TopologicalLink.IsValid())
		{
			return ConnectedEdges.Num();
		}
		else
		{
			int32 Count = 0;
			for (const FTopologicalVertex* Vertex : GetLink()->GetTwinEntities())
			{
				Count += Vertex->ConnectedEdges.Num();
			}
			return Count;
		}
	}

	/**
	 *
	 */
	CADKERNEL_API void GetConnectedEdges(const FTopologicalVertex& OtherVertex, TArray<FTopologicalEdge*>& Edges) const;
};

} // namespace UE::CADKernel
