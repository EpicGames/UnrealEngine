// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#ifdef CADKERNEL_DEV
#include "Topo/TopomakerReport.h"
#endif

namespace UE::CADKernel
{

class FModel;
class FSession;
class FShell;
class FTopologicalEdge;
class FTopologicalFace;
class FTopologicalVertex;

enum class ESewOption : uint8
{
	None = 0x00u,	// No flags.

	ForceJoining = 0x01u,
	RemoveThinFaces = 0x02u, 
	RemoveDuplicatedFaces = 0x04u,

	All = 0x07u
};

ENUM_CLASS_FLAGS(ESewOption);

namespace SewOption
{

static ESewOption GetFromOptions(bool bGStitchingForceSew, bool bGStitchingRemoveThinFaces, bool bGStitchingRemoveDuplicatedFaces)
{
	ESewOption Option = ESewOption::None;

	if (bGStitchingForceSew)
	{
		Option |= ESewOption::ForceJoining;
	}

	if (bGStitchingRemoveThinFaces)
	{
		Option |= ESewOption::RemoveThinFaces;
	}

	if (bGStitchingRemoveDuplicatedFaces)
	{
		Option |= ESewOption::RemoveDuplicatedFaces;
	}

	return Option;
}

}

namespace TopomakerTools
{

/**
 * Merge Border Vertices with other vertices.
 * @param Vertices: the initial array of active vertices to process, this array is updated at the end of the process
 */
void MergeCoincidentVertices(TArray<FTopologicalVertex*>& VerticesToMerge, double Tolerance);

}

struct FTopomakerOptions
{
	ESewOption SewOptions;
	double Tolerance;
	double ForceJoinFactor;

	FTopomakerOptions(ESewOption InSewOptions, double InTolerance, double InForceJoinFactor)
		: SewOptions(InSewOptions)
		, Tolerance(InTolerance)
		, ForceJoinFactor(InForceJoinFactor)
	{}
};


class CADKERNEL_API FTopomaker
{

protected:

	FSession& Session;

	TArray<FShell*> Shells;
	TArray<TSharedPtr<FTopologicalFace>> Faces;

#ifdef CADKERNEL_DEV
	FTopomakerReport Report;
#endif

public:

	FTopomaker(FSession& InSession, const FTopomakerOptions& InOptions);
	FTopomaker(FSession& InSession, const TArray<TSharedPtr<FShell>>& Shells, const FTopomakerOptions& InOptions);
	FTopomaker(FSession& InSession, const TArray<TSharedPtr<FTopologicalFace>>& Faces, const FTopomakerOptions& InOptions);

	void SetTolerance(const FTopomakerOptions Options)
	{
		SewOptions = Options.SewOptions;
		Tolerance = Options.Tolerance;
		ForceJoinFactor = Options.ForceJoinFactor;

		SewTolerance = Tolerance * UE_DOUBLE_SQRT_2;

		// StartVertex (SV) is linked to TwinStartVertex (TSV), the max distance between vertices is SewTolerance
		// EndVertex (EV) is linked to TwinEndVertex (TEV), the max distance between vertices is SewTolerance
		//
		//           * TSV * TEV
		//     SV *-----------* EV
		// So the max length is 2*SewTolerance
		EdgeLengthTolerance = SewTolerance * 2.;

		SewToleranceToForceJoin = Tolerance * ForceJoinFactor;
		LargeEdgeLengthTolerance = SewToleranceToForceJoin * 2.;

		ThinFaceWidth = Tolerance * ForceJoinFactor * 0.5;
	}

	void Sew();

	/**
	 * Split into connected shell and put each shell into the appropriate body
	 */
	void SplitIntoConnectedShells();

	void OrientShells();

	/**
	 * Unlink Non-Manifold Vertex i.e. Vertex belong to tow or more shell
	 */
	void UnlinkNonManifoldVertex();

	/**
	 * Mandatory: UnlinkNonManifoldVertex has to be call before
	 */
	void UnlinkFromOther();

	void RemoveThinFaces();

	/**
	 * Mandatory: 
	 * UnlinkNonManifoldVertex has to be call after
	 */
	void DeleteNonmanifoldLink();

#ifdef CADKERNEL_DEV
	void PrintSewReport()
	{
		Report.PrintSewReport();
	}

	void PrintRemoveThinFacesReport()
	{
		Report.PrintRemoveThinFacesReport();
	}
#endif

private:

	/**
	 * Call by constructor.
	 * For each shell, add their faces into Faces array, complete the metadata and set the states for the joining process
	 */
	void InitFaces();

	void EmptyShells();

	void RemoveEmptyShells();

	void RemoveDuplicatedFaces();

	/**
	 * Return an array of active vertices.
	 */
	void GetVertices(TArray<FTopologicalVertex*>& Vertices);

	/**
	 * Return an array of active border vertices.
	 */
	void GetBorderVertices(TArray<FTopologicalVertex*>& BorderVertices);

	void CheckSelfConnectedEdge(double MaxLengthOfDegeneratedEdge, TArray<FTopologicalVertex*>& OutBorderVertices);
	void RemoveIsolatedEdges(); // Useful ???

	void SetSelfConnectedEdgeDegenerated(TArray<FTopologicalVertex*>& Vertices);

	/**
	 * For each loop of each surface, check if successive edges are unconnected and if their common vertices are connected only to them (UV vs CV: vertex connected to many faces ).
	 * These edges must be tangent together.
	 * These edges are merged into one edge.
	 * E.g. :
	 * Face A has 3 successive unconnected edges. If these 3 edges are merged to give only one edge, the new edge could be linked to its parallel edge of Face B
	 * 
	 *  CV: Connected vertex i.e. linked to different faces
	 * 
	 *
	 *              \                          Face A                                   |    Face C
	 *               \                                                                  |
	 *    Face E     CV ------------------ UV -------------------- UV ---------------- CV --------------------
	 *               CV -------------------------------------------------------------- CV --------------------
	 *               /                         Face B                                   |    Face D
	 *              /                                                                   |
	 */
	void MergeUnconnectedSuccessiveEdges();

	void SetToProcessMarkerOfFaces();
	void ResetMarkersOfFaces();

private:

	ESewOption SewOptions;

	double Tolerance;
	double ForceJoinFactor;

	double SewTolerance;
	double SewToleranceToForceJoin; // SewTolerance x ForceJoinFactor;

	double EdgeLengthTolerance; // 2x SewTolerance
	double LargeEdgeLengthTolerance; // 2x SewToleranceToForceJoin

	double ThinFaceWidth;
};

} // namespace UE::CADKernel
