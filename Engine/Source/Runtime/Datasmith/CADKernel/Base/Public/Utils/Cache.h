// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"

namespace UE::CADKernel
{
template<class ObjectType>
class CADKERNEL_API TCache
{
protected:
	ObjectType Value;
	bool bReady;

public:
	TCache()
		: Value()
		, bReady(false)
	{
	}

	template<typename... InArgTypes>
	TCache(InArgTypes&&... Args)
		: Value(Forward<InArgTypes>(Args)...)
		, bReady(true)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, TCache<ObjectType>& Cache)
	{
		Ar << Cache.Value;
		Ar << Cache.bReady;
		return Ar;
	}

	void Set(const ObjectType& NewValue)
	{
		Value = NewValue;
		bReady = true;
	}

	template<typename... InArgTypes>
	void Set(InArgTypes&&... Args)
	{
		Value.Set(Forward<InArgTypes>(Args)...);
		bReady = true;
	}

	void operator=(const ObjectType& NewValue)
	{
		Value = NewValue;
		bReady = true;
	}

	void operator+=(const ObjectType& NewValue)
	{
		ensureCADKernel(bReady);
		Value += NewValue;
	}

	ObjectType* operator->()
	{
		return &Value;
	}

	operator ObjectType& ()
	{
		return Value;
	}

	const ObjectType& operator*() const
	{
		return Value;
	}

	void SetReady()
	{
		bReady = true;
	}

	bool IsValid() const
	{
		return bReady;
	}

	void Empty()
	{
		bReady = false;
	}
};
}
