// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

namespace UE::CADKernel
{
/**
 * Finder of the index of the segment of the polyline coordinates containing the input coordinate
 * This abstract class return the lower index of the segment i.e.
 * InPolylineCoordinates[OutIndex] <= InCoordinate <= InPolylineCoordinates[OutIndex + 1]
 * As input coordinates are increasing, the algorithme is optimize by restarting form the last coordinate found.
 *
 * Class derived from it allows to optimized polyline functions needing the index of the segment containing the input coordinate (@see ApproximatePoint).
 * @see FLinearFinder to optimize the search when the next points to search is nearest the previous.
 * @see FFinderByDichotomy to optimize the search when there is no simple rule, so a dichotomy algorithm is used.
 * This class is dedicated for a unique or mess up set of input coordinates
 */
class CADKERNEL_API FIndexOfCoordinateFinder
{
protected:
	const TArray<double>& Coordinates;
public:
	FIndexOfCoordinateFinder(const TArray<double>& InPolylineCoordinates)
		: Coordinates(InPolylineCoordinates)
	{
	}

	virtual int32 Find(const double InCoordinate) = 0;
};

class CADKERNEL_API FLinearFinder : public FIndexOfCoordinateFinder
{
	int32& Index;

public:
	FLinearFinder(const TArray<double>& InPolylineCoordinates, int32& InOutIndex)
		: FIndexOfCoordinateFinder(InPolylineCoordinates)
		, Index(InOutIndex)
	{
	}

	virtual int32 Find(const double InCoordinate) override
	{
		if ((Coordinates.Num() < 3) || (InCoordinate < Coordinates[0]))
		{
			Index = 0;
			return Index;
		}

		if (InCoordinate > Coordinates.Last())
		{
			Index = Coordinates.Num() - 2;
			return Index;
		}

		if (InCoordinate < Coordinates[Index])
		{
			// find the previous segment containing the coordinate
			while (Index > 0)
			{
				--Index;
				if (InCoordinate + DOUBLE_SMALL_NUMBER >= Coordinates[Index])
				{
					return Index;
				}
			}
			return Index;
		}

		// find the next segment containing the coordinate
		for (; Index < Coordinates.Num() - 2; ++Index)
		{
			if (InCoordinate <= Coordinates[Index + 1])
			{
				return Index;
			}
		}
		Index = Coordinates.Num() - 2;
		return Index;
	}
};

class CADKERNEL_API FDichotomyFinder : public FIndexOfCoordinateFinder
{

public:
	FDichotomyFinder(const TArray<double>& InPolylineCoordinates)
		: FIndexOfCoordinateFinder(InPolylineCoordinates)
		, StartLower(0)
		, StartUpper(InPolylineCoordinates.Num() - 2)
	{
	}

	FDichotomyFinder(const TArray<double>& InPolylineCoordinates, int32 InStartLowerBound, int32 InStartUpperBound)
		: FIndexOfCoordinateFinder(InPolylineCoordinates)
		, StartLower(InStartLowerBound)
		, StartUpper(InStartUpperBound)
	{
	}

	int32 StartLower;
	int32 StartUpper;

	virtual int32 Find(const double InCoordinate) override
	{
		if (StartUpper < 0)
		{
			StartUpper = Coordinates.Num() - 2;
		}

		if ((StartUpper - StartLower < 1) || (InCoordinate < Coordinates[StartLower + 1]))
		{
			return StartLower;
		}

		if (InCoordinate > Coordinates[StartUpper])
		{
			StartLower = StartUpper;
			return StartUpper;
		}

		int32 LowerBound = StartLower;
		int32 UpperBound = StartUpper;

		while (LowerBound <= UpperBound)
		{
			int32 Center = (LowerBound + UpperBound) >> 1;
			if (InCoordinate >= Coordinates[Center] && InCoordinate <= Coordinates[Center + 1])
			{
				StartLower = Center;
				return Center;
			}
			else if (InCoordinate < Coordinates[Center])
			{
				UpperBound = Center;
			}
			else
			{
				LowerBound = Center + 1;
			}
		}
		return StartUpper;
	}
};
} // namespace UE::CADKernel

