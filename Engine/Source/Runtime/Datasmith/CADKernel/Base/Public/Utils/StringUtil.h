// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/Types.h"

#include "Core/Types.h"
#include "Math/MathConst.h"
#include "Math/Point.h"

#include <list>
#include <locale>
#include <string>
#include <sstream>
#include <stdio.h>
#include <stdarg.h>

namespace UE::CADKernel::Utils
{

inline std::string ToString(const TCHAR* WideString)
{
	return TCHAR_TO_UTF8(WideString);
}

template<typename ValueType>
inline std::string ToString(const ValueType& v)
{
	std::stringstream stream;
	stream << v;
	return stream.str();
}

inline std::string ToString(const FVector& p)
{
	std::string Message = "dm.FVector(";
	Message += Utils::ToString(p.X) + "," + Utils::ToString(p.Y) + "," + Utils::ToString(p.Z) + ")";
	return Message;
}

inline FString ToFString(const char* Text)
{
	return UTF8_TO_TCHAR(Text);
}

inline FString ToFString(const double& Value)
{
	return FString::Printf(TEXT("%f"), Value);
}

inline FString ToFString(const int32& Value)
{
	return FString::Printf(TEXT("%d"), Value);
}

inline FString ToFString(const uint32& Value)
{
	return FString::Printf(TEXT("%u"), Value);
}

inline FString ToFString(const bool& Value)
{
	return FString::Printf(TEXT("%d"), Value);
}

inline FString ToFString(const FVector& p)
{
	return FString::Printf(TEXT("dm.FVector(%f, %f, %f)"), p.X, p.Y, p.Z);
}

void Explode(const FString& FullString, const TCHAR* Separator, TArray<FString>& StringArray);

inline FString EscapeBackSlashes(const FString& InputString)
{
	FString ret;
	ret.Reserve(InputString.Len());
	for (TCHAR Char : InputString)
	{
		if (Char == TEXT('\\')) {
			ret.AppendChar(TEXT('\\'));
		}
		ret.AppendChar(Char);
	}
	return ret;
}

inline void RemoveUnwantedChar(FString& String, const TCHAR UnwantedChar)
{
	FString NewString;
	NewString.Reserve(String.Len());
	for (const TCHAR& Char : String)
	{
		if (Char != UnwantedChar)
		{
			NewString.AppendChar(Char);
		}
	}
	Move(String, NewString);
}

}
