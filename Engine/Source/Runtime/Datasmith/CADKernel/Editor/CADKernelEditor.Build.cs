// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class CADKernelEditor : ModuleRules
	{
		public CADKernelEditor(ReadOnlyTargetRules Target)
			: base(Target)
		{
			bEnableExceptions = true;
			bUseUnity = false; // TechSoftLibrary.cpp cannot be compiled with the other cpp files.
			//OptimizeCode = CodeOptimization.Never;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CADKernel",
					"CADKernelEngine",
					"Engine",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"ContentBrowser",
						"EditorFramework",
						"Slate",
						"SlateCore",
						"StaticMeshEditor",
						"ToolMenus",
					}
				);
			}
		}
	}
}