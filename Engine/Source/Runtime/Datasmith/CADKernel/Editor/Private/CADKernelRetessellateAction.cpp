// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelRetessellateAction.h"

#include "LegacyParametricSurfaceData.h"

#include "CADKernelEngine.h"
#include "ParametricSurfaceData.h"

#include "Algo/AnyOf.h"
#include "Async/ParallelFor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Logging/LogMacros.h"

#if WITH_EDITOR
#include "ContentBrowserMenuContexts.h"
#include "IStaticMeshEditor.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "ToolMenus.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogCADKernelRetessellation, Log, All)

#define LOCTEXT_NAMESPACE "CADKernelRetessellateAction"

static const FName CADKernelTessellationTag("CADKernel");

static bool GCADKernelRetessellation = false;
FAutoConsoleVariableRef GCADKernelDebugUseEngine(
	TEXT("CADKernel.Debug.Retessellation"),
	GCADKernelRetessellation,
	TEXT(""),
	ECVF_Default);

namespace UE::CADKernel
{
	class FRetessellationUtils
	{
	public:
		static bool ConfirmAndUpdateTessellation(FCADKernelRetessellationSettings& Settings)
		{
			return false;
		}

		static UParametricSurfaceData* GetParametricSurfaceData(UStaticMesh* StaticMesh)
		{
			UParametricSurfaceData* ParametricSurfaceData = nullptr;

			if (IInterface_AssetUserData* AssetUserData = Cast<IInterface_AssetUserData>(StaticMesh))
			{
				ParametricSurfaceData = AssetUserData->GetAssetUserData<UParametricSurfaceData>();
			}

			return ParametricSurfaceData;
		}

		static bool Retessellate(UStaticMesh& StaticMesh, const FLegacyParametricSurfaceData* LegacyData = nullptr)
		{
			IInterface_AssetUserData* AssetUserData = Cast<IInterface_AssetUserData>(&StaticMesh);
			if (!ensure(AssetUserData))
			{
				return false;
			}

			UParametricSurfaceData* ParametricSurfaceData = LegacyData ? LegacyData->ToCADKernel() : GetParametricSurfaceData(&StaticMesh);
			if (!ensure(ParametricSurfaceData))
			{
				return false;
			}

			if (GCADKernelRetessellation)
			{
				// If StaticMesh has been generated using the Datasmith importer, add ParametricSurfaceData to StaticMesh's AssetUserData
				if (LegacyData)
				{
					ParametricSurfaceData->Rename(nullptr, &StaticMesh, REN_DontCreateRedirectors | REN_NonTransactional);
					AssetUserData->AddAssetUserData(ParametricSurfaceData);
				}
			}

			return GCADKernelRetessellation;
		}

#if WITH_EDITOR
		static void FinalizeChanges(UStaticMesh* StaticMesh)
		{
			StaticMesh->PostEditChange();
			StaticMesh->MarkPackageDirty();

			// Refresh associated editor 
			TSharedPtr<IToolkit> EditingToolkit = FToolkitManager::Get().FindEditorForAsset(StaticMesh);
			if (IStaticMeshEditor* StaticMeshEditorInUse = StaticCastSharedPtr<IStaticMeshEditor>(EditingToolkit).Get())
			{
				StaticMeshEditorInUse->RefreshTool();
			}
		}
#endif
	};
}

bool FCADKernelRetessellateAction::CanRetessellate(UStaticMesh* StaticMesh)
{
	using namespace UE::CADKernel;
	return FRetessellationUtils::GetParametricSurfaceData(StaticMesh) != nullptr;
}

bool FCADKernelRetessellateAction::RetessellateArray(const TArray<UStaticMesh*>& StaticMeshes)
{
	using namespace UE::CADKernel;

	return false;
}

bool FCADKernelRetessellateAction::RetessellateArray(const TArray<FAssetData>& Assets)
{
	TArray<UStaticMesh*> StaticMeshes;
	StaticMeshes.Reserve(Assets.Num());

	for (const FAssetData& AssetData : Assets)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(AssetData.GetAsset()))
		{
			StaticMeshes.Add(StaticMesh);
		}
	}

	return RetessellateArray(StaticMeshes);
}

bool FCADKernelRetessellateAction::RetessellateArray(const TArray<AStaticMeshActor*>& StaticMeshActors)
{
	TSet<UStaticMesh*> StaticMeshes;
	StaticMeshes.Reserve(StaticMeshActors.Num());

	for (AStaticMeshActor* StaticMeshActor : StaticMeshActors)
	{
		if (StaticMeshActor)
		{
			if (UStaticMeshComponent* MeshComponent = StaticMeshActor->GetStaticMeshComponent())
			{
				if (UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh())
				{
					StaticMeshes.Add(StaticMesh);
				}
			}
		}
	}

	return RetessellateArray(StaticMeshes.Array());
}

bool FCADKernelRetessellateAction::RetessellateLegacy(UStaticMesh& StaticMesh, FArchive& Ar)
{
	using namespace UE::CADKernel;

	ensure(Ar.IsLoading());

	FLegacyParametricSurfaceData LegacyParametricSurfaceData;
	LegacyParametricSurfaceData.Serialize(Ar);

	return FRetessellationUtils::Retessellate(StaticMesh, &LegacyParametricSurfaceData);
}

#if WITH_EDITOR
namespace MenuExtension_CADKernel_StaticMesh
{
	static const TAttribute<FText> Label = LOCTEXT("CADKernelRetessellateAction_Label", "Retessellate (New)");
	static const TAttribute<FText> ToolTip = LOCTEXT("CADKernelRetessellateAction_Tooltip", "This will execute the action for your type.");
	static const FSlateIcon Icon = FSlateIcon();

	static void ExecuteRetessellation(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
		FCADKernelRetessellateAction::RetessellateArray(CBContext->LoadSelectedObjects<UStaticMesh>());
	}

	static bool DisplayMenuEntry(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		for (UStaticMesh* StaticMesh : CBContext->LoadSelectedObjects<UStaticMesh>())
		{
			if (FCADKernelRetessellateAction::CanRetessellate(StaticMesh))
			{
				return GCADKernelRetessellation;
			}
		}

		return false;
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
		{
			UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
				{
					FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
					UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UStaticMesh::StaticClass());

					FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
					Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
						{
							if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
							{
								bool bCanRetessellate = Algo::AnyOf(CBContext->SelectedAssets, [](const FAssetData& InAssetData)
									{
										int32 Value;
										return InAssetData.GetTagValue(CADKernelTessellationTag, Value);
									}
								);

								FToolUIAction UIAction;
								UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&DisplayMenuEntry);
								UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRetessellation);
								InSection.AddMenuEntry("CADKernelRetessellateAction_Entry", Label, ToolTip, Icon, UIAction);
							}
						})
					);
				})
			);
		}
	);
}
#endif

#undef LOCTEXT_NAMESPACE
