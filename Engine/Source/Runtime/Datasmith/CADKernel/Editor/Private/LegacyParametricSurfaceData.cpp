// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyParametricSurfaceData.h"

UParametricSurfaceData* FLegacyParametricSurfaceData::ToCADKernel() const
{
	UParametricSurfaceData* ParametricSurfaceData = NewObject<UParametricSurfaceData>();

	ParametricSurfaceData->SetModelParameters(SceneParameters.ToCADKernel());
	ParametricSurfaceData->SetMeshParameters(MeshParameters.ToCADKernel());
	ParametricSurfaceData->SetLastTessellationSettings(LastTessellationOptions.ToCADKernel());

	ParametricSurfaceData->SetRawData(RawData, LastTessellationOptions.Tessellator == ECADKernelTessellator::TechSoft);

	return ParametricSurfaceData;
}

void FLegacyParametricSurfaceData::Serialize(FArchive& Ar)
{
	ensure(Ar.IsLoading());

	Ar << SceneParameters.ModelCoordSys;
	Ar << SceneParameters.MetricUnit;
	Ar << SceneParameters.ScaleFactor;

	Ar << MeshParameters.bNeedSwapOrientation;
	Ar << MeshParameters.bIsSymmetric;
	Ar << MeshParameters.SymmetricOrigin;
	Ar << MeshParameters.SymmetricNormal;

	Ar << LastTessellationOptions.ChordTolerance;
	Ar << LastTessellationOptions.MaxEdgeLength;
	Ar << LastTessellationOptions.NormalTolerance;
	Ar << LastTessellationOptions.StitchingTechnique;
	Ar << LastTessellationOptions.Tessellator;
	Ar << LastTessellationOptions.GeometricTolerance;
	Ar << LastTessellationOptions.StitchingTolerance;

	Ar << RawData;
}
