// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ParametricSurfaceData.h"

namespace UE::CADKernel
{
	class FRetessellationUtils;
}

struct FLegacyParametricSceneParameters
{
	uint8 ModelCoordSys = (uint8)ECADKernelModelCoordSystem::ZUp_LeftHanded;
	float MetricUnit = 0.01f;
	float ScaleFactor = 1.0f;

	FCADKernelModelParameters ToCADKernel() const
	{
		return {
			ModelCoordSys,
			MetricUnit,
			ScaleFactor
		};
	}
};

struct FLegacyParametricMeshParameters
{
	bool bNeedSwapOrientation = false;
	bool bIsSymmetric = false;
	FVector SymmetricOrigin = FVector::ZeroVector;
	FVector SymmetricNormal = FVector::ZeroVector;

	FCADKernelMeshParameters ToCADKernel() const
	{
		return {
			bNeedSwapOrientation,
			bIsSymmetric,
			SymmetricOrigin,
			SymmetricNormal,
		};
	}
};

struct FLegacyTessellationOptions
{
	float ChordTolerance;
	float MaxEdgeLength;
	float NormalTolerance;
	uint8 StitchingTechnique = (uint8)ECADKernelStitchingTechnique::StitchingNone;
	uint8 Tessellator = (uint8)ECADKernelTessellator::CADKernel;
	double GeometricTolerance = 0.001;
	double StitchingTolerance = 0.001;

	FCADKernelTessellationSettings ToCADKernel() const
	{
		return FCADKernelTessellationSettings(
			ChordTolerance,
			MaxEdgeLength,
			NormalTolerance,
			static_cast<ECADKernelStitchingTechnique>(StitchingTechnique),
			GeometricTolerance,
			StitchingTolerance,
			static_cast<ECADKernelTessellator>(Tessellator)
		);
	}
};

class FLegacyParametricSurfaceData
{
	UParametricSurfaceData* ToCADKernel() const;

	void Serialize(FArchive& Ar);

	FLegacyParametricSceneParameters SceneParameters;
	FLegacyParametricMeshParameters MeshParameters;
	FLegacyTessellationOptions LastTessellationOptions;
	TArray<uint8> RawData;

	friend UE::CADKernel::FRetessellationUtils;
	friend class FCADKernelRetessellateAction;
};

