// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricSurfaceData.h"

#include "CADKernelEngine.h"

#include "Core/Session.h"
#include "Topo/Model.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "UObject/EnterpriseObjectVersion.h"

#define LOCTEXT_NAMESPACE "ParametricSurfaceData"

namespace UE::CADKernel::Editor
{
	namespace Private
	{
#if PLATFORM_DESKTOP
		using namespace UE::CADKernel;

		template<typename MeshType>
		bool Tessellate(UParametricSurfaceData& Data, MeshType& MeshOut)
		{
			using namespace UE::CADKernel;

			FTessellationContext Context(Data.GetModelParameters(), Data.GetMeshParameters(), Data.GetLastTessellationSettings());

			if (Context.TessellationSettings.UseCADKernel())
			{
				TSharedPtr<FModel> Model = Data.GetModel();
				if (Model.IsValid())
				{
					return FCADKernelUtilities::Tessellate(*Model, Context, MeshOut);
				}
			}
			else if (FTechSoftLibrary::Initialize())
			{
				A3DRiRepresentationItem* Representation = Data.GetRepresentation();
				return FTechSoftUtilities::Tessellate(Representation, Context, MeshOut);
			}

			return false;
		}

		template<typename MeshType>
		bool Retessellate(UParametricSurfaceData& Data, const FCADKernelRetessellationSettings& Settings, MeshType& MeshOut)
		{
			using namespace UE::CADKernel;
			using namespace UE::CADKernel::MeshUtilities;

			FTessellationContext Context(Data.GetModelParameters(), Data.GetMeshParameters(), Settings);

			if (Context.TessellationSettings.UseCADKernel())
			{
				TSharedPtr<FModel> Model = Data.GetModel();
				if (Model.IsValid())
				{
					const bool bSkipDeletedFaces = Settings.RetessellationRule == ECADKernelRetessellationRule::SkipDeletedFaces;
					return FCADKernelUtilities::Tessellate(*Model, Context, MeshOut, bSkipDeletedFaces, true);
				}
			}
			else if (FTechSoftLibrary::Initialize())
			{
				if (A3DRiRepresentationItem* Representation = Data.GetRepresentation())
				{
					const bool bSkipDeletedFaces = Settings.RetessellationRule == ECADKernelRetessellationRule::SkipDeletedFaces;
					return FTechSoftUtilities::Tessellate(Representation, Context, MeshOut, bSkipDeletedFaces, true);
				}
			}

			return false;
		}
#endif
	}
}

void UParametricSurfaceData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FEnterpriseObjectVersion::GUID) < FEnterpriseObjectVersion::AddedParametricSurfaceData)
	{
		// #cad_import_todo : Put code to deserialize UDatasmithParametricSurfaceData
		 return;
	}

	Super::Serialize(Ar);

	Ar << CADKernelRawData;
	Ar << TechSoftRawData;
}

TSharedPtr<UE::CADKernel::FModel> UParametricSurfaceData::GetModel()
{
	using namespace UE::CADKernel;

	if (CADKernelRawData.IsEmpty())
	{
		return {};
	}

	TSharedRef<FSession> Session = MakeShared<FSession>(LastTessellationSettings.GetGeometricTolerance(true));
	Session->AddDatabase(CADKernelRawData);

	return Session->GetModelAsShared();
}

bool UParametricSurfaceData::SetFromFile(const TCHAR* FilePath, bool bForTechSoft)
{
	if (FPaths::FileExists(FilePath))
	{
		TArray<uint8> ByteArray;

		if (FFileHelper::LoadFileToArray(ByteArray, FilePath))
		{
			TArray<uint8>& RawData = bForTechSoft ? TechSoftRawData : CADKernelRawData;
			RawData = MoveTemp(ByteArray);
			return true;
		}
	}

	return false;
}

#if PLATFORM_DESKTOP
bool UParametricSurfaceData::SetModel(TSharedPtr<UE::CADKernel::FModel>& Model, double UnitModelToCentimeter)
{
	ModelParameters.ModelUnitToCentimeter = UnitModelToCentimeter;
	return UE::CADKernel::FCADKernelUtilities::ToRawData(Model, CADKernelRawData);
}

A3DRiRepresentationItem* UParametricSurfaceData::GetRepresentation()
{
	using namespace UE::CADKernel;

	return TechSoftRawData.IsEmpty() ? nullptr : FTechSoftUtilities::GetRepresentation(TechSoftRawData);
}

bool UParametricSurfaceData::SetRepresentation(A3DRiRepresentationItem* Representation, int32 MaterialID, double UnitRepresentationToCentimeter)
{
#ifdef WITH_HOOPS
	ModelParameters.ModelUnitToCentimeter = UnitRepresentationToCentimeter;
	return UE::CADKernel::FTechSoftUtilities::ToRawData(Representation, MaterialID, TechSoftRawData);
#else
	return false;
#endif
}

bool UParametricSurfaceData::Tessellate(UE::Geometry::FDynamicMesh3& MeshOut)
{
	return UE::CADKernel::Editor::Private::Tessellate(*this, MeshOut);
}

bool UParametricSurfaceData::Tessellate(FMeshDescription& MeshOut)
{
	return UE::CADKernel::Editor::Private::Tessellate(*this, MeshOut);
}

bool UParametricSurfaceData::Retessellate(const FCADKernelRetessellationSettings& Settings, UE::Geometry::FDynamicMesh3& MeshOut)
{
	return UE::CADKernel::Editor::Private::Retessellate(*this, Settings, MeshOut);
}

bool UParametricSurfaceData::Retessellate(const FCADKernelRetessellationSettings& Settings, FMeshDescription& MeshOut)
{
	return UE::CADKernel::Editor::Private::Retessellate(*this, Settings, MeshOut);
}
#endif

#undef LOCTEXT_NAMESPACE // "ParametricSurfaceData"
