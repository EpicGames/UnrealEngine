// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetRegistry/AssetData.h"


class UStaticMesh;
class AStaticMeshActor;

class FCADKernelRetessellateAction
{
public:
	static CADKERNELEDITOR_API bool CanRetessellate(UStaticMesh* StaticMesh);

	static CADKERNELEDITOR_API bool RetessellateArray(const TArray<UStaticMesh*>& StaticMeshes);
	static CADKERNELEDITOR_API bool Retessellate(UStaticMesh* StaticMesh)
	{
		return RetessellateArray(TArray<UStaticMesh*>{ StaticMesh });
	}

	static CADKERNELEDITOR_API bool RetessellateArray(const TArray<FAssetData>& Assets);
	static CADKERNELEDITOR_API bool Retessellate(const FAssetData& Asset)
	{
		return RetessellateArray({ Asset });
	}

	static CADKERNELEDITOR_API bool RetessellateArray(const TArray<AStaticMeshActor*>& StaticMeshActors);
	static CADKERNELEDITOR_API bool Retessellate(AStaticMeshActor* StaticMeshActor)
	{
		return RetessellateArray({ StaticMeshActor });
	}

	static CADKERNELEDITOR_API bool RetessellateLegacy(UStaticMesh& StaticMesh, FArchive& Ar);
};
