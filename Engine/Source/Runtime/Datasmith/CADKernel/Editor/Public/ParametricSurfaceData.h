// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TechSoftIncludes.h"
#include "CADKernelEngineDefinitions.h"

#include "Engine/AssetUserData.h"

#include "ParametricSurfaceData.generated.h"

namespace UE::CADKernel
{
	class FModel;
}

namespace UE::Geometry
{
	class FDynamicMesh3;
}

class UStaticMesh;
struct FMeshDescription;

typedef void A3DRiRepresentationItem;
typedef void A3DAsmModelFile;

UCLASS(meta = (DisplayName = "CADKernel Parametric Surface Data"))
class CADKERNELEDITOR_API UParametricSurfaceData : public UAssetUserData
{
	GENERATED_BODY()

public:

	virtual bool IsValid() { return CADKernelRawData.Num() > 0 || TechSoftRawData.Num() > 0; }

	void SetModelParameters(const FCADKernelModelParameters& InModelParameters) { ModelParameters = InModelParameters; }
	const FCADKernelModelParameters& GetModelParameters() const { return ModelParameters; }
	FCADKernelModelParameters& GetModelParameters() { return ModelParameters; }

	void SetMeshParameters(const FCADKernelMeshParameters& InMeshParameters) { MeshParameters = InMeshParameters; }
	const FCADKernelMeshParameters& GetMeshParameters() const { return MeshParameters; }
	FCADKernelMeshParameters& GetMeshParameters() { return MeshParameters; }

	void SetLastTessellationSettings(const FCADKernelTessellationSettings& InTessellationSettings) { LastTessellationSettings = InTessellationSettings; }
	const FCADKernelTessellationSettings& GetLastTessellationSettings() const { return LastTessellationSettings; }
	
	TSharedPtr<UE::CADKernel::FModel> GetModel();

	virtual bool SetFromFile(const TCHAR* FilePath, bool bForTechSoft = false);

#if PLATFORM_DESKTOP
	bool SetModel(TSharedPtr<UE::CADKernel::FModel>& Model, double UnitModelToCentimeter = 1.);

	A3DRiRepresentationItem* GetRepresentation();
	bool SetRepresentation(A3DRiRepresentationItem* Representation, int32 MaterialID, double UnitRepresentationToCentimeter = 1.);

	bool Tessellate(UE::Geometry::FDynamicMesh3& MeshOut);
	bool Tessellate(FMeshDescription& MeshOut);

	bool Retessellate(const FCADKernelRetessellationSettings& Settings, UE::Geometry::FDynamicMesh3& MeshOut);
	bool Retessellate(const FCADKernelRetessellationSettings& Settings, FMeshDescription& MeshOut);
#endif

	/*
	 ** The SetRawData is only for internal use to facilitate the transition
	 ** out of UDatasmithParametricSurfaceData into the new UParametricSurfaceData class
	 */
	virtual void SetRawData(const TArray<uint8>& InRawData, bool bForTechSoft = false)
	{
		TArray<uint8>& RawData = bForTechSoft ? TechSoftRawData : CADKernelRawData;
		RawData = InRawData;
	}

protected:
	virtual void Serialize(FArchive& Ar) override;

protected:
	UPROPERTY()
	FCADKernelModelParameters ModelParameters;

	UPROPERTY()
	FCADKernelMeshParameters MeshParameters;

	UPROPERTY(EditAnywhere, Category = NURBS)
	FCADKernelTessellationSettings LastTessellationSettings;

	TArray<uint8> CADKernelRawData;
	TArray<uint8> TechSoftRawData;
};
