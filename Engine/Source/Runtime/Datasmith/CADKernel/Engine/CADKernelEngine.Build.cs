// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class CADKernelEngine : ModuleRules
	{
		public CADKernelEngine(ReadOnlyTargetRules Target)
			: base(Target)
		{
			bEnableExceptions = true;
			bUseUnity = false; // TechSoftLibrary.cpp cannot be compiled with the other cpp files.
			//OptimizeCode = CodeOptimization.Never;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CADKernel",
					"GeometryCore",
					"Json",
					"MeshConversion",
					"MeshDescription",
					"StaticMeshDescription",
				}
			);

			// TechSoft related API needs the TechSoft headers to be accessible
			bool bHasTechSoft = System.Type.GetType("TechSoft") != null;

			if ((Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux) && bHasTechSoft)
			{
				// #cad_import: Verifiy that this is correct
				PublicDependencyModuleNames.Add("TechSoft");
			}
		}
	}
}