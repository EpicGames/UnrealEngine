// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelEngine.h"

#if PLATFORM_DESKTOP
#include "CADKernelEnginePrivate.h"
#include "MeshUtilities.h"

#include "Core/Session.h"
#include "Topo/Body.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalFace.h"

#include "CADKernelEngineLog.h"

#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY(LogCADKernelEngine);

#ifdef WITH_HOOPS
bool FCADKernelTessellationSettings::bWithHoops = true;
#else
bool FCADKernelTessellationSettings::bWithHoops = false;
#endif

namespace UE::CADKernel
{
	const double FUnitConverter::CentimeterToMillimeter = 10.;
	const double FUnitConverter::MillimeterToCentimeter = 0.1;
	const double FUnitConverter::CentimeterToMeter = 0.01;
	const double FUnitConverter::MeterToCentimeter = 100.;
	const double FUnitConverter::MillimeterToMeter = 0.001;
	const double FUnitConverter::MeterToMillimeter = 1000.;
}

namespace UE::CADKernel
{
	using namespace UE::CADKernel::MeshUtilities;

	static bool GUseEngine = false;
	FAutoConsoleVariableRef GCADKernelDebugUseEngine(
		TEXT("CADKernel.Debug.UseEngine"),
		GUseEngine,
		TEXT(""),
		ECVF_Default);

	FTessellationContext::FTessellationContext(const FCADKernelModelParameters& InModelParams, const FCADKernelMeshParameters& InMeshParams, const FCADKernelRetessellationSettings& Settings)
	{
		ModelParams = InModelParams;
		MeshParams = InMeshParams;
		TessellationSettings = Settings;
		bResolveTJunctions = TessellationSettings.bResolveTJunctions;
	}

	bool FCADKernelUtilities::Save(TSharedPtr<UE::CADKernel::FModel>& Model, const FString& FilePath)
	{
		TSharedPtr<FSession> Session = FEntity::MakeShared<FSession>(0.01);
		Session->GetModel().Copy(*Model);

		return Session->SaveDatabase(*FilePath);
	}

	bool FCADKernelUtilities::Load(TSharedPtr<UE::CADKernel::FModel>& Model, const FString& FilePath)
	{
		TSharedPtr<FSession> Session = FEntity::MakeShared<FSession>(0.01);

		if (Session->LoadDatabase(*FilePath))
		{
			Model->Copy(Session->GetModel());
			return true;
		}

		return false;
	}

	bool FCADKernelUtilities::Tessellate(UE::CADKernel::FModel& Model, const FTessellationContext& Context, FMeshDescription& Mesh, bool bSkipDeletedFaces, bool bEmptyMesh)
	{
		using namespace UE::CADKernel::MeshUtilities;

		if (bSkipDeletedFaces)
		{
			GetExistingFaceGroups(Mesh, Context.FaceGroupsToExtract);
		}

		TSharedPtr<FMeshWrapperAbstract> MeshWrapper = FMeshWrapperAbstract::MakeWrapper(Context, Mesh);
		return Private::Tessellate(Model, Context, *MeshWrapper, bEmptyMesh);
	}

	bool FCADKernelUtilities::Tessellate(UE::CADKernel::FModel& Model, const FTessellationContext& Context, UE::Geometry::FDynamicMesh3& Mesh, bool bSkipDeletedFaces, bool bEmptyMesh)
	{
		using namespace UE::CADKernel::MeshUtilities;

		if (bSkipDeletedFaces)
		{
			GetExistingFaceGroups(Mesh, Context.FaceGroupsToExtract);
		}

		TSharedPtr<FMeshWrapperAbstract> MeshWrapper = FMeshWrapperAbstract::MakeWrapper(Context, Mesh);
		return Private::Tessellate(Model, Context, *MeshWrapper, bEmptyMesh);
	}

	bool FCADKernelUtilities::ToRawData(TSharedPtr<UE::CADKernel::FModel>& Model, TArray<uint8>& RawDataOut)
	{
		return false;
	}
}
#endif
