// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelEnginePrivate.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngine.h"
#include "MeshUtilities.h"

#include "Geo/Curves/CurveUtilities.h"
#include "Mesh/Criteria/Criterion.h"
#include "Mesh/Meshers/Mesher.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Topo/Model.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalFaceUtilities.h"
#include "Topo/TopologicalLoop.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "Algo/Reverse.h"
#include "MeshDescription.h"

namespace UE::CADKernel::Private
{
	using namespace UE::CADKernel;
	using namespace UE::CADKernel::MeshUtilities;
	using namespace UE::Geometry;

	void DefineMeshCriteria(FModelMesh& MeshModel, const FCADKernelTessellationSettings& TessellationSettings)
	{
		using namespace UE::CADKernel;

		constexpr bool bInMillimeter = true;

		{
			TSharedPtr<FCriterion> CurvatureCriterion = FCriterion::CreateCriterion(ECriterion::CADCurvature);
			MeshModel.AddCriterion(CurvatureCriterion);

			const double GeometricTolerance = TessellationSettings.GetGeometricTolerance(bInMillimeter);
			TSharedPtr<FCriterion> MinSizeCriterion = FCriterion::CreateCriterion(ECriterion::MinSize, 2. * GeometricTolerance);
			MeshModel.AddCriterion(MinSizeCriterion);
		}

		if (TessellationSettings.GetMaxEdgeLength(bInMillimeter) > SMALL_NUMBER)
		{
			TSharedPtr<FCriterion> MaxSizeCriterion = FCriterion::CreateCriterion(ECriterion::MaxSize, TessellationSettings.GetMaxEdgeLength(bInMillimeter));
			MeshModel.AddCriterion(MaxSizeCriterion);
		}

		if (TessellationSettings.GetChordTolerance(bInMillimeter) > SMALL_NUMBER)
		{
			TSharedPtr<FCriterion> ChordCriterion = FCriterion::CreateCriterion(ECriterion::Sag, TessellationSettings.GetChordTolerance(bInMillimeter));
			MeshModel.AddCriterion(ChordCriterion);
		}

		if (TessellationSettings.NormalTolerance > SMALL_NUMBER)
		{
			TSharedPtr<FCriterion> MaxNormalAngleCriterion = FCriterion::CreateCriterion(ECriterion::Angle, TessellationSettings.NormalTolerance);
			MeshModel.AddCriterion(MaxNormalAngleCriterion);
		}
	}

	bool Tessellate(FModel& Model, const FTessellationContext& Context, FMeshWrapperAbstract& MeshWrapper, bool bEmptyMesh)
	{
		// Tessellate the model
		TSharedRef<FModelMesh> ModelMesh = FEntity::MakeShared<FModelMesh>();

		const double GeometriTolerance = Context.TessellationSettings.GetGeometricTolerance(true);
		FMesher Mesher(*ModelMesh, GeometriTolerance, false/*bActivateThinZoneMeshing*/);

		DefineMeshCriteria(*ModelMesh, Context.TessellationSettings);
		// #cad_import: FMesher::MeshEntity should take a const FModel&
		Mesher.MeshEntity(Model);

		if (bEmptyMesh)
		{
			MeshWrapper.ClearMesh();
		}

		return AddModelMesh(*ModelMesh, MeshWrapper);
	}

	bool GetFaceTrimmingCurves(const FModel& Model, const FTopologicalFace& Face, TArray<TArray<TArray<FVector>>>& CurvesOut)
	{
		using namespace UE::CADKernel;

		const int32 CurvesCount = CurvesOut.Num();

		CurvesOut.Reserve(CurvesCount + Face.GetLoops().Num());

		for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
		{
			TArray<TArray<FVector>> Polylines;
			Polylines.Reserve(Loop->GetEdges().Num());

			for (const FOrientedEdge& Edge : Loop->GetEdges())
			{
				TArray<FVector> Polyline;

				Polyline = CurveUtilities::GetPoles(*Edge.Entity->GetCurve()->Get2DCurve());

				
				if (Polyline.Num() > 1)
				{
					if (Edge.Direction == EOrientation::Back)
					{
						if (Polyline.Num() == 2)
						{
							FVector Reserve = Polyline[0];
							Polyline[0] = MoveTemp(Polyline[1]);
							Polyline[1] = MoveTemp(Reserve);
						}
						else
						{
							Algo::Reverse(Polyline);
						}
					}

					Polylines.Emplace(MoveTemp(Polyline));
				}
			}

			if (Polylines.Num() > 1)
			{
				CurvesOut.Add(MoveTemp(Polylines));
			}
		}

		return CurvesOut.Num() > CurvesCount;
	}

	bool GetFaceTrimming2DPolylines(const FModel& Model, const FTopologicalFace& Face, TArray<TArray<FVector2d>>& PolylinesOut)
	{
		using namespace UE::CADKernel;

		const int32 PolylinesCount = PolylinesOut.Num();

		PolylinesOut.Reserve(PolylinesCount + Face.GetLoops().Num());

		for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
		{
			TArray<FVector2d> Polyline;

			Polyline = TopologicalFaceUtilities::Get2DPolyline(*Loop);

			if (Polyline.Num() > 1)
			{
				PolylinesOut.Add(MoveTemp(Polyline));
			}
		}

		return PolylinesOut.Num() > PolylinesCount;
	}

	bool GetFaceTrimming3DPolylines(const FModel& Model, const FTopologicalFace& Face, TArray<TArray<FVector>>& PolylinesOut)
	{
		using namespace UE::CADKernel;

		const int32 PolylinesCount = PolylinesOut.Num();

		PolylinesOut.Reserve(PolylinesCount + Face.GetLoops().Num());

		for (const TSharedPtr<FTopologicalLoop>& Loop : Face.GetLoops())
		{
			TArray<FVector> Polyline;

			Polyline = TopologicalFaceUtilities::Get3DPolyline(*Loop);

			if (Polyline.Num() > 1)
			{
				PolylinesOut.Add(MoveTemp(Polyline));
			}
		}

		return PolylinesOut.Num() > PolylinesCount;
	}
}
#endif