// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngine.h"
#include "MeshUtilities.h"

namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::CADKernel
{
	class FModel;

	struct FTessellationContext;

	namespace MeshUtilities
	{
		class FMeshWrapperAbstract;
	}

	namespace Private
	{
		bool Tessellate(FModel& Model, const UE::CADKernel::FTessellationContext& Context, UE::CADKernel::MeshUtilities::FMeshWrapperAbstract& MeshWrapper, bool bEmptyMesh = false);

		// For future use
		bool GetFaceTrimmingCurves(const UE::CADKernel::FModel& Model, const UE::CADKernel::FTopologicalFace& Face, TArray<TArray<TArray<FVector>>>& CurvesOut);
		bool GetFaceTrimming2DPolylines(const UE::CADKernel::FModel& Model, const UE::CADKernel::FTopologicalFace& Face, TArray<TArray<FVector2d>>& PolylinesOut);
		bool GetFaceTrimming3DPolylines(const UE::CADKernel::FModel& Model, const UE::CADKernel::FTopologicalFace& Face, TArray<TArray<FVector>>& PolylinesOut);

		bool AddModelMesh(const UE::CADKernel::FModelMesh& ModelMesh, UE::CADKernel::MeshUtilities::FMeshWrapperAbstract& MeshWrapper);
	}
}
#endif
