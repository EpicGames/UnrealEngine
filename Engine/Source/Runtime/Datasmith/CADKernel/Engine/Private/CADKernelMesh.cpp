// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelEngine.h"
#if PLATFORM_DESKTOP

#include "CADKernelEnginePrivate.h"
#include "MeshUtilities.h"

//#include "Geo/Curves/CurveUtilities.h"
//#include "Mesh/Criteria/Criterion.h"
//#include "Mesh/Meshers/Mesher.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Mesh/Structure/FaceMesh.h"
//#include "Topo/Model.h"
//#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalFaceUtilities.h"
//#include "Topo/TopologicalLoop.h"

namespace UE::CADKernel::Private
{
	using namespace UE::CADKernel;
	using namespace UE::CADKernel::MeshUtilities;

	class FModelMeshConverter
	{
	private:
		const FModelMesh& ModelMesh;
		FMeshWrapperAbstract& MeshWrapper;

	public:
		FModelMeshConverter(const FModelMesh& InModelMesh, FMeshWrapperAbstract& InMeshWrapper)
			: ModelMesh(InModelMesh)
			, MeshWrapper(InMeshWrapper)
		{}

		void AddTrianglesFromFaceMesh(const FFaceMesh& FaceMesh)
		{
			const FTopologicalFace& Face = static_cast<const FTopologicalFace&>(FaceMesh.GetGeometricEntity());
			const int32 GroupID = Face.GetPatchId();

			if (Face.IsDegenerated() || !MeshWrapper.IsFaceGroupValid(GroupID))
			{
				return;
			}

			const uint32 MaterialID = Face.GetColorId();

			const TArray<int32>& TriangleVertexIndices = FaceMesh.TrianglesVerticesIndex;

			TArray<FMeshWrapperAbstract::FFaceTriangle> FaceTriangles;
			FaceTriangles.Reserve(TriangleVertexIndices.Num() / 3);

			MeshWrapper.StartFaceTriangles(FaceTriangles.Num(), FaceMesh.Normals, FaceMesh.UVMap);

			const int32* GlobalVtxIDs = FaceMesh.VerticesGlobalIndex.GetData();
			const int32* TriVtxIDs = TriangleVertexIndices.GetData();

			for (int32 Index = 0; Index < TriangleVertexIndices.Num(); Index += 3, TriVtxIDs += 3)
			{
				FArray3i Normals(TriVtxIDs[0], TriVtxIDs[1], TriVtxIDs[2]);
				FArray3i TexCoords = Normals;
				FArray3i VertexIndices(GlobalVtxIDs[TriVtxIDs[0]], GlobalVtxIDs[TriVtxIDs[1]], GlobalVtxIDs[TriVtxIDs[2]]);

				FaceTriangles.Emplace(GroupID, MaterialID, VertexIndices, Normals, TexCoords);
			}

			MeshWrapper.AddFaceTriangles(FaceTriangles);

			MeshWrapper.EndFaceTriangles();
		}

		bool Convert()
		{
			{
				const TArray<TArray<FVector>*>& GlobalPointCloud = ModelMesh.GetGlobalPointCloud();

				int32 VertexCount = 0;
				for (const TArray<FVector>* PointArray : GlobalPointCloud)
				{
					VertexCount += PointArray->Num();
				}

				TArray<FVector> Vertices;
				Vertices.Reserve(VertexCount);

				for (const TArray<FVector>* PointArray : GlobalPointCloud)
				{
					Vertices.Append(*PointArray);
				}

				MeshWrapper.SetVertices(MoveTemp(Vertices));
			}

			{

				for (const FFaceMesh* FaceMesh : ModelMesh.GetFaceMeshes())
				{
					AddTrianglesFromFaceMesh(*FaceMesh);
				}
			}

			MeshWrapper.Complete();

			return true;
		}
	};

	bool AddModelMesh(const FModelMesh& ModelMesh, FMeshWrapperAbstract& MeshWrapper)
	{
		FModelMeshConverter Converter(ModelMesh, MeshWrapper);
		return Converter.Convert();
	}
}
#endif
