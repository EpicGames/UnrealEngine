// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshWrapper.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngine.h"

#include "Algo/Reverse.h"

namespace UE::CADKernel::MeshUtilities
{
	using namespace UE::CADKernel;
	using namespace UE::Geometry;

	TSharedPtr<FMeshWrapperAbstract> FMeshWrapperAbstract::MakeWrapper(const FMeshExtractionContext& Context, FDynamicMesh3& Mesh)
	{
		return MakeShared<FDynamicMeshWrapper>(Context, Mesh);
	}

	FDynamicMeshWrapper::FDynamicMeshWrapper(const FMeshExtractionContext& InContext, FDynamicMesh3& InMesh)
		: FMeshWrapperAbstract(InContext)
		, MeshOut(InMesh)
	{
	}

	void FDynamicMeshWrapper::ClearMesh()
	{
		// #cad_import: Clear mesh
		MeshOut.Clear();
	}

	bool FDynamicMeshWrapper::AddNewVertices(TArray<FVector>&& InVertexArray)
	{
		if (bAreVerticesSet)
		{
			return false;
		}

		TArray<FVector> VertexArray = MoveTemp(InVertexArray);

		VertexIdOffset = VertexMapping.Num();

		const int32 VertexCount = VertexArray.Num();
		VertexMapping.Reserve(VertexIdOffset + VertexCount);
		VertIDMap.SetNumUninitialized(VertexIdOffset + VertexCount);

		for (const FVector& Vertex : VertexArray)
		{
			const int32 NewVertIdx = MeshOut.AppendVertex(Vertex);
			VertIDMap[NewVertIdx] = NewVertIdx;
			VertexMapping.Add(NewVertIdx);
		}

		bNewVerticesAdded = true;

		return true;
	}

	bool FDynamicMeshWrapper::SetVertices(TArray<FVector>&& InVertices)
	{
		VertexIdOffset = VertexMapping.Num();
		ensure(VertexIdOffset == 0);

		TArray<FVector> Vertices = MoveTemp(InVertices);
		AddNewVertices(MoveTemp(Vertices));

		bAreVerticesSet = true;

		return true;
	}

	void FDynamicMeshWrapper::FinalizeMesh()
	{
		TDynamicVector<FVector3d>& Positions = const_cast<TDynamicVector<FVector3d>&>(MeshOut.GetVerticesBuffer());
		ConvertVectorArray(Context.ModelParams.ModelCoordSys, Positions);

		if (!FMath::IsNearlyEqual(Context.ModelParams.ModelUnitToCentimeter, 1.f))
		{
			for (FVector3d& Position : Positions)
			{
				Position *= Context.ModelParams.ModelUnitToCentimeter;
			}
		}

		UE::CADKernel::MathUtils::ConvertVectorArray(Context.ModelParams.ModelCoordSys, Normals);

		AddTriangles();
	}

	bool FDynamicMeshWrapper::ReserveNewTriangles(int32 TriangleCount)
	{
		Normals.Reserve(Normals.Num() + TriangleCount * 3);
		TexCoords.Reserve(TexCoords.Num() + TriangleCount * 3);

		return true;
	}

	bool  FDynamicMeshWrapper::AddTriangle(int32 GroupID, uint32 MaterialID, const FArray3i& VertexIndices, const TArrayView<FVector3f>& InNormals, const TArrayView<FVector2f>& InTexCoords)
	{
		ensure(InNormals.Num() == 3 && InTexCoords.Num() == 3);

		const bool bNeedSwapOrientation = Context.MeshParams.bNeedSwapOrientation;
		const FArray3i& Orientation = bNeedSwapOrientation ? CounterClockwise : Clockwise;

		LastNormalIndex = Normals.Num();
		Normals.Append(InNormals);
		TexCoords.Append(InTexCoords);

		GroupIDSet.Add(GroupID);
		if (!MaterialMapping.Contains(MaterialID))
		{
			MaterialMapping.Add(MaterialID, ++MaterialIDCount);
		}

		FIndex3i VertexIDs{
			VertexMapping[VertexIdOffset + VertexIndices[Orientation[0]]],
			VertexMapping[VertexIdOffset + VertexIndices[Orientation[1]]],
			VertexMapping[VertexIdOffset + VertexIndices[Orientation[2]]]
		};


		int NewTriangleID = AppendTriangle(VertexIDs, GroupID);

		//-- already seen this triangle for some reason.. or the MeshDecription had a degenerate edge
		if (NewTriangleID == FDynamicMesh3::InvalidID)
		{
			return false;;
		}

		FArray3i NormalIndices{
			LastNormalIndex + LastNormalIndex + Orientation[0],
			LastNormalIndex + LastNormalIndex + Orientation[1],
			LastNormalIndex + LastNormalIndex + Orientation[2]
		};

		FArray3i TexCoordIndices{
			LastNormalIndex + LastNormalIndex + Orientation[0],
			LastNormalIndex + LastNormalIndex + Orientation[1],
			LastNormalIndex + LastNormalIndex + Orientation[2]
		};

		TriangleDataSet.Emplace(GroupID, MaterialID, NormalIndices, TexCoordIndices);

		return true;
	}

	void FDynamicMeshWrapper::AddSymmetry()
	{
		const int32 VertexCount = MeshOut.VertexCount();
		const int32 TriangleCount = MeshOut.TriangleCount();

		FMatrix44f SymmetricMatrix = (FMatrix44f)GetSymmetricMatrix(Context.MeshParams.SymmetricOrigin, Context.MeshParams.SymmetricNormal);
		TArray<int32> SymmetricVertexIds;

		SymmetricVertexIds.SetNum(VertexCount);

		for (int32 Index = 0; Index < VertexCount; ++Index)
		{
			FVector4f SymmetricPosition = FVector4f(SymmetricMatrix.TransformPosition((FVector3f)MeshOut.GetVertex(Index)));
			int32 NewVertIdx = MeshOut.AppendVertex(FVector3d(SymmetricPosition));
			SymmetricVertexIds[Index] = NewVertIdx;
		}

		int32 NewTriID;
		for (int32 TriID = 0; TriID < TriangleCount; ++TriID)
		{
			const int32 GroupID = MeshOut.GetTriangleGroup(TriID);
			{
				const FIndex3i& VertexIDs = MeshOut.GetTriangle(TriID);
				FIndex3i NewVertexIDs{
					SymmetricVertexIds[VertexIDs[2]],
					SymmetricVertexIds[VertexIDs[1]],
					SymmetricVertexIds[VertexIDs[0]],
				};
				NewTriID = MeshOut.AppendTriangle(NewVertexIDs, GroupID);
				ensure(NewTriID != FDynamicMesh3::DuplicateTriangleID && NewTriID != FDynamicMesh3::InvalidID && NewTriID != FDynamicMesh3::NonManifoldID);
			}

			{
				const FIndex3i& VertexIDs = UVOverlay->GetTriangle(TriID);
				FIndex3i NewVertexIDs;

				NewVertexIDs[0] = UVOverlay->AppendElement(UVOverlay->GetElement(VertexIDs[2]));
				NewVertexIDs[1] = UVOverlay->AppendElement(UVOverlay->GetElement(VertexIDs[1]));
				NewVertexIDs[2] = UVOverlay->AppendElement(UVOverlay->GetElement(VertexIDs[0]));

				// set the triangle in the overlay
				UVOverlay->SetTriangle(NewTriID, NewVertexIDs);
			}

			{
				const FIndex3i& VertexIDs = NormalOverlay->GetTriangle(TriID);
				FVector3f TrianglesNormals[3]{
					SymmetricMatrix.TransformVector(NormalOverlay->GetElement(VertexIDs[2])),
					SymmetricMatrix.TransformVector(NormalOverlay->GetElement(VertexIDs[1])),
					SymmetricMatrix.TransformVector(NormalOverlay->GetElement(VertexIDs[0])),
				};

				FIndex3i NewVertexIDs;
				NewVertexIDs[0] = NormalOverlay->AppendElement(&TrianglesNormals[0].X);
				NewVertexIDs[1] = NormalOverlay->AppendElement(&TrianglesNormals[1].X);
				NewVertexIDs[2] = NormalOverlay->AppendElement(&TrianglesNormals[2].X);

				NormalOverlay->SetTriangle(NewTriID, NewVertexIDs);
			}

			TangentOverlay->SetTriangle(NewTriID, { TangentOverlayID , TangentOverlayID, TangentOverlayID });
			BiTangentOverlay->SetTriangle(NewTriID, { BiTangentOverlayID , BiTangentOverlayID, BiTangentOverlayID });
			ColorOverlay->SetTriangle(NewTriID, { ColorOverlayID , ColorOverlayID, ColorOverlayID });

			// Add the triangle as a polygon to the mesh description
			int32 MaterialIndex = MaterialIDAttrib->GetValue(TriID);
			MaterialIDAttrib->SetValue(NewTriID, &MaterialIndex);

			// Set patch id attribute
			MeshOut.SetTriangleGroup(NewTriID, GroupID);
		}
	}

	void FDynamicMeshWrapper::InitializeAttributes()
	{
		int32 TriangleCount = MeshOut.TriangleCount();

		// Enable attributes to extract normals, tangents, etc...
		MeshOut.EnableAttributes();
		FDynamicMeshAttributeSet* Attributes = MeshOut.Attributes();
		ensure(Attributes);


		// Normals
		constexpr int32 NormalLayerCount = 3;
		Attributes->SetNumNormalLayers(NormalLayerCount);
		for (int32 Index = 0; Index < NormalLayerCount; ++Index)
		{
			Attributes->GetNormalLayer(Index)->InitializeTriangles(TriangleCount);
		}

		NormalOverlay = Attributes->PrimaryNormals();
		TangentOverlay = Attributes->PrimaryTangents();
		BiTangentOverlay = Attributes->PrimaryBiTangents();

		static FVector3f ZeroVector(ForceInitToZero);
		TangentOverlayID = TangentOverlay->AppendElement(&ZeroVector.X);
		BiTangentOverlayID = BiTangentOverlay->AppendElement(&ZeroVector.X);

		// Only one UV channel 
		Attributes->SetNumUVLayers(1);
		Attributes->GetUVLayer(0)->InitializeTriangles(TriangleCount);
		UVOverlay = MeshOut.Attributes()->GetUVLayer(0);

		Attributes->EnablePrimaryColors();
		ColorOverlay = Attributes->PrimaryColors();
		ColorOverlayID = ColorOverlay->AppendElement(&ZeroVector.X);

		// always enable Material ID if there are any attributes
		Attributes->EnableMaterialID();
		MaterialIDAttrib = Attributes->GetMaterialID();

		//
		MeshOut.EnableTriangleGroups();
		Attributes->SetNumPolygroupLayers(GroupIDSet.Num());
		LayerMapping.Reserve(GroupIDSet.Num());

		constexpr int32 InvalidGroupID = INDEX_NONE;
		int32 Index = -1;
		for (int32 GroupID : GroupIDSet)
		{
			FDynamicMeshPolygroupAttribute* Layer = Attributes->GetPolygroupLayer(++Index);
			Layer->SetName(*FString::Printf(TEXT("Face #%d"), GroupID));

			LayerMapping.Add(GroupID, Layer);
		}
	}

	void FDynamicMeshWrapper::AddTriangles()
	{
		if (TriangleDataSet.IsEmpty())
		{
			return;
		}

		InitializeAttributes();

		FIndex3i TriVertexIDs;

		auto TriangleDataIter = TriangleDataSet.CreateIterator();
		for (int TriID : MeshOut.TriangleIndicesItr())
		{
			FIndex3i Triangle = MeshOut.GetTriangle(TriID);

			{
				TriVertexIDs[0] = UVOverlay->AppendElement(FVector2f(TexCoords[TriangleDataIter->TexCoordIndices[0]] * static_cast<float>(ScaleUV)));
				TriVertexIDs[1] = UVOverlay->AppendElement(FVector2f(TexCoords[TriangleDataIter->TexCoordIndices[1]] * static_cast<float>(ScaleUV)));
				TriVertexIDs[2] = UVOverlay->AppendElement(FVector2f(TexCoords[TriangleDataIter->TexCoordIndices[2]] * static_cast<float>(ScaleUV)));

				// set the triangle in the overlay
				UVOverlay->SetTriangle(TriID, TriVertexIDs);

				
			}

			// #cad_import: FNormalWelder????
			{
				FVector3f TrianglesNormals[3]{
					Normals[TriangleDataIter->NormalIndices[0]],
					Normals[TriangleDataIter->NormalIndices[0]],
					Normals[TriangleDataIter->NormalIndices[0]],
				};

				TriVertexIDs[0] = NormalOverlay->AppendElement(&TrianglesNormals[0].X);
				TriVertexIDs[1] = NormalOverlay->AppendElement(&TrianglesNormals[1].X);
				TriVertexIDs[2] = NormalOverlay->AppendElement(&TrianglesNormals[2].X);

				NormalOverlay->SetTriangle(TriID, TriVertexIDs);
			}

			TangentOverlay->SetTriangle(TriID, { TangentOverlayID , TangentOverlayID, TangentOverlayID });
			BiTangentOverlay->SetTriangle(TriID, { BiTangentOverlayID , BiTangentOverlayID, BiTangentOverlayID });
			ColorOverlay->SetTriangle(TriID, { ColorOverlayID , ColorOverlayID, ColorOverlayID });

			// Add the triangle as a polygon to the mesh description
			MaterialIDAttrib->SetValue(TriID, MaterialMapping.Find(TriangleDataIter->MaterialID));

			// Set patch id attribute
			MeshOut.SetTriangleGroup(TriID, TriangleDataIter->GroupID);

			++TriangleDataIter;
		}

		TriangleDataSet.Empty(0);
	}

	bool FDynamicMeshWrapper::StartFaceTriangles(int32 TriangleCount, const TArray<FVector3f>& InNormals, const TArray<FVector2f>& InTexCoords)
	{
		ensure(InNormals.Num() == InTexCoords.Num() && InNormals.Num()%3 == 0);

		LastNormalIndex = Normals.Num();
		Normals.Append(InNormals);
		TexCoords.Append(InTexCoords);

		return true;
	}

	bool FDynamicMeshWrapper::StartFaceTriangles(const TArrayView<FVector>& InNormals, const TArrayView<FVector2d>& InTexCoords)
	{
		LastNormalIndex = Normals.Num();

		const int32 ArraySize = InNormals.Num();
		ensure(ArraySize == InTexCoords.Num());

		Normals.Reserve(LastNormalIndex + ArraySize);
		TexCoords.Reserve(LastNormalIndex + ArraySize);

		for (int32 Index = 0; Index < ArraySize; ++Index)
		{
			Normals.Emplace(InNormals[Index].X, InNormals[Index].Y, InNormals[Index].Z);
			TexCoords.Emplace(InTexCoords[Index].X, InTexCoords[Index].Y);
		}

		return true;
	}

	int32 FDynamicMeshWrapper::AppendTriangle(FIndex3i& VertexIDs, int32 GroupID)
	{
		int32 NewTriangleID = MeshOut.AppendTriangle(VertexIDs, GroupID);

		//-- already seen this triangle for some reason.. or the MeshDecription had a degenerate tri
		if (NewTriangleID == FDynamicMesh3::DuplicateTriangleID || NewTriangleID == FDynamicMesh3::InvalidID)
		{
			return FDynamicMesh3::InvalidID;
		}

		//-- non manifold 
		// if append failed due to non-manifold, duplicate verts
		if (NewTriangleID == FDynamicMesh3::NonManifoldID)
		{
			int e0 = MeshOut.FindEdge(VertexIDs[0], VertexIDs[1]);
			int e1 = MeshOut.FindEdge(VertexIDs[1], VertexIDs[2]);
			int e2 = MeshOut.FindEdge(VertexIDs[2], VertexIDs[0]);

			// determine which verts need to be duplicated
			bool bDuplicate[3] = { false, false, false };
			if (e0 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e0) == false)
			{
				bDuplicate[0] = true;
				bDuplicate[1] = true;
			}
			if (e1 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e1) == false)
			{
				bDuplicate[1] = true;
				bDuplicate[2] = true;
			}
			if (e2 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e2) == false)
			{
				bDuplicate[2] = true;
				bDuplicate[0] = true;
			}
			for (int32 Index = 0; Index < 3; ++Index)
			{
				if (bDuplicate[Index])
				{
					const FVector Position = MeshOut.GetVertex(VertexIDs[Index]);
					const int32 NewVertIdx = MeshOut.AppendVertex(Position);
					VertIDMap.SetNumUninitialized(NewVertIdx + 1);
					VertIDMap[NewVertIdx] = VertexIDs[Index];
					VertexMapping[VertexIDs[Index]] = NewVertIdx;
					VertexIDs[Index] = NewVertIdx;
				}
			}

			NewTriangleID = MeshOut.AppendTriangle(VertexIDs, GroupID);
		}

		return NewTriangleID;
	}

	bool FDynamicMeshWrapper::AddFaceTriangles(const TArray<FFaceTriangle>& FaceTriangles)
	{
		const bool bNeedSwapOrientation = Context.MeshParams.bNeedSwapOrientation;
		const FArray3i& Orientation = bNeedSwapOrientation ? CounterClockwise : Clockwise;

		bool bSrcIsManifold = true;
		TriangleDataSet.Reserve(TriangleDataSet.Num() + FaceTriangles.Num());

		for (const FFaceTriangle& FaceTriangle : FaceTriangles)
		{
			GroupIDSet.Add(FaceTriangle.GroupID);
			if (!MaterialMapping.Contains(FaceTriangle.MaterialID))
			{
				MaterialMapping.Add(FaceTriangle.MaterialID, ++MaterialIDCount);
			}

			FIndex3i VertexIDs{
				VertexMapping[VertexIdOffset + FaceTriangle.VertexIndices[Orientation[0]]],
				VertexMapping[VertexIdOffset + FaceTriangle.VertexIndices[Orientation[1]]],
				VertexMapping[VertexIdOffset + FaceTriangle.VertexIndices[Orientation[2]]]
			};

			int NewTriangleID = AppendTriangle(VertexIDs, FaceTriangle.GroupID);

			//-- already seen this triangle for some reason.. or the MeshDecription had a degenerate tri
			if (NewTriangleID == FDynamicMesh3::InvalidID)
			{
				continue;
			}

			FArray3i NormalIndices{
				LastNormalIndex + FaceTriangle.Normals[Orientation[0]],
				LastNormalIndex + FaceTriangle.Normals[Orientation[1]],
				LastNormalIndex + FaceTriangle.Normals[Orientation[2]]
			};

			FArray3i TexCoordIndices{
				LastNormalIndex + FaceTriangle.TexCoords[Orientation[0]],
				LastNormalIndex + FaceTriangle.TexCoords[Orientation[1]],
				LastNormalIndex + FaceTriangle.TexCoords[Orientation[2]]
			};

			TriangleDataSet.Emplace(FaceTriangle.GroupID, FaceTriangle.MaterialID, NormalIndices, TexCoordIndices);
		}

		return true;
	}

	void FDynamicMeshWrapper::EndFaceTriangles()
	{
	}

	void FDynamicMeshWrapper::RecomputeNullNormal()
	{
		// #cad_import: TO DO
	}

	void FDynamicMeshWrapper::OrientMesh()
	{
		// #cad_import: TO DO
	}

	void FDynamicMeshWrapper::ResolveTJunctions()
	{
		// #cad_import: TO DO
	}


	void GetExistingFaceGroups(UE::Geometry::FDynamicMesh3& Mesh, TSet<int32>& FaceGroupsOut)
	{
		// #cad_import: TO DO
	}
}
#endif
