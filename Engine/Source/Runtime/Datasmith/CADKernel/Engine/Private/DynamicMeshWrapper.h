// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#if PLATFORM_DESKTOP

#include "MeshUtilities.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE::CADKernel::MeshUtilities
{
	class FDynamicMeshWrapper : public FMeshWrapperAbstract
	{
	public:
		FDynamicMeshWrapper(const FMeshExtractionContext& InContext, UE::Geometry::FDynamicMesh3& InMesh);
		
		virtual void ClearMesh() override;
		virtual bool ReserveNewTriangles(int32 AdditionalTriangleCount) override;
		virtual bool SetVertices(TArray<FVector>&& Vertices) override;
		virtual bool AddNewVertices(TArray<FVector>&& Vertices) override;
		virtual bool AddTriangle(int32 GroupID, uint32 MaterialID, const FArray3i& VertexIndices, const TArrayView<FVector3f>& Normals, const TArrayView<FVector2f>& TexCoords) override;
		virtual bool StartFaceTriangles(int32 TriangleCount, const TArray<FVector3f>& Normals, const TArray<FVector2f>& TexCoords) override;
		virtual bool StartFaceTriangles(const TArrayView<FVector>& Normals, const TArrayView<FVector2d>& TexCoords) override;
		virtual bool AddFaceTriangles(const TArray<FFaceTriangle>& FaceTriangles) override;
		virtual bool AddFaceTriangle(const FFaceTriangle& FaceTriangle) override
		{
			return AddFaceTriangles({ FaceTriangle });
		}
		virtual void EndFaceTriangles() override;

		virtual void FinalizeMesh() override;
		virtual void AddSymmetry() override;
		virtual void RecomputeNullNormal() override;
		virtual void OrientMesh() override;
		virtual void ResolveTJunctions() override;

	private:
		void InitializeAttributes();
		void AddTriangles();
		int32 AppendTriangle(UE::Geometry::FIndex3i& VertexIDs, int32 GroupID);

	private:
		UE::Geometry::FDynamicMesh3& MeshOut;

		TMap<uint32, int32> MaterialMapping;
		/**
		* map from DynamicMesh vertex Id to MeshDecription FVertexID.
		* NB: due to vertex splitting, multiple DynamicMesh vertex ids
		* may map to the same MeshDescription FVertexID.
		*  ( a vertex split is a result of reconciling non-manifold MeshDescription vertex )
		*/
		TArray<int32> VertIDMap;
		TMap<int32, UE::Geometry::FDynamicMeshPolygroupAttribute*> LayerMapping;

		UE::Geometry::FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		UE::Geometry::FDynamicMeshMaterialAttribute* MaterialIDAttrib = nullptr;
		UE::Geometry::FDynamicMeshUVOverlay* UVOverlay = nullptr;
		UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		UE::Geometry::FDynamicMeshNormalOverlay* TangentOverlay = nullptr;
		UE::Geometry::FDynamicMeshNormalOverlay* BiTangentOverlay = nullptr;
		int32 TangentOverlayID = INDEX_NONE;
		int32 BiTangentOverlayID = INDEX_NONE;
		int32 ColorOverlayID = INDEX_NONE;

		bool bNewVerticesAdded = false;
		int32 VertexIdOffset = 0;
		TArray<int32> VertexMapping;

		struct FTriangleData
		{
			int32 GroupID;
			uint32 MaterialID;
			FArray3i NormalIndices;
			FArray3i TexCoordIndices;
			FTriangleData(int32 InGroupID, uint32 InMaterialID, const FArray3i& InNormalIndices, const FArray3i& InTexCoordIndices)
				: GroupID(InGroupID)
				, MaterialID(InMaterialID)
				, NormalIndices(InNormalIndices)
				, TexCoordIndices(InTexCoordIndices)
			{}
		};
		TArray<FTriangleData> TriangleDataSet;
		TSet<int32> GroupIDSet;
		TArray<FVector3f> Normals;
		TArray<FVector2f> TexCoords;
		int32 LastNormalIndex = 0;
		int32 MaterialIDCount = 0;

		bool bIsFinalized = false;
	};
}
#endif
