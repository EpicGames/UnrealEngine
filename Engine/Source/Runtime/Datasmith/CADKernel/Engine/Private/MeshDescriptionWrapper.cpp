// Copyright Epic Games, Inc. All Rights Reserved. 

#include "MeshDescriptionWrapper.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngine.h"
#include "CADKernelEngineLog.h"
#include "MeshUtilities.h"
#include "MeshTopologyHelper.h"

#include "Containers/Queue.h"
#include "MeshDescriptionBuilder.h"
#include "StaticMeshOperations.h"

namespace UE::CADKernel::MeshUtilities
{
	TSharedPtr<FMeshWrapperAbstract> FMeshWrapperAbstract::MakeWrapper(const FMeshExtractionContext& Context, FMeshDescription& Mesh)
	{
		return MakeShared<FMeshDescriptionWrapper>(Context, Mesh);
	}

	FMeshDescriptionWrapper::FMeshDescriptionWrapper(const FMeshExtractionContext& InContext, FMeshDescription& InMesh)
		: FMeshWrapperAbstract(InContext)
		, Attributes(InMesh)
		, Mesh(InMesh)
	{
		Attributes.Register();
		if (Attributes.IsValid())
		{
			VertexPositions = Mesh.GetVertexPositions();
			VertexInstanceToVertex = Attributes.GetVertexInstanceVertexIndices();
			VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
			VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
			VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
			VertexInstanceColors = Attributes.GetVertexInstanceColors();
			VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
			PolygonAttributes = Attributes.GetPolygonGroups();
			PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
		}
	}

	void FMeshDescriptionWrapper::ClearMesh()
	{
		// # cad_import: Maybe cache the material sections???
		Mesh.Empty();
	}

	void FMeshDescriptionWrapper::FinalizeMesh()
	{
		if (bIsFinalized)
		{
			return;
		}

		TArrayView<FVector3f> Positions = VertexPositions.GetRawArray();
		UE::CADKernel::MathUtils::ConvertVectorArray(Context.ModelParams.ModelCoordSys, Positions);

		const float ScaleFactor = Context.ModelParams.ModelUnitToCentimeter * Context.MeshParams.ScaleFactor;
		if (!FMath::IsNearlyEqual(ScaleFactor, 1.f))
		{
			for (FVector3f& Position : Positions)
			{
				Position *= ScaleFactor;
			}
		}

		if (Context.MeshParams.bNeedSwapOrientation)
		{
			for (const FVertexInstanceID VertexInstanceID : Mesh.VertexInstances().GetElementIDs())
			{
				VertexInstanceNormals[VertexInstanceID] *= -1.f;
			}
		}

		for (const FVertexInstanceID VertexInstanceID : Mesh.VertexInstances().GetElementIDs())
		{
			VertexInstanceColors[VertexInstanceID] = FLinearColor::White;
			VertexInstanceTangents[VertexInstanceID] = FVector3f(ForceInitToZero);
			VertexInstanceBinormalSigns[VertexInstanceID] = 0.0f;
		}

		FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(Mesh);

		bIsFinalized = true;
	}

	void FMeshDescriptionWrapper::AddSymmetry()
	{
		FMatrix44f SymmetricMatrix = (FMatrix44f)GetSymmetricMatrix(Context.MeshParams.SymmetricOrigin, Context.MeshParams.SymmetricNormal);

		TMap<FVertexID, FVertexID> VertexMapping;
		VertexMapping.Reserve(VertexPositions.GetNumElements());

		for (const FVertexID VertexID : Mesh.Vertices().GetElementIDs())
		{
			const FVector4f SymmetricPosition = FVector4f(SymmetricMatrix.TransformPosition(VertexPositions[VertexID]));

			const FVertexID NewVertexID = Mesh.CreateVertex();
			VertexPositions[NewVertexID] = FVector3f(SymmetricPosition);

			VertexMapping.Add(VertexID, NewVertexID);
		}

		FVertexID3 NewVertices;
		FVertexInstanceID3 NewVertexInstanceIDs;
		TArray<FVertexInstanceID, TInlineAllocator<3>> VertexInstancesIDs;

		for (const FPolygonID PolygonID : Mesh.Polygons().GetElementIDs())
		{
			Mesh.GetPolygonVertexInstances(PolygonID, VertexInstancesIDs);

			NewVertices[0] = VertexMapping[Mesh.GetVertexInstanceVertex(VertexInstancesIDs[2])];
			NewVertices[1] = VertexMapping[Mesh.GetVertexInstanceVertex(VertexInstancesIDs[1])];
			NewVertices[2] = VertexMapping[Mesh.GetVertexInstanceVertex(VertexInstancesIDs[0])];

			NewVertexInstanceIDs[0] = Mesh.CreateVertexInstance(NewVertices[0]);
			NewVertexInstanceIDs[1] = Mesh.CreateVertexInstance(NewVertices[1]);
			NewVertexInstanceIDs[2] = Mesh.CreateVertexInstance(NewVertices[2]);

			VertexInstanceUVs.Set(NewVertexInstanceIDs[0], 0/*UVChannel*/, VertexInstanceUVs.Get(VertexInstancesIDs[2]));
			VertexInstanceUVs.Set(NewVertexInstanceIDs[1], 0/*UVChannel*/, VertexInstanceUVs.Get(VertexInstancesIDs[1]));
			VertexInstanceUVs.Set(NewVertexInstanceIDs[2], 0/*UVChannel*/, VertexInstanceUVs.Get(VertexInstancesIDs[0]));

			VertexInstanceNormals[NewVertexInstanceIDs[0]] = SymmetricMatrix.TransformVector(VertexInstanceNormals[VertexInstancesIDs[2]]);
			VertexInstanceNormals[NewVertexInstanceIDs[1]] = SymmetricMatrix.TransformVector(VertexInstanceNormals[VertexInstancesIDs[1]]);
			VertexInstanceNormals[NewVertexInstanceIDs[2]] = SymmetricMatrix.TransformVector(VertexInstanceNormals[VertexInstancesIDs[0]]);

			// Add the triangle as a polygon to the mesh description
			const FPolygonID NewPolygonID = Mesh.CreatePolygon(Mesh.GetPolygonPolygonGroup(PolygonID), NewVertexInstanceIDs.ABC);

			// Set patch id attribute
			PolygonAttributes[NewPolygonID] = PolygonAttributes[PolygonID];
		}
	}

	// Inspired from FDynamicMesh3::AppendTriangle and its usage
	bool FMeshDescriptionWrapper::GetVertexInstances(const FArray3i& Vertices, FVertexInstanceID3& VertexInstances)
	{
		FVertexID3 Triangle(VertexIDs[Vertices[0]], VertexIDs[Vertices[1]], VertexIDs[Vertices[2]]);

		if (Triangle.A == Triangle.B || Triangle.A == Triangle.C || Triangle.C == Triangle.B)
		{
			// Degenerated triangle
			return false;
		}

		FEdgeID Edges[3];
		bool IsBoundary[3]{ true, true, true };

		Edges[0] = FindEdge(Triangle.A, Triangle.B, IsBoundary[0]);
		Edges[1] = FindEdge(Triangle.B, Triangle.C, IsBoundary[1]);
		Edges[2] = FindEdge(Triangle.C, Triangle.A, IsBoundary[2]);

		// At least one edge is non-manifold
		if (IsBoundary[0] == false || IsBoundary[1] == false || IsBoundary[2] == false)
		{
			UE_LOGF(LogCADKernelEngine, Verbose, "FMeshDescriptionWrapper::GetVertexInstances: Detected a non-manifold edge");

			bool bDuplicate[3]{ false, false, false };
			if (IsBoundary[0] == false)
			{
				bDuplicate[0] = true;
				bDuplicate[1] = true;
			}
			if (IsBoundary[1] == false)
			{
				bDuplicate[1] = true;
				bDuplicate[2] = true;
			}
			if (IsBoundary[2] == false)
			{
				bDuplicate[2] = true;
				bDuplicate[0] = true;
			}

			FVertexInstanceID3 NewTrianle(Triangle.A, Triangle.B, Triangle.C);
			for (int32 Index = 0; Index < 3; ++Index)
			{
				if (bDuplicate[Index])
				{
					const FVertexID VertexID = Mesh.CreateVertex();
					VertexPositions[VertexID] = VertexPositions[Triangle[Index]];
					NewTrianle[Index] = VertexID;

					// Make sure new triangle using faulty vertex index uses the newly created one.
					VertexIDs[Vertices[Index]] = VertexID;
				}
			}

			VertexInstances.A = Mesh.CreateVertexInstance(NewTrianle.A);
			VertexInstances.B = Mesh.CreateVertexInstance(NewTrianle.B);
			VertexInstances.C = Mesh.CreateVertexInstance(NewTrianle.C);
		}
		else
		{
			VertexInstances.A = Mesh.CreateVertexInstance(Triangle.A);
			VertexInstances.B = Mesh.CreateVertexInstance(Triangle.B);
			VertexInstances.C = Mesh.CreateVertexInstance(Triangle.C);
		}

		return true;
	}

	bool FMeshDescriptionWrapper::SetVertices(TArray<FVector>&& InVertices)
	{
		VertexIndexOffset = VertexPositions.GetNumElements();
		ensure(bAreVerticesSet == false && VertexIndexOffset == 0);

		TArray<FVector> Vertices = MoveTemp(InVertices);;
		AddNewVertices(MoveTemp(Vertices));

		bAreVerticesSet = true;

		return true;
	}

	bool FMeshDescriptionWrapper::AddNewVertices(TArray<FVector>&& InVertices)
	{
		using namespace UE::CADKernel;

		if (bAreVerticesSet)
		{
			return false;
		}

		VertexIndexOffset = VertexPositions.GetNumElements();

		const int32 VertexCount = InVertices.Num();

		Mesh.ReserveNewVertices(Context.MeshParams.bIsSymmetric ? VertexCount * 2 : VertexCount);
		VertexIDs.Empty(VertexCount);

		for (const FVector& Vertex : InVertices)
		{
			const FVertexID VertexID = Mesh.CreateVertex();
			VertexPositions[VertexID] = FVector3f(Vertex);
			VertexIDs.Add(VertexID);
		}

		return true;
	}

	bool FMeshDescriptionWrapper::ReserveNewTriangles(int32 TriangleCount)
	{
		if (Context.MeshParams.bIsSymmetric)
		{
			TriangleCount *= 2;
		}

		Mesh.ReserveNewPolygons(TriangleCount);
		Mesh.ReserveNewVertexInstances(TriangleCount * 3);
		Mesh.ReserveNewUVs(TriangleCount * 3);
		Mesh.ReserveNewEdges(TriangleCount * 3);

		if (MaterialToPolygonGroupMapping.Num() >= NANITE_MAX_CLUSTER_MATERIALS)
		{
			UE_LOGF(LogCADKernelEngine, Warning, "The main UE5 rendering systems do not support more than %d materials per mesh. Only the first %d materials are kept. The others are replaced by the last one", NANITE_MAX_CLUSTER_MATERIALS, NANITE_MAX_CLUSTER_MATERIALS);
		}

		// Add to the mesh, a polygon groups per material
		int32 PolyGroupIndex = 0;
		FPolygonGroupID PolyGroupID = 0;
		for (TPair<uint32, FPolygonGroupID>& Material : MaterialToPolygonGroupMapping)
		{
			if (PolyGroupIndex < NANITE_MAX_CLUSTER_MATERIALS)
			{
				uint32 MaterialHash = Material.Key;
				FName ImportedSlotName = *LexToString(MaterialHash);
				PolyGroupID = Mesh.CreatePolygonGroup();
				PolygonGroupImportedMaterialSlotNames[PolyGroupID] = ImportedSlotName;
				PolyGroupIndex++;
			}

			Material.Value = PolyGroupID;
		}

		return true;
	}

	bool FMeshDescriptionWrapper::AddTriangle(int32 GroupID, uint32 MaterialID, const FArray3i& VertexIndices, const TArrayView<FVector3f>& InNormals, const TArrayView<FVector2f>& InTexCoords)
	{
		ensure(InNormals.Num() == 3 && InTexCoords.Num() == 3);

		const bool bNeedSwapOrientation = Context.MeshParams.bNeedSwapOrientation;
		const FArray3i& Orientation = bNeedSwapOrientation ? CounterClockwise : Clockwise;

		FPolygonGroupID PolygonGroupID = GetPolygonGroupID(MaterialID);
		FVertexInstanceID3 VertexInstanceIDs;
		FArray3i Vertices{
			VertexIndexOffset + VertexIndices[Orientation[0]],
			VertexIndexOffset + VertexIndices[Orientation[1]],
			VertexIndexOffset + VertexIndices[Orientation[2]]
		};

		if (!GetVertexInstances(Vertices, VertexInstanceIDs))
		{
			return false;
		}

		// Fill up arrays associated with vertex instances
		VertexInstanceUVs.Set(VertexInstanceIDs[0], 0/*UVChannel*/, FVector2f(InTexCoords[Orientation[0]] * static_cast<float>(ScaleUV)));
		VertexInstanceUVs.Set(VertexInstanceIDs[1], 0/*UVChannel*/, FVector2f(InTexCoords[Orientation[1]] * static_cast<float>(ScaleUV)));
		VertexInstanceUVs.Set(VertexInstanceIDs[2], 0/*UVChannel*/, FVector2f(InTexCoords[Orientation[2]] * static_cast<float>(ScaleUV)));

		VertexInstanceNormals[VertexInstanceIDs[0]] = FVector3f(InNormals[Orientation[0]]);
		VertexInstanceNormals[VertexInstanceIDs[1]] = FVector3f(InNormals[Orientation[1]]);
		VertexInstanceNormals[VertexInstanceIDs[2]] = FVector3f(InNormals[Orientation[2]]);

		// Add the triangle as a polygon to the mesh description
		const FPolygonID PolygonID = Mesh.CreatePolygon(PolygonGroupID, VertexInstanceIDs.ABC);

		// Set group attribute
		PolygonAttributes[PolygonID] = GroupID;

		return true;
	}

	bool FMeshDescriptionWrapper::StartFaceTriangles(int32 TriangleCount, const TArray<FVector3f>& InNormals, const TArray<FVector2f>& InTexCoords)
	{
		const int32 ArraySize = InNormals.Num();
		ensure(InTexCoords.IsEmpty() || (ArraySize == InTexCoords.Num()));

		Normals = InNormals;
		for (FVector3f& Normal : Normals)
		{
			Normal = Normal.GetSafeNormal();
		}

		UE::CADKernel::MathUtils::ConvertVectorArray(Context.ModelParams.ModelCoordSys, Normals);

		TexCoords = InTexCoords;

		return true;
	}

	bool FMeshDescriptionWrapper::StartFaceTriangles(const TArrayView<FVector>& InNormals, const TArrayView<FVector2d>& InTexCoords)
	{
		const int32 ArraySize = InNormals.Num();
		ensure(InTexCoords.IsEmpty() || (ArraySize == InTexCoords.Num()));

		Normals.Reserve(ArraySize);
		TexCoords.Reserve(ArraySize);

		for (int32 Index = 0; Index < ArraySize; ++Index)
		{
			Normals.Emplace(FVector3f(InNormals[Index].X, InNormals[Index].Y, InNormals[Index].Z).GetSafeNormal());
			if (!InTexCoords.IsEmpty())
			{
				TexCoords.Emplace(InTexCoords[Index].X, InTexCoords[Index].Y);
			}
		}

		UE::CADKernel::MathUtils::ConvertVectorArray(Context.ModelParams.ModelCoordSys, Normals);

		return true;
	}

	bool FMeshDescriptionWrapper::AddFaceTriangles(const TArray<FFaceTriangle>& FaceTriangles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionWrapper::AddFaceTriangles);

		const bool bNeedSwapOrientation = Context.MeshParams.bNeedSwapOrientation;
		const FArray3i& Orientation = bNeedSwapOrientation ? CounterClockwise : Clockwise;
		
		FPolygonGroupID PolygonGroupID;
		FVertexInstanceID3 VertexInstanceIDs;

		for (const FFaceTriangle& FaceTriangle : FaceTriangles)
		{
			FArray3i Vertices{
				VertexIndexOffset + FaceTriangle.VertexIndices[Orientation[0]],
				VertexIndexOffset + FaceTriangle.VertexIndices[Orientation[1]],
				VertexIndexOffset + FaceTriangle.VertexIndices[Orientation[2]]
			};

			if (!GetVertexInstances(Vertices, VertexInstanceIDs))
			{
				return false;
			}

			// Fill up arrays associated with vertex instances
			if (!TexCoords.IsEmpty())
			{
				VertexInstanceUVs.Set(VertexInstanceIDs[0], 0/*UVChannel*/, FVector2f(TexCoords[FaceTriangle.TexCoords[Orientation[0]]] * static_cast<float>(ScaleUV)));
				VertexInstanceUVs.Set(VertexInstanceIDs[1], 0/*UVChannel*/, FVector2f(TexCoords[FaceTriangle.TexCoords[Orientation[1]]] * static_cast<float>(ScaleUV)));
				VertexInstanceUVs.Set(VertexInstanceIDs[2], 0/*UVChannel*/, FVector2f(TexCoords[FaceTriangle.TexCoords[Orientation[2]]] * static_cast<float>(ScaleUV)));
			}

			VertexInstanceNormals[VertexInstanceIDs[0]] = Normals[FaceTriangle.Normals[Orientation[0]]];
			VertexInstanceNormals[VertexInstanceIDs[1]] = Normals[FaceTriangle.Normals[Orientation[1]]];
			VertexInstanceNormals[VertexInstanceIDs[2]] = Normals[FaceTriangle.Normals[Orientation[2]]];

			// Add the triangle as a polygon to the mesh description
			PolygonGroupID = GetPolygonGroupID(FaceTriangle.MaterialID);
			const FPolygonID PolygonID = Mesh.CreatePolygon(PolygonGroupID, VertexInstanceIDs.ABC);

			// Set group attribute
			PolygonAttributes[PolygonID] = FaceTriangle.GroupID;
		}

		return true;
	}

	void FMeshDescriptionWrapper::EndFaceTriangles()
	{
		Normals.Reset(Normals.Num());
		TexCoords.Reset(TexCoords.Num());
	}

	void FMeshDescriptionWrapper::RecomputeNullNormal()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionWrapper::RecomputeNullNormal);

		constexpr float SquareNormalThreshold = UE_KINDA_SMALL_NUMBER * UE_KINDA_SMALL_NUMBER;

		constexpr int32 TriangleIndex[3][3] = { {0,1,2},{1,2,0},{2,0,1} };
		FVector NewNormal;

		for (FTriangleID Triangle : Mesh.Triangles().GetElementIDs())
		{
			TArrayView<const FVertexInstanceID> Vertices = Mesh.GetTriangleVertexInstances(Triangle);
			for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
			{
				FVector3f& Normal = VertexInstanceNormals[Vertices[VertexIndex]];

				if (Normal.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
				{
					NewNormal = FVector::ZeroVector;

					const FVertexInstanceID VertexInstanceID = Vertices[VertexIndex];
					FVertexID VertexID = Mesh.GetVertexInstanceVertex(VertexInstanceID);
					const TArrayView<const FTriangleID> VertexConnectedTriangles = Mesh.GetVertexInstanceConnectedTriangleIDs(VertexInstanceID);

					bool bNormalComputed = false;

					// compute the weighted sum of normals of the "partition star" according to the corner angle
					for (const FTriangleID& TriangleID : VertexConnectedTriangles)
					{
						// compute face normal at the vertex
						TArrayView<const FVertexID> TriangleVertices = Mesh.GetTriangleVertices(TriangleID);

						int32 ApexIndex = 0;
						for (; ApexIndex < 3; ++ApexIndex)
						{
							if (TriangleVertices[ApexIndex] == VertexID)
							{
								break;
							}
						}

						const FVector Position0 = FVector(VertexPositions[TriangleVertices[TriangleIndex[ApexIndex][0]]]);
						FVector DPosition1 = FVector(VertexPositions[TriangleVertices[TriangleIndex[ApexIndex][1]]]) - Position0;
						FVector DPosition2 = FVector(VertexPositions[TriangleVertices[TriangleIndex[ApexIndex][2]]]) - Position0;

						// to avoid numerical issue due to small vector
						DPosition1.Normalize();
						DPosition2.Normalize();

						// We have a left-handed coordinate system, but a counter-clockwise winding order
						// Hence normal calculation has to take the triangle vectors cross product in reverse.
						FVector TriangleNormal = DPosition2 ^ DPosition1;
						double SinOfApexAngle = TriangleNormal.Length();
						double ApexAngle = FMath::Asin(SinOfApexAngle);

						if (TriangleNormal.Normalize(SquareNormalThreshold))
						{
							NewNormal += (TriangleNormal * ApexAngle);
							bNormalComputed = true;
						}
					}

					if (bNormalComputed)
					{
						if (NewNormal.Normalize(SquareNormalThreshold))
						{
							Normal = FVector3f(NewNormal);
						}
					}
					else
					{
						// the vertex is a vertex of degenerated triangles, the vertex normal doesn't matter as the triangle is flat, but a non null normal is needed.
						Normal = FVector3f::UpVector;
					}
				}
			}
		}
	}

	void FMeshDescriptionWrapper::OrientMesh()
	{
		static const FVector MaxVector(-MAX_flt, -MAX_flt, -MAX_flt);
		static const FVector MinVector( MAX_flt, MAX_flt, MAX_flt );
		static const FIntVector UnInitVec( INDEX_NONE, INDEX_NONE, INDEX_NONE );

		FMeshTopologyHelper MeshHelper(Mesh);

		TQueue<FTriangleID> Front;
		TQueue<FTriangleID> BadOrientationFront;

		FTriangleID AdjacentTriangle;
		FEdgeID Edge;

		TArrayView < const FVertexInstanceID> Vertices;

		FVector MinCorner;
		FVector MaxCorner;
		FIntVector HighestVertex;
		FIntVector LowestVertex;

		uint32 NbTriangles = Mesh.Triangles().Num();

		TArray<FTriangleID> ConnectedTriangles;
		ConnectedTriangles.Reserve(NbTriangles);

		for (FTriangleID Triangle : Mesh.Triangles().GetElementIDs())
		{
			if (MeshHelper.IsTriangleMarked(Triangle))
			{
				continue;
			}

			MaxCorner = MaxVector;
			MinCorner = MinVector;
			HighestVertex = UnInitVec;
			LowestVertex = UnInitVec;

			MeshHelper.SetTriangleMarked(Triangle);

			MeshHelper.GetTriangleVertexExtremities(Triangle, MinCorner, MaxCorner, HighestVertex, LowestVertex);

			Front.Enqueue(Triangle);
			ConnectedTriangles.Add(Triangle);

			int32 NbConnectedFaces = 1;
			int32 NbBorderEdges = 0;
			int32 NbSurfaceEdges = 0;
			int32 NbSwappedTriangles = 0;
			while (!Front.IsEmpty())
			{
				while (!Front.IsEmpty())
				{
					Front.Dequeue(Triangle);

					TArrayView<const FEdgeID> EdgeSet = Mesh.GetTriangleEdges(Triangle);

					for (int32 IEdge = 0; IEdge < 3; IEdge++)
					{
						Edge = EdgeSet[IEdge];

						if (!MeshHelper.IsEdgeOfType(Edge, EElementType::Surface))
						{
							NbBorderEdges++;
							continue;
						}

						AdjacentTriangle = MeshHelper.GetOtherTriangleAtEdge(Edge, Triangle);
						if (MeshHelper.IsTriangleMarked(AdjacentTriangle))
						{
							continue;
						}

						NbSurfaceEdges++;
						NbConnectedFaces++;

						ConnectedTriangles.Add(AdjacentTriangle);

						MeshHelper.SetTriangleMarked(AdjacentTriangle);
						MeshHelper.GetTriangleVertexExtremities(AdjacentTriangle, MinCorner, MaxCorner, HighestVertex, LowestVertex);

						if (MeshHelper.GetEdgeDirectionInTriangle(Edge, 0) == MeshHelper.GetEdgeDirectionInTriangle(Edge, 1))
						{
							MeshHelper.SwapTriangleOrientation(AdjacentTriangle);
							NbSwappedTriangles++;
							BadOrientationFront.Enqueue(AdjacentTriangle);
						}
						else
						{
							Front.Enqueue(AdjacentTriangle);
						}
					}
				}

				while (!BadOrientationFront.IsEmpty())
				{
					BadOrientationFront.Dequeue(Triangle);

					TArrayView<const FEdgeID> EdgeSet = Mesh.GetTriangleEdges(Triangle);

					for (int32 IEdge = 0; IEdge < 3; IEdge++)
					{
						Edge = EdgeSet[IEdge];

						if (!MeshHelper.IsEdgeOfType(Edge, EElementType::Surface))
						{
							NbBorderEdges++;
							continue;
						}

						AdjacentTriangle = MeshHelper.GetOtherTriangleAtEdge(Edge, Triangle);
						if (MeshHelper.IsTriangleMarked(AdjacentTriangle))
						{
							continue;
						}

						NbSurfaceEdges++;
						NbConnectedFaces++;

						ConnectedTriangles.Add(AdjacentTriangle);

						MeshHelper.SetTriangleMarked(AdjacentTriangle);
						MeshHelper.GetTriangleVertexExtremities(AdjacentTriangle, MinCorner, MaxCorner, HighestVertex, LowestVertex);
						if (MeshHelper.GetEdgeDirectionInTriangle(Edge, 0) == MeshHelper.GetEdgeDirectionInTriangle(Edge, 1))
						{
							BadOrientationFront.Enqueue(AdjacentTriangle);
							MeshHelper.SwapTriangleOrientation(AdjacentTriangle);
							NbSwappedTriangles++;
						}
						else
						{
							Front.Enqueue(AdjacentTriangle);
						}
					}
				}
			}

			// Check if the mesh orientation need to be swapped
			int NbInverted = 0;
			int NbNotInverted = 0;
			if (NbBorderEdges == 0 || NbBorderEdges * 20 < NbSurfaceEdges)  // NbBorderEdges * 20 < nbSurfaceEdges => basic rule to define if a mesh is a surface mesh or if the mesh is a volume mesh with gaps 
			{
				// case of volume mesh
				// A vertex can have many normal (one by vertex instance) e.g. the corner of a box that have 3 normal and could be the highest 
				// vertex of a mesh. At the highest vertex, a folding of the mesh can give a vertex with two opposite normal.
				// The normal most parallel to the axis is preferred
				// To avoid mistake, we check for each highest vertex and trust the majority 

				if (HighestVertex[0] != INDEX_NONE)
				{
					for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
					{
						if (MeshHelper.IsVertexOfType(HighestVertex[VertexIndex], EElementType::Surface))
						{
							FVertexID VertexID = Mesh.GetVertexInstanceVertex(HighestVertex[VertexIndex]);
							TArrayView<const FVertexInstanceID> CoincidentVertexInstanceIdSet = Mesh.GetVertexVertexInstanceIDs(VertexID);
							float MaxComponent = 0;
							for (const FVertexInstanceID VertexInstanceID : CoincidentVertexInstanceIdSet)
							{
								FVector Normal = (FVector)VertexInstanceNormals[VertexInstanceID];
								if (FMath::Abs(MaxComponent) < FMath::Abs(Normal[VertexIndex]))
								{
									MaxComponent = Normal[VertexIndex];
								}
							}

							if (0 > MaxComponent)
							{
								NbInverted++;
							}
							else
							{
								NbNotInverted++;
							}
						}

						if (MeshHelper.IsVertexOfType(LowestVertex[VertexIndex], EElementType::Surface))
						{
							FVertexID VertexID = Mesh.GetVertexInstanceVertex(LowestVertex[VertexIndex]);
							TArrayView<const FVertexInstanceID> CoincidentVertexInstanceIdSet = Mesh.GetVertexVertexInstanceIDs(VertexID);
							float MaxComponent = 0;
							for (const FVertexInstanceID VertexInstanceID : CoincidentVertexInstanceIdSet)
							{
								FVector Normal = (FVector)VertexInstanceNormals[VertexInstanceID];
								if (FMath::Abs(MaxComponent) < FMath::Abs(Normal[VertexIndex]))
								{
									MaxComponent = Normal[VertexIndex];
								}
							}

							if (0 < MaxComponent)
							{
								NbInverted++;
							}
							else
							{
								NbNotInverted++;
							}
						}
					}
				}
			}
			else if (NbSwappedTriangles * 2 > NbConnectedFaces)
			{
				// case of surface mesh
				// this means that more triangles of surface shape have been swapped than no swapped, the good orientation has been reversed.
				// The mesh need to be re swapped
				NbInverted++;
			}

			// if needed swap all the mesh
			if (NbInverted > NbNotInverted)
			{
				for (auto Tri : ConnectedTriangles)
				{
					MeshHelper.SwapTriangleOrientation(Tri);
				}
			}
			ConnectedTriangles.Empty(NbTriangles);
		}
	}

	void FMeshDescriptionWrapper::ResolveTJunctions()
	{
	}

	void FCADKernelStaticMeshAttributes::Register(bool bKeepExistingAttribute)
	{
		FStaticMeshAttributes::Register(bKeepExistingAttribute);

		using namespace ExtendedMeshAttribute;

		if (!MeshDescription.PolygonAttributes().HasAttribute(PolyTriGroups) || !bKeepExistingAttribute)
		{
			MeshDescription.PolygonAttributes().RegisterAttribute<int32>(PolyTriGroups, 1, 0, EMeshAttributeFlags::AutoGenerated);
		}

		ensure(IsValid());
	}

	TPolygonAttributesRef<int32> FCADKernelStaticMeshAttributes::GetPolygonGroups()
	{
		using namespace ExtendedMeshAttribute;
		return MeshDescription.PolygonAttributes().GetAttributesRef<int32>(PolyTriGroups);
	}

	TPolygonAttributesConstRef<int32> FCADKernelStaticMeshAttributes::GetPolygonGroups() const
	{
		using namespace ExtendedMeshAttribute;
		return MeshDescription.PolygonAttributes().GetAttributesRef<int32>(PolyTriGroups);
	}

	void GetExistingFaceGroups(FMeshDescription& Mesh, TSet<int32>& FaceGroupsOut)
	{
		using namespace ExtendedMeshAttribute;
		TPolygonAttributesRef<int32> ElementToGroups = Mesh.PolygonAttributes().GetAttributesRef<int32>(PolyTriGroups);
		int32 LastPatchId = -1;
		for (const FPolygonID TriangleID : Mesh.Polygons().GetElementIDs())
		{
			int32 PatchId = ElementToGroups[TriangleID];
			if (PatchId != LastPatchId)
			{
				FaceGroupsOut.Add(PatchId);
				LastPatchId = PatchId;
			}
		}
	}
}
#endif
