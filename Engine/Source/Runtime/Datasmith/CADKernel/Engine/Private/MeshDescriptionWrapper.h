// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "MeshUtilities.h"
#if PLATFORM_DESKTOP

#include "MeshDescription.h"
#include "NaniteDefinitions.h"

namespace UE::CADKernel::MeshUtilities
{
	class FMeshDescriptionWrapper : public FMeshWrapperAbstract
	{
	public:
		FMeshDescriptionWrapper(const FMeshExtractionContext& InContext, FMeshDescription& InMesh);

		virtual void ClearMesh() override;
		virtual bool ReserveNewTriangles(int32 AdditionalTriangleCount) override;
		virtual bool SetVertices(TArray<FVector>&& Vertices) override;
		virtual bool AddNewVertices(TArray<FVector>&& Vertices) override;
		virtual bool AddTriangle(int32 GroupID, uint32 MaterialID, const FArray3i& VertexIndices, const TArrayView<FVector3f>& Normals, const TArrayView<FVector2f>& TexCoords) override;
		virtual bool StartFaceTriangles(int32 TriangleCount, const TArray<FVector3f>& Normals, const TArray<FVector2f>& TexCoords) override;
		virtual bool StartFaceTriangles(const TArrayView<FVector>& Normals, const TArrayView<FVector2d>& TexCoords) override;
		virtual bool AddFaceTriangles(const TArray<FFaceTriangle>& FaceTriangles) override;
		virtual bool AddFaceTriangle(const FFaceTriangle& FaceTriangle) override
		{
			return AddFaceTriangles({ FaceTriangle });
		}
		virtual void EndFaceTriangles() override;

		virtual void FinalizeMesh() override;
		virtual void AddSymmetry() override;
		virtual void RecomputeNullNormal() override;
		virtual void OrientMesh() override;
		virtual void ResolveTJunctions() override;

	private:
		template<typename VertexType>
		struct TMeshDescEntity
		{
			union
			{
				struct
				{
					VertexType A;
					VertexType B;
					VertexType C;
				};

				VertexType ABC[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
			};

			TMeshDescEntity() {}
			TMeshDescEntity(VertexType ValA, VertexType ValB, VertexType ValC)
				: A(ValA), B(ValB), C(ValC)
			{}

			VertexType& operator[](int Index)
			{
				return ABC[Index];
			}

			const VertexType& operator[](int Index) const
			{
				return ABC[Index];
			}
		};

		using FVertexID3 = TMeshDescEntity<FVertexID>;
		using FVertexInstanceID3 = TMeshDescEntity<FVertexInstanceID>;

		FPolygonGroupID GetPolygonGroupID(uint32 MaterialID)
		{
			if (FPolygonGroupID* PolygonGroupIDPtr = MaterialToPolygonGroupMapping.Find(MaterialID))
			{
				return *PolygonGroupIDPtr;
			}

			if (MaterialToPolygonGroupMapping.Num() < NANITE_MAX_CLUSTER_MATERIALS)
			{
				FName ImportedSlotName = *LexToString(MaterialID);
				LastPolygonGroupID = Mesh.CreatePolygonGroup();
				PolygonGroupImportedMaterialSlotNames[LastPolygonGroupID] = ImportedSlotName;
				MaterialToPolygonGroupMapping.Add(MaterialID, LastPolygonGroupID);
			}

			return LastPolygonGroupID;
		}

		// bIsBoundary is only modified if an edge exists
		FEdgeID FindEdge(const FVertexID& Start, const FVertexID& End, bool& bIsBoundary)
		{
			FEdgeID EdgeID = Mesh.GetVertexPairEdge(Start, End);
			if (EdgeID != INDEX_NONE)
			{
				TArrayView<const FTriangleID> Triangles = Mesh.GetEdgeConnectedTriangleIDs(EdgeID);
				bIsBoundary = Triangles.Num() < 2;
			}

			return EdgeID;
		}

		bool GetVertexInstances(const FArray3i& Vertices, FVertexInstanceID3& VertexInstances);

	private:
		int32 VertexIndexOffset = 0;
		FCADKernelStaticMeshAttributes Attributes;
		TVertexAttributesRef<FVector3f> VertexPositions;
		TVertexInstanceAttributesRef<FVertexID> VertexInstanceToVertex;
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals;
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents;
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns;
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors;
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs;
		TPolygonAttributesRef<int32> PolygonAttributes;
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames;

		bool bSelectiveExtraction = false;
		TMap<uint32, FPolygonGroupID> MaterialToPolygonGroupMapping;
		FPolygonGroupID LastPolygonGroupID = -1;
		TArray<FVertexID> VertexIDs;
		TArray<FVector3f> Normals;
		TArray<FVector2f> TexCoords;

		FMeshDescription& Mesh;
		bool bIsFinalized = false;
	};
}
#endif