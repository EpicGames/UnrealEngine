// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTopologyHelper.h"

#if PLATFORM_DESKTOP
#include "MeshAttributes.h"
#include "StaticMeshAttributes.h"

namespace UE::CADKernel::MeshUtilities
{
	FMeshTopologyHelper::FMeshTopologyHelper(FMeshDescription& InMeshDescription)
		: Mesh(InMeshDescription)
	{
		UpdateMeshWrapper();
	}

	void FMeshTopologyHelper::UpdateMeshWrapper()
	{
		VertexTypeSet.SetNumZeroed(Mesh.Vertices().GetArraySize());
		EdgeTypeSet.SetNumZeroed(Mesh.Edges().GetArraySize());

		TriangleMarkers.Init(false, Mesh.Triangles().GetArraySize());

		// If there are more than one polygon groups, edges between groups are considered feature lines
		for (const FEdgeID EdgeID : Mesh.Edges().GetElementIDs())
		{
			DefineEdgeTopology(EdgeID);
		}

		for (const FVertexID VertexID : Mesh.Vertices().GetElementIDs())
		{
			DefineVertexTopologyApproximation(VertexID);
		}

		VertexInstanceMarker.Init(false, Mesh.VertexInstances().GetArraySize());
	}


	void FMeshTopologyHelper::DefineEdgeTopology(FEdgeID EdgeID)
	{
		if (EdgeID == INDEX_NONE)
		{
			return;
		}

		TArrayView<const FTriangleID> EdgeConnectedPolygons = Mesh.GetEdgeConnectedTriangleIDs(EdgeID);

		switch (EdgeConnectedPolygons.Num())
		{
		case 0:
			EdgeTypeSet[EdgeID] = EElementType::Line;
			break;
		case 1:
			EdgeTypeSet[EdgeID] = EElementType::Border;
			break;
		case 2:
			EdgeTypeSet[EdgeID] = EElementType::Surface;
			break;
		default:
			EdgeTypeSet[EdgeID] = EElementType::NonManifold;
			break;
		}
	}

	void FMeshTopologyHelper::DefineVertexTopologyApproximation(FVertexID VertexID)
	{
		if (VertexID == INDEX_NONE)
		{
			return;
		}

		TArrayView<const FEdgeID> VertexConnectedEdgeIDs = Mesh.GetVertexConnectedEdgeIDs(VertexID);

		switch (VertexConnectedEdgeIDs.Num())
		{
		case 0:
			VertexTypeSet[VertexID] = EElementType::Free;
			break;

		case 1:
			VertexTypeSet[VertexID] = EElementType::Border;
			break;
		default:
		{
			int CountPerCategory[8] = { 0 };

			for (const FEdgeID EdgeID : VertexConnectedEdgeIDs)
			{
				CountPerCategory[type_width(EdgeTypeSet[EdgeID])]++;
			}

			if (CountPerCategory[type_width(EElementType::NonManifold)] > 0)
			{
				VertexTypeSet[VertexID] = EElementType::NonManifold;
			}
			else if (CountPerCategory[type_width(EElementType::Border)] > 0)
			{
				VertexTypeSet[VertexID] = EElementType::NonSurface;

			}
			else if (CountPerCategory[type_width(EElementType::Line)] > 0)
			{
				if (CountPerCategory[type_width(EElementType::Line)] == 2 && VertexConnectedEdgeIDs.Num() == 2)
				{
					VertexTypeSet[VertexID] = EElementType::Line;
				}
				else
				{
					VertexTypeSet[VertexID] = EElementType::NonManifold;
				}
			}
			else if (CountPerCategory[type_width(EElementType::Surface)] > 0)
			{
				const int32 EdgeCount = VertexConnectedEdgeIDs.Num();
				const FEdgeID FirstEdgeID(VertexConnectedEdgeIDs[0]);
				FEdgeID EdgeID(FirstEdgeID);

				FTriangleID TriangleID(INDEX_NONE);
				int32 TriangleCount = 0;

				do
				{
					TArrayView<const FTriangleID> EdgeConnectedPolygons = Mesh.GetEdgeConnectedTriangleIDs(EdgeID);
					// Border edge no more triangles to process, exit from the loop
					if (EdgeConnectedPolygons.Num() < 2)
					{
						break;
					}

					TriangleID = (TriangleID == EdgeConnectedPolygons[0]) ? EdgeConnectedPolygons[1] : EdgeConnectedPolygons[0];
					++TriangleCount;

					TArrayView<const FEdgeID> TriangleEdges = Mesh.GetTriangleEdges(TriangleID);

					for (int32 Corner = 0; Corner < 3; ++Corner)
					{
						if (EdgeID != TriangleEdges[Corner])
						{
							const FVertexID EdgeVertexID0 = Mesh.GetEdgeVertex(TriangleEdges[Corner], 0);
							const FVertexID EdgeVertexID1 = Mesh.GetEdgeVertex(TriangleEdges[Corner], 1);
							if (EdgeVertexID0 == VertexID || EdgeVertexID1 == VertexID)
							{
								EdgeID = TriangleEdges[Corner];
								break;
							}
						}
					}
				} while (EdgeID != FirstEdgeID);

				VertexTypeSet[VertexID] = TriangleCount == EdgeCount ? EElementType::Surface : EElementType::NonManifold;
			}
			else
			{
				// we don`t need to know exactly if the node is a border or non manifold node. Non surface is enough
				VertexTypeSet[VertexID] = EElementType::NonSurface;
			}
		}
		break;
		}
	}

	FMeshTopologyHelper::~FMeshTopologyHelper()
	{

	}

	bool FMeshTopologyHelper::IsVertexOfType(FVertexInstanceID VertexInstanceID, EElementType Type) const
	{
		FVertexID VertexID = Mesh.GetVertexInstanceVertex(VertexInstanceID);
		return EnumHasAnyFlags( VertexTypeSet[VertexID], Type);
	}

	// Triangle
	void FMeshTopologyHelper::GetTriangleVertexExtremities(FTriangleID Triangle, FVector& MinCorner, FVector& MaxCorner, FIntVector& HighestVertex, FIntVector& LowestVertex) const
	{

		TArrayView < const FVertexInstanceID> Vertices = Mesh.GetTriangleVertexInstances(Triangle);

		GetVertexExtremities(Vertices[0], MinCorner, MaxCorner, HighestVertex, LowestVertex);
		GetVertexExtremities(Vertices[1], MinCorner, MaxCorner, HighestVertex, LowestVertex);
		GetVertexExtremities(Vertices[2], MinCorner, MaxCorner, HighestVertex, LowestVertex);
	}

	void FMeshTopologyHelper::GetVertexExtremities(FVertexInstanceID VertexInstanceID, FVector& MinCorner, FVector& MaxCorner, FIntVector& HighestVertex, FIntVector& LowestVertex) const
	{
		FVertexID VertexID = Mesh.GetVertexInstanceVertex(VertexInstanceID);
		const FVector VertexPosition = (FVector)Mesh.GetVertexPositions()[VertexID];

		if (MaxCorner[0] < VertexPosition[0])
		{
			HighestVertex[0] = VertexInstanceID;
			MaxCorner[0] = VertexPosition[0];
		}
		if (MaxCorner[1] < VertexPosition[1])
		{
			HighestVertex[1] = VertexInstanceID;
			MaxCorner[1] = VertexPosition[1];
		}
		if (MaxCorner[2] < VertexPosition[2])
		{
			HighestVertex[2] = VertexInstanceID;
			MaxCorner[2] = VertexPosition[2];
		}

		if (MinCorner[0] > VertexPosition[0])
		{
			LowestVertex[0] = VertexInstanceID;
			MinCorner[0] = VertexPosition[0];
		}
		if (MinCorner[1] > VertexPosition[1])
		{
			LowestVertex[1] = VertexInstanceID;
			MinCorner[1] = VertexPosition[1];
		}
		if (MinCorner[2] > VertexPosition[2])
		{
			LowestVertex[2] = VertexInstanceID;
			MinCorner[2] = VertexPosition[2];
		}
	}

	void FMeshTopologyHelper::SwapTriangleOrientation(FTriangleID Triangle)
	{
		Mesh.ReverseTriangleFacing(Triangle);

		TArrayView<const FVertexInstanceID> TriVertexInstances = Mesh.GetTriangleVertexInstances(Triangle);
		for (int32 IVertex = 0; IVertex < 3; IVertex++)
		{
			const FVertexInstanceID& InstanceID = TriVertexInstances[IVertex];
			if (!VertexInstanceMarker[InstanceID])
			{
				SwapVertexNormal(InstanceID);
				VertexInstanceMarker[InstanceID] = true;
			}
		}
	}

	// Edge
	const FTriangleID FMeshTopologyHelper::GetOtherTriangleAtEdge(FEdgeID EdgeID, FTriangleID Triangle) const
	{
		TArrayView<const FTriangleID> EdgeConnectedPolygons = Mesh.GetEdgeConnectedTriangleIDs(EdgeID);
		return EdgeConnectedPolygons.Num() < 2 ? INDEX_NONE : (EdgeConnectedPolygons[0] == Triangle ? EdgeConnectedPolygons[1] : EdgeConnectedPolygons[0]);
	}

	bool FMeshTopologyHelper::GetEdgeDirectionInTriangle(FEdgeID EdgeID, int32 TriangleIndex) const
	{
		TArrayView<const FTriangleID> EdgeConnectedTriangles = Mesh.GetEdgeConnectedTriangleIDs(EdgeID);

		if (EdgeConnectedTriangles.Num() > TriangleIndex)
		{
			const FTriangleID TriangleID = EdgeConnectedTriangles[TriangleIndex];
			TArrayView<const FVertexInstanceID> VertexInstanceIDs = Mesh.GetTriangleVertexInstances(TriangleID);
			TArrayView<const FEdgeID> TriangleEdges = Mesh.GetTriangleEdges(TriangleID);

			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				if (TriangleEdges[Corner] == EdgeID)
				{
					const FVertexID VertexID = Mesh.GetVertexInstanceVertex(VertexInstanceIDs[Corner]);
					return Mesh.GetEdgeVertex(EdgeID, 0) == VertexID ? true : false;
				}
			}
		}
		return true;
	}

	// Vertex
	void FMeshTopologyHelper::SwapVertexNormal(FVertexInstanceID VertexInstanceID)
	{
		FStaticMeshAttributes StaticMeshAttributes(Mesh);
		FVector3f Normal = StaticMeshAttributes.GetVertexInstanceNormals()[VertexInstanceID];
		Normal *= -1;

		StaticMeshAttributes.GetVertexInstanceNormals()[VertexInstanceID] = Normal;
	}
}
#endif
