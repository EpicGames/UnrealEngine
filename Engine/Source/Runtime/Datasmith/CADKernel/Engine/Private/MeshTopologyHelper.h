// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if PLATFORM_DESKTOP

#include "MeshTypes.h"
#include "MeshDescription.h"

#include <bit>

#define ELEMENT_FIRST_MARKER_MASK	0x02

namespace UE::CADKernel::MeshUtilities
{
	struct FElementMetaData
	{
		uint16 Category : 4;
		uint16 Markers : 4;
		uint16 Extras : 4;
	};

	enum class EElementType : uint8_t
	{
		Unused       = 0,
		Free         = 1 << 0,
		Line         = 1 << 1,
		Surface      = 1 << 2,
		Border       = 1 << 3,
		NonManifold  = 1 << 4,
		NonSurface   = 1 << 5,
		Max			 = 1 << 7,
	};
	ENUM_CLASS_FLAGS(EElementType)

	class FMeshTopologyHelper
	{

	private:
		FMeshDescription& Mesh;

		TArray<EElementType> VertexTypeSet;
		TArray<EElementType> EdgeTypeSet;

		TBitArray<> VertexInstanceMarker;
		TBitArray<> TriangleMarkers;

	private:
		static int32 type_width(EElementType Type)
		{
			return std::bit_width((unsigned char)Type);
		}

	public:
		FMeshTopologyHelper(FMeshDescription& InMeshDescription);
		~FMeshTopologyHelper();

		void UpdateMeshWrapper();

		bool IsTriangleMarked(FTriangleID Triangle) const
		{
			return TriangleMarkers[Triangle];
		}
		void SetTriangleMarked(FTriangleID Triangle)
		{
			TriangleMarkers[Triangle] = true;
		}

		EElementType GetEdgeType(FEdgeID Edge) const
		{
			return EdgeTypeSet[Edge];
		}

		bool IsEdgeOfType(FEdgeID Edge, EElementType Type) const
		{
			return EnumHasAnyFlags(EdgeTypeSet[Edge], Type);
		}

		bool IsVertexOfType(FVertexInstanceID Vertex, EElementType Type) const;

		// Triangle
		void GetTriangleVertexExtremities(FTriangleID Triangle, FVector& MinCorner, FVector& MaxCorner, FIntVector& HighestVertex, FIntVector& LowestVertex) const;

		void SwapTriangleOrientation(FTriangleID Triangle);

		// Edge
		const FTriangleID GetOtherTriangleAtEdge(FEdgeID Edge, FTriangleID Triangle) const;

		/**
		 * @param Edge           The edge defined by its FEdgeID to get direction in its connected triangle
		 * @param TriangleIndex  The triangle connected to the edge, 0 for the first, 1 for the second, ...
		 *
		 */
		bool GetEdgeDirectionInTriangle(FEdgeID Edge, int32 TriangleIndex) const;
		void DefineEdgeTopology(FEdgeID EdgeID);

		// Vertex
		void SwapVertexNormal(FVertexInstanceID VertexID);
		void GetVertexExtremities(FVertexInstanceID Vertex, FVector& MinCorner, FVector& MaxCorner, FIntVector& HighestVertex, FIntVector& LowestVertex) const;

		void DefineVertexTopologyApproximation(FVertexID VertexID);
	};
}
#endif
