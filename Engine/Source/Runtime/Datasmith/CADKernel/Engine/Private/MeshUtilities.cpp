// Copyright Epic Games, Inc. All Rights Reserved. 

#include "MeshUtilities.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngineLog.h"

#include "MeshDescription.h"
#include "MeshDescriptionBuilder.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

namespace UE::CADKernel
{
	void FCADKernelUtilities::ApplyExtractionContext(const FMeshExtractionContext& Context, FMeshDescription& MeshInOut)
	{
		using namespace MeshUtilities;

		TSharedPtr<FMeshWrapperAbstract> MeshWrapper = FMeshWrapperAbstract::MakeWrapper(Context, MeshInOut);
		MeshWrapper->Complete();
	}

	void FCADKernelUtilities::ApplyExtractionContext(const FMeshExtractionContext& Context, UE::Geometry::FDynamicMesh3& MeshInOut)
	{
		using namespace MeshUtilities;

		TSharedPtr<FMeshWrapperAbstract> MeshWrapper = FMeshWrapperAbstract::MakeWrapper(Context, MeshInOut);
		MeshWrapper->Complete();
	}

	void FCADKernelUtilities::RegisterAttributes(FMeshDescription& MeshInOut, bool bKeepExistingAttribute)
	{
		MeshUtilities::FCADKernelStaticMeshAttributes Attributes(MeshInOut);
		Attributes.Register(bKeepExistingAttribute);
	}


	namespace MeshUtilities
	{
		bool FMeshOperations::OrientMesh(FMeshDescription& MeshDescription)
		{
			// #cad_import: TODO - Refactor MeshOperator::OrientMesh
			return true;
		}

		void FMeshOperations::ResolveTJunctions(FMeshDescription& MeshDescription, double Tolerance)
		{
			// #cad_import: TODO - Refactor MeshOperator::ResolveTJunctions
		}

		void FMeshOperations::RecomputeNullNormal(FMeshDescription& MeshDescriptione)
		{
			// #cad_import: TODO Refactor MeshOperator::RecomputeNullNormal
		}
	}
}
#endif
