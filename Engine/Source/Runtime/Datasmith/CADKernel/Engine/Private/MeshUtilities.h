// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CADKernelEngine.h"
#if PLATFORM_DESKTOP
#include "CADKernelEngineDefinitions.h"

#include "IntVectorTypes.h"
#include "StaticMeshAttributes.h"
#include "Util/DynamicVector.h"

namespace UE::CADKernel
{
	class FModelMesh;
	struct FMeshExtractionContext;
}

namespace UE::Geometry
{
	class FDynamicMesh3;
}

class FMeshDecription;

using FArray3i = UE::Geometry::FVector3i;

namespace UE::CADKernel::MeshUtilities
{
	class FCADKernelStaticMeshAttributes : public FStaticMeshAttributes
	{
	public:

		explicit FCADKernelStaticMeshAttributes(FMeshDescription& InMeshDescription) : FStaticMeshAttributes(InMeshDescription) {}

		virtual void Register(bool bKeepExistingAttribute = false) override;

		bool IsValid() const
		{
			return GetVertexInstanceNormals().IsValid() &&
				GetVertexInstanceTangents().IsValid() &&
				GetVertexInstanceBinormalSigns().IsValid() &&
				GetVertexInstanceColors().IsValid() &&
				GetVertexInstanceUVs().IsValid() &&
				GetPolygonGroupMaterialSlotNames().IsValid() &&
				GetPolygonGroups().IsValid();
		}

		/**
		 * Enable per-triangle integer attribute named PolyTriGroups (see ExtendedMeshAttribute::PolyTriGroups)
		 * This integer defines the identifier of the PolyTriGroup containing the triangle.
		 * In case of mesh coming from a CAD file, a PolyTriGroup is associated to a CAD topological face
		 */
		TPolygonAttributesRef<int32> GetPolygonGroups();
		TPolygonAttributesConstRef<int32> GetPolygonGroups() const;
	};

	class FMeshWrapperAbstract
	{
	public:
		struct FFaceTriangle
		{
			int32 GroupID;
			uint32 MaterialID;
			
			// Indices referencing the array of positions set calling SetVertices or the latest call to AddNewVertices
			const FArray3i VertexIndices;

			// Indices referencing the array of normals, Normals, set calling StartFaceTriangles.
			const FArray3i Normals;

			// Indices referencing the array of texture coordinates, TexCoords, set calling StartFaceTriangles.
			const FArray3i TexCoords;

			FFaceTriangle(int32 InGroupID, uint32 InMaterialID, const FArray3i& InVertexIndices, const FArray3i& InNormals, const FArray3i& InTexCoords)
				: GroupID(InGroupID)
				, MaterialID(InMaterialID)
				, VertexIndices(InVertexIndices)
				, Normals(InNormals)
				, TexCoords(InTexCoords)
			{
			}
		};

		FMeshWrapperAbstract(const FMeshExtractionContext& InContext) : Context(InContext)
		{
			bHasFaceGroupsToSkip = !InContext.FaceGroupsToExtract.IsEmpty();
		}

		/*
		 ** Call this method when the building of the mesh is completed.
		 ** Must be called before the pointer to the child class is deleted.
		 */
		void Complete()
		{
			if (bIsComplete)
			{
				return;
			}

			if (Context.MeshParams.bIsSymmetric)
			{
				AddSymmetry();
			}

			FinalizeMesh();

			// Workaround for SDHE-19725 (Declined): Compute any null normals.
			RecomputeNullNormal();

			OrientMesh();
			
			if (Context.bResolveTJunctions)
			{
				ResolveTJunctions();
			}

			bIsComplete = true;
		}

		virtual ~FMeshWrapperAbstract() {}

		virtual void ClearMesh() = 0;

		virtual bool SetVertices(TArray<FVector>&& Vertices) = 0;

		virtual bool AddNewVertices(TArray<FVector>&& Vertices) = 0;

		virtual bool ReserveNewTriangles(int32 AdditionalTriangleCount) = 0;

		/*
		 ** Expected Normals.Num() == TexCoords.Num() == 3 * TriangleCount
		 */
		virtual bool StartFaceTriangles(int32 TriangleCount, const TArray<FVector3f>& Normals, const TArray<FVector2f>& TexCoords) = 0;
		virtual bool StartFaceTriangles(const TArrayView<FVector>& Normals, const TArrayView<FVector2d>& TexCoords) = 0;

		virtual bool AddFaceTriangles(const TArray<FFaceTriangle>& FaceTriangles) = 0;

		virtual bool AddFaceTriangle(const FFaceTriangle& FaceTriangle) = 0;

		virtual void EndFaceTriangles() = 0;


		/*
		 * Normals and TexCoords array are expected to be arrays of 3 elements
		 * Each value in those arrays is associated to the vertex in VertexIndices at the same index value, 0, 1 and 2.
		 */
		virtual bool AddTriangle(int32 GroupID, uint32 MaterialID, const FArray3i& VertexIndices, const TArrayView<FVector3f>& Normals, const TArrayView<FVector2f>& TexCoords) = 0;

		bool IsFaceGroupValid(int32 GroupID)
		{
			return bHasFaceGroupsToSkip ? Context.FaceGroupsToExtract.Contains(GroupID) : true;
		}

		static TSharedPtr<FMeshWrapperAbstract> MakeWrapper(const FMeshExtractionContext& InContext, FMeshDescription& Mesh);
		static TSharedPtr<FMeshWrapperAbstract> MakeWrapper(const FMeshExtractionContext& Context, UE::Geometry::FDynamicMesh3& Mesh);

	protected:
		static constexpr FArray3i Clockwise{ 0, 1, 2 };
		static constexpr FArray3i CounterClockwise{ 0, 2, 1 };

		const FMeshExtractionContext& Context;
		bool bAreVerticesSet = false;

		// #cad_import: The UV scaling should be done in CADKernel
		static constexpr double ScaleUV = 0.001; // mm to m

	protected:
		virtual void AddSymmetry() = 0;
		virtual void FinalizeMesh() = 0;
		virtual void RecomputeNullNormal() = 0;
		virtual void OrientMesh() = 0;
		virtual void ResolveTJunctions() = 0;

	private:
		bool bHasFaceGroupsToSkip = false;
		bool bIsComplete = false;
	};

	void GetExistingFaceGroups(FMeshDescription& Mesh, TSet<int32>& FaceGroupsOut);
	void GetExistingFaceGroups(UE::Geometry::FDynamicMesh3& Mesh, TSet<int32>& FaceGroupsOut);

	template<typename VecType>
	void ConvertVectorArray(uint8 ModelCoordSys, UE::Geometry::TDynamicVector<VecType>& Array)
	{
		switch (ECADKernelModelCoordSystem(ModelCoordSys))
		{
			case ECADKernelModelCoordSystem::YUp_LeftHanded:
			for (VecType& Vector : Array)
			{
				Vector.Set(Vector[2], Vector[0], Vector[1]);
			}
			break;

			case ECADKernelModelCoordSystem::YUp_RightHanded:
			for (VecType& Vector : Array)
			{
				Vector.Set(-Vector[2], Vector[0], Vector[1]);
			}
			break;

			case ECADKernelModelCoordSystem::ZUp_RightHanded:
			for (VecType& Vector : Array)
			{
				Vector.Set(-Vector[0], Vector[1], Vector[2]);
			}
			break;

			case ECADKernelModelCoordSystem::ZUp_RightHanded_FBXLegacy:
			for (VecType& Vector : Array)
			{
				Vector.Set(Vector[0], -Vector[1], Vector[2]);
			}
			break;

			case ECADKernelModelCoordSystem::ZUp_LeftHanded:
			default:
			break;
		}
	}

	template<typename Type>
	UE::Math::TMatrix<Type> GetSymmetricMatrix(const UE::Math::TVector<Type>& Origin, const UE::Math::TVector<Type>& Normal)
	{
		using namespace UE::Math;
		//Calculate symmetry matrix
		//(Px, Py, Pz) = normal
		// -Px*Px + Pz*Pz + Py*Py  |  - 2 * Px * Py           |  - 2 * Px * Pz
		// - 2 * Py * Px           |  - Py*Py + Px*Px + Pz*Pz |  - 2 * Py * Pz
		// - 2 * Pz * Px           |  - 2 * Pz * Py           |  - Pz*Pz + Py*Py + Px*Px

		TVector<Type> LocOrigin = Origin;

		Type NormalXSqr = Normal.X * Normal.X;
		Type NormalYSqr = Normal.Y * Normal.Y;
		Type NormalZSqr = Normal.Z * Normal.Z;

		TMatrix<Type> OSymmetricMatrix;
		OSymmetricMatrix.SetIdentity();
		TVector<Type> Axis0(-NormalXSqr + NormalZSqr + NormalYSqr, -2 * Normal.X * Normal.Y, -2 * Normal.X * Normal.Z);
		TVector<Type> Axis1(-2 * Normal.Y * Normal.X, -NormalYSqr + NormalXSqr + NormalZSqr, -2 * Normal.Y * Normal.Z);
		TVector<Type> Axis2(-2 * Normal.Z * Normal.X, -2 * Normal.Z * Normal.Y, -NormalZSqr + NormalYSqr + NormalXSqr);
		OSymmetricMatrix.SetAxes(&Axis0, &Axis1, &Axis2);

		TMatrix<Type> SymmetricMatrix;
		SymmetricMatrix.SetIdentity();

		//Translate to 0, 0, 0
		LocOrigin *= -1.;
		SymmetricMatrix.SetOrigin(LocOrigin);

		//// Apply Symmetric
		SymmetricMatrix *= OSymmetricMatrix;

		//Translate to original position
		LocOrigin *= -1.;
		TMatrix<Type> OrigTranslation;
		OrigTranslation.SetIdentity();
		OrigTranslation.SetOrigin(LocOrigin);
		SymmetricMatrix *= OrigTranslation;

		return SymmetricMatrix;
	}

	class FMeshOperations
	{
		static bool OrientMesh(FMeshDescription& MeshDescription);
		static void ResolveTJunctions(FMeshDescription& MeshDescription, double Tolerance);
		static void RecomputeNullNormal(FMeshDescription& MeshDescriptione);
	};
}
#endif