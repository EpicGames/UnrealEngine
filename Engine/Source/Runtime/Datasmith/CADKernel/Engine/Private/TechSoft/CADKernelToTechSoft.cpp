// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelEngine.h"

#if PLATFORM_DESKTOP
#ifdef WITH_HOOPS
#include "TechSoftUniqueObjectImpl.h"

namespace UE::CADKernel
{
	A3DRiRepresentationItem* FTechSoftUtilities::CADKernelToTechSoft(TSharedPtr<FModel>& Model)
	{
		if (!FTechSoftLibrary::IsInitialized())
		{
			return nullptr;
		}

		return nullptr;
	}
}
#else
namespace UE::CADKernel
{
	A3DRiRepresentationItem* FTechSoftUtilities::CADKernelToTechSoft(TSharedPtr<FModel>& Model)
	{
		return nullptr;
	}
}
#endif
#endif

