// Copyright Epic Games, Inc. All Rights Reserved.

// Required to define INITIALIZE_A3D_API in order to call A3DSDKLoadLibraryA
#define INITIALIZE_A3D_API

#include "CADKernelEngine.h"
#if PLATFORM_DESKTOP

#ifdef WITH_HOOPS
#include "hoops_license.h"
#endif
#include "TechSoftUniqueObjectImpl.h"

#include "CADKernelEngineLog.h"

#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

namespace UE::CADKernel
{
	bool FTechSoftLibrary::bIsInitialized = false;

#ifdef WITH_HOOPS
	bool FTechSoftLibrary::Initialize()
	{
		if (bIsInitialized)
		{
			return true;
		}

		// #cad_import: Cover case when called in a game
		FString TechSoftDllPath = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/TechSoft"), FPlatformProcess::GetBinariesSubdirectory());
		TechSoftDllPath = FPaths::ConvertRelativePathToFull(TechSoftDllPath);

		if (A3DSDKLoadLibraryA(TCHAR_TO_UTF8(*TechSoftDllPath)))
		{
			A3DStatus Status = A3DLicPutUnifiedLicense(HOOPS_LICENSE);

			A3DInt32 iMajorVersion = 0, iMinorVersion = 0;
			Status = A3DDllGetVersion(&iMajorVersion, &iMinorVersion);
			if (Status == A3D_SUCCESS)
			{
				Status = A3DDllInitialize(A3D_DLL_MAJORVERSION, A3D_DLL_MINORVERSION);
				if (Status == A3D_SUCCESS || Status == A3D_INITIALIZE_ALREADY_CALLED)
				{
					bIsInitialized = true;
					return true;
				}
			}
		}

		return false;
	}

	const TCHAR* FTechSoftLibrary::GetVersion()
	{
		static const FString Version = TEXT("TechSoft ") + FString::FromInt(A3D_DLL_MAJORVERSION) + TEXT(".") + FString::FromInt(A3D_DLL_MINORVERSION) + TEXT(".") + FString::FromInt(A3D_DLL_UPDATEVERSION);
		return bIsInitialized ? *Version : TEXT("TechSoft uninitialized");
	}

	A3DRiRepresentationItem* FTechSoftLibrary::CreateRIBRep(const TArray<A3DTopoShell*>& TopoShells)
	{
		if (!FTechSoftLibrary::Initialize() || TopoShells.IsEmpty())
		{
			return nullptr;
		}

		A3DTopoConnex* TopoConnexPtr = nullptr;
		{
			TechSoft::TUniqueObject<A3DTopoConnexData> TopoConnexData;
			TopoConnexData->m_ppShells = (A3DTopoShell**)TopoShells.GetData();
			TopoConnexData->m_uiShellSize = TopoShells.Num();
			if (A3DTopoConnexCreate(TopoConnexData.GetPtr(), &TopoConnexPtr) != A3DStatus::A3D_SUCCESS)
			{
				return nullptr;
			}
		}

		A3DTopoBrepData* TopoBRepDataPtr = nullptr;
		{
			TechSoft::TUniqueObject<A3DTopoBrepDataData> TopoBRepData;
			TopoBRepData->m_uiConnexSize = 1;
			TopoBRepData->m_ppConnexes = &TopoConnexPtr;
			if (A3DTopoBrepDataCreate(TopoBRepData.GetPtr(), &TopoBRepDataPtr) != A3DStatus::A3D_SUCCESS)
			{
				return nullptr;
			}
		}

		TechSoft::TUniqueObject<A3DRiBrepModelData> RiBRepModelData;
		RiBRepModelData->m_pBrepData = TopoBRepDataPtr;
		RiBRepModelData->m_bSolid = false;
		A3DRiBrepModel* RiBrepModelPtr = nullptr;
		if (A3DRiBrepModelCreate(RiBRepModelData.GetPtr(), &RiBrepModelPtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}

		return RiBrepModelPtr;
	}

	A3DTopoFace* FTechSoftLibrary::CreateTopoFaceWithNaturalLoop(A3DSurfBase* CarrierSurface)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DTopoFaceData> Face;
		Face->m_pSurface = CarrierSurface;
		Face->m_bHasTrimDomain = false;
		Face->m_ppLoops = nullptr;
		Face->m_uiLoopSize = 0;
		Face->m_uiOuterLoopIndex = 0;
		Face->m_dTolerance = 0.01; //mm

		return CreateTopoFace(*Face);
	}

	A3DCrvNurbs* FTechSoftLibrary::CreateTrimNurbsCurve(A3DCrvNurbs* CurveNurbsPtr, double UMin, double UMax, bool bIs2D)
	{
		if (!FTechSoftLibrary::Initialize() || CurveNurbsPtr == nullptr)
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DCrvTransformData> TransformCurveData;

		TransformCurveData->m_bIs2D = bIs2D;
		TransformCurveData->m_sParam.m_sInterval.m_dMin = UMin;
		TransformCurveData->m_sParam.m_sInterval.m_dMax = UMax;
		TransformCurveData->m_sParam.m_dCoeffA = 1.;
		TransformCurveData->m_sParam.m_dCoeffB = 0.;
		TransformCurveData->m_pBasisCrv = CurveNurbsPtr;
		TransformCurveData->m_pTransfo = nullptr;

		TransformCurveData->m_sTrsf.m_sXVector.m_dX = 1.;
		TransformCurveData->m_sTrsf.m_sYVector.m_dY = 1.;
		TransformCurveData->m_sTrsf.m_sScale.m_dX = 1.;
		TransformCurveData->m_sTrsf.m_sScale.m_dY = 1.;
		TransformCurveData->m_sTrsf.m_sScale.m_dZ = 1.;

		A3DCrvTransform* CurveTransformPtr = nullptr;
		if (A3DCrvTransformCreate(TransformCurveData.GetPtr(), &CurveTransformPtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DCrvNurbsData> NurbsCurveData;
		if (A3DCrvBaseGetAsNurbs(CurveTransformPtr, 0.01 /*mm*/, /*bUseSameParameterization*/ true, NurbsCurveData.GetPtr()) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}

		return CreateCurveNurbs(*NurbsCurveData);
	}

	A3DTopoShell* FTechSoftLibrary::CreateTopoShell(A3DTopoShellData& TopoShellData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DTopoShell* TopoShellPtr = nullptr;
		if (A3DTopoShellCreate(&TopoShellData, &TopoShellPtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return TopoShellPtr;
	}

	A3DSurfNurbs* FTechSoftLibrary::CreateSurfaceNurbs(A3DSurfNurbsData& SurfaceNurbsData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DSurfNurbs* SurfaceNurbsPtr = nullptr;
		if (A3DSurfNurbsCreate(&SurfaceNurbsData, &SurfaceNurbsPtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return SurfaceNurbsPtr;
	}

	A3DStatus FTechSoftLibrary::SetEntityGraphicsColor(A3DEntity* Entity, FColor Color)
	{
		if (!FTechSoftLibrary::Initialize() || Entity == nullptr)
		{
			return A3DStatus::A3D_NOT_AVAILABLE;
		}

		TechSoft::TUniqueObject<A3DRootBaseWithGraphicsData> BaseWithGraphicsData(Entity);

		//Create a style color
		TechSoft::TUniqueObjectFromIndex<A3DGraphRgbColorData> RgbColor;

		RgbColor->m_dRed = Color.R / 255.;
		RgbColor->m_dGreen = Color.G / 255.;
		RgbColor->m_dBlue = Color.B / 255.;
		
		A3DUns32 ColorIndex = A3D_DEFAULT_COLOR_INDEX;
		if (A3DGlobalInsertGraphRgbColor(RgbColor.GetPtr(), &ColorIndex) != A3DStatus::A3D_SUCCESS)
		{
			ColorIndex = A3D_DEFAULT_COLOR_INDEX;
		}

		A3DUns32 StyleIndex = 0;
		TechSoft::TUniqueObjectFromIndex<A3DGraphStyleData> StyleData;
		StyleData->m_bMaterial = false;
		StyleData->m_bVPicture = false;
		StyleData->m_dWidth = 0.1; // default
		A3DUns8 Alpha = Color.A;
		if (Alpha < 255)
		{
			StyleData->m_bIsTransparencyDefined = true;
			StyleData->m_ucTransparency = 255 - Alpha;
		}
		else
		{
			StyleData->m_bIsTransparencyDefined = false;
			StyleData->m_ucTransparency = 0;
		}

		StyleData->m_bSpecialCulling = false;
		StyleData->m_bBackCulling = false;
		StyleData->m_uiRgbColorIndex = ColorIndex;

		StyleIndex = A3D_DEFAULT_STYLE_INDEX;
		if (A3DGlobalInsertGraphStyle(StyleData.GetPtr(), &StyleIndex) != A3DStatus::A3D_SUCCESS)
		{
			StyleIndex = A3D_DEFAULT_STYLE_INDEX;
		}

		TechSoft::TUniqueObject<A3DGraphicsData> GraphicsData;

		GraphicsData->m_uiStyleIndex = StyleIndex;
		GraphicsData->m_usBehaviour = kA3DGraphicsShow;
		GraphicsData->m_usBehaviour |= kA3DGraphicsSonHeritColor;

		BaseWithGraphicsData->m_pGraphics = nullptr;
		if (A3DGraphicsCreate(GraphicsData.GetPtr(), &BaseWithGraphicsData->m_pGraphics) != A3DStatus::A3D_SUCCESS || BaseWithGraphicsData->m_pGraphics == nullptr)
		{
			return A3DStatus::A3D_ERROR;
		}

		return A3DRootBaseWithGraphicsSet(Entity, BaseWithGraphicsData.GetPtr());
	}

	A3DTopoFace* FTechSoftLibrary::CreateTopoFace(A3DTopoFaceData& TopoFaceData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DTopoFace* TopoFacePtr = nullptr;
		if (A3DTopoFaceCreate(&TopoFaceData, &TopoFacePtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return TopoFacePtr;
	}

	A3DTopoLoop* FTechSoftLibrary::CreateTopoLoop(A3DTopoLoopData& TopoLoopData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DTopoLoop* TopoLoopPtr = nullptr;
		if (A3DTopoLoopCreate(&TopoLoopData, &TopoLoopPtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return TopoLoopPtr;
	}

	A3DTopoEdge* FTechSoftLibrary::CreateTopoEdge()
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DTopoEdgeData> EdgeData;
		return CreateTopoEdge(*EdgeData);
	}

	A3DTopoEdge* FTechSoftLibrary::CreateTopoEdge(A3DTopoEdgeData& TopoEdgeData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DTopoEdge* TopoEdgePtr = nullptr;
		if (A3DTopoEdgeCreate(&TopoEdgeData, &TopoEdgePtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return TopoEdgePtr;
	}

	A3DTopoCoEdge* FTechSoftLibrary::CreateTopoCoEdge(A3DTopoCoEdgeData& TopoCoEdgeData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DTopoCoEdge* TopoCoEdgePtr = nullptr;
		if (A3DTopoCoEdgeCreate(&TopoCoEdgeData, &TopoCoEdgePtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return TopoCoEdgePtr;
	}

	A3DCrvNurbs* FTechSoftLibrary::CreateCurveNurbs(A3DCrvNurbsData& CurveNurbsData)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return nullptr;
		}

		A3DCrvNurbs* CurveNurbsPtr = nullptr;
		if (A3DCrvNurbsCreate(&CurveNurbsData, &CurveNurbsPtr) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}

		return CurveNurbsPtr;
	}

	A3DStatus FTechSoftLibrary::LinkCoEdges(A3DTopoCoEdge* CoEdgePtr, A3DTopoCoEdge* NeighbourCoEdgePtr)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return A3DStatus::A3D_NOT_AVAILABLE;
		}

		return A3DTopoCoEdgeSetNeighbour(CoEdgePtr, NeighbourCoEdgePtr);
	}

	A3DAsmModelFile* FTechSoftLibrary::LoadModelFileFromFile(const A3DImport& Import, const TCHAR* Filename)
	{
		A3DAsmModelFile* ModelFile = nullptr;
		A3DStatus Status;
	#if !PLATFORM_EXCEPTIONS_DISABLED
		try
	#endif
		{
			Status = A3DAsmModelFileLoadFromFile(Import.GetFilePath(), &Import.m_sLoadData, &ModelFile);
		}
	#if !PLATFORM_EXCEPTIONS_DISABLED
		catch (...)
		{
			UE_LOGF(LogCADKernelEngine, Warning, "Failed to load %ls. An exception is thrown.", Filename);
			return nullptr;
		}
	#endif

		UE_LOGF(LogCADKernelEngine, Log, "A3DAsmModelFileLoadFromFile for '%ls' returned %d status.", Filename, Status);

		switch (Status)
		{
		case A3D_LOAD_MULTI_MODELS_CADFILE: //if the file contains multiple entries (see A3DRWParamsMultiEntriesData).
		case A3D_LOAD_MISSING_COMPONENTS: //_[I don't know about this one]_
		case A3D_SUCCESS:
			if (!ModelFile)
			{
				UE_LOGF(LogCADKernelEngine, Warning, "File %ls hasn't been loaded with success Status: %d.", Filename, Status);
			}
			return ModelFile;

		case A3DStatus::A3D_LOAD_FILE_TOO_OLD:
		{
			UE_LOGF(LogCADKernelEngine, Warning, "File %ls hasn't been loaded because the version is less than the oldest supported version.", Filename);
			break;
		}

		case A3DStatus::A3D_LOAD_FILE_TOO_RECENT:
		{
			UE_LOGF(LogCADKernelEngine, Warning, "File %ls hasn't been loaded because the version is more recent than supported version.", Filename);
			break;
		}

		case A3DStatus::A3D_LOAD_CANNOT_ACCESS_CADFILE:
		{
			UE_LOGF(LogCADKernelEngine, Warning, "File %ls hasn't been loaded because the input path cannot be opened by the running process for reading.", Filename);
			break;
		}

		case A3DStatus::A3D_LOAD_INVALID_FILE_FORMAT:
		{
			UE_LOGF(LogCADKernelEngine, Warning, "File %ls hasn't been loaded because the format is not supported.", Filename);
			break;
		}

		default:
			break;
		}

		return nullptr;
	}

	A3DAsmModelFile* FTechSoftLibrary::LoadModelFile(const TCHAR* Filename, const FTechSoftImportOverrides& LoadOverrides, FString& OutReason)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			// TODO: Update OutReason
			return nullptr;
		}
		
		A3DImport LoadOptions(TCHAR_TO_UTF8(Filename));
		// Specify reading mode
		LoadOptions.m_sLoadData.m_sGeneral.m_eReadGeomTessMode = LoadOverrides.bLoadGeometryOnly ? kA3DReadGeomOnly : kA3DReadGeomAndTess;
		
		// Specify loading of the input file in incremental mode
		LoadOptions.m_sLoadData.m_sIncremental.m_bLoadNoDependencies = LoadOverrides.bLoadLoadNoDependency ? A3D_TRUE : A3D_FALSE;
		LoadOptions.m_sLoadData.m_sIncremental.m_bLoadStructureOnly = LoadOverrides.bLoadStructureOnly ? A3D_TRUE : A3D_FALSE;


		// A3DRWParamsGeneralData Importer.m_sGeneral
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadSolids = A3D_TRUE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadSurfaces = A3D_TRUE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadWireframes = A3D_FALSE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadPmis = A3D_FALSE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadAttributes = A3D_TRUE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadHiddenObjects = A3D_TRUE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadConstructionAndReferences = A3D_FALSE;
		LoadOptions.m_sLoadData.m_sGeneral.m_bReadActiveFilter = A3D_FALSE;
		LoadOptions.m_sLoadData.m_sGeneral.m_eReadingMode2D3D = kA3DRead_3D;

		LoadOptions.m_sLoadData.m_sGeneral.m_bReadFeature = A3D_FALSE;

		LoadOptions.m_sLoadData.m_sGeneral.m_bReadConstraints = A3D_FALSE;


		A3DAsmModelFile* ModelFile = nullptr;
		A3DStatus LoadStatus = A3DStatus::A3D_SUCCESS;
#if !PLATFORM_EXCEPTIONS_DISABLED
		try
#endif
		{
			LoadStatus = A3DAsmModelFileLoadFromFile(LoadOptions.GetFilePath(), &LoadOptions.m_sLoadData, &ModelFile);
		}
#if !PLATFORM_EXCEPTIONS_DISABLED
		catch (...)
		{
			// TODO: Update OutReason
			return nullptr;
		}
#endif
		switch (LoadStatus)
		{
		case A3D_LOAD_MULTI_MODELS_CADFILE: //if the file contains multiple entries (see A3DRWParamsMultiEntriesData).
		case A3D_LOAD_MISSING_COMPONENTS: //_[I don't know about this one]_
		case A3D_SUCCESS:
			return ModelFile;
		default:
			// TODO: Update OutReason
			break;
		}
		return nullptr;
	}

	A3DAsmModelFile* FTechSoftLibrary::LoadModelFileFromPrcFile(const A3DUTF8Char* CADFileName, A3DRWParamsPrcReadHelper** ReadHelper)
	{
		A3DAsmModelFile* ModelFile = nullptr;
		if (A3DAsmModelFileLoadFromPrcFile(CADFileName, ReadHelper, &ModelFile) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return ModelFile;
	}

	A3DStatus FTechSoftLibrary::AdaptBRepInModelFile(A3DAsmModelFile* ModelFile, const A3DCopyAndAdaptBrepModelData& Setting, int32& ErrorCount, A3DCopyAndAdaptBrepModelErrorData** Errors)
	{
		A3DUns32 NbErrors;
		A3DStatus Status = A3DAdaptAndReplaceAllBrepInModelFileAdvanced(ModelFile, &Setting, &NbErrors, Errors);
		ErrorCount = NbErrors;
		return Status;
	}

	A3DStatus FTechSoftLibrary::DeleteModelFile(A3DAsmModelFile* ModelFile)
	{
		return A3DAsmModelFileDelete(ModelFile);
	}

	A3DStatus FTechSoftLibrary::DeleteEntity(A3DEntity* EntityPtr)
	{
		return A3DEntityDelete(EntityPtr);
	}

	double FTechSoftLibrary::GetModelFileUnit(const A3DAsmModelFile* ModelFile)
	{
		double FileUnit = 0.1;
	#if !PLATFORM_EXCEPTIONS_DISABLED
		try
	#endif
		{
			if (A3DAsmModelFileGetUnit(ModelFile, &FileUnit) != A3DStatus::A3D_SUCCESS)
			{
				return 0.1;
			}
		}
	#if !PLATFORM_EXCEPTIONS_DISABLED
		catch (...)
		{
				return 0.1;
		}
	#endif
		return FileUnit * 0.1;
	}

	A3DStatus FTechSoftLibrary::AdaptBRepModel(A3DAsmModelFile* ModelFile, const TCHAR* Filename)
	{
		using namespace UE::CADKernel;
		TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::AdaptBRepModel);

		const A3DUns32 ValidSurfaceCount = 9;
		A3DUns32 AcceptedSurfaces[ValidSurfaceCount] = {
			//kA3DTypeSurfBlend01,
			//kA3DTypeSurfBlend02,
			//kA3DTypeSurfBlend03,
			kA3DTypeSurfNurbs,
			kA3DTypeSurfCone,
			kA3DTypeSurfCylinder,
			kA3DTypeSurfCylindrical,
			//kA3DTypeSurfOffset,
			//kA3DTypeSurfPipe,
			kA3DTypeSurfPlane,
			kA3DTypeSurfRuled,
			kA3DTypeSurfSphere,
			kA3DTypeSurfRevolution,
			//kA3DTypeSurfExtrusion,
			//kA3DTypeSurfFromCurves,
			kA3DTypeSurfTorus,
			//kA3DTypeSurfTransform,
		};

		const A3DUns32 ValidCurveCount = 7;
		A3DUns32 AcceptedCurves[ValidCurveCount] = {
			//kA3DTypeCrvBase,
			//kA3DTypeCrvBlend02Boundary,
			kA3DTypeCrvNurbs,
			kA3DTypeCrvCircle,
			//kA3DTypeCrvComposite,
			//kA3DTypeCrvOnSurf,
			kA3DTypeCrvEllipse,
			//kA3DTypeCrvEquation,
			//kA3DTypeCrvHelix,
			kA3DTypeCrvHyperbola,
			//kA3DTypeCrvIntersection,
			kA3DTypeCrvLine,
			//kA3DTypeCrvOffset,
			kA3DTypeCrvParabola,
			kA3DTypeCrvPolyLine,
			//kA3DTypeCrvTransform,
		};

		TechSoft::TUniqueObject<A3DCopyAndAdaptBrepModelData> CopyAndAdaptBrepModelData;
		CopyAndAdaptBrepModelData->m_bUseSameParam = false;                        // If `A3D_TRUE`, surfaces will keep their parametrization when converted to NURBS.       
		CopyAndAdaptBrepModelData->m_dTol = 1e-3;                                  // Tolerance value of resulting B-rep. The value is relative to the scale of the model.
		CopyAndAdaptBrepModelData->m_bDeleteCrossingUV = false;                    // If `A3D_TRUE`, UV curves that cross seams of periodic surfaces are replaced by 3D curves 
		CopyAndAdaptBrepModelData->m_bSplitFaces = true;                           // If `A3D_TRUE`, the faces with a periodic basis surface are split on parametric seams
		CopyAndAdaptBrepModelData->m_bSplitClosedFaces = false;                    // If `A3D_TRUE`, the faces with a closed basis surface are split into faces at the parametric seam and mid-parameter
		CopyAndAdaptBrepModelData->m_bForceComputeUV = true;                       // If `A3D_TRUE`, UV curves are computed from the B-rep data
		CopyAndAdaptBrepModelData->m_bAllowUVCrossingSeams = true;                 // If `A3D_TRUE` and m_bForceComputeUV is set to `A3D_TRUE`, computed UV curves can cross seams.
		CopyAndAdaptBrepModelData->m_bForceCompute3D = false;                      // If `A3D_TRUE`, 3D curves are computed from the B-rep data
		CopyAndAdaptBrepModelData->m_bContinueOnError = true;                      // Continue processing even if an error occurs. Use \ref A3DCopyAndAdaptBrepModelAdvanced to get the error status.
		CopyAndAdaptBrepModelData->m_bClampTolerantUVCurvesInsideUVDomain = false; // If `A3D_FALSE`, UV curves may stray outside the UV domain as long as the 3D edge tolerance is respected. If set to `A3D_TRUE`, the UV curves will be clamped to the UV domain (if the clamp still leaves them within the edge tolerance). */
		CopyAndAdaptBrepModelData->m_bForceDuplicateGeometries = false;            // If `A3D_TRUE`, break the sharing of surfaces and curves into topologies.*/

		CopyAndAdaptBrepModelData->m_uiAcceptableSurfacesSize = ValidSurfaceCount;
		CopyAndAdaptBrepModelData->m_puiAcceptableSurfaces = &AcceptedSurfaces[0];
		CopyAndAdaptBrepModelData->m_uiAcceptableCurvesSize = ValidCurveCount;
		CopyAndAdaptBrepModelData->m_puiAcceptableCurves = &AcceptedCurves[0];

		int32 ErrorCount = 0;
		A3DCopyAndAdaptBrepModelErrorData* Errors = nullptr;
		A3DStatus Ret = FTechSoftLibrary::AdaptBRepInModelFile(ModelFile, *CopyAndAdaptBrepModelData, ErrorCount, &Errors);
		if ((Ret == A3D_SUCCESS || Ret == A3D_TOOLS_CONTINUE_ON_ERROR) && ErrorCount > 0)
		{
			// Add message about non-critical errors during the adaptation
			UE_LOGF(LogCADKernelEngine, Warning, "File %ls had %d non-critical error(s) during BRep adaptation step.", Filename, ErrorCount);
		}
		else if (Ret != A3D_SUCCESS)
		{
			UE_LOGF(LogCADKernelEngine, Warning, "File %ls failed during BRep adaptation step.", Filename);
			return A3D_ERROR;
		}
		return A3D_SUCCESS;
	}



	// Does it need to be moved? Seems too specific
	FString FTechSoftLibrary::CleanLabel(const FString& Name)
	{
		int32 Index;
		if (Name.FindLastChar(TEXT('['), Index))
		{
			return Name.Left(Index);
		}
		return Name;
	}

	void FTechSoftLibrary::GetOccurrenceChildren(const A3DAsmProductOccurrence* Node, TArray<const A3DAsmProductOccurrence*>& OutChildren)
	{
		// inspired by A3DProductOccurrenceConnector::CollectSons
		TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> OccurrenceData(Node);

		// Get a node in prototype chain with children, in case there are no children in current node
		int32 POccurrencesSize = OccurrenceData->m_uiPOccurrencesSize;
		A3DAsmProductOccurrence** POccurrences = OccurrenceData->m_ppPOccurrences;
		A3DAsmProductOccurrence* Prototype = OccurrenceData->m_pPrototype;

		// test:
		while ((POccurrencesSize == 0) && (Prototype != nullptr))
		{
			TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> PrototypeData(Prototype);
			if (!ensure(PrototypeData.IsValid()))
			{
				return;
			}

			Prototype = PrototypeData->m_pPrototype;
			POccurrencesSize = PrototypeData->m_uiPOccurrencesSize;
			POccurrences = PrototypeData->m_ppPOccurrences;
		}

		OutChildren.Append(POccurrences, POccurrencesSize);

		// test:
		if (OccurrenceData->m_pExternalData)
		{
			if (OutChildren.IsEmpty())
			{
				GetOccurrenceChildren(OccurrenceData->m_pExternalData, OutChildren);
			}
			else
			{
				OutChildren.Add(OccurrenceData->m_pExternalData);
			}
		}
	}

	void FTechSoftLibrary::ExtractGraphicsProperties(const A3DEntity* Entity, FGraphicsProperties& Result)
	{
		if (A3DEntityIsBaseWithGraphicsType(Entity))
		{
			UE::CADKernel::TechSoft::TUniqueObject<A3DRootBaseWithGraphicsData> MetaDataWithGraphics(Entity);
			if (MetaDataWithGraphics.IsValid())
			{
				if (MetaDataWithGraphics->m_pGraphics != NULL)
				{
					// FTechSoftFileParser::ExtractGraphicProperties
					UE::CADKernel::TechSoft::TUniqueObject<A3DGraphicsData> GraphicsData(MetaDataWithGraphics->m_pGraphics);
					if (GraphicsData.IsValid())
					{
						Result.bIsRemoved = GraphicsData->m_usBehaviour & kA3DGraphicsRemoved;
						Result.bShow = GraphicsData->m_usBehaviour & kA3DGraphicsShow;

						if (GraphicsData->m_usBehaviour & kA3DGraphicsFatherHeritColor)
						{
							Result.MaterialInheritance = FGraphicsProperties::EInheritance::Father;
						}
						else if (GraphicsData->m_usBehaviour & kA3DGraphicsSonHeritColor)
						{
							Result.MaterialInheritance = FGraphicsProperties::EInheritance::Child;
						}

						// may be A3D_DEFAULT_STYLE_INDEX)
						Result.StyleIndex = GraphicsData->m_uiStyleIndex;
					}
				}
			}
		}
	}

	bool FTechSoftLibrary::ExtractMaterial(uint32 TechSoftMaterialIndex, const A3DGraphStyleData& GraphStyleData, FTechSoftLibrary::FMaterial& ParsedMaterial)
	{
		UE::CADKernel::TechSoft::TUniqueObjectFromIndex<A3DGraphMaterialData> MaterialData(TechSoftMaterialIndex);
		if(MaterialData.IsValid())
		{
			ParsedMaterial.Diffuse = GetColorAt(MaterialData->m_uiDiffuse);
			ParsedMaterial.Ambient = GetColorAt(MaterialData->m_uiAmbient);
			ParsedMaterial.Specular = GetColorAt(MaterialData->m_uiSpecular);
			ParsedMaterial.Shininess = MaterialData->m_dShininess;
			ParsedMaterial.Transparency = GraphStyleData.m_bIsTransparencyDefined
				                              ? 1. - GraphStyleData.m_ucTransparency/255.f
				                              : 0;
			// todo: find how to convert Emissive color into ? reflexion coef...
			// Material.Emissive = GetColor(MaterialData->m_uiEmissive);
			// Material.Reflexion;
			return true;
		}
		return false;
	}

	FColor FTechSoftLibrary::GetColorAt(uint32 ColorIndex)
	{
		using namespace UE::CADKernel;
		TechSoft::TUniqueObjectFromIndex<A3DGraphRgbColorData> ColorData(ColorIndex);
		if (ColorData.IsValid())
		{
			return FColor((uint8)(ColorData->m_dRed * 255)
				, (uint8)(ColorData->m_dGreen * 255)
				, (uint8)(ColorData->m_dBlue * 255));
		}
		return FColor(200, 200, 200);
	}


	TArray<uint32> FTechSoftLibrary::GetStylesUsedOnFaces(A3DRiBrepModel* RepresentationItem)
	{
		UE::CADKernel::TechSoft::TUniqueObject<A3DRiRepresentationItemData> RepresentationData(RepresentationItem);

		using namespace UE::CADKernel;
		TArray<uint32> MaterialsUsedOnFaces;
		TechSoft::TUniqueObject<A3DTess3DData> Tess3DData(RepresentationData->m_pTessBase);
		if (Tess3DData.IsValid() && Tess3DData->m_uiFaceTessSize != 0)
		{
			for (uint32 FaceTessIndex = 0; FaceTessIndex < Tess3DData->m_uiFaceTessSize; ++FaceTessIndex)
			{
				const A3DTessFaceData& TessFaceData = Tess3DData->m_psFaceTessData[FaceTessIndex];

				if (TessFaceData.m_uiStyleIndexesSize != 0)
				{
					A3DUns32 StyleIndex = TessFaceData.m_puiStyleIndexes[0];
					MaterialsUsedOnFaces.Add(StyleIndex);
				}
			}
		}

		TechSoft::TUniqueObject<A3DRiBrepModelData> BRepModelData(RepresentationItem);

		if (BRepModelData.IsValid())
		{
			A3DTopoBrepData* BRepData = BRepModelData->m_pBrepData;

			TechSoft::TUniqueObject<A3DTopoBrepDataData> TopoBrepData(BRepData);
			if (TopoBrepData.IsValid())
			{
				for (A3DUns32 Index = 0; Index < TopoBrepData->m_uiConnexSize; ++Index)
				{
					TechSoft::TUniqueObject<A3DTopoConnexData> TopoConnexData(TopoBrepData->m_ppConnexes[Index]);
					if (TopoConnexData.IsValid())
					{
						for (A3DUns32 Sndex = 0; Sndex < TopoConnexData->m_uiShellSize; ++Sndex)
						{
							TechSoft::TUniqueObject<A3DTopoShellData> ShellData(TopoConnexData->m_ppShells[Sndex]);
							if (ShellData.IsValid())
							{
								for (A3DUns32 Fndex = 0; Fndex < ShellData->m_uiFaceSize; ++Fndex)
								{
									A3DTopoFace* TopoFace = ShellData->m_ppFaces[Fndex];
									FTechSoftLibrary::FGraphicsProperties Result;
									FTechSoftLibrary::ExtractGraphicsProperties(TopoFace, Result);

									if (!Result.bIsRemoved && Result.bShow)
									{
										A3DUns32 StyleIndex = Result.StyleIndex;

										MaterialsUsedOnFaces.Add(StyleIndex);
									}
								}
							}
						}
					}
				}
			}
		}

		return MoveTemp(MaterialsUsedOnFaces);
	}


	bool FTechSoftLibrary::ParseRootBaseData(const A3DEntity* Entity, TMap<FString, FString>& MetaData, FString& UniqueID, FString& Label)
	{
		using namespace UE::CADKernel;
		TechSoft::TUniqueObject<A3DRootBaseData> RootBaseData(Entity);
		if (RootBaseData.IsValid())
		{
			if (RootBaseData->m_pcName && RootBaseData->m_pcName[0] != '\0')
			{
				FString Name = UTF8_TO_TCHAR(RootBaseData->m_pcName);

				// todo: this sounds strange - check?
				// "unnamed" is create by Techsoft. This name is ignored
				if(Name != TEXT("unnamed"))  
				{
					Label = FTechSoftLibrary::CleanLabel(Name);
				} 
			}

			UniqueID = TEXT("TechSoft::") + FString::FromInt(RootBaseData->m_uiPersistentId); // RootBaseData->m_uiPersistentId is unique

			TechSoft::TUniqueObject<A3DMiscAttributeData> AttributeData;
			for (A3DUns32 Index = 0; Index < RootBaseData->m_uiSize; ++Index)
			{
				AttributeData.FillFrom(RootBaseData->m_ppAttributes[Index]);
				if (AttributeData.IsValid())
				{
					TraverseAttribute(*AttributeData, MetaData);
				}
			}
			return true;
		}
		return false;
	}

	void FTechSoftLibrary::TraverseAttribute(const A3DMiscAttributeData& AttributeData, TMap<FString, FString>& OutMetaData)
	{
		FString AttributeFamillyName;
		if (AttributeData.m_bTitleIsInt)
		{
			A3DUns32 UnsignedValue = 0;
			memcpy(&UnsignedValue, AttributeData.m_pcTitle, sizeof(A3DUns32));
			AttributeFamillyName = FString::Printf(TEXT("%u"), UnsignedValue);
		}
		else if (AttributeData.m_pcTitle && AttributeData.m_pcTitle[0] != '\0')
		{
			AttributeFamillyName = UTF8_TO_TCHAR(AttributeData.m_pcTitle);
		}

		for (A3DUns32 Index = 0; Index < AttributeData.m_uiSize; ++Index)
		{
			FString AttributeName = AttributeFamillyName;
			{
				FString AttributeTitle = UTF8_TO_TCHAR(AttributeData.m_asSingleAttributesData[Index].m_pcTitle);
				if (AttributeTitle.Len())
				{
					AttributeName += TEXT(" ") + AttributeTitle;
				}
				else if(Index > 0)
				{
					AttributeName += TEXT(" ") + FString::FromInt((int32)Index);
				}
			}

			FString AttributeValue;
			switch (AttributeData.m_asSingleAttributesData[Index].m_eType)
			{
			case kA3DModellerAttributeTypeTime:
			case kA3DModellerAttributeTypeInt:
			{
				A3DInt32 Value;
				memcpy(&Value, AttributeData.m_asSingleAttributesData[Index].m_pcData, sizeof(A3DInt32));
				AttributeValue = FString::Printf(TEXT("%d"), Value);
				break;
			}

			case kA3DModellerAttributeTypeReal:
			{
				A3DDouble Value;
				memcpy(&Value, AttributeData.m_asSingleAttributesData[Index].m_pcData, sizeof(A3DDouble));
				AttributeValue = FString::Printf(TEXT("%f"), Value);
				break;
			}

			case kA3DModellerAttributeTypeString:
			{
				if (AttributeData.m_asSingleAttributesData[Index].m_pcData && AttributeData.m_asSingleAttributesData[Index].m_pcData[0] != '\0')
				{
					AttributeValue = UTF8_TO_TCHAR(AttributeData.m_asSingleAttributesData[Index].m_pcData);
				}
				break;
			}

			default:
				break;
			}

			if (AttributeName.Len())
			{
				OutMetaData.Emplace(AttributeName, AttributeValue);
			}
		}
	}

	bool FTechSoftLibrary::IsMaterialTexture(const uint32 MaterialIndex)
	{
		A3DBool bIsTexture = false;
		return A3DGlobalIsMaterialTexture(MaterialIndex, &bIsTexture) == A3DStatus::A3D_SUCCESS ? bool(bIsTexture) : false;
	}

	bool FTechSoftLibrary::GetEntityType(A3DRiRepresentationItem* RepresentationItem, A3DEEntityType& Type)
	{
		return A3DEntityGetType(RepresentationItem, &Type) == A3D_SUCCESS;
	}

	A3DAsmProductOccurrence* FTechSoftLibrary::FindConfiguration(const A3DAsmProductOccurrence* ConfigurationSetOccurrencePtr, TFunctionRef<bool(A3DAsmProductOccurrenceData& ConfigurationData)> Callback)
	{
		TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> ConfigurationSetData(ConfigurationSetOccurrencePtr);
		if (ConfigurationSetData.IsValid())
		{
			for (uint32 Index = 0; Index < ConfigurationSetData->m_uiPOccurrencesSize; ++Index)
			{
				A3DAsmProductOccurrence* ConfigurationOccurrence = ConfigurationSetData->m_ppPOccurrences[Index];
				TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> ConfigurationData(ConfigurationOccurrence);
				if (!ConfigurationData.IsValid())
				{
					continue;
				}

				if (ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONFIG )
				{
					if (Callback(*ConfigurationData))
					{
						return ConfigurationOccurrence;
					}
				}
			}
		}
		return nullptr;
	};

	bool FTechSoftLibrary::IsConfigurationSet(UE::CADKernel::ECADFormat Format, const A3DAsmProductOccurrence* Occurrence)
	{
		switch (Format)
		{
		case ECADFormat::CATIAV4:
		case ECADFormat::N_X:
		case ECADFormat::SOLIDWORKS:
		{
			TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> OccurrenceData(Occurrence);
			if (!OccurrenceData.IsValid())
			{
				return false;
			}

			return OccurrenceData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONTAINER;
		}

		default : 
			return false;
		}
	}


#else
	bool FTechSoftLibrary::Initialize()
	{
		return false;
	}

	const TCHAR* FTechSoftLibrary::GetVersion()
	{
		return TEXT("TechSoft unavailable");
	}
#endif
}
#endif