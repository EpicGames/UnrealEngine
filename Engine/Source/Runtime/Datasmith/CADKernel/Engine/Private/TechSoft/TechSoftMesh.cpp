// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelEngine.h"
#include "TechSoftUtilities.h"

#ifdef WITH_HOOPS
#include "MeshUtilities.h"
#include "TechSoftUniqueObjectImpl.h"

namespace UE::CADKernel::TechSoftUtilities
{
	using namespace UE::CADKernel;
	using namespace UE::CADKernel::MeshUtilities;

	class FFaceTriangleCollector
	{
	private:
		const A3DUns32* TriangulatedIndexes = nullptr;
		const TArrayView<FVector>& TessellationNormals;
		const TArrayView<FVector2d>& TessellationTexCoords;
		
		int32 LastVertexIndex = 0;
		int32 GroupID = INDEX_NONE;
		uint32 MaterialID = INDEX_NONE;
		TArray<FVector3f> FaceNormals;
		TArray<FVector2f> FaceTexCoords;
		TArray<FMeshWrapperAbstract::FFaceTriangle> FaceTriangles;
		FVector2d UVScale;

	public:
		struct FCollectionResult
		{
			TArray<FVector3f> Normals;
			TArray<FVector2f> TexCoords;
			TArray<FMeshWrapperAbstract::FFaceTriangle> FaceTriangles;
		};

		FFaceTriangleCollector(const A3DUns32* InTriangulatedIndexes, const TArrayView<FVector>& InNormals, const TArrayView<FVector2d>& InTexCoords)
			: TriangulatedIndexes(InTriangulatedIndexes)
			, TessellationNormals(InNormals)
			, TessellationTexCoords(InTexCoords)
		{
		}

		bool CollectTriangles(const A3DTessFaceData& FaceTessData, const A3DTopoFace* TopoFace, int32 InGroupID, double TextureUnit, FCollectionResult& Result);

		void AddFaceTriangle(const uint32 TriangleCount, uint32& StartIndexInOut);
		void AddFaceTriangleWithUniqueNormal(const uint32 TriCount, uint32& StartIndexInOut);
		void AddFaceTriangleWithUniqueNormalAndTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut);
		void AddFaceTriangleWithTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut);

		void AddFaceTriangleFan(const uint32 TriCount, uint32& StartIndexInOut);
		void AddFaceTriangleFanWithUniqueNormal(const uint32 TriCount, uint32& StartIndexInOut);
		void AddFaceTriangleFanWithUniqueNormalAndTexture(const uint32 TriCout, const uint32 TexOffset, uint32& StartIndexInOut);
		void AddFaceTriangleFanWithTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut);

		void AddFaceTriangleStripe(const uint32 TriCount, uint32& StartIndexInOut);
		void AddFaceTriangleStripeWithTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut);
		void AddFaceTriangleStripeWithUniqueNormal(const uint32 TriCount, uint32& StartIndexInOut);
		void AddFaceTriangleStripeWithUniqueNormalAndTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut);

		void AddTriangle(const FArray3i& VertexIndices, const FArray3i& NormalIndices)
		{
			const int32 NextIndex = FaceNormals.Num();
			const FArray3i OtherIndices{ NextIndex + 0, NextIndex + 1, NextIndex + 2 };

			const FVector3f Normals[3]{
				{FVector3f(TessellationNormals[NormalIndices[0] / 3])},
				{FVector3f(TessellationNormals[NormalIndices[1] / 3])},
				{FVector3f(TessellationNormals[NormalIndices[2] / 3])},
			};
			FaceNormals.Append(Normals, 3);

			FaceTriangles.Emplace(GroupID, MaterialID, VertexIndices, OtherIndices, OtherIndices);
		}

		void AddTriangle(const FArray3i& VertexIndices, const FArray3i& NormalIndices, const FArray3i& TexCoordIndicies)
		{
			const int32 NextIndex = FaceNormals.Num();
			const FArray3i OtherIndices{ NextIndex + 0, NextIndex + 1, NextIndex + 2 };

			const FVector3f Normals[3]{
				{FVector3f(TessellationNormals[NormalIndices[0] / 3])},
				{FVector3f(TessellationNormals[NormalIndices[1] / 3])},
				{FVector3f(TessellationNormals[NormalIndices[2] / 3])},
			};
			FaceNormals.Append(Normals, 3);

			const FVector2f TexCoords[3]{
				{FVector2f(TessellationTexCoords[TexCoordIndicies[0] / 2] * UVScale)},
				{FVector2f(TessellationTexCoords[TexCoordIndicies[1] / 2] * UVScale)},
				{FVector2f(TessellationTexCoords[TexCoordIndicies[2] / 2] * UVScale)},
			};
			FaceTexCoords.Append(TexCoords, 3);

			FaceTriangles.Emplace(GroupID, MaterialID, VertexIndices, OtherIndices, OtherIndices);
		}
	};

	void FFaceTriangleCollector::AddFaceTriangle(const uint32 TriCount, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;

		// Get Triangles
		for (uint32 Index = 0; Index < TriCount; ++Index)
		{
			NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
			NormalIndices[1] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;
			NormalIndices[2] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleWithUniqueNormal(const uint32 TriCount, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;

		for (uint32 Index = 0; Index < TriCount; ++Index)
		{
			NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
			NormalIndices[1] = NormalIndices[0];
			NormalIndices[2] = NormalIndices[0];

			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
			VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleWithUniqueNormalAndTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;
		FArray3i TexCoordIndices;

		// Get Triangles
		for (uint32 FArray3i = 0; FArray3i < TriCount; FArray3i++)
		{
			NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
			NormalIndices[1] = NormalIndices[0];
			NormalIndices[2] = NormalIndices[0];

			TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

			TexCoordIndices[1] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

			TexCoordIndices[2] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleWithTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;
		FArray3i TexCoordIndices;

		for (uint32 Index = 0; Index < TriCount; ++Index)
		{
			NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
			TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

			NormalIndices[1] = TriangulatedIndexes[StartIndexInOut++];
			TexCoordIndices[1] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

			NormalIndices[2] = TriangulatedIndexes[StartIndexInOut++];
			TexCoordIndices[2] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleFan(const uint32 TriCount, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
		NormalIndices[1] = TriangulatedIndexes[StartIndexInOut++];
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			NormalIndices[2] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);

			NormalIndices[1] = NormalIndices[2];
			VertexIndices[1] = VertexIndices[2];
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleFanWithUniqueNormal(const uint32 TriCount, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;


		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		NormalIndices[1] = NormalIndices[0];
		NormalIndices[2] = NormalIndices[0];

		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		// Get Triangles
		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);

			VertexIndices[1] = VertexIndices[2];
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleFanWithUniqueNormalAndTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;
		FArray3i TexCoordIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		NormalIndices[1] = NormalIndices[0];
		NormalIndices[2] = NormalIndices[0];

		TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

		TexCoordIndices[1] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			TexCoordIndices[2] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);

			VertexIndices[1] = VertexIndices[2];
			TexCoordIndices[1] = TexCoordIndices[2];
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleFanWithTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;
		FArray3i TexCoordIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

		NormalIndices[1] = TriangulatedIndexes[StartIndexInOut++];
		TexCoordIndices[1] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			NormalIndices[2] = TriangulatedIndexes[StartIndexInOut++];
			TexCoordIndices[2] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);

			NormalIndices[1] = NormalIndices[2];
			TexCoordIndices[1] = TexCoordIndices[2];
			VertexIndices[1] = VertexIndices[2];
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleStripe(const uint32 TriCount, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
		NormalIndices[1] = TriangulatedIndexes[StartIndexInOut++];
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			NormalIndices[2] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);

			Index++;
			if (Index == TriCount)
			{
				break;
			}

			Swap(VertexIndices[1], VertexIndices[2]);
			Swap(NormalIndices[1], NormalIndices[2]);

			NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);

			Swap(VertexIndices[0], VertexIndices[1]);
			Swap(NormalIndices[0], NormalIndices[1]);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleStripeWithTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;
		FArray3i TexCoordIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
		NormalIndices[1] = TriangulatedIndexes[StartIndexInOut++];
		TexCoordIndices[1] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			NormalIndices[2] = TriangulatedIndexes[StartIndexInOut++];
			TexCoordIndices[2] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);

			Index++;
			if (Index == TriCount)
			{
				break;
			}

			Swap(VertexIndices[1], VertexIndices[2]);
			Swap(NormalIndices[1], NormalIndices[2]);
			Swap(TexCoordIndices[1], TexCoordIndices[2]);

			NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);

			Swap(VertexIndices[0], VertexIndices[1]);
			Swap(NormalIndices[0], NormalIndices[1]);
			Swap(TexCoordIndices[0], TexCoordIndices[1]);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleStripeWithUniqueNormal(const uint32 TriCount, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		NormalIndices[1] = NormalIndices[0];
		NormalIndices[2] = NormalIndices[0];

		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);

			Index++;
			if (Index == TriCount)
			{
				break;
			}

			Swap(VertexIndices[1], VertexIndices[2]);

			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices);

			Swap(VertexIndices[0], VertexIndices[1]);
		}
	}

	void FFaceTriangleCollector::AddFaceTriangleStripeWithUniqueNormalAndTexture(const uint32 TriCount, const uint32 TexOffset, uint32& StartIndexInOut)
	{
		FArray3i VertexIndices;
		FArray3i NormalIndices;
		FArray3i TexCoordIndices;

		NormalIndices[0] = TriangulatedIndexes[StartIndexInOut++];
		NormalIndices[1] = NormalIndices[0];
		NormalIndices[2] = NormalIndices[0];

		TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

		TexCoordIndices[1] = TriangulatedIndexes[StartIndexInOut];
		StartIndexInOut += TexOffset;
		VertexIndices[1] = TriangulatedIndexes[StartIndexInOut++] / 3;

		for (uint32 Index = 2; Index < TriCount; ++Index)
		{
			TexCoordIndices[2] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[2] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);

			++Index;
			if (Index == TriCount)
			{
				break;
			}

			Swap(VertexIndices[1], VertexIndices[2]);
			Swap(TexCoordIndices[1], TexCoordIndices[2]);

			TexCoordIndices[0] = TriangulatedIndexes[StartIndexInOut];
			StartIndexInOut += TexOffset;
			VertexIndices[0] = TriangulatedIndexes[StartIndexInOut++] / 3;

			AddTriangle(VertexIndices, NormalIndices, TexCoordIndices);

			Swap(VertexIndices[0], VertexIndices[1]);
			Swap(TexCoordIndices[0], TexCoordIndices[1]);
		}
	}

	bool FFaceTriangleCollector::CollectTriangles(const A3DTessFaceData& FaceTessData, const A3DTopoFace* TopoFace, int32 InGroupID, double TextureUnit, FFaceTriangleCollector::FCollectionResult& Result)
	{
		FaceNormals.Empty(FaceNormals.Num());
		FaceTexCoords.Empty(FaceTexCoords.Num());
		FaceTriangles.Empty(FaceTriangles.Num());
		UVScale = FVector2d::UnitVector;

		GroupID = InGroupID;
		MaterialID = 0;
		if (FaceTessData.m_uiStyleIndexesSize != 0)
		{
			// Store the StyleIndex on the MaterialName. It will be processed after tessellation
			MaterialID = FaceTessData.m_puiStyleIndexes[0];
		}

		// #cad_import: Where TextureUnit should come from?
		UVScale = GetUVScale(TopoFace, TextureUnit);

		LastVertexIndex = 0;
		uint32 FaceSetIndex = 0;
		bool bMustProcess = true;

		uint32 LastTriangulatedIndex = FaceTessData.m_uiStartTriangulated;
		uint32 UsedEntitiesFlags = FaceTessData.m_usUsedEntitiesFlags;
		LastVertexIndex = 0;

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangle)
		{
			AddFaceTriangle(FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], LastTriangulatedIndex);
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleOneNormal)
		{
			AddFaceTriangleWithUniqueNormal(FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], LastTriangulatedIndex);
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleTextured)
		{
			AddFaceTriangleWithTexture(FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize, LastTriangulatedIndex);
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleOneNormalTextured)
		{
			AddFaceTriangleWithUniqueNormalAndTexture(FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize, LastTriangulatedIndex);
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFan)
		{
			uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
			for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
			{
				uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				AddFaceTriangleFan(VertexCount, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFanOneNormal)
		{
			uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
			for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
			{
				ensure((FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalSingle) != 0);

				uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				AddFaceTriangleFanWithUniqueNormal(VertexCount, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFanTextured)
		{
			uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
			for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
			{
				uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				AddFaceTriangleFanWithTexture(VertexCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFanOneNormalTextured)
		{
			uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
			for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
			{
				ensure((FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalSingle) != 0);

				uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				AddFaceTriangleFanWithUniqueNormalAndTexture(VertexCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripe)
		{
			A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
			for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
			{
				A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				AddFaceTriangleStripe(PointCount, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeOneNormal)
		{
			A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
			for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
			{
				bool bIsOneNormal = (FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalSingle) != 0;
				A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;

				// Is there only one normal for the entire stripe?
				if (bIsOneNormal == false)
				{
					AddFaceTriangleStripe(PointCount, LastTriangulatedIndex);
					continue;
				}

				AddFaceTriangleStripeWithUniqueNormal(PointCount, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeTextured)
		{
			A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
			for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
			{
				A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				AddFaceTriangleStripeWithTexture(PointCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeOneNormalTextured)
		{
			A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
			for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
			{
				A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				AddFaceTriangleStripeWithUniqueNormalAndTexture(PointCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTriangulatedIndex);
			}
			bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
		}

		ensure(!bMustProcess);

		Result.Normals = MoveTemp(FaceNormals);
		Result.TexCoords = MoveTemp(FaceTexCoords);
		Result.FaceTriangles = MoveTemp(FaceTriangles);

		return true;
	}

	class FRepresentationConverter
	{
	public:
		FRepresentationConverter(const A3DRiRepresentationItem* RepresentationItem, double InTextureUnit, MeshUtilities::FMeshWrapperAbstract& InMeshWrapper)
			: MeshWrapper(InMeshWrapper)
			, TextureUnit(InTextureUnit)
		{
			TechSoft::TUniqueObject<A3DRiRepresentationItemData> RepresentationItemData(RepresentationItem);
			if (!RepresentationItemData.IsValid())
			{
				return;
			}

			A3DEEntityType Type;
			A3DEntityGetType(RepresentationItemData->m_pTessBase, &Type);
			if (Type != kA3DTypeTess3D)
			{
				return;
			}

			TessellationBase = RepresentationItemData->m_pTessBase;

			if (TessellationBase)
			{
				A3DEntityGetType(RepresentationItem, &Type);
				if (Type == kA3DTypeRiBrepModel)
				{
					TechSoft::TUniqueObject<A3DRiBrepModelData> BRepModelData(RepresentationItem);
					if (BRepModelData.IsValid())
					{
						BRepData = BRepModelData->m_pBrepData;
					}
				}
			}
		}

		bool IsValid() const { return TessellationBase != nullptr; }

		bool Convert();

	private:
		struct FTriangleContext
		{
			int32 GroupID = 0;
			uint32 MaterialID = 0;
			FVector2f UVScale;
		};

		bool CollectBRepFaces(TArray<A3DTopoFace*>& BRepFaces, int32& TriangleCount);
		int32 GetTriangleCount(const A3DTessFaceData& FaceTessData);
		bool ParseTriangles(const TArray<A3DTopoFace*>& BRepFaces);

	private:
		MeshUtilities::FMeshWrapperAbstract& MeshWrapper;
		const A3DTess3D* TessellationBase = nullptr;
		const A3DTopoBrepData* BRepData = nullptr;
		double TextureUnit = 1.;
	};

	bool FRepresentationConverter::CollectBRepFaces(TArray<A3DTopoFace*>& BRepFaces, int32& TriangleCount)
	{
		BRepFaces.Empty();
		TriangleCount = 0;

		if (BRepData == nullptr)
		{
			return false;
		}

		// #cad_import: Check what this is all about
		//TechSoft::TUniqueObject<A3DTopoBodyData> TopoBodyData(BRepData);
		//if (TopoBodyData.IsValid())
		//{
		//	if (TopoBodyData->m_pContext)
		//	{
		//		TechSoft::TUniqueObject<A3DTopoContextData> TopoContextData(TopoBodyData->m_pContext);
		//		if (TopoContextData.IsValid())
		//		{
		//			if (TopoContextData->m_bHaveScale)
		//			{
		//				TextureUnit *= TopoContextData->m_dScale;
		//			}
		//		}
		//	}
		//}

		TechSoft::TUniqueObject<A3DTopoBrepDataData> TopoBrepData(BRepData);
		if (!TopoBrepData.IsValid())
		{
			return false;
		}

		for (A3DUns32 Index = 0; Index < TopoBrepData->m_uiConnexSize; ++Index)
		{
			TechSoft::TUniqueObject<A3DTopoConnexData> TopoConnexData(TopoBrepData->m_ppConnexes[Index]);
			if (TopoConnexData.IsValid())
			{
				for (A3DUns32 Sndex = 0; Sndex < TopoConnexData->m_uiShellSize; ++Sndex)
				{
					TechSoft::TUniqueObject<A3DTopoShellData> ShellData(TopoConnexData->m_ppShells[Sndex]);
					if (ShellData.IsValid())
					{
						for (A3DUns32 Fndex = 0; Fndex < ShellData->m_uiFaceSize; ++Fndex)
						{
							BRepFaces.Add(ShellData->m_ppFaces[Fndex]);
						}
					}
				}
			}
		}

		TechSoft::TUniqueObject<A3DTess3DData> TessellationData(TessellationBase);

		if (!TessellationData.IsValid() || TessellationData->m_uiFaceTessSize == 0)
		{
			return false;
		}

		for (A3DUns32 Index = 0; Index < TessellationData->m_uiFaceTessSize; ++Index)
		{
			TriangleCount += GetTriangleCount(TessellationData->m_psFaceTessData[Index]);
		}

		return TriangleCount > 0;
	}

	int32 FRepresentationConverter::GetTriangleCount(const A3DTessFaceData& FaceTessData)
	{
		constexpr int32 TessellationFaceDataWithTriangle   = 0x2222;
		constexpr int32 TessellationFaceDataWithFan        = 0x4444;
		constexpr int32 TessellationFaceDataWithStripe     = 0x8888;
		constexpr int32 TessellationFaceDataWithOneNormal  = 0xE0E0;

		uint32 UsedEntitiesFlags = FaceTessData.m_usUsedEntitiesFlags;

		int32 TriangleCount = 0;
		uint32 FaceSetIndex = 0;
		
		if (UsedEntitiesFlags & TessellationFaceDataWithTriangle)
		{
			TriangleCount += FaceTessData.m_puiSizesTriangulated[FaceSetIndex];
			FaceSetIndex++;
		}

		if (FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex)
		{
			if (UsedEntitiesFlags & TessellationFaceDataWithFan)
			{
				uint32 LastFanIndex = 1 + FaceSetIndex + FaceTessData.m_puiSizesTriangulated[FaceSetIndex];
				FaceSetIndex++;
				for (; FaceSetIndex < LastFanIndex; FaceSetIndex++)
				{
					int32 FanSize = (int32)(FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalMask);
					TriangleCount += (FanSize - 2);
				}
			}
		}

		if (FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex)
		{
			FaceSetIndex++;
			for (; FaceSetIndex < FaceTessData.m_uiSizesTriangulatedSize; FaceSetIndex++)
			{
				int32 StripeSize = (int32)(FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalMask);
				TriangleCount += (StripeSize - 2);
			}
		}

		return TriangleCount;
	}

	bool FRepresentationConverter::ParseTriangles(const TArray<A3DTopoFace*>& BRepFaces)
	{
		TechSoft::TUniqueObject<A3DTess3DData> TessellationData(TessellationBase);

		ensure((TessellationData->m_uiNormalSize % 3) == 0 && (TessellationData->m_uiTextureCoordSize % 2) == 0);
		TArrayView<FVector> TriangleNormals((FVector*)TessellationData->m_pdNormals, TessellationData->m_uiNormalSize / 3);
		TArrayView<FVector2d> TriangleTexCoords((FVector2d*)TessellationData->m_pdTextureCoords, TessellationData->m_uiTextureCoordSize / 2);

		FFaceTriangleCollector TriangleCollector(TessellationData->m_puiTriangulatedIndexes, TriangleNormals, TriangleTexCoords);
		FFaceTriangleCollector::FCollectionResult CollectionResult;

		for (A3DUns32 Index = 0; Index < TessellationData->m_uiFaceTessSize; ++Index)
		{
			const A3DTessFaceData& FaceTessData = TessellationData->m_psFaceTessData[Index];

			if (FaceTessData.m_uiSizesTriangulatedSize == 0)
			{
				continue;
			}

			const A3DTopoFace* TopoFace = BRepFaces.IsValidIndex(Index) ? BRepFaces[Index] : nullptr;

			TechSoft::TUniqueObject<A3DRootBaseData> RootBaseData(TopoFace);
			int32 GroupID = RootBaseData.IsValid() ? (int32)RootBaseData->m_uiPersistentId : Index;

			if (MeshWrapper.IsFaceGroupValid(GroupID))
			{
				TriangleCollector.CollectTriangles(FaceTessData, TopoFace, GroupID, TextureUnit, CollectionResult);

				MeshWrapper.StartFaceTriangles(0, CollectionResult.Normals, CollectionResult.TexCoords);
				MeshWrapper.AddFaceTriangles(CollectionResult.FaceTriangles);
				MeshWrapper.EndFaceTriangles();
			}
		}

		return true;
	}

	bool FRepresentationConverter::Convert()
	{
		using namespace UE::CADKernel;

		TechSoft::TUniqueObject<A3DTessBaseData> TessellationBaseData(TessellationBase);

		if (!TessellationBaseData.IsValid() || TessellationBaseData->m_uiCoordSize == 0)
		{
			return false;
		}

		TArray<FVector> Vertices;
		const int32 VertexCount = TessellationBaseData->m_uiCoordSize / 3;
		
		Vertices.Reserve(VertexCount);

		double* Coordinates = TessellationBaseData->m_pdCoords;

		for (int32 Index = 0; Index < VertexCount; ++Index, Coordinates += 3)
		{
			Vertices.Emplace(Coordinates[0], Coordinates[1], Coordinates[2]);
		}

		if (!MeshWrapper.AddNewVertices(MoveTemp(Vertices)))
		{
			return false;
		}

		int32 TriangleCount;
		TArray<A3DTopoFace*> BRepFaces;
		CollectBRepFaces(BRepFaces, TriangleCount);
		ParseTriangles(BRepFaces);

		return true;
	}

	bool AddRepresentation(A3DRiRepresentationItem* RepresentationItem, double ModelUnitToMeter, MeshUtilities::FMeshWrapperAbstract& MeshWrapper)
	{
		FRepresentationConverter Converter(RepresentationItem, ModelUnitToMeter, MeshWrapper);

		return Converter.IsValid() ? Converter.Convert() : false;
	}
}
#endif