// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelEngine.h"

#ifdef WITH_HOOPS
#include "TechSoftUniqueObjectImpl.h"

#include "Geo/Curves/Curve.h"
#include "Geo/Curves/NURBSCurveData.h"
#include "Geo/Surfaces/NurbsSurfaceData.h"
#include "Geo/Surfaces/Surface.h"

#include "Topo/Body.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"

#include "Topo/Model.h"

namespace UE::CADKernel::TechSoftUtilities
{
	
class FUVReparameterization
{
private:
	double Scale[2] = { 1., 1. };
	double Offset[2] = { 0., 0. };
	bool bSwapUV = false;
	bool bNeedApply = false;
	bool bNeedSwapOrientation = false;

public:

	FUVReparameterization()
	{
	}

	void SetCoef(const double InUScale, const double InUOffset, const double InVScale, const double InVOffset)
	{
		Scale[EIso::IsoU] = InUScale;
		Scale[EIso::IsoV] = InVScale;
		Offset[EIso::IsoU] = InUOffset;
		Offset[EIso::IsoV] = InVOffset;
		SetNeedApply();
	}

	bool GetNeedApply() const
	{
		return bNeedApply;
	}

	bool GetSwapUV() const
	{
		return bSwapUV;
	}

	bool GetNeedSwapOrientation() const
	{
		return bNeedSwapOrientation != bSwapUV;
	}

	void SetNeedSwapOrientation()
	{
		bNeedSwapOrientation = true;
	}

	void SetNeedApply()
	{
		if (!FMath::IsNearlyEqual(Scale[EIso::IsoU], 1.) || !FMath::IsNearlyEqual(Scale[EIso::IsoV], 1.) || !FMath::IsNearlyEqual(Offset[EIso::IsoU], 0.) || !FMath::IsNearlyEqual(Offset[EIso::IsoV], 0.))
		{
			bNeedApply = true;
		}
		else
		{
			bNeedApply = false;
		}
	}

	void ScaleUVTransform(double InUScale, double InVScale)
	{
		if (bSwapUV)
		{
			Swap(InUScale, InVScale);
		}
		Scale[EIso::IsoU] *= InUScale;
		Scale[EIso::IsoV] *= InVScale;
		Offset[EIso::IsoU] *= InUScale;
		Offset[EIso::IsoV] *= InVScale;
		SetNeedApply();
	}

	void Process(TArray<FVector>& Poles) const
	{
		using namespace UE::CADKernel;

		if (bNeedApply)
		{
			for (FVector& Point : Poles)
			{
				Apply(Point);
			}
		}
		if (bSwapUV)
		{
			for (FVector& Point : Poles)
			{
				GetSwapUV(Point);
			}
		}
	}

	void AddUVTransform(A3DUVParameterizationData& Transform)
	{
		bSwapUV = (bool) Transform.m_bSwapUV;

		Scale[0] = Scale[0] * Transform.m_dUCoeffA;
		Scale[1] = Scale[1] * Transform.m_dVCoeffA;
		Offset[0] = Offset[0] * Transform.m_dUCoeffA + Transform.m_dUCoeffB;
		Offset[1] = Offset[1] * Transform.m_dVCoeffA + Transform.m_dVCoeffB;
		SetNeedApply();
	}

	void Apply(FVector2d& Point) const
	{
		using namespace UE::CADKernel;

		Point.X = Scale[EIso::IsoU] * Point.X + Offset[EIso::IsoU];
		Point.Y = Scale[EIso::IsoV] * Point.Y + Offset[EIso::IsoV];
	}

private:
	void Apply(FVector& Point) const
	{
		using namespace UE::CADKernel;

		Point.X = Scale[EIso::IsoU] * Point.X + Offset[EIso::IsoU];
		Point.Y = Scale[EIso::IsoV] * Point.Y + Offset[EIso::IsoV];
	}

	void GetSwapUV(FVector& Point) const
	{
		Swap(Point.X, Point.Y);
	}

};

template<typename... InArgTypes>
A3DStatus GetCurveAsNurbs(const A3DCrvBase* A3DCurve, A3DCrvNurbsData* DataPtr, InArgTypes&&... Args)
{
	auto [Tolerance, bUseSameParameterization] = TTuple<InArgTypes...>(Args...);
	return A3DCrvBaseGetAsNurbs(A3DCurve, Tolerance, bUseSameParameterization, DataPtr);
};

template<typename... InArgTypes>
A3DStatus GetSurfaceAsNurbs(const A3DSurfBase* A3DSurface, A3DSurfNurbsData* DataPtr, InArgTypes&&... Args)
{
	auto [Tolerance, bUseSameParameterization] = TTuple<InArgTypes...>(Args...);
	return A3DSurfBaseWithDomainGetAsNurbs(A3DSurface, nullptr, Tolerance, bUseSameParameterization, DataPtr);
};

FMatrixH CreateCoordinateSystem(const A3DMiscCartesianTransformationData& Transformation, double UnitScale = 1.0)
{
	using namespace UE::CADKernel;

	FVector Origin(Transformation.m_sOrigin.m_dX, Transformation.m_sOrigin.m_dY, Transformation.m_sOrigin.m_dZ);
	FVector Ox(Transformation.m_sXVector.m_dX, Transformation.m_sXVector.m_dY, Transformation.m_sXVector.m_dZ);
	FVector Oy(Transformation.m_sYVector.m_dX, Transformation.m_sYVector.m_dY, Transformation.m_sYVector.m_dZ);

	Ox.Normalize();
	Oy.Normalize();

	if (!FMath::IsNearlyEqual(UnitScale, 1.))
	{
		Origin *= UnitScale;
	}
	FVector Oz = Ox ^ Oy;

	FMatrixH Matrix = FMatrixH(Origin, Ox, Oy, Oz);

	if (!FMath::IsNearlyEqual(Transformation.m_sScale.m_dX, 1.) || !FMath::IsNearlyEqual(Transformation.m_sScale.m_dY, 1.) || !FMath::IsNearlyEqual(Transformation.m_sScale.m_dZ, 1.))
	{
		FMatrixH Scale = FMatrixH::MakeScaleMatrix(Transformation.m_sScale.m_dX, Transformation.m_sScale.m_dY, Transformation.m_sScale.m_dZ);
		Matrix *= Scale;
	}
	return Matrix;
}

void FillInt32Array(const int32 Count, const A3DInt32* Values, TArray<int32>& OutInt32Array)
{
	OutInt32Array.Reserve(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutInt32Array.Add(Values[Index]);
	}
};

void FillDoubleArray(const int32 Count, const double* Values, TArray<double>& OutDoubleArray)
{
	OutDoubleArray.Reserve(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutDoubleArray.Add(Values[Index]);
	}
};

void FillDoubleArray(const int32 UCount, const int32 VCount, const double* Values, TArray<double>& OutDoubleArray)
{
	OutDoubleArray.SetNum(UCount * VCount);
	for (int32 Undex = 0, ValueIndex = 0; Undex < UCount; ++Undex)
	{
		int32 Index = Undex;
		for (int32 Vndex = 0; Vndex < VCount; ++Vndex, Index += UCount, ++ValueIndex)
		{
			OutDoubleArray[Index] = Values[ValueIndex];
		}
	}
}
void FillPointArray(const int32 Count, const A3DVector3dData* Points, TArray<FVector>& OutPointsArray, double UnitScale = 1.0)
{
	using namespace UE::CADKernel;

	OutPointsArray.Reserve(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutPointsArray.Emplace(Points[Index].m_dX, Points[Index].m_dY, Points[Index].m_dZ);
	}

	if (!FMath::IsNearlyEqual(UnitScale, 1.))
	{
		for (FVector& Point : OutPointsArray)
		{
			Point *= UnitScale;
		}
	}
};

void FillPointArray(const int32 UCount, const int32 VCount, const A3DVector3dData* Points, TArray<FVector>& OutPointsArray, double UnitScale = 1.0)
{
	using namespace UE::CADKernel;

	OutPointsArray.SetNum(UCount * VCount);

	for (int32 Undex = 0, PointIndex = 0; Undex < UCount; ++Undex)
	{
		int32 Index = Undex;
		for (int32 Vndex = 0; Vndex < VCount; ++Vndex, Index += UCount, ++PointIndex)
		{
			OutPointsArray[Index].Set(Points[PointIndex].m_dX, Points[PointIndex].m_dY, Points[PointIndex].m_dZ);
		}
	}

	if (!FMath::IsNearlyEqual(UnitScale, 1.))
	{
		for (FVector& Point : OutPointsArray)
		{
			Point *= UnitScale;
		}
	}
};

FSurfacicBoundary GetSurfacicBoundary(A3DDomainData& Domain, const FUVReparameterization& UVReparameterization)
{
	using namespace UE::CADKernel;


	FVector2d Min(Domain.m_sMin.m_dX, Domain.m_sMin.m_dY);
	FVector2d Max(Domain.m_sMax.m_dX, Domain.m_sMax.m_dY);

	if (UVReparameterization.GetNeedApply())
	{
		UVReparameterization.Apply(Min);
		UVReparameterization.Apply(Max);
	}

	EIso UIndex = UVReparameterization.GetSwapUV() ? EIso::IsoV : EIso::IsoU;
	EIso VIndex = UVReparameterization.GetSwapUV() ? EIso::IsoU : EIso::IsoV;

	FSurfacicBoundary Boundary;
	Boundary[UIndex].Min = FMath::Min(Min.X, Max.X);
	Boundary[UIndex].Max = FMath::Max(Min.X, Max.X);
	Boundary[VIndex].Min = FMath::Min(Min.Y, Max.Y);
	Boundary[VIndex].Max = FMath::Max(Min.Y, Max.Y);

	return Boundary;
}

FLinearBoundary GetLinearBoundary(A3DIntervalData& A3DDomain)
{
	FLinearBoundary Domain(A3DDomain.m_dMin, A3DDomain.m_dMax);
	return Domain;
}

FLinearBoundary GetLinearBoundary(const A3DCrvBase* A3DCurve)
{
	TechSoft::TUniqueObject<A3DIntervalData> A3DDomain(A3DCurve);
	FLinearBoundary Domain(A3DDomain->m_dMin, A3DDomain->m_dMax);
	return Domain;
}



// inspired by FTechSoftBridge
class FRepresentationToModel
{
	constexpr static bool bUseCurveAsNurbs = true;
	constexpr static bool bUseSurfaceAsNurbs = true;


	const double GeometricTolerance;
	const double EdgeLengthTolerance;
	const double SquareJoiningVertexTolerance;

	TMap<const A3DEntity*, TSharedPtr<FBody>> TechSoftToCADKernel;
	TMap<FBody*, const A3DEntity*> CADKernelToTechSoft;
	TMap<const A3DTopoCoEdge*, TSharedPtr<FTopologicalEdge>> A3DEdgeToEdge;

	double BodyScale = 1;

	bool bConvertionFailed = false;


public:
	FRepresentationToModel(double InGeometricTolerance)
		: GeometricTolerance(InGeometricTolerance)
		, EdgeLengthTolerance(GeometricTolerance * 2.)
		, SquareJoiningVertexTolerance(FMath::Square(GeometricTolerance) * 2)
	{
	}

	TSharedPtr<FBody> Convert(A3DRiBrepModel* A3DBRepModel, const FString* InName, double Unit)
	{
		//UE::CADKernel working unit is mm
		BodyScale = Unit * 10.;

		FString Name;
		uint32 MaterialId;
		GetEntityInfo(A3DBRepModel, Name, MaterialId);
		if (InName)
		{
			Name = *InName;
		}

		if (TSharedPtr<FBody>* BodyPtr = TechSoftToCADKernel.Find(A3DBRepModel))
		{
			return (*BodyPtr)->IsDeleted() ? nullptr : *BodyPtr;
		}

		TSharedRef<FBody> Body = FEntity::MakeShared<FBody>();

		Body->SetName(*Name);
		Body->SetDisplayData(MaterialId, MaterialId);

		TechSoft::TUniqueObject<A3DRiBrepModelData> BRepModelData(A3DBRepModel);
		if (BRepModelData.IsValid())
		{
			TraverseBrepData(BRepModelData->m_pBrepData, Body);
		}

		if (Body->FaceCount() == 0 || bConvertionFailed)
		{
			Body->Delete();
			return nullptr;
		}

		return Body;
	}

private:

	static void GetEntityInfo(const A3DEntity* Entity, FString& OutName, uint32& OutStyleIndex)
	{
		using namespace UE::CADKernel;
		TMap<FString, FString> MetaData;
		FString UniqueID;
		FString Label;
		FTechSoftLibrary::ParseRootBaseData(Entity, MetaData, UniqueID, Label);

		// // Also was part of FTechSoftFileParser::ExtractMetaData
		// // but parses "Graphics" entity to extract object visibility and Material/Color
		FTechSoftLibrary::FGraphicsProperties GraphicsProperties;
		FTechSoftLibrary::ExtractGraphicsProperties(Entity, GraphicsProperties);

		FString* Name = MetaData.Find(TEXT("Name"));
		if (Name != nullptr)
		{
			OutName = *Name;
		}
		OutStyleIndex = GraphicsProperties.StyleIndex;
	};

	void TraverseBrepData(const A3DTopoBrepData* A3DBrepData, TSharedRef<FBody>& Body)
	{
		{
			TechSoft::TUniqueObject<A3DTopoBodyData> TopoBodyData(A3DBrepData);
			if (TopoBodyData.IsValid())
			{
				if (TopoBodyData->m_pContext)
				{
					TechSoft::TUniqueObject<A3DTopoContextData> TopoContextData(TopoBodyData->m_pContext);
					if (TopoContextData.IsValid())
					{
						if (TopoContextData->m_bHaveScale)
						{
							BodyScale *= TopoContextData->m_dScale;
						}
					}
				}
			}
		}

		TechSoft::TUniqueObject<A3DTopoBrepDataData> TopoBrepData(A3DBrepData);
		if (TopoBrepData.IsValid())
		{
			for (A3DUns32 Index = 0; Index < TopoBrepData->m_uiConnexSize; ++Index)
			{
				TraverseConnex(TopoBrepData->m_ppConnexes[Index], Body);
				if (bConvertionFailed)
				{
					return;
				}
			}
		}
	}

	void TraverseConnex(const A3DTopoConnex* A3DTopoConnex, TSharedRef<FBody>& Body)
	{
		TechSoft::TUniqueObject<A3DTopoConnexData> TopoConnexData(A3DTopoConnex);
		if (TopoConnexData.IsValid())
		{
			for (A3DUns32 Index = 0; Index < TopoConnexData->m_uiShellSize; ++Index)
			{
				TraverseShell(TopoConnexData->m_ppShells[Index], Body);
				if (bConvertionFailed)
				{
					return;
				}
			}
		}
	}

	void TraverseShell(const A3DTopoShell* A3DShell, TSharedRef<FBody>& Body)
	{
		FString Name;
		uint32 MaterialId;
		GetEntityInfo(A3DShell, Name, MaterialId);

		TSharedRef<FShell> Shell = FEntity::MakeShared<FShell>();
		Body->AddShell(Shell);

		Shell->SetDisplayData(*Body);

		Shell->SetName(*Name);
		Shell->SetDisplayData(MaterialId, MaterialId);

		TechSoft::TUniqueObject<A3DTopoShellData> ShellData(A3DShell);

		if (ShellData.IsValid())
		{
			A3DEdgeToEdge.Empty();
			for (A3DUns32 Index = 0; Index < ShellData->m_uiFaceSize; ++Index)
			{
				AddFace(ShellData->m_ppFaces[Index], ShellData->m_pucOrientationWithShell[Index] == 1 ? Front : Back, Shell, Index);
				if (bConvertionFailed)
				{
					return;
				}
			}
		}
	}

	TSharedPtr<FCurve> AddCurve(const A3DCrvBase* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		TSharedPtr<FCurve> Curve = TSharedPtr<FCurve>();
		A3DEEntityType eType;
		A3DInt32 Ret = A3DEntityGetType(A3DCurve, &eType);
		if (Ret == A3D_SUCCESS)
		{
			switch (eType)
			{
			case kA3DTypeCrvNurbs:
				Curve = AddCurveNurbs(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvLine:
				Curve = AddCurveLine(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvCircle:
				Curve = AddCurveCircle(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvEllipse:
				Curve = AddCurveEllipse(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvParabola:
				Curve = AddCurveParabola(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvHyperbola:
				Curve = AddCurveHyperbola(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvHelix:
				Curve = AddCurveHelix(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvPolyLine:
				Curve = AddCurvePolyLine(A3DCurve, UVReparameterization);
				break;
			case kA3DTypeCrvComposite:
				Curve = AddCurveComposite(A3DCurve, UVReparameterization);
				break;
			default:
				Curve = AddCurveAsNurbs(A3DCurve, UVReparameterization);
				break;
			}
		}

		FLinearBoundary Boundary = GetLinearBoundary(A3DCurve);

		return Curve;
	}

	TSharedPtr<FCurve> AddCurveLine(const A3DCrvLine* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}


		TechSoft::TUniqueObject<A3DCrvLineData> CrvLineData(A3DCurve);
		if (!CrvLineData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvLineData->m_bIs2D;
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveCircle(const A3DCrvCircle* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvCircleData> CrvCircleData(A3DCurve);
		if (!CrvCircleData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvCircleData->m_bIs2D;
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveEllipse(const A3DCrvEllipse* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvEllipseData> CrvEllipseData(A3DCurve);
		if (!CrvEllipseData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvEllipseData->m_bIs2D;
		ensure(!bIs2D);
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveParabola(const A3DCrvParabola* A3DCurve, const FUVReparameterization& UVReparameterization)
	{

		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvParabolaData> CrvParabolaData(A3DCurve);
		if (!CrvParabolaData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvParabolaData->m_bIs2D;
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveHyperbola(const A3DCrvHyperbola* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvHyperbolaData> CrvHyperbolaData(A3DCurve);
		if (!CrvHyperbolaData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvHyperbolaData->m_bIs2D;
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveHelix(const A3DCrvHelix* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvHelixData> CrvHelixData(A3DCurve);
		if (!CrvHelixData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvHelixData->m_bIs2D;

		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurvePolyLine(const A3DCrvPolyLine* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvPolyLineData> CrvPolyLineData(A3DCurve);
		if (!CrvPolyLineData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvPolyLineData->m_bIs2D;
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveComposite(const A3DCrvComposite* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		if (bUseCurveAsNurbs)
		{
			return AddCurveAsNurbs(A3DCurve, UVReparameterization);
		}

		TechSoft::TUniqueObject<A3DCrvCompositeData> CrvCompositeData(A3DCurve);
		if (!CrvCompositeData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		bool bIs2D = (bool)CrvCompositeData->m_bIs2D;
		// Todo
		return TSharedPtr<FCurve>();
	}

	TSharedPtr<FCurve> AddCurveNurbsFromData(A3DCrvNurbsData& A3DNurbs, const FUVReparameterization& UVReparameterization)
	{
		FNurbsCurveData Nurbs;
		Nurbs.Dimension = A3DNurbs.m_bIs2D ? 2 : 3;
		Nurbs.bIsRational = (bool)A3DNurbs.m_bRational;
		Nurbs.Degree = A3DNurbs.m_uiDegree;

		FillPointArray(A3DNurbs.m_uiCtrlSize, A3DNurbs.m_pCtrlPts, Nurbs.Poles);
		if (Nurbs.Dimension == 2)
		{
			UVReparameterization.Process(Nurbs.Poles);
		}

		FillDoubleArray(A3DNurbs.m_uiKnotSize, A3DNurbs.m_pdKnots, Nurbs.NodalVector);
		if (Nurbs.bIsRational)
		{
			FillDoubleArray(A3DNurbs.m_uiCtrlSize, A3DNurbs.m_pdWeights, Nurbs.Weights);
		}

		A3DCrvNurbsGet(NULL, &A3DNurbs);

		return FCurve::MakeNurbsCurve(Nurbs);
	}

	TSharedPtr<FCurve> AddCurveNurbs(const A3DCrvNurbs* A3DNurbs, const FUVReparameterization& UVReparameterization)
	{
		TechSoft::TUniqueObject<A3DCrvNurbsData> CrvNurbsData(A3DNurbs);
		if (!CrvNurbsData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		return AddCurveNurbsFromData(*CrvNurbsData, UVReparameterization);
	}

	TSharedPtr<FCurve> AddCurveAsNurbs(const A3DCrvBase* A3DCurve, const FUVReparameterization& UVReparameterization)
	{
		TechSoft::TUniqueObject<A3DCrvNurbsData> NurbsData;

		// todo: propagate value from user options set in tessellation params
		A3DDouble Tolerance = 1e-3;
		A3DBool bUseSameParameterization = true;
		NurbsData.FillWith(&GetCurveAsNurbs, A3DCurve, Tolerance, bUseSameParameterization);


		if (!NurbsData.IsValid())
		{
			return TSharedPtr<FCurve>();
		}

		return AddCurveNurbsFromData(*NurbsData, UVReparameterization);
	}

	TSharedPtr<FTopologicalEdge> AddEdge(const A3DTopoCoEdge* A3DCoedge, const TSharedRef<FSurface>& Surface, const FUVReparameterization& UVReparameterization, EOrientation& OutOrientation)
	{
		TechSoft::TUniqueObject<A3DTopoCoEdgeData> CoEdgeData(A3DCoedge);
		if (!CoEdgeData.IsValid())
		{
			return TSharedPtr<FTopologicalEdge>();
		}

		if (CoEdgeData->m_pUVCurve == nullptr)
		{
			bConvertionFailed = true;
			// todo:
			// FailureReason = EFailureReason::Curve3D;
			return TSharedPtr<FTopologicalEdge>();
		}

		TSharedPtr<FCurve> Curve = AddCurve(CoEdgeData->m_pUVCurve, UVReparameterization);
		if (!Curve.IsValid())
		{
			return TSharedPtr<FTopologicalEdge>();
		}

		TSharedRef<FRestrictionCurve> RestrictionCurve = FEntity::MakeShared<FRestrictionCurve>(Surface, Curve.ToSharedRef());

		TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(RestrictionCurve);
		if (!Edge.IsValid())
		{
			return TSharedPtr<FTopologicalEdge>();
		}

		A3DEdgeToEdge.Emplace(A3DCoedge, Edge);

		OutOrientation = CoEdgeData->m_ucOrientationUVWithLoop > 0 ? EOrientation::Front : EOrientation::Back;

		return Edge;
	}

	TSharedPtr<FTopologicalLoop> AddLoop(const A3DTopoLoop* A3DLoop, const TSharedRef<FSurface>& Surface, const FUVReparameterization& UVReparameterization, const bool bIsExternalLoop)
	{
		TArray<TSharedPtr<FTopologicalEdge>> Edges;
		TArray<EOrientation> Directions;

		TechSoft::TUniqueObject<A3DTopoLoopData> TopoLoopData(A3DLoop);
		if (!TopoLoopData.IsValid())
		{
			return TSharedPtr<FTopologicalLoop>();
		}

		bool bLoopOrientation = (bool)TopoLoopData->m_ucOrientationWithSurface;
		for (A3DUns32 Index = 0; Index < TopoLoopData->m_uiCoEdgeSize; ++Index)
		{
			EOrientation Orientation;
			TSharedPtr<FTopologicalEdge> Edge = AddEdge(TopoLoopData->m_ppCoEdges[Index], Surface, UVReparameterization, Orientation);
			if (!Edge.IsValid())
			{
				continue;
			}

			Edges.Emplace(Edge);
			Directions.Emplace(Orientation);
		}

		if (Edges.Num() == 0)
		{
			return TSharedPtr<FTopologicalLoop>();
		}

		TSharedPtr<FTopologicalLoop> Loop = FTopologicalLoop::Make(Edges, Directions, bIsExternalLoop, GeometricTolerance);

		// Link the edges of the loop with their neighbors if possible
		for (A3DUns32 Index = 0; Index < TopoLoopData->m_uiCoEdgeSize; ++Index)
		{
			const A3DTopoCoEdge* A3DCoedge = TopoLoopData->m_ppCoEdges[Index];
			TSharedPtr<FTopologicalEdge>* Edge = A3DEdgeToEdge.Find(A3DCoedge);
			if (Edge == nullptr || !Edge->IsValid() || (*Edge)->IsDeleted())
			{
				continue;
			}

			TechSoft::TUniqueObject<A3DTopoCoEdgeData> CoEdgeData(A3DCoedge);
			if (!CoEdgeData.IsValid())
			{
				continue;
			}

			if (CoEdgeData->m_pNeighbor)
			{
				const A3DTopoCoEdge* Neighbor = CoEdgeData->m_pNeighbor;
				while (Neighbor && Neighbor != A3DCoedge)
				{
					TSharedPtr<FTopologicalEdge>* TwinEdge = A3DEdgeToEdge.Find(Neighbor);
					if (TwinEdge != nullptr && TwinEdge->IsValid() && !(*TwinEdge)->IsDeleted())
					{
						(*Edge)->LinkIfCoincident(*TwinEdge->Get(), EdgeLengthTolerance, SquareJoiningVertexTolerance);
					}

					// Next
					TechSoft::TUniqueObject<A3DTopoCoEdgeData> NeighborData(Neighbor);
					if (NeighborData.IsValid())
					{
						Neighbor = NeighborData->m_pNeighbor;
					}
					else
					{
						break;
					}
				}
			}
		}

		return Loop;
	}

	void AddFace(const A3DTopoFace* A3DFace, EOrientation Orientation, TSharedRef<FShell>& Shell, uint32 ShellIndex)
	{
		using namespace UE::CADKernel;

		FString Name;
		uint32 MaterialId;
		GetEntityInfo(A3DFace, Name, MaterialId);

		TechSoft::TUniqueObject<A3DTopoFaceData> TopoFaceData(A3DFace);
		if (!TopoFaceData.IsValid())
		{
			return;
		}

		const A3DSurfBase* A3DSurface = TopoFaceData->m_pSurface;
		FUVReparameterization UVReparameterization;
		TSharedPtr<FSurface> SurfacePtr = AddSurface(A3DSurface, UVReparameterization);
		if (!SurfacePtr.IsValid())
		{
			return;
		}

		if (UVReparameterization.GetNeedSwapOrientation())
		{
			SwapOrientation(Orientation);
		}

		TSharedRef<FSurface> Surface = SurfacePtr.ToSharedRef();
		TSharedRef<FTopologicalFace> Face = FEntity::MakeShared<FTopologicalFace>(Surface);

		TechSoft::TUniqueObject<A3DRootBaseData> RootBaseData(A3DFace);
		Face->SetPatchId(RootBaseData.IsValid() ? (int32)RootBaseData->m_uiPersistentId : ShellIndex);

		if (TopoFaceData->m_bHasTrimDomain)
		{
			FSurfacicBoundary SurfaceBoundary = GetSurfacicBoundary(TopoFaceData->m_sSurfaceDomain, UVReparameterization);
			Surface->TrimBoundaryTo(SurfaceBoundary);
		}

		if (!TopoFaceData->m_uiLoopSize)
		{
			Face->ApplyNaturalLoops();
		}
		else
		{
			TArray<TSharedPtr<FTopologicalLoop>> Loops;

			const uint32 OuterLoopIndex = TopoFaceData->m_uiOuterLoopIndex;

			for (A3DUns32 Index = 0; Index < TopoFaceData->m_uiLoopSize; ++Index)
			{
				const bool bIsExternalLoop = (Index == OuterLoopIndex);
				TSharedPtr<FTopologicalLoop> Loop = AddLoop(TopoFaceData->m_ppLoops[Index], Surface, UVReparameterization, bIsExternalLoop);
				if (!Loop.IsValid())
				{
					continue;
				}

				TArray<FVector2d> LoopSampling;
				Loop->Get2DSampling(LoopSampling);
				FAABB2D Boundary;
				Boundary += LoopSampling;
				Loop->Boundary.Set(Boundary.GetMin(), Boundary.GetMax());

				// Check if the loop is not composed with only degenerated edge
				bool bDegeneratedLoop = true;
				for (const FOrientedEdge& Edge : Loop->GetEdges())
				{
					if (!Edge.Entity->IsDegenerated())
					{
						bDegeneratedLoop = false;
						break;
					}
				}
				if (bDegeneratedLoop)
				{
					continue;
				}

				Loops.Add(Loop);
			}

			if (Loops.Num() > 1)
			{
				// Check external loop
				TSharedPtr<FTopologicalLoop> ExternalLoop;
				FSurfacicBoundary ExternalBoundary;
				ExternalBoundary.Init();
				for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
				{
					// fast but not accurate test to check if the loop is inside an other loop based of the bounding box 
					switch (Loop->Boundary.IsInside(ExternalBoundary, Surface->GetIsoTolerances()))
					{
					case ESituation::Undefined:
					{
						// accurate test to check if the loop is inside an other loop 
						if (!Loop->IsInside(*ExternalLoop))
						{
							ExternalBoundary = Loop->Boundary;
							ExternalLoop = Loop;
						}
						break;
					}

					case ESituation::Outside:
					{
						ExternalBoundary = Loop->Boundary;
						ExternalLoop = Loop;
						break;
					}

					default:
						break;
					}
				}

				if (!ExternalLoop->IsExternal())
				{
					for (TSharedPtr<FTopologicalLoop>& Loop : Loops)
					{
						if (Loop->IsExternal())
						{
							Loop->SetInternal();
							break;
						}
					}
					ExternalLoop->SetExternal();
				}
			}

			if (Loops.Num() == 0)
			{
				Face->SetAsDegenerated();
				Face->Delete();
				return;
			}
			else
			{
				int32 DoubtfulLoopOrientationCount = 0;
				Face->AddLoops(Loops, DoubtfulLoopOrientationCount);
			}
		}

		if (Face->GetLoops().Num() == 0)
		{
			Face->SetAsDegenerated();
			Face->Delete();
			return;
		}

		Face->SetName(*Name);
		Face->SetDisplayData(MaterialId, MaterialId);
		Face->CompleteMetaData();

		Face->SetHostId(ShellIndex);
		Shell->Add(Face, Orientation);
	}

	TSharedPtr<FSurface> AddSurface(const A3DSurfBase* A3DSurface, FUVReparameterization& OutUVReparameterization)
	{
		A3DEEntityType Type;
		int32 Ret = A3DEntityGetType(A3DSurface, &Type);
		if (Ret == A3D_SUCCESS)
		{
			switch (Type)
			{
			case kA3DTypeSurfBlend01:
				return AddBlend01Surface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfBlend02:
				return AddBlend02Surface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfBlend03:
				return AddBlend03Surface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfNurbs:
				return AddNurbsSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfCone:
				return AddConeSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfCylinder:
				return AddCylinderSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfCylindrical:
				return AddCylindricalSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfOffset:
				return AddOffsetSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfPipe:
				return AddPipeSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfPlane:
				return AddPlaneSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfRuled:
				return AddRuledSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfSphere:
				return AddSphereSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfRevolution:
				return AddRevolutionSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfExtrusion:
				return AddExtrusionSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfFromCurves:
				return AddSurfaceFromCurves(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfTorus:
				return AddTorusSurface(A3DSurface, OutUVReparameterization);

			case kA3DTypeSurfTransform:
				return AddTransformSurface(A3DSurface, OutUVReparameterization);

			default:
				return AddSurfaceAsNurbs(A3DSurface, OutUVReparameterization);
			}
		}
		else if (Ret == A3D_NOT_IMPLEMENTED)
		{
			return AddSurfaceAsNurbs(A3DSurface, OutUVReparameterization);
		}
		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddConeSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfConeData> A3DConeData(Surface);
		if (!A3DConeData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		OutUVReparameterization.AddUVTransform(A3DConeData->m_sParam);
		OutUVReparameterization.ScaleUVTransform(1, BodyScale);
		if (A3DConeData->m_dSemiAngle < 0)
		{
			OutUVReparameterization.SetNeedSwapOrientation();
		}

		FMatrixH CoordinateSystem = CreateCoordinateSystem(A3DConeData->m_sTrsf, BodyScale);
		FSurfacicBoundary Boundary = GetSurfacicBoundary(A3DConeData->m_sParam.m_sUVDomain, OutUVReparameterization);
		return FSurface::MakeConeSurface(GeometricTolerance, CoordinateSystem, A3DConeData->m_dRadius * BodyScale, A3DConeData->m_dSemiAngle, Boundary);
	}

	TSharedPtr<FSurface> AddCylinderSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfCylinderData> A3DCylinderData(Surface);
		if (!A3DCylinderData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		OutUVReparameterization.AddUVTransform(A3DCylinderData->m_sParam);
		OutUVReparameterization.ScaleUVTransform(1, BodyScale);

		FMatrixH CoordinateSystem = CreateCoordinateSystem(A3DCylinderData->m_sTrsf, BodyScale);
		FSurfacicBoundary Boundary = GetSurfacicBoundary(A3DCylinderData->m_sParam.m_sUVDomain, OutUVReparameterization);
		return FSurface::MakeCylinderSurface(GeometricTolerance, CoordinateSystem, A3DCylinderData->m_dRadius * BodyScale, Boundary);
	}

	TSharedPtr<FSurface> AddLinearTransfoSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}
		// Todo
		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddNurbsSurface(const A3DSurfNurbs* Nurbs, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfNurbsData> A3DNurbsData(Nurbs);
		if (!A3DNurbsData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return AddSurfaceNurbs(*A3DNurbsData, OutUVReparameterization);
	}

	TSharedPtr<FSurface> AddOffsetSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddPlaneSurface(const A3DSurfPlane* Surface, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfPlaneData> A3DPlaneData(Surface);
		if (!A3DPlaneData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		OutUVReparameterization.AddUVTransform(A3DPlaneData->m_sParam);
		OutUVReparameterization.ScaleUVTransform(BodyScale, BodyScale);

		FMatrixH CoordinateSystem = CreateCoordinateSystem(A3DPlaneData->m_sTrsf, BodyScale);
		FSurfacicBoundary Boundary = GetSurfacicBoundary(A3DPlaneData->m_sParam.m_sUVDomain, OutUVReparameterization);
		return FSurface::MakePlaneSurface(GeometricTolerance, CoordinateSystem, Boundary);
	}

	TSharedPtr<FSurface> AddRevolutionSurface(const A3DSurfRevolution* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfRevolutionData> A3DRevolutionData(Surface);
		if (!A3DRevolutionData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddRuledSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfRuledData> A3DRuledData(Surface);
		if (!A3DRuledData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddSphereSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfSphereData> A3DSphereData(Surface);
		if (!A3DSphereData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		OutUVReparameterization.AddUVTransform(A3DSphereData->m_sParam);

		FMatrixH CoordinateSystem = CreateCoordinateSystem(A3DSphereData->m_sTrsf, BodyScale);
		FSurfacicBoundary Boundary = GetSurfacicBoundary(A3DSphereData->m_sParam.m_sUVDomain, OutUVReparameterization);
		return FSurface::MakeSphericalSurface(GeometricTolerance, CoordinateSystem, A3DSphereData->m_dRadius * BodyScale, Boundary);
	}

	TSharedPtr<FSurface> AddTorusSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfTorusData> A3DTorusData(Surface);
		if (!A3DTorusData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		OutUVReparameterization.AddUVTransform(A3DTorusData->m_sParam);
		FMatrixH CoordinateSystem = CreateCoordinateSystem(A3DTorusData->m_sTrsf, BodyScale);
		FSurfacicBoundary Boundary = GetSurfacicBoundary(A3DTorusData->m_sParam.m_sUVDomain, OutUVReparameterization);
		return FSurface::MakeTorusSurface(GeometricTolerance, CoordinateSystem, A3DTorusData->m_dMajorRadius * BodyScale, A3DTorusData->m_dMinorRadius * BodyScale, Boundary);
	}

	TSharedPtr<FSurface> AddBlend01Surface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfBlend01Data> A3DBlend01Data(Surface);
		if (!A3DBlend01Data.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddBlend02Surface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfBlend02Data> A3DBlend02Data(Surface);
		if (!A3DBlend02Data.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddBlend03Surface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddCylindricalSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfCylindricalData> A3DCylindricalData(Surface);
		if (!A3DCylindricalData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddPipeSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfPipeData> A3DPipeData(Surface);
		if (!A3DPipeData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddExtrusionSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfExtrusionData> A3DExtrusionData(Surface);
		if (!A3DExtrusionData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddSurfaceFromCurves(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfFromCurvesData> A3DFromCurvesData(Surface);
		if (!A3DFromCurvesData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddTransformSurface(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		if (bUseSurfaceAsNurbs)
		{
			return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
		}

		TechSoft::TUniqueObject<A3DSurfFromCurvesData> A3DTransformData(Surface);
		if (!A3DTransformData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return TSharedPtr<FSurface>();
	}

	TSharedPtr<FSurface> AddSurfaceNurbs(const A3DSurfNurbsData& A3DNurbsData, FUVReparameterization& OutUVReparameterization)
	{
		using namespace UE::CADKernel;

		FNurbsSurfaceData NurbsData;

		NurbsData.PoleUCount = A3DNurbsData.m_uiUCtrlSize;
		NurbsData.PoleVCount = A3DNurbsData.m_uiVCtrlSize;
		int32 PoleCount = A3DNurbsData.m_uiUCtrlSize * A3DNurbsData.m_uiVCtrlSize;

		NurbsData.UDegree = A3DNurbsData.m_uiUDegree;
		NurbsData.VDegree = A3DNurbsData.m_uiVDegree;

		FillDoubleArray(A3DNurbsData.m_uiUKnotSize, A3DNurbsData.m_pdUKnots, NurbsData.UNodalVector);
		FillDoubleArray(A3DNurbsData.m_uiVKnotSize, A3DNurbsData.m_pdVKnots, NurbsData.VNodalVector);

		TArray<FVector> Poles;
		FillPointArray(NurbsData.PoleUCount, NurbsData.PoleVCount, A3DNurbsData.m_pCtrlPts, NurbsData.Poles);
		if (!FMath::IsNearlyEqual(BodyScale, 1.))
		{
			for (FVector& Point : NurbsData.Poles)
			{
				Point *= BodyScale;
			}
		}

		bool bIsRational = false;
		if (A3DNurbsData.m_pdWeights)
		{
			bIsRational = true;
			FillDoubleArray(NurbsData.PoleUCount, NurbsData.PoleVCount, A3DNurbsData.m_pdWeights, NurbsData.Weights);
		}

		return FSurface::MakeNurbsSurface(GeometricTolerance, NurbsData);
	}

	TSharedPtr<FSurface> AddSurfaceAsNurbs(const A3DSurfBase* Surface, FUVReparameterization& OutUVReparameterization)
	{
		TechSoft::TUniqueObject<A3DSurfNurbsData> A3DNurbsData;

		A3DDouble Tolerance = 1e-3;
		A3DBool bUseSameParameterization = true;
		A3DNurbsData.FillWith(&GetSurfaceAsNurbs, Surface, Tolerance, bUseSameParameterization);

		if (!A3DNurbsData.IsValid())
		{
			return TSharedPtr<FSurface>();
		}

		return AddSurfaceNurbs(*A3DNurbsData, OutUVReparameterization);

	}
};

}

namespace UE::CADKernel
{

TSharedPtr<FModel> FTechSoftUtilities::TechSoftToCADKernel(A3DRiRepresentationItem* Representation, double Unit, double InGeometricTolerance)
{
	if (!FTechSoftLibrary::IsInitialized())
	{
		return {};
	}

	TechSoftUtilities::FRepresentationToModel Converter(InGeometricTolerance);

	TSharedPtr<FBody> Body = Converter.Convert(Representation, nullptr, Unit);

	if (Body.IsValid())
	{
		TSharedRef<FModel> Model = FEntity::MakeShared<FModel>();
		Model->Add(Body);

		return Model;
	}
	return nullptr;
}

}
#else
#endif
