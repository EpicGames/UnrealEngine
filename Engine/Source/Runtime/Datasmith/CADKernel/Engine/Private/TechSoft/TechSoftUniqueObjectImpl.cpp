// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftUniqueObjectImpl.h"

#ifdef WITH_HOOPS
namespace UE::CADKernel::TechSoft
{

	// TUniqueObject InitializeData -----------------------------------
	template<>
	void TUniqueObject<A3DAsmModelFileData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DAsmModelFileData, Data);
	}
	template<>
	void TUniqueObject<A3DAsmPartDefinitionData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DAsmPartDefinitionData, Data);
	}
	template<>
	void TUniqueObject<A3DAsmProductOccurrenceData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceData, Data);
	}
	template<>
	void TUniqueObject<A3DAsmProductOccurrenceDataCV5>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceDataCV5, Data);
	}
	template<>
	void TUniqueObject<A3DAsmProductOccurrenceDataSLW>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceDataSLW, Data);
	}
	template<>
	void TUniqueObject<A3DAsmProductOccurrenceDataNX>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceDataNX, Data);
	}
	template<>
	void TUniqueObject<A3DBoundingBoxData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DBoundingBoxData, Data);
	}
	template<>
	void TUniqueObject<A3DCopyAndAdaptBrepModelData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCopyAndAdaptBrepModelData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvCircleData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvCircleData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvCompositeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvCompositeData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvEllipseData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvEllipseData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvHelixData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvHelixData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvHyperbolaData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvHyperbolaData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvLineData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvLineData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvNurbsData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvNurbsData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvParabolaData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvParabolaData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvPolyLineData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvPolyLineData, Data);
	}
	template<>
	void TUniqueObject<A3DDomainData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DDomainData, Data);
	}
	template<>
	void TUniqueObject<A3DCrvTransformData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DCrvTransformData, Data);
	}
	template<>
	void TUniqueObject<A3DGlobalData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGlobalData, Data);
	}
	template<>
	void TUniqueObject<A3DGraphicsData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphicsData, Data);
	}
	template<>
	void TUniqueObject<A3DIntervalData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DIntervalData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscAttributeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscAttributeData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscCartesianTransformationData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscCartesianTransformationData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscEntityReferenceData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscEntityReferenceData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscGeneralTransformationData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscGeneralTransformationData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscMaterialPropertiesData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscMaterialPropertiesData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscReferenceOnCsysItemData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscReferenceOnCsysItemData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscReferenceOnTessData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscReferenceOnTessData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscReferenceOnTopologyData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscReferenceOnTopologyData, Data);
	}
	template<>
	void TUniqueObject<A3DMiscSingleAttributeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DMiscSingleAttributeData, Data);
	}
	template<>
	void TUniqueObject<A3DRWParamsExportPrcData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRWParamsExportPrcData, Data);
	}
	template<>
	void TUniqueObject<A3DRiBrepModelData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRiBrepModelData, Data);
	}
	template<>
	void TUniqueObject<A3DRiCoordinateSystemData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRiCoordinateSystemData, Data);
	}
	template<>
	void TUniqueObject<A3DRiDirectionData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRiDirectionData, Data);
	}
	template<>
	void TUniqueObject<A3DRiPolyBrepModelData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRiPolyBrepModelData, Data);
	}
	template<>
	void TUniqueObject<A3DRiRepresentationItemData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRiRepresentationItemData, Data);
	}
	template<>
	void TUniqueObject<A3DRiSetData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRiSetData, Data);
	}
	template<>
	void TUniqueObject<A3DRootBaseData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRootBaseData, Data);
	}
	template<>
	void TUniqueObject<A3DRootBaseWithGraphicsData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRootBaseWithGraphicsData, Data);
	}
	template<>
	void TUniqueObject<A3DRWParamsTessellationData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DRWParamsTessellationData, Data);
	}
	template<>
	void TUniqueObject<A3DSewOptionsData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSewOptionsData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfBlend01Data>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfBlend01Data, Data);
	}
	template<>
	void TUniqueObject<A3DSurfBlend02Data>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfBlend02Data, Data);
	}
	template<>
	void TUniqueObject<A3DSurfBlend03Data>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfBlend03Data, Data);
	}
	template<>
	void TUniqueObject<A3DSurfConeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfConeData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfCylinderData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfCylinderData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfCylindricalData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfCylindricalData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfExtrusionData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfExtrusionData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfFromCurvesData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfFromCurvesData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfNurbsData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfNurbsData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfPipeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfPipeData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfPlaneData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfPlaneData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfRevolutionData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfRevolutionData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfRuledData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfRuledData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfSphereData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfSphereData, Data);
	}
	template<>
	void TUniqueObject<A3DSurfTorusData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DSurfTorusData, Data);
	}
	template<>
	void TUniqueObject<A3DTess3DData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTess3DData, Data);
	}
	template<>
	void TUniqueObject<A3DTessBaseData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTessBaseData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoBodyData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoBodyData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoBrepDataData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoBrepDataData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoCoEdgeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoCoEdgeData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoConnexData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoConnexData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoContextData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoContextData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoEdgeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoEdgeData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoFaceData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoFaceData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoLoopData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoLoopData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoShellData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoShellData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoUniqueVertexData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoUniqueVertexData, Data);
	}
	template<>
	void TUniqueObject<A3DTopoWireEdgeData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DTopoWireEdgeData, Data);
	}
	template<>
	void TUniqueObject<A3DUTF8Char*>::InitializeData()
	{
		Data = nullptr;
	}
	template<>
	void TUniqueObject<A3DVector2dData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DVector2dData, Data);
	}
	template<>
	void TUniqueObject<A3DVector3dData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DVector3dData, Data);
	}


	// TUniqueObjectFromIndex InitializeData -----------------------------------
	template<>
	void TUniqueObjectFromIndex<A3DGraphMaterialData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphMaterialData, Data);
	}
	template<>
	void TUniqueObjectFromIndex<A3DGraphPictureData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphPictureData, Data);
	}
	template<>
	void TUniqueObjectFromIndex<A3DGraphRgbColorData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphRgbColorData, Data);
	}
	template<>
	void TUniqueObjectFromIndex<A3DGraphStyleData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphStyleData, Data);
	}
	template<>
	void TUniqueObjectFromIndex<A3DGraphTextureApplicationData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphTextureApplicationData, Data);
	}
	template<>
	void TUniqueObjectFromIndex<A3DGraphTextureDefinitionData>::InitializeData()
	{
		A3D_INITIALIZE_DATA(A3DGraphTextureDefinitionData, Data);
	}



	// TUniqueObject GetData -----------------------------------

	template<>
	A3DStatus TUniqueObject<A3DAsmModelFileData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DAsmModelFileGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DAsmPartDefinitionData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DAsmPartDefinitionGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DAsmProductOccurrenceData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DAsmProductOccurrenceGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DAsmProductOccurrenceDataCV5>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DAsmProductOccurrenceGetCV5(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DAsmProductOccurrenceDataSLW>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DAsmProductOccurrenceGetSLW(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DAsmProductOccurrenceDataNX>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DAsmProductOccurrenceGetNX(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DBoundingBoxData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscGetBoundingBox(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCopyAndAdaptBrepModelData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvCircleData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvCircleGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvCompositeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvCompositeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvEllipseData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvEllipseGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvHelixData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvHelixGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvHyperbolaData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvHyperbolaGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvLineData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvLineGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvNurbsData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvNurbsGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvParabolaData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvParabolaGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvPolyLineData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvPolyLineGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DDomainData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DCrvTransformData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvTransformGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DGlobalData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DGlobalGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DGraphicsData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DGraphicsGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DIntervalData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DCrvGetInterval(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscAttributeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscAttributeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscCartesianTransformationData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscCartesianTransformationGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscEntityReferenceData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscEntityReferenceGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscGeneralTransformationData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscGeneralTransformationGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscMaterialPropertiesData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscGetMaterialProperties(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscReferenceOnCsysItemData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscReferenceOnCsysItemGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscReferenceOnTessData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscReferenceOnTessGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscReferenceOnTopologyData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DMiscReferenceOnTopologyGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DMiscSingleAttributeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DRWParamsExportPrcData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DRiBrepModelData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRiBrepModelGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRiCoordinateSystemData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRiCoordinateSystemGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRiDirectionData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRiDirectionGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRiPolyBrepModelData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRiPolyBrepModelGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRiRepresentationItemData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRiRepresentationItemGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRiSetData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRiSetGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRootBaseData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRootBaseGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRootBaseWithGraphicsData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DRootBaseWithGraphicsGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DRWParamsTessellationData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DSewOptionsData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfBlend01Data>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfBlend01Get(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfBlend02Data>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfBlend02Get(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfBlend03Data>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfBlend03Get(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfConeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfConeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfCylinderData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfCylinderGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfCylindricalData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfCylindricalGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfExtrusionData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfExtrusionGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfFromCurvesData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfFromCurvesGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfNurbsData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfNurbsGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfPipeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfPipeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfPlaneData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfPlaneGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfRevolutionData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfRevolutionGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfRuledData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfRuledGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfSphereData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfSphereGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DSurfTorusData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DSurfTorusGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTess3DData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTess3DGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTessBaseData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTessBaseGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoBodyData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoBodyGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoBrepDataData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoBrepDataGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoCoEdgeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoCoEdgeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoConnexData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoConnexGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoContextData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoContextGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoEdgeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoEdgeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoFaceData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoFaceGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoLoopData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoLoopGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoShellData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoShellGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoUniqueVertexData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoUniqueVertexGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DTopoWireEdgeData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DTopoWireEdgeGet(InEntityPtr, &Data);
	}

	template<>
	A3DStatus TUniqueObject<A3DUTF8Char*>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DVector2dData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}

	template<>
	A3DStatus TUniqueObject<A3DVector3dData>::GetData(const A3DEntity* InEntityPtr)
	{
		return A3DStatus::A3D_ERROR;
	}


	// TUniqueObjectFromIndex GetData -----------------------------------
	template<>
	A3DStatus TUniqueObjectFromIndex<A3DGraphMaterialData>::GetData(const uint32 InEntityIndex)
	{
		return A3DGlobalGetGraphMaterialData(InEntityIndex, &Data);
	}
	template<>
	A3DStatus TUniqueObjectFromIndex<A3DGraphPictureData>::GetData(const uint32 InEntityIndex)
	{
		return A3DGlobalGetGraphPictureData(InEntityIndex, &Data);
	}
	template<>
	A3DStatus TUniqueObjectFromIndex<A3DGraphRgbColorData>::GetData(const uint32 InEntityIndex)
	{
		return A3DGlobalGetGraphRgbColorData(InEntityIndex, &Data);
	}
	template<>
	A3DStatus TUniqueObjectFromIndex<A3DGraphStyleData>::GetData(const uint32 InEntityIndex)
	{
		return A3DGlobalGetGraphStyleData(InEntityIndex, &Data);
	}
	template<>
	A3DStatus TUniqueObjectFromIndex<A3DGraphTextureApplicationData>::GetData(const uint32 InEntityIndex)
	{
		return A3DGlobalGetGraphTextureApplicationData(InEntityIndex, &Data);
	}
	template<>
	A3DStatus TUniqueObjectFromIndex<A3DGraphTextureDefinitionData>::GetData(const uint32 InEntityIndex)
	{
		return A3DGlobalGetGraphTextureDefinitionData(InEntityIndex, &Data);
	}

	// DefaultValue -----------------------------------
	template<>
	const A3DEntity* TUniqueObject<A3DAsmModelFileData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DAsmPartDefinitionData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceDataCV5>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceDataSLW>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceDataNX>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DBoundingBoxData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCopyAndAdaptBrepModelData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvCircleData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvCompositeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvEllipseData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvHelixData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvHyperbolaData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvLineData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvNurbsData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvParabolaData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvPolyLineData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DDomainData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DCrvTransformData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DGlobalData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DGraphicsData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DIntervalData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscAttributeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscCartesianTransformationData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscEntityReferenceData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscGeneralTransformationData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscMaterialPropertiesData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscReferenceOnCsysItemData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscReferenceOnTessData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscReferenceOnTopologyData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DMiscSingleAttributeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRWParamsExportPrcData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRiBrepModelData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRiCoordinateSystemData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRiDirectionData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRiPolyBrepModelData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRiRepresentationItemData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRiSetData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRootBaseData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRootBaseWithGraphicsData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DRWParamsTessellationData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSewOptionsData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfBlend01Data>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfBlend02Data>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfBlend03Data>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfConeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfCylinderData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfCylindricalData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfExtrusionData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfFromCurvesData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfNurbsData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfPipeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfPlaneData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfRevolutionData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfRuledData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfSphereData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DSurfTorusData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTess3DData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTessBaseData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoBodyData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoBrepDataData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoCoEdgeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoConnexData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoContextData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoEdgeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoFaceData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoLoopData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoShellData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoUniqueVertexData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DTopoWireEdgeData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DVector2dData>::GetDefaultIndexerValue() const { return nullptr; }
	template<>
	const A3DEntity* TUniqueObject<A3DVector3dData>::GetDefaultIndexerValue() const { return nullptr; }

	template<>
	uint32 TUniqueObjectFromIndex<A3DGraphMaterialData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_MATERIAL_INDEX; }
	template<>
	uint32 TUniqueObjectFromIndex<A3DGraphPictureData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_PICTURE_INDEX; }
	template<>
	uint32 TUniqueObjectFromIndex<A3DGraphRgbColorData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_COLOR_INDEX; }
	template<>
	uint32 TUniqueObjectFromIndex<A3DGraphStyleData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_STYLE_INDEX; }
	template<>
	uint32 TUniqueObjectFromIndex<A3DGraphTextureApplicationData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_TEXTURE_APPLICATION_INDEX; }
	template<>
	uint32 TUniqueObjectFromIndex<A3DGraphTextureDefinitionData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_TEXTURE_DEFINITION_INDEX; }
	template<>
	const A3DEntity* TUniqueObject<A3DUTF8Char*>::GetDefaultIndexerValue() const { return nullptr; }

	const uint32 FTechSoftDefaultValue::Material = A3D_DEFAULT_MATERIAL_INDEX;
	const uint32 FTechSoftDefaultValue::Picture = A3D_DEFAULT_PICTURE_INDEX;
	const uint32 FTechSoftDefaultValue::RgbColor = A3D_DEFAULT_COLOR_INDEX;
	const uint32 FTechSoftDefaultValue::Style = A3D_DEFAULT_STYLE_INDEX;
	const uint32 FTechSoftDefaultValue::TextureApplication = A3D_DEFAULT_TEXTURE_APPLICATION_INDEX;
	const uint32 FTechSoftDefaultValue::TextureDefinition = A3D_DEFAULT_TEXTURE_DEFINITION_INDEX;

}
#endif
