// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TechSoftIncludes.h"
#include "TechSoftUniqueObject.h"

#ifdef WITH_HOOPS

namespace UE::CADKernel::TechSoft
{

	// Single-ownership smart TeshSoft object
	// Use this when you need to manage TechSoft object's lifetime.
	//
	// TechSoft give access to void pointers
	// According to the context, the class name of the void pointer is known but the class is unknown
	// i.e. A3DSDKTypes.h defines all type like :
	// 	   typedef void A3DEntity;		
	// 	   typedef void A3DAsmModelFile; ...
	// 
	// From a pointer, TechSoft give access to a copy of the associated structure :
	//
	// const A3DXXXXX* pPointer;
	// A3DXXXXXData sData; // the structure
	// A3D_INITIALIZE_DATA(A3DXXXXXData, sData); // initialization of the structure
	// A3DXXXXXXGet(pPointer, &sData); // Copy of the data of the pointer in the structure
	// ...
	// A3DXXXXXXGet(NULL, &sData); // Free the structure
	//
	// A3D_INITIALIZE_DATA, and all A3DXXXXXXGet methods are TechSoft macro
	//


	// TUniqueObject -----------------------------------

	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DAsmModelFileData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DAsmPartDefinitionData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DAsmProductOccurrenceData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DAsmProductOccurrenceDataCV5>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DAsmProductOccurrenceDataSLW>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DAsmProductOccurrenceDataNX>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DBoundingBoxData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCopyAndAdaptBrepModelData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvCircleData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvCompositeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvEllipseData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvHelixData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvHyperbolaData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvLineData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvNurbsData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvParabolaData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvPolyLineData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DCrvTransformData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DDomainData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DGlobalData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DGraphicsData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DIntervalData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscAttributeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscCartesianTransformationData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscEntityReferenceData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscGeneralTransformationData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscMaterialPropertiesData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscReferenceOnCsysItemData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscReferenceOnTessData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscReferenceOnTopologyData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DMiscSingleAttributeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRWParamsExportPrcData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRiBrepModelData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRiCoordinateSystemData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRiDirectionData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRiPolyBrepModelData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRiRepresentationItemData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRiSetData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRootBaseData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRootBaseWithGraphicsData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DRWParamsTessellationData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSewOptionsData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfBlend01Data>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfBlend02Data>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfBlend03Data>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfConeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfCylinderData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfCylindricalData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfExtrusionData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfFromCurvesData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfNurbsData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfPipeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfPlaneData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfRevolutionData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfRuledData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfSphereData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DSurfTorusData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTess3DData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTessBaseData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoBodyData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoBrepDataData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoCoEdgeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoConnexData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoContextData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoEdgeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoFaceData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoLoopData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoShellData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoUniqueVertexData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoMultipleVertexData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DTopoWireEdgeData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DVector2dData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DVector3dData>::InitializeData();

	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DAsmModelFileData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DAsmPartDefinitionData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DAsmProductOccurrenceData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DAsmProductOccurrenceDataCV5>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DAsmProductOccurrenceDataSLW>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DAsmProductOccurrenceDataNX>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DBoundingBoxData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCopyAndAdaptBrepModelData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvCircleData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvCompositeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvEllipseData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvHelixData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvHyperbolaData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvLineData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvNurbsData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvParabolaData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvPolyLineData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DCrvTransformData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DDomainData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DGlobalData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DGraphicsData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DIntervalData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscAttributeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscCartesianTransformationData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscEntityReferenceData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscGeneralTransformationData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscMaterialPropertiesData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscReferenceOnCsysItemData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscReferenceOnTessData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscReferenceOnTopologyData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DMiscSingleAttributeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRWParamsExportPrcData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRiBrepModelData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRiCoordinateSystemData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRiDirectionData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRiPolyBrepModelData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRiRepresentationItemData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRiSetData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRootBaseData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRootBaseWithGraphicsData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DRWParamsTessellationData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSewOptionsData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfBlend01Data>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfBlend02Data>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfBlend03Data>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfConeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfCylinderData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfCylindricalData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfExtrusionData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfFromCurvesData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfNurbsData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfPipeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfPlaneData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfRevolutionData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfRuledData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfSphereData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DSurfTorusData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTess3DData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTessBaseData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoBodyData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoBrepDataData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoCoEdgeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoConnexData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoContextData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoEdgeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoFaceData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoLoopData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoShellData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoUniqueVertexData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoMultipleVertexData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DTopoWireEdgeData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DVector2dData>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DVector3dData>::GetData(const A3DEntity* InEntityPtr);

	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DAsmModelFileData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DAsmPartDefinitionData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceDataCV5>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceDataSLW>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DAsmProductOccurrenceDataNX>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DBoundingBoxData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCopyAndAdaptBrepModelData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvCircleData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvCompositeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvEllipseData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvHelixData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvHyperbolaData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvLineData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvNurbsData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvParabolaData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvPolyLineData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DCrvTransformData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DDomainData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DGlobalData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DGraphicsData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DIntervalData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscAttributeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscCartesianTransformationData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscEntityReferenceData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscGeneralTransformationData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscMaterialPropertiesData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscReferenceOnCsysItemData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscReferenceOnTessData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscReferenceOnTopologyData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DMiscSingleAttributeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRWParamsExportPrcData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRiBrepModelData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRiCoordinateSystemData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRiDirectionData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRiPolyBrepModelData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRiRepresentationItemData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRiSetData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRootBaseData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRootBaseWithGraphicsData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DRWParamsTessellationData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSewOptionsData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfBlend01Data>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfBlend02Data>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfBlend03Data>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfConeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfCylinderData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfCylindricalData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfExtrusionData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfFromCurvesData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfNurbsData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfPipeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfPlaneData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfRevolutionData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfRuledData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfSphereData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DSurfTorusData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTess3DData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTessBaseData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoBodyData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoBrepDataData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoCoEdgeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoConnexData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoContextData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoEdgeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoFaceData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoLoopData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoShellData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoUniqueVertexData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoMultipleVertexData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DTopoWireEdgeData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DVector2dData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DVector3dData>::GetDefaultIndexerValue() const;

	// TUniqueObjectFromIndex -----------------------------------

	template<>
	CADKERNELENGINE_API void TUniqueObjectFromIndex<A3DGraphMaterialData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObjectFromIndex<A3DGraphPictureData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObjectFromIndex<A3DGraphRgbColorData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObjectFromIndex<A3DGraphStyleData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObjectFromIndex<A3DGraphTextureApplicationData>::InitializeData();
	template<>
	CADKERNELENGINE_API void TUniqueObjectFromIndex<A3DGraphTextureDefinitionData>::InitializeData();

	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObjectFromIndex<A3DGraphMaterialData>::GetData(const uint32 InEntityIndex);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObjectFromIndex<A3DGraphPictureData>::GetData(const uint32 InEntityIndex);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObjectFromIndex<A3DGraphRgbColorData>::GetData(const uint32 InEntityIndex);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObjectFromIndex<A3DGraphStyleData>::GetData(const uint32 InEntityIndex);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObjectFromIndex<A3DGraphTextureApplicationData>::GetData(const uint32 InEntityIndex);
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObjectFromIndex<A3DGraphTextureDefinitionData>::GetData(const uint32 InEntityIndex);

	template<>
	CADKERNELENGINE_API uint32 TUniqueObjectFromIndex<A3DGraphMaterialData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API uint32 TUniqueObjectFromIndex<A3DGraphPictureData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API uint32 TUniqueObjectFromIndex<A3DGraphRgbColorData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API uint32 TUniqueObjectFromIndex<A3DGraphStyleData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API uint32 TUniqueObjectFromIndex<A3DGraphTextureApplicationData>::GetDefaultIndexerValue() const;
	template<>
	CADKERNELENGINE_API uint32 TUniqueObjectFromIndex<A3DGraphTextureDefinitionData>::GetDefaultIndexerValue() const;

	// A3DUTF8Char* -----------------------------------

	template<>
	CADKERNELENGINE_API void TUniqueObject<A3DUTF8Char*>::InitializeData();
	template<>
	CADKERNELENGINE_API A3DStatus TUniqueObject<A3DUTF8Char*>::GetData(const A3DEntity* InEntityPtr);
	template<>
	CADKERNELENGINE_API const A3DEntity* TUniqueObject<A3DUTF8Char*>::GetDefaultIndexerValue() const;

}
#endif
