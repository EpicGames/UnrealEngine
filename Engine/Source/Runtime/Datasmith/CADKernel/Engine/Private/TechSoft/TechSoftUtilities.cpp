// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftUtilities.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngine.h"

#ifdef WITH_HOOPS

#include "TechSoftUniqueObjectImpl.h"

#include "MeshUtilities.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

namespace UE::CADKernel
{
	bool FTechSoftUtilities::Save(const TArray<A3DRiRepresentationItem*>& Representations, const FString& FilePath, const FString& AttributesStr)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return false;
		}

		// Create PartDefinition
		TechSoft::TUniqueObject<A3DAsmPartDefinitionData> PartDefinitionData;
		PartDefinitionData->m_uiRepItemsSize = Representations.Num();
		PartDefinitionData->m_ppRepItems = (A3DRiRepresentationItem**)Representations.GetData();
		A3DAsmPartDefinition* PartDefinition = nullptr;
		if (A3DAsmPartDefinitionCreate(PartDefinitionData.GetPtr(), &PartDefinition) != A3DStatus::A3D_SUCCESS || PartDefinition == nullptr)
		{
			return false;
		}

		TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> ProductOccurrenceData;
		ProductOccurrenceData->m_pPart = PartDefinition;
		A3DAsmProductOccurrence* ProductOccurrence = nullptr;
		if (A3DAsmProductOccurrenceCreate(ProductOccurrenceData.GetPtr(), &ProductOccurrence) != A3DStatus::A3D_SUCCESS)
		{
			return false;
		}

		// Add MaterialTable as attribute to ProductOccurrence
		std::string StringAnsi(TCHAR_TO_UTF8(*AttributesStr));

		TechSoft::TUniqueObject<A3DMiscSingleAttributeData> SingleAttributeData;
		SingleAttributeData->m_eType = kA3DModellerAttributeTypeString;
		SingleAttributeData->m_pcTitle = (char*)"Attributes";
		SingleAttributeData->m_pcData = (char*)StringAnsi.c_str();

		TechSoft::TUniqueObject<A3DMiscAttributeData> AttributesData;
		AttributesData->m_pcTitle = SingleAttributeData->m_pcTitle;
		AttributesData->m_asSingleAttributesData = SingleAttributeData.GetPtr();
		AttributesData->m_uiSize = 1;
		A3DMiscAttribute* Attributes = nullptr;
		if (A3DMiscAttributeCreate(AttributesData.GetPtr(), &Attributes) != A3DStatus::A3D_SUCCESS)
		{
			return false;
		}

		TechSoft::TUniqueObject<A3DRootBaseData> RootBaseData;
		RootBaseData->m_pcName = SingleAttributeData->m_pcTitle;
		RootBaseData->m_ppAttributes = &Attributes;
		RootBaseData->m_uiSize = 1;
		if (A3DRootBaseSet(ProductOccurrence, RootBaseData.GetPtr()) != A3DStatus::A3D_SUCCESS)
		{
			return false;
		}

		// Create ModelFile
		TechSoft::TUniqueObject<A3DAsmModelFileData> ModelFileData;
		ModelFileData->m_uiPOccurrencesSize = 1;
		ModelFileData->m_dUnit = 1.0;
		ModelFileData->m_ppPOccurrences = &ProductOccurrence;
		A3DAsmModelFile* ModelFile = nullptr;
		if (A3DAsmModelFileCreate(ModelFileData.GetPtr(), &ModelFile) != A3DStatus::A3D_SUCCESS)
		{
			return false;
		}

		// Save ModelFile to Pcr file
		TechSoft::TUniqueObject<A3DRWParamsExportPrcData> ParamsExportData;
		ParamsExportData->m_bCompressBrep = false;
		ParamsExportData->m_bCompressTessellation = false;

#if PLATFORM_WINDOWS
		A3DUTF8Char HsfFileName[MAX_PATH];
		FCStringAnsi::Strncpy(HsfFileName, TCHAR_TO_UTF8(*FilePath), MAX_PATH);
#elif PLATFORM_LINUX
		A3DUTF8Char HsfFileName[PATH_MAX];
		FCStringAnsi::Strncpy(HsfFileName, TCHAR_TO_UTF8(*FilePath), PATH_MAX);
#else
#error Platform not supported
#endif // PLATFORM_WINDOWS

		if (A3DAsmModelFileExportToPrcFile(ModelFile, ParamsExportData.GetPtr(), HsfFileName, nullptr) != A3DStatus::A3D_SUCCESS)
		{
			return false;
		}

		A3DAsmModelFileUnloadParts(ModelFile, 1, &ProductOccurrence);

		// #ueent_techsoft: Deleting the model seems to delete the entire content. To be double-checked
		//A3DEntityDelete(Attributes);

		return true;
	}

	bool FTechSoftUtilities::Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, FMeshDescription& Mesh, bool bSkipDeletedFaces, bool bEmptyMesh)
	{
		using namespace UE::CADKernel::MeshUtilities;

		if (!FTechSoftLibrary::Initialize())
		{
			return false;
		}

		if (bSkipDeletedFaces)
		{
			GetExistingFaceGroups(Mesh, Context.FaceGroupsToExtract);
		}

		TSharedPtr<FMeshWrapperAbstract> MeshWrapper = FMeshWrapperAbstract::MakeWrapper(Context, Mesh);
		return TechSoftUtilities::Tessellate(Representation, Context, *MeshWrapper, bEmptyMesh);
	}

	bool FTechSoftUtilities::Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, UE::Geometry::FDynamicMesh3& Mesh, bool bSkipDeletedFaces, bool bEmptyMesh)
	{
		using namespace UE::CADKernel::MeshUtilities;

		if (!FTechSoftLibrary::Initialize())
		{
			return false;
		}

		if (bSkipDeletedFaces)
		{
			GetExistingFaceGroups(Mesh, Context.FaceGroupsToExtract);
		}

		TSharedPtr<FMeshWrapperAbstract> MeshWrapper = FMeshWrapperAbstract::MakeWrapper(Context, Mesh);
		return TechSoftUtilities::Tessellate(Representation, Context, *MeshWrapper, bEmptyMesh);
	}

	A3DRiRepresentationItem* FTechSoftUtilities::GetRepresentation(const TArray<uint8>& TechSoftRawData)
	{
		const FString CachePath = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("Retessellate"));
		IFileManager::Get().MakeDirectory(*CachePath, true);

		FString ResourceFile = FPaths::CreateTempFilename(*CachePath, TEXT(""), TEXT(".prc"));
		FPaths::ConvertRelativePathToFull(ResourceFile);

		FFileHelper::SaveArrayToFile(TechSoftRawData, *ResourceFile);

		A3DRWParamsPrcReadHelper* ReadHelper = nullptr;

		A3DAsmModelFile* ModelFile = nullptr;
		if (A3DAsmModelFileLoadFromPrcFile(TCHAR_TO_UTF8(*ResourceFile), &ReadHelper, &ModelFile) != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DAsmModelFileData> ModelFileData(ModelFile);
		if (!ModelFileData.IsValid() || ModelFileData->m_uiPOccurrencesSize != 1)
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DAsmProductOccurrenceData> OccurenceData(ModelFileData->m_ppPOccurrences[0]);
		if (!OccurenceData.IsValid() || !OccurenceData->m_pPart)
		{
			return nullptr;
		}

		TechSoft::TUniqueObject<A3DAsmPartDefinitionData> PartDefinitionData(OccurenceData->m_pPart);
		if (!PartDefinitionData.IsValid() || PartDefinitionData->m_uiRepItemsSize == 0)
		{
			return nullptr;
		}

		return PartDefinitionData->m_ppRepItems[0];
	}

	bool TechSoftUtilities::Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, MeshUtilities::FMeshWrapperAbstract& MeshWrapper, bool bEmptyMesh)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return false;
		}

		using namespace UE::CADKernel;

		static FCriticalSection TechSoftCriticalSection;

		// #cad_import: Track when TechSoft tessellation is thread safe
		FScopeLock Lock(&TechSoftCriticalSection);

		// #cad_import: Create set of helpers IsA[Whatever] in FTechSoftLibrary
		// Here it would FTechSoftLibrary::IsATriangulatedMesh
		A3DEEntityType Type;
		A3DEntityGetType(Representation, &Type);
		if (Type == kA3DTypeRiPolyBrepModel)
		{
			return AddRepresentation(Representation, Context.ModelParams.ModelUnitToCentimeter, MeshWrapper);
		}

		TechSoft::TUniqueObject<A3DRiRepresentationItemData> RepresentationData(Representation);

		if (!RepresentationData.IsValid())
		{
			return false;
}

		TArray<A3DRiBrepModel*> NewBReps;
		const FCADKernelTessellationSettings& TessellationSettings = Context.TessellationSettings;

		if (TessellationSettings.StitchingTechnique == ECADKernelStitchingTechnique::StitchingHeal)
		{
			// cad_import: Review unit conversion - mm or cm to model unit
			const double SewingTolerance = TessellationSettings.GetStitchingTolerance() / TessellationSettings.UnitMultiplier;
			if (!SewBReps({ Representation }, SewingTolerance, NewBReps))
			{
				// Log message
			}
		}

		if (bEmptyMesh)
		{
			MeshWrapper.ClearMesh();
		}

		if (!NewBReps.IsEmpty())
		{
			for (A3DRiBrepModel* BrepModel : NewBReps)
			{
				// It is ok to use BrepModel as a A3DRiRepresentationItem.
				// TechSoft is feeling the required structure according to the data structure asked for
				if (TessellateRepresentation(BrepModel, TessellationSettings))
				{
					AddRepresentation(BrepModel, Context.ModelParams.ModelUnitToCentimeter, MeshWrapper);
				}
			}
		}
		else
		{
			if (TessellateRepresentation(Representation, TessellationSettings))
			{
				AddRepresentation(Representation, Context.ModelParams.ModelUnitToCentimeter, MeshWrapper);
			}
		}

		MeshWrapper.Complete();

		return true;
	}

	bool TechSoftUtilities::TessellateRepresentation(A3DRiRepresentationItem* Representation, const FCADKernelTessellationSettings& Settings)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return false;
		}

		// TUniqueTechSoftObj does not work in this case
		TechSoft::TUniqueObject<A3DRWParamsTessellationData> TessellationParameters;

		TessellationParameters->m_eTessellationLevelOfDetail = kA3DTessLODUserDefined; // Enum to specify predefined values for some following members.
		TessellationParameters->m_bUseHeightInsteadOfRatio = A3D_TRUE;
		TessellationParameters->m_dMaxChordHeight = Settings.GetChordTolerance(); // This is centimeters
		if (!FMath::IsNearlyZero(Settings.UnitMultiplier))
		{
			// Convert back to unit of imported data
			TessellationParameters->m_dMaxChordHeight /= Settings.UnitMultiplier;
		}

		TessellationParameters->m_dAngleToleranceDeg = Settings.NormalTolerance;
		TessellationParameters->m_dMaximalTriangleEdgeLength = 0; //Settings.MaxEdgeLength;

		TessellationParameters->m_bAccurateTessellation = A3D_FALSE;  // A3D_FALSE' indicates the tessellation is set for visualization
		TessellationParameters->m_bAccurateTessellationWithGrid = A3D_FALSE; // Enable accurate tessellation with faces inner points on a grid.
		TessellationParameters->m_dAccurateTessellationWithGridMaximumStitchLength = 0; 	// Maximal grid stitch length. Disabled if value is 0. Be careful, a too small value can generate a huge tessellation.

		TessellationParameters->m_bKeepUVPoints = A3D_TRUE; // Keep parametric points as texture points.

		// Get the tessellation
		static FCriticalSection TechSoftCriticalSection;

		A3DStatus Status = A3DRiRepresentationItemComputeTessellation(Representation, TessellationParameters.GetPtr());
		if (!ensure(Status == A3D_SUCCESS))
		{
			return false;
		}

		TechSoft::TUniqueObject<A3DRiRepresentationItemData> RepresentationItemData(Representation);
		if (!RepresentationItemData.IsValid())
		{
			return false;
		}

		A3DEEntityType Type;
		A3DEntityGetType(RepresentationItemData->m_pTessBase, &Type);

		return Type == kA3DTypeTess3D ? true : false;
	}

	bool TechSoftUtilities::SewBReps(const TArray<A3DRiBrepModel*>& BRepsIn, double Tolerance, TArray<A3DRiBrepModel*>& BRepsOut)
	{
		if (!FTechSoftLibrary::Initialize())
		{
			return false;
		}

		TechSoft::TUniqueObject<A3DSewOptionsData> SewData;
		SewData->m_bComputePreferredOpenShellOrientation = false;

		A3DUns32 NewBRepCount = 0;
		A3DRiBrepModel** NewBReps = nullptr;
		A3DRiBrepModel** BRepsToSew = const_cast<A3DRiBrepModel**>(BRepsIn.GetData());

		A3DStatus Status = A3DSewBrep(&BRepsToSew, BRepsIn.Num(), Tolerance, SewData.GetPtr(), &NewBReps, &NewBRepCount);

		if (Status == A3DStatus::A3D_SUCCESS && NewBRepCount > 0)
		{
			BRepsOut.Append(NewBReps, NewBRepCount);
		}

		return Status == A3DStatus::A3D_SUCCESS ? true : false;
	}

	FVector2d TechSoftUtilities::GetUVScale(const A3DTopoFace* TopoFace, double TextureUnit)
	{
		TechSoft::TUniqueObject<A3DTopoFaceData> TopoFaceData(TopoFace);
		if (!TopoFaceData.IsValid())
		{
			return FVector2d::UnitVector;
		}

		TechSoft::TUniqueObject<A3DDomainData> Domain;
		if (TopoFaceData->m_bHasTrimDomain)
		{
			*Domain = TopoFaceData->m_sSurfaceDomain;
		}
		else
		{
			A3DStatus Status = A3DSurfGetDomain(TopoFaceData->m_pSurface, Domain.GetPtr());
			if (Status != A3D_SUCCESS)
			{
				return FVector2d::UnitVector;
			}
		}

		constexpr int32 IsoCurveCount = 7;
		const double DeltaU = (Domain->m_sMax.m_dX - Domain->m_sMin.m_dX) / (IsoCurveCount - 1.);
		const double DeltaV = (Domain->m_sMax.m_dY - Domain->m_sMin.m_dY) / (IsoCurveCount - 1.);
		if (DeltaU < UE_DOUBLE_SMALL_NUMBER || DeltaV < UE_DOUBLE_SMALL_NUMBER)
		{
			return FVector2d::UnitVector;
		}

		const A3DSurfBase* A3DSurface = TopoFaceData->m_pSurface;

		FVector NodeMatrix[IsoCurveCount * IsoCurveCount];

		TechSoft::TUniqueObject<A3DVector3dData> Point3D;
		TechSoft::TUniqueObject<A3DVector2dData> CoordinateObj;
		A3DVector2dData& Coordinate = *CoordinateObj;
		Coordinate.m_dX = Domain->m_sMin.m_dX;
		Coordinate.m_dY = Domain->m_sMin.m_dY;

		for (int32 IndexI = 0; IndexI < IsoCurveCount; IndexI++)
		{
			for (int32 IndexJ = 0; IndexJ < IsoCurveCount; IndexJ++)
			{
				if (A3DSurfEvaluate(A3DSurface, &Coordinate, 0, Point3D.GetPtr()) == A3D_SUCCESS)
				{
					NodeMatrix[IndexI * IsoCurveCount + IndexJ].X = Point3D->m_dX;
					NodeMatrix[IndexI * IsoCurveCount + IndexJ].Y = Point3D->m_dY;
					NodeMatrix[IndexI * IsoCurveCount + IndexJ].Z = Point3D->m_dZ;
					Coordinate.m_dY += DeltaV;
				}
			}
			Coordinate.m_dX += DeltaU;
			Coordinate.m_dY = Domain->m_sMin.m_dY;
		}

		// Compute length of 7 iso V line
		double LengthU[IsoCurveCount];
		double LengthUMax = 0;
		double LengthUMed = 0;

		for (int32 IndexJ = 0; IndexJ < IsoCurveCount; IndexJ++)
		{
			LengthU[IndexJ] = 0;
			for (int32 IndexI = 0; IndexI < (IsoCurveCount - 1); IndexI++)
			{
				LengthU[IndexJ] += FVector::Distance(NodeMatrix[IndexI * IsoCurveCount + IndexJ], NodeMatrix[(IndexI + 1) * IsoCurveCount + IndexJ]);
			}
			LengthUMed += LengthU[IndexJ];
			LengthUMax = FMath::Max(LengthU[IndexJ], LengthUMax);
		}
		LengthUMed /= IsoCurveCount;
		LengthUMed = LengthUMed * 2 / 3 + LengthUMax / 3;

		// Compute length of 7 iso U line
		double LengthV[IsoCurveCount];
		double LengthVMax = 0;
		double LengthVMed = 0;

		for (int32 IndexI = 0; IndexI < IsoCurveCount; IndexI++)
		{
			LengthV[IndexI] = 0;
			for (int32 IndexJ = 0; IndexJ < (IsoCurveCount - 1); IndexJ++)
			{
				LengthV[IndexI] += FVector::Distance(NodeMatrix[IndexI * IsoCurveCount + IndexJ], NodeMatrix[IndexI * IsoCurveCount + IndexJ + 1]);
			}
			LengthVMed += LengthV[IndexI];
			LengthVMax = FMath::Max(LengthV[IndexI], LengthVMax);
		}
		LengthVMed /= IsoCurveCount;
		LengthVMed = LengthVMed * 2 / 3 + LengthVMax / 3;

		// Texture unit is meter, Coord unit from TechSoft is mm, so TextureScale = 0.01 to convert cm into m
		// #cad_import: Verify assumption above and where TextureUnit can come from
		constexpr double TextureScale = 0.01;

		return {
			TextureUnit * TextureScale * LengthUMed / (Domain->m_sMax.m_dX - Domain->m_sMin.m_dX),
			TextureUnit * TextureScale * LengthVMed / (Domain->m_sMax.m_dY - Domain->m_sMin.m_dY)
		};
	}

	TSharedPtr<FJsonObject> TechSoftUtilities::GetJsonObject(A3DEntity* Entity, bool bIsLegacy)
	{
		TechSoft::TUniqueObject<A3DRootBaseData> RootBaseData(Entity);

		if (RootBaseData.IsValid() && RootBaseData->m_uiSize > 0)
		{
			if (bIsLegacy)
			{
				TechSoft::TUniqueObject<A3DMiscAttributeData> AttributeData(RootBaseData->m_ppAttributes[0]);
				if (AttributeData->m_uiSize > 0 && AttributeData->m_asSingleAttributesData[0].m_eType == kA3DModellerAttributeTypeString)
				{
					FString JsonString = UTF8_TO_TCHAR(AttributeData->m_asSingleAttributesData[0].m_pcData);

					TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);

					TSharedPtr<FJsonObject> JsonObject;
					if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
					{
						return JsonObject;
					}
				}
			}
		}

		return {};
	}

	bool FTechSoftUtilities::ToRawData(A3DRiRepresentationItem* Representation, int32 MaterialID, TArray<uint8>& RawDataOut)
	{
		// See TechSoftUtils::SaveBodiesToPrcFile
		// Set attributes on representation not ProductOccurence
		return false;
	}

}
#else
namespace UE::CADKernel
{
	bool FTechSoftUtilities::Save(const TArray<A3DRiRepresentationItem*>& Representations, const FString& FilePath, const FString& Attributes)
	{
		return false;
	}

	bool FTechSoftUtilities::Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, FMeshDescription& Mesh, bool bSkipDeletedFaces, bool bEmptyMesh)
	{
		return false;
	}

	bool FTechSoftUtilities::Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, UE::Geometry::FDynamicMesh3& Mesh, bool bSkipDeletedFaces, bool bEmptyMesh)
	{
		return false;
	}

	A3DRiRepresentationItem* FTechSoftUtilities::GetRepresentation(const TArray<uint8>& TechSoftRawData)
	{
		return nullptr;
	}

	TSharedPtr<UE::CADKernel::FModel> FTechSoftUtilities::TechSoftToCADKernel(A3DRiRepresentationItem* Representation, double Unit, double InGeometricTolerance)
	{
		return {};
	}

	bool FTechSoftUtilities::ToRawData(A3DRiRepresentationItem* Representation, int32 MaterialID, TArray<uint8>& RawDataOut)
	{
		return false;
	}
}
#endif
#endif
