// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if PLATFORM_DESKTOP
#include "TechSoftIncludes.h"

#ifdef WITH_HOOPS

class FJsonObject;
struct FCADKernelTessellationSettings;

namespace UE::CADKernel
{
	namespace MeshUtilities
	{
		class FMeshWrapperAbstract;
	}

	struct FTessellationContext;

	namespace TechSoftUtilities
	{
		bool Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Settings, MeshUtilities::FMeshWrapperAbstract& MeshWrapper, bool bEmptyMesh = false);
		bool TessellateRepresentation(A3DRiRepresentationItem* Representation, const FCADKernelTessellationSettings& Settings);
		bool AddRepresentation(A3DRiRepresentationItem* Representation, double ModelUnitToMeter, MeshUtilities::FMeshWrapperAbstract& MeshWrapper);
		FVector2d GetUVScale(const A3DTopoFace* TopoFace, double TextureUnit);
		TSharedPtr<FJsonObject> GetJsonObject(A3DEntity* Entity, bool bIsLegacy = false);


		/*
		 ** Sew the in coming BReps together
		 * @param: BRepsIn
		 * @param: Tolerance is file's unit with which to operate the sewing
		 * @param: BRepsOut
		 */
		bool SewBReps(const TArray<A3DRiBrepModel*>& BRepsIn, double Tolerance, TArray<A3DRiBrepModel*>& BRepsOut);
	};
}
#endif
#endif
