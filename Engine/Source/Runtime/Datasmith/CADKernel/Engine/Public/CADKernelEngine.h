// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if PLATFORM_DESKTOP
#include "CADKernelEngineDefinitions.h"
#include "TechSoftIncludes.h"

namespace UE::CADKernel
{
	class FModel;
	class FModelMesh;
	class FTopologicalFace;
}

namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::CADKernel::MeshUtilities
{
	class FMeshWrapperAbstract;
}

struct FMeshDescription;

namespace UE::CADKernel
{
	class FCADKernelUtilities
	{
	public:
		static bool CADKERNELENGINE_API Save(TSharedPtr<UE::CADKernel::FModel>& Model, const FString& FilePath);
		static bool CADKERNELENGINE_API Load(TSharedPtr<UE::CADKernel::FModel>& Model, const FString& FilePath);
		static bool CADKERNELENGINE_API Tessellate(UE::CADKernel::FModel& Model, const FTessellationContext& Context, FMeshDescription& MeshOut, bool bSkipDeletedFaces = false, bool bEmptyMesh = false);
		static bool CADKERNELENGINE_API Tessellate(UE::CADKernel::FModel& Model, const FTessellationContext& Context, UE::Geometry::FDynamicMesh3& MeshOut, bool bSkipDeletedFaces = false, bool bEmptyMesh = false);
		
		/* Note: Meshes are expected to be in their native coordinates system and unit */
		static void CADKERNELENGINE_API ApplyExtractionContext(const FMeshExtractionContext& Context, FMeshDescription& MeshInOut);
		static void CADKERNELENGINE_API ApplyExtractionContext(const FMeshExtractionContext& Context, UE::Geometry::FDynamicMesh3& MeshInOut);
		// Set the proper attributes for FMeshDescription created from CAD data.
		static void CADKERNELENGINE_API RegisterAttributes(FMeshDescription& MeshInOut, bool bKeepExistingAttribute = false);
		static bool CADKERNELENGINE_API ToRawData(TSharedPtr<UE::CADKernel::FModel>& Model, TArray<uint8>& RawDataOut);
	};

	enum class ECADFormat
	{
		ACIS,
		AUTOCAD,
		CATIA,
		CATIA_CGR,
		CATIA_3DXML,
		CATIAV4,
		CREO,
		DWG,
		DGN,
		TECHSOFT,
		IFC,
		IGES,
		INVENTOR,
		JT,
		N_X,  
		MICROSTATION,
		PARASOLID,
		SOLID_EDGE,
		SOLIDWORKS,
		STEP,
		OTHER
	};


	class FTechSoftLibrary
	{
	public:
		static CADKERNELENGINE_API const TCHAR* GetVersion();
		static bool CADKERNELENGINE_API Initialize();
		static bool CADKERNELENGINE_API IsInitialized() { return bIsInitialized; }


#ifdef WITH_HOOPS
		static CADKERNELENGINE_API A3DRiRepresentationItem* CreateRIBRep(const TArray<A3DTopoShell*>& TopoShells);

		static CADKERNELENGINE_API A3DTopoEdge* CreateTopoEdge();
		static CADKERNELENGINE_API A3DTopoEdge* CreateTopoEdge(A3DTopoEdgeData& TopoEdgeData);
		static CADKERNELENGINE_API A3DTopoFace* CreateTopoFaceWithNaturalLoop(A3DSurfBase* CarrierSurface);
		static CADKERNELENGINE_API A3DTopoShell* CreateTopoShell(A3DTopoShellData& TopoShellData);
		static CADKERNELENGINE_API A3DTopoFace* CreateTopoFace(A3DTopoFaceData& TopoFaceData);
		static CADKERNELENGINE_API A3DTopoLoop* CreateTopoLoop(A3DTopoLoopData& TopoLoopData);
		static CADKERNELENGINE_API A3DTopoCoEdge* CreateTopoCoEdge(A3DTopoCoEdgeData& TopoCoEdgeData);
		static CADKERNELENGINE_API A3DStatus LinkCoEdges(A3DTopoCoEdge* CoEdgePtr, A3DTopoCoEdge* NeighbourCoEdgePtr);

		static CADKERNELENGINE_API A3DCrvNurbs* CreateTrimNurbsCurve(A3DCrvNurbs* CurveNurbsPtr, double UMin, double UMax, bool bIs2D);
		static CADKERNELENGINE_API A3DSurfNurbs* CreateSurfaceNurbs(A3DSurfNurbsData& SurfaceNurbsData);
		static CADKERNELENGINE_API A3DCrvNurbs* CreateCurveNurbs(A3DCrvNurbsData& CurveNurbsData);

		static CADKERNELENGINE_API A3DStatus SetEntityGraphicsColor(A3DEntity* InEntity, FColor Color);


		static CADKERNELENGINE_API A3DAsmModelFile* LoadModelFileFromFile(const A3DImport& Import, const TCHAR* Filename);
		static CADKERNELENGINE_API A3DAsmModelFile* LoadModelFile(const TCHAR* Filename, const FTechSoftImportOverrides& LoadOverrides, FString& OutReason);

		static CADKERNELENGINE_API A3DAsmModelFile* LoadModelFileFromPrcFile(const A3DUTF8Char* CADFileName, A3DRWParamsPrcReadHelper** ReadHelper);
		static CADKERNELENGINE_API A3DStatus AdaptBRepInModelFile(A3DAsmModelFile* ModelFile, const A3DCopyAndAdaptBrepModelData& Setting, int32& ErrorCount, A3DCopyAndAdaptBrepModelErrorData** Errors);
		static CADKERNELENGINE_API A3DStatus AdaptBRepModel(A3DAsmModelFile* ModelFile, const TCHAR* Filename);
		static CADKERNELENGINE_API A3DStatus DeleteModelFile(A3DAsmModelFile* ModelFile);
		static CADKERNELENGINE_API A3DStatus DeleteEntity(A3DEntity* EntityPtr);
		static CADKERNELENGINE_API double GetModelFileUnit(const A3DAsmModelFile* ModelFile);

		static CADKERNELENGINE_API FString CleanLabel(const FString& Name);
		static CADKERNELENGINE_API void GetOccurrenceChildren(const A3DAsmProductOccurrence* Node, TArray<const A3DAsmProductOccurrence*>& OutChildren);

		struct FGraphicsProperties
		{
			bool bIsRemoved = false;
			bool bShow = true;

			int32 ColorUid = 0;
			int32 StyleIndex = 0;

			enum class EInheritance
			{
				Unset,
				Father,
				Child
			};

			EInheritance MaterialInheritance = EInheritance::Unset;

		};
		static CADKERNELENGINE_API void ExtractGraphicsProperties(const A3DEntity* Entity, FGraphicsProperties& Result);

		struct FMaterial
		{
			FColor Diffuse;
			FColor Ambient;
			FColor Specular;
			float Shininess;
			float Transparency = 0;
		};

		static CADKERNELENGINE_API bool ExtractMaterial(uint32 TechSoftMaterialIndex, const A3DGraphStyleData& GraphStyleData, FMaterial& ParsedMaterial);
		static CADKERNELENGINE_API FColor GetColorAt(uint32 ColorIndex);
		static CADKERNELENGINE_API TArray<uint32> GetStylesUsedOnFaces(A3DRiBrepModel* RepresentationItem);

		static CADKERNELENGINE_API bool ParseRootBaseData(const A3DEntity* Entity, TMap<FString, FString>& MetaData, FString& UniqueID, FString& Label);
		static CADKERNELENGINE_API void TraverseAttribute(const A3DMiscAttributeData& AttributeData, TMap<FString, FString>& OutMetaData);


		static CADKERNELENGINE_API bool IsMaterialTexture(const uint32 MaterialIndex);
		static CADKERNELENGINE_API bool GetEntityType(A3DRiRepresentationItem* RepresentationItem, A3DEEntityType& Type);

		static CADKERNELENGINE_API bool IsConfigurationSet(ECADFormat Format, const A3DAsmProductOccurrence* Occurrence);
		static CADKERNELENGINE_API A3DAsmProductOccurrence* FindConfiguration(const A3DAsmProductOccurrence* ConfigurationSetOccurrencePtr, TFunctionRef<bool(A3DAsmProductOccurrenceData& ConfigurationData)> Callback);

#endif

	private:
		static bool bIsInitialized;
	};

	class FTechSoftUtilities
	{
	public:
		/*
		 ** @param FilePath: Absolute path to the prc file to load
		 ** @param bSewModel: Optional argument to force a stitching of the model
		 ** @param StitchingTolerance: Optional stitching tolerance in mm
		 */
		static CADKERNELENGINE_API bool Save(const TArray<A3DRiRepresentationItem*>& Representations, const FString& FilePath, const FString& Attributes = FString());
		static CADKERNELENGINE_API bool Save(A3DRiRepresentationItem* Representation, const FString& FilePath, const FString& Attributes = FString())
		{
			return Save(TArray<A3DRiRepresentationItem*>( &Representation, 1 ), FilePath, Attributes);
		}
		static CADKERNELENGINE_API bool Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, FMeshDescription& MeshOut, bool bSkipDeletedFaces = false, bool bEmptyMesh = false);
		static CADKERNELENGINE_API bool Tessellate(A3DRiRepresentationItem* Representation, const FTessellationContext& Context, UE::Geometry::FDynamicMesh3& MeshOut, bool bSkipDeletedFaces = false, bool bEmptyMesh = false);

		static CADKERNELENGINE_API TSharedPtr<UE::CADKernel::FModel> TechSoftToCADKernel(A3DRiRepresentationItem* Representation, double Unit, double InGeometricTolerance);
		/* Limitation: Non of the implicit geometry is supported */
		static CADKERNELENGINE_API A3DRiRepresentationItem* CADKernelToTechSoft(TSharedPtr<UE::CADKernel::FModel>& Model);
		static CADKERNELENGINE_API A3DRiRepresentationItem* GetRepresentation(const TArray<uint8>& TechSoftRawData);
		static CADKERNELENGINE_API bool ToRawData(A3DRiRepresentationItem* Representation, int32 MaterialID, TArray<uint8>& RawDataOut);

	};

	namespace MathUtils
	{
		/*
		** Converts a transform from a given coordinates system to UE's one
		** UE's coordinates system is ZUp_LeftHanded
		*/
		inline FTransform ConvertTransform(ECADKernelModelCoordSystem SourceCoordSystem, const FTransform& LocalTransform)
		{
			const FTransform RightHanded(FRotator(0.0f, 0.0f, 0.0f), FVector(0.0f, 0.0f, 0.0f), FVector(-1.0f, 1.0f, 1.0f));
			const FTransform RightHandedLegacy(FRotator(0.0f, 0.0f, 0.0f), FVector(0.0f, 0.0f, 0.0f), FVector(1.0f, -1.0f, 1.0f));
			const FTransform YUpMatrix(FMatrix(FVector(0.0f, 0.0f, 1.0f), FVector(1.0f, 0.0f, 0.0f), FVector(0.0f, 1.0f, 0.0f), FVector(0.0f, 0.0f, 0.0f)));
			const FTransform YUpMatrixInv(YUpMatrix.Inverse());

			switch (SourceCoordSystem)
			{
				case ECADKernelModelCoordSystem::ZUp_RightHanded:
				{
					return RightHanded * LocalTransform * RightHanded;
				}
				case ECADKernelModelCoordSystem::YUp_LeftHanded:
				{
					return YUpMatrix * LocalTransform * YUpMatrixInv;
				}
				case ECADKernelModelCoordSystem::YUp_RightHanded:
				{
					return RightHanded * YUpMatrix * LocalTransform * YUpMatrixInv * RightHanded;
				}
				case ECADKernelModelCoordSystem::ZUp_RightHanded_FBXLegacy:
				{
					return RightHandedLegacy * LocalTransform * RightHandedLegacy;
				}
				default:
				{
					return LocalTransform;
				}
			}
		}

		template<typename VecType>
		void ConvertVectorArray(ECADKernelModelCoordSystem ModelCoordSys, TArrayView<VecType>& Array)
		{
			switch (ModelCoordSys)
			{
			case ECADKernelModelCoordSystem::YUp_LeftHanded:
				for (VecType& Vector : Array)
				{
					Vector.Set(Vector[2], Vector[0], Vector[1]);
				}
				break;

			case ECADKernelModelCoordSystem::YUp_RightHanded:
				for (VecType& Vector : Array)
				{
					Vector.Set(-Vector[2], Vector[0], Vector[1]);
				}
				break;

			case ECADKernelModelCoordSystem::ZUp_RightHanded:
				for (VecType& Vector : Array)
				{
					Vector.Set(-Vector[0], Vector[1], Vector[2]);
				}
				break;

			case ECADKernelModelCoordSystem::ZUp_RightHanded_FBXLegacy:
				for (VecType& Vector : Array)
				{
					Vector.Set(Vector[0], -Vector[1], Vector[2]);
				}
				break;

			case ECADKernelModelCoordSystem::ZUp_LeftHanded:
			default:
				break;
			}
		}

		template<typename VecType>
		void ConvertVectorArray(uint8 ModelCoordSys, TArrayView<VecType>& Array)
		{
			ConvertVectorArray(static_cast<ECADKernelModelCoordSystem>(ModelCoordSys), Array);
		}

		template<typename VecType>
		void ConvertVectorArray(ECADKernelModelCoordSystem ModelCoordSys, TArray<VecType>& Array)
		{
			TArrayView<VecType> ArrayView(Array);
			ConvertVectorArray(ModelCoordSys, ArrayView);
		}

		template<typename VecType>
		void ConvertVectorArray(uint8 ModelCoordSys, TArray<VecType>& Array)
		{
			TArrayView<VecType> ArrayView(Array);
			ConvertVectorArray(static_cast<ECADKernelModelCoordSystem>(ModelCoordSys), ArrayView);
		}

		template<typename VecType>
		VecType ConvertVector(ECADKernelModelCoordSystem ModelCoordSys, const VecType& V)
		{
			switch (ModelCoordSys)
			{
			case ECADKernelModelCoordSystem::YUp_LeftHanded:
				return VecType(V[2], V[0], V[1]);

			case ECADKernelModelCoordSystem::YUp_RightHanded:
				return VecType(-V[2], V[0], V[1]);

			case ECADKernelModelCoordSystem::ZUp_RightHanded:
				return VecType(-V[0], V[1], V[2]);

			case ECADKernelModelCoordSystem::ZUp_RightHanded_FBXLegacy:
				return VecType(V[0], -V[1], V[2]);

			case ECADKernelModelCoordSystem::ZUp_LeftHanded:
			default:
				return VecType(V[0], V[1], V[2]);
			}
		}

		template<typename VecType>
		VecType ConvertVector(uint8 ModelCoordSys, const VecType& V)
		{
			return ConvertVector(static_cast<ECADKernelModelCoordSystem>(ModelCoordSys), V);
		}
	}
}
#endif
