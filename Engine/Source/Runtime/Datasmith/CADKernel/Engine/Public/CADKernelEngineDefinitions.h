// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "CADKernelEngineDefinitions.generated.h"

UENUM(BlueprintType)
enum class ECADKernelModelCoordSystem : uint8
{
	ZUp_LeftHanded,
	ZUp_RightHanded,
	YUp_LeftHanded,
	YUp_RightHanded,
	ZUp_RightHanded_FBXLegacy,
};

UENUM(BlueprintType)
enum ECADKernelStitchingTechnique : uint8
{
	StitchingNone = 0,
	StitchingHeal,
	StitchingSew,
};

UENUM(BlueprintType)
enum ECADKernelTessellator : uint8
{
	CADKernel = 0,
	TechSoft,
};

USTRUCT(BlueprintType)
struct CADKERNELENGINE_API FCADKernelModelParameters
{
	GENERATED_BODY()

	// value from FDatasmithUtils::ECADKernelModelCoordSystem
	UPROPERTY()
	uint8 ModelCoordSys = uint8(ECADKernelModelCoordSystem::ZUp_LeftHanded);

	UPROPERTY()
	float ModelUnitToCentimeter = 1.; // Model is in centimeters

	UPROPERTY()
	float ScaleFactor = 1.0f;
	
	bool FromJson(const FString& JsonString) const { return false; }
	FString ToJson() const { return {}; }
};

USTRUCT(BlueprintType)
struct CADKERNELENGINE_API FCADKernelMeshParameters
{
	GENERATED_BODY()

	UPROPERTY()
	bool bNeedSwapOrientation = false;

	UPROPERTY()
	bool bIsSymmetric = false;

	UPROPERTY()
	FVector SymmetricOrigin = FVector::ZeroVector;

	UPROPERTY()
	FVector SymmetricNormal = FVector::ZeroVector;

	UPROPERTY()
	float ScaleFactor = 1.0f;

	bool FromJson(const FString& JsonString) const { return false; }
	FString ToJson() const { return {}; }
};

USTRUCT(BlueprintType)
struct CADKERNELENGINE_API FCADKernelTessellationSettings
{
	GENERATED_BODY()

	FCADKernelTessellationSettings(
			double InChordTolerance = 0.2,
			double InMaxEdgeLength = 0.0,
			double InNormalTolerance = 20.0,
			ECADKernelStitchingTechnique InStitchingTechnique = ECADKernelStitchingTechnique::StitchingSew,
			double InGeometricTolerance = 0.001,
			double InStichingTolerance = 0.001,
			ECADKernelTessellator InTessellator = ECADKernelTessellator::CADKernel
		)
		: ChordTolerance(InChordTolerance)
		, MaxEdgeLength(InMaxEdgeLength)
		, NormalTolerance(InNormalTolerance)
		, StitchingTechnique(InStitchingTechnique)
		, Tessellator(InTessellator)
		, GeometricTolerance(InGeometricTolerance)
		, StitchingTolerance(InStichingTolerance)
	{
		Tessellator = bWithTechSoft ? InTessellator : ECADKernelTessellator::CADKernel;
	}

protected:
	/**
	 * Maximum distance between any point on a triangle generated by the tessellation process and the actual surface.
	 * The lower the value the more triangles.
	 * Default value is 0.2, minimal value is 0.005 cm.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry & Tessellation Options", meta = (Units = cm, ToolTip = "Maximum distance between any generated triangle and the original surface. Smaller values make more triangles.", ClampMin = "0.005"))
	double ChordTolerance;

	/**
	 * Maximum length of edges of triangles generated by the tessellation process.
	 * The length is in scene/model unit. The smaller the more triangles are generated.
	 * Value of 0 means no constraint on length of edges
	 * Default value is 0 to disable this criteria, and 1. cm is its minimal value if enable.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = cm, DisplayName = "Max Edge Length", ToolTip = "Maximum length of any edge in the generated triangles. Smaller values make more triangles.", ClampMin = "0.0"))
	double MaxEdgeLength;

public:
	/**
	 * Maximum angle between the normal of two triangles generated by the tessellation process.
	 * The angle is expressed in degree. The smaller the more triangles are generated.
	 * Default value is 20 degrees, min value is 5 degrees.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = deg, ToolTip = "Maximum angle between adjacent triangles. Smaller values make more triangles.", ClampMin = "5.0", ClampMax = "90.0"))
	double NormalTolerance;


	/**
	 * Stitching technique applied on neighboring surfaces before tessellation.
	 * None : No stitching applied. This is the default.
	 * Sewing : Connects surfaces which physically share a boundary but not topologically within a set of objects.
	 *          This technique can modify the structure of the model by removing and adding objects.
	 * Healing : Connects surfaces which physically share a boundary but not topologically within an object.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (ToolTip = "Stitching technique applied on model before tessellation. Sewing could impact number of objects."))
	TEnumAsByte<ECADKernelStitchingTechnique> StitchingTechnique;

	/**
	 * Indicates which tessellator to use: CADKernel or TechSoft. By default it is CADKernel.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Tooltip = "Select tessellator to use. Default to CADKernel. Disabled if TechSoft is not available.", editCondition = "bWithTechSoft == true"))
	TEnumAsByte<ECADKernelTessellator> Tessellator = ECADKernelTessellator::CADKernel;
	
	/**
	 * T-Junctions are edge configurations where a border vertex lies on an opposing border edge.
	 * If true, the mesh is analyzed and all T-Junctions found on the mesh will be resolved.
	 * Only applicable with the TechSoft tessellator. Default is false
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (EditCondition = "Tessellator == ECADKernelTessellator::TechSoft", EditConditionHides, Tooltip = "If true, enables the removal of T-Junctions from the resulting mesh. Default is false."))
	bool bResolveTJunctions = false;

protected:
	/**
	 * Tolerance used to determine if a surface should be tessellate or not.
	 * Any surface which is narrower than the geometric tolerance
	 * in one of the iso direction will not be tessellated
	 * Value is in centimeter
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = cm, ToolTip = "Tolerance used to determine if a surface should be tessellated or not."))
	double GeometricTolerance = 0.001;

	/**
	 * Tolerance used to determine if two surfaces should be stitched.
	 * Value is in centimeter
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = cm, ToolTip = "Tolerance used to determine if two surfaces should be stitched.", editCondition = "StitchingTechnique!=ECADKernelStitchingTechnique::StitchingNone"))
	double StitchingTolerance = 0.001;

public:
	/*
	 ** Value to convert imported geometrical values to cm which is UE's unit.
	 ** Default value is 0.1 as CADKernel's and most CAD formats' unit is millimeter.
	 */
	// #cadkernel_check: Verify that this new logic is properly used everywhere.
	UPROPERTY()
	double UnitMultiplier = 1.;

	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bWithTechSoft = bWithHoops;

	static bool bWithHoops;

	bool operator == (const FCADKernelTessellationSettings& Other) const
	{
		return FMath::IsNearlyEqual(ChordTolerance, Other.ChordTolerance)
			&& FMath::IsNearlyEqual(MaxEdgeLength, Other.MaxEdgeLength)
			&& FMath::IsNearlyEqual(NormalTolerance, Other.NormalTolerance)
			&& StitchingTechnique == Other.StitchingTechnique
			&& Tessellator == Other.Tessellator
			&& FMath::IsNearlyEqual(GeometricTolerance, Other.GeometricTolerance)
			&& FMath::IsNearlyEqual(StitchingTolerance, Other.StitchingTolerance);
	}

	uint32 GetHash() const
	{
		uint32 Hash = HashCombine(GetTypeHash(StitchingTechnique), GetTypeHash(Tessellator));
		for (double Param : {ChordTolerance, MaxEdgeLength, NormalTolerance, GeometricTolerance, StitchingTolerance})
		{
			Hash = HashCombine(Hash, GetTypeHash(Param));
		}
		return Hash;
	}
	/** Helper functions to get geometrical values in the right unit, cm (native) or mm */
	double GetGeometricTolerance(bool bInMillimeter = false) const { return bInMillimeter ? GeometricTolerance * 10. : GeometricTolerance; }
	double GetStitchingTolerance(bool bInMillimeter = false) const { return bInMillimeter ? StitchingTolerance * 10. : StitchingTolerance; }
	double GetChordTolerance(bool bInMillimeter = false) const { return bInMillimeter ? ChordTolerance * 10. : ChordTolerance; }
	double GetMaxEdgeLength(bool bInMillimeter = false) const { return bInMillimeter ? MaxEdgeLength * 10. : MaxEdgeLength; }

	bool FromJson(const FString& JsonString) const { return false; }
	FString ToJson() const { return {}; }

	bool UseCADKernel() const { return Tessellator == ECADKernelTessellator::CADKernel; }
};

UENUM()
enum class ECADKernelRetessellationRule : uint8
{
	AllFaces = 0,
	SkipDeletedFaces,
};

USTRUCT(BlueprintType)
struct CADKERNELENGINE_API FCADKernelRetessellationSettings : public FCADKernelTessellationSettings
{
	GENERATED_BODY()

	FCADKernelRetessellationSettings(double InChordTolerance = 0.2f, double InMaxEdgeLength = 0.0f, double InNormalTolerance = 20.0f, ECADKernelStitchingTechnique InStitchingTechnique = ECADKernelStitchingTechnique::StitchingSew, ECADKernelRetessellationRule InRetessellationRule = ECADKernelRetessellationRule::AllFaces)
		: FCADKernelTessellationSettings(InChordTolerance, InMaxEdgeLength, InNormalTolerance, InStitchingTechnique)
		, RetessellationRule(InRetessellationRule)
	{
	}

	FCADKernelRetessellationSettings(const FCADKernelTessellationSettings& TessellationSettings, ECADKernelRetessellationRule InRetessellationRule = ECADKernelRetessellationRule::AllFaces)
		: FCADKernelTessellationSettings(TessellationSettings)
		, RetessellationRule(InRetessellationRule)
	{
	}

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Retessellation Settings", meta = (ToolTip = "Regenerate deleted surfaces during retesselate or ignore them"))
	ECADKernelRetessellationRule RetessellationRule = ECADKernelRetessellationRule::AllFaces;
};

namespace UE::CADKernel
{
	struct FUnitConverter
	{
		static const double CADKERNELENGINE_API CentimeterToMillimeter;
		static const double CADKERNELENGINE_API MillimeterToCentimeter;
		static const double CADKERNELENGINE_API CentimeterToMeter;
		static const double CADKERNELENGINE_API MeterToCentimeter;
		static const double CADKERNELENGINE_API MillimeterToMeter;
		static const double CADKERNELENGINE_API MeterToMillimeter;
	};

	struct FTechSoftImportOverrides
	{
		bool bLoadGeometryOnly = true;
		bool bLoadLoadNoDependency = false;
		bool bLoadStructureOnly = false;
	};

	struct FMeshExtractionContext
	{
		FCADKernelModelParameters ModelParams;
		FCADKernelMeshParameters MeshParams;
		mutable TSet<int32> FaceGroupsToExtract;
		bool bResolveTJunctions = false;
	};

	struct FTessellationContext : public FMeshExtractionContext
	{
		FCADKernelTessellationSettings TessellationSettings;
		CADKERNELENGINE_API FTessellationContext() {};
		CADKERNELENGINE_API FTessellationContext(const FCADKernelModelParameters& ModelParams, const FCADKernelMeshParameters& MeshParams, const FCADKernelRetessellationSettings& Settings);
	};
}
