// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

#ifdef USE_TECHSOFT_SDK

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#pragma push_macro("TEXT")
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS // unsafe sprintf
#include "A3DSDKIncludes.h"
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
#pragma pop_macro("TEXT")
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define WITH_HOOPS
#else
typedef void A3DRiRepresentationItem;
#endif

