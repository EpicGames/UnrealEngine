// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TechSoftIncludes.h"

#ifdef WITH_HOOPS
namespace UE::CADKernel::TechSoft
{
	// Single-ownership smart TeshSoft object
	// Use this when you need to manage TechSoft object's lifetime.
	//
	// TechSoft give access to void pointers
	// According to the context, the class name of the void pointer is known but the class is unknown
	// i.e. A3DSDKTypes.h defines all type like :
	// 	   typedef void A3DEntity;		
	// 	   typedef void A3DAsmModelFile; ...
	// 
	// From a pointer, TechSoft give access to a copy of the associated structure :
	//
	// const A3DXXXXX* pPointer;
	// A3DXXXXXData sData; // the structure
	// A3D_INITIALIZE_DATA(A3DXXXXXData, sData); // initialization of the structure
	// A3DXXXXXXGet(pPointer, &sData); // Copy of the data of the pointer in the structure
	// ...
	// A3DXXXXXXGet(NULL, &sData); // Free the structure
	//
	// A3D_INITIALIZE_DATA, and all A3DXXXXXXGet methods are TechSoft macro
	//

	template<class ObjectType, class IndexerType>
	class TUniqueObjectBase
	{
	public:

		/**
		 * Constructor of an initialized ObjectType object
		 */
		TUniqueObjectBase()
			: bDataFromTechSoft(false)
			, Status(A3DStatus::A3D_SUCCESS)
		{
			InitializeData();
		}

		/**
		 * Constructor of an filled ObjectType object with the data of DataPtr
		 * @param DataPtr: the pointer of the data to copy
		 */
		explicit TUniqueObjectBase(IndexerType DataPtr)
			: bDataFromTechSoft(false)
		{
			InitializeData();
			FillFrom(DataPtr);
		}

		~TUniqueObjectBase()
		{
			ResetData();
		}

		/**
		 * Fill the structure with the data of a new DataPtr
		 */
		A3DStatus FillFrom(IndexerType EntityPtr)
		{
			ResetData();

			if (EntityPtr == GetDefaultIndexerValue())
			{
				Status = A3DStatus::A3D_ERROR;
			}
			else
			{
				Status = GetData(EntityPtr);
				if (Status == A3DStatus::A3D_SUCCESS)
				{
					bDataFromTechSoft = true;
				}
			}
			return Status;
		}

		template<typename... InArgTypes>
		A3DStatus FillWith(A3DStatus(*Getter)(const A3DEntity*, ObjectType*, InArgTypes&&...), const A3DEntity* EntityPtr, InArgTypes&&... Args)
		{
			ResetData();

			if (EntityPtr == GetDefaultIndexerValue())
			{
				Status = A3DStatus::A3D_ERROR;
			}
			else
			{
				Status = Getter(EntityPtr, &Data, Forward<InArgTypes>(Args)...);
				if (Status == A3DStatus::A3D_SUCCESS)
				{
					bDataFromTechSoft = true;
				}
			}

			return Status;
		}

		/**
		 * Empty the structure
		 */
		void Reset()
		{
			ResetData();
		}

		/**
		 * Return
		 *  - A3DStatus::A3D_SUCCESS if the data is filled
		 *  - A3DStatus::A3D_ERROR if the data is empty
		 */
		A3DStatus GetStatus()
		{
			return Status;
		}

		/**
		 * Return true if the data is filled
		 */
		const bool IsValid() const
		{
			return Status == A3DStatus::A3D_SUCCESS;
		}

		// Non-copyable
		TUniqueObjectBase(const TUniqueObjectBase&) = delete;
		TUniqueObjectBase& operator=(const TUniqueObjectBase&) = delete;

		// Conversion methods

		const ObjectType& operator*() const
		{
			return Data;
		}

		ObjectType& operator*()
		{
			check(IsValid());
			return Data;
		}

		const ObjectType* operator->() const
		{
			check(IsValid());
			return &Data;
		}

		ObjectType* operator->()
		{
			check(IsValid());
			return &Data;
		}

		/**
		 * Return the structure pointer
		 */
		ObjectType* GetPtr()
		{
			if (Status != A3DStatus::A3D_SUCCESS)
			{
				return nullptr;
			}
			return &Data;
		}

	private:
		ObjectType Data;
		bool bDataFromTechSoft = false;
		A3DStatus Status = A3DStatus::A3D_ERROR;

		/**
		 * DefaultValue is used to initialize "Data" with GetData method
		 * According to IndexerType, the value is either nullptr for const A3DEntity* either something like "A3D_DEFAULT_MATERIAL_INDEX" ((A3DUns16)-1) for uint32
		 * @see ResetData
		 */
		CADKERNELENGINE_API IndexerType GetDefaultIndexerValue() const;

		CADKERNELENGINE_API void InitializeData();

		CADKERNELENGINE_API A3DStatus GetData(IndexerType AsmModelFilePtr);

		void ResetData()
		{
			if (bDataFromTechSoft)
			{
				GetData(GetDefaultIndexerValue());
			}
			else
			{
				InitializeData();
			}
			Status = A3DStatus::A3D_SUCCESS;
			bDataFromTechSoft = false;
		}
	};

	template<class ObjectType>
	using TUniqueObject = TUniqueObjectBase<ObjectType, const A3DEntity*>;

	template<class ObjectType>
	using TUniqueObjectFromIndex = TUniqueObjectBase<ObjectType, uint32>;

	class CADKERNELENGINE_API FTechSoftDefaultValue
	{
	public:
		static const uint32 Material;
		static const uint32 Picture;
		static const uint32 RgbColor;
		static const uint32 Style;
		static const uint32 TextureApplication;
		static const uint32 TextureDefinition;
	};

}
#endif

