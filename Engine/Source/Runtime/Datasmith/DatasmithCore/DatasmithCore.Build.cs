// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithCore : ModuleRules
	{
		public DatasmithCore(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DirectLink",
					"MeshDescription",
					"RawMesh",
					"StaticMeshDescription",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Json",
					"XmlParser",
				}
			);

			bDisableAutoRTFMInstrumentation = true;
		}
	}
}
