// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithAnimationElementsImpl.h"
#include "DatasmithSceneElementsImpl.h"

#include "DatasmithDefinitions.h"

FDatasmithLevelSequenceElementImpl::FDatasmithLevelSequenceElementImpl(const TCHAR* InName)
	: FDatasmithElementImpl(InName, EDatasmithElementType::LevelSequence)
	, FrameRate(30.f)
{
	Store.RegisterParameter(File,      "File"     );
	Store.RegisterParameter(FileHash,  "FileHash" );
	Store.RegisterParameter(FrameRate, "FrameRate");
}

FDatasmithTransformAnimationElementImpl::FDatasmithTransformAnimationElementImpl(const TCHAR* InName)
	: FDatasmithBaseAnimationElementImpl(InName, EDatasmithElementType::Animation, EDatasmithElementAnimationSubType::TransformAnimation)
	, EnabledChannels(EDatasmithTransformChannels::All)
{
	TransformCurveInterpMode[0] = EDatasmithCurveInterpMode::Linear;
	TransformCurveInterpMode[1] = EDatasmithCurveInterpMode::Linear;
	TransformCurveInterpMode[2] = EDatasmithCurveInterpMode::Linear;
}

FDatasmithVisibilityAnimationElementImpl::FDatasmithVisibilityAnimationElementImpl(const TCHAR* InName)
	: FDatasmithBaseAnimationElementImpl(InName, EDatasmithElementType::Animation, EDatasmithElementAnimationSubType::VisibilityAnimation)
	, bPropagate(false)
{
	CurveInterpMode = EDatasmithCurveInterpMode::Linear;
}

FDatasmithSubsequenceAnimationElementImpl::FDatasmithSubsequenceAnimationElementImpl(const TCHAR* InName)
	: FDatasmithBaseAnimationElementImpl(InName, EDatasmithElementType::Animation, EDatasmithElementAnimationSubType::SubsequenceAnimation)
	, StartTime(FFrameNumber(0))
	, Duration(0)
	, TimeScale(1.0f)
	, Subsequence(nullptr)
{
}