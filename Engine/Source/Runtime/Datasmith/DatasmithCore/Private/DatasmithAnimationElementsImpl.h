// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithAnimationElements.h"
#include "DatasmithSceneElementsImpl.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class FDatasmithLevelSequenceElementImpl : public FDatasmithElementImpl< IDatasmithLevelSequenceElement  >
{
public:
	explicit FDatasmithLevelSequenceElementImpl(const TCHAR* InName);

	virtual const TCHAR* GetFile() const override { return *File.Get(); }
	virtual void SetFile(const TCHAR* InFile) override { File = InFile; };

	virtual FMD5Hash GetFileHash() const override { return FileHash; }
	virtual void SetFileHash(FMD5Hash Hash) override { FileHash = Hash; }

	virtual float GetFrameRate() const override { return FrameRate; }
	virtual void SetFrameRate(float FramePerSecs) override { FrameRate = FramePerSecs; }

	virtual void AddAnimation(const TSharedRef< IDatasmithBaseAnimationElement >& InAnimation) override { Animations.Add(InAnimation); }
	virtual int32 GetAnimationsCount() const override { return Animations.Num(); }

	virtual TSharedPtr< IDatasmithBaseAnimationElement > GetAnimation(int32 InIndex) override
	{
		return Animations.IsValidIndex(InIndex) ? Animations[InIndex] : TSharedPtr< IDatasmithBaseAnimationElement >();
	}

	virtual void RemoveAnimation(const TSharedRef< IDatasmithBaseAnimationElement >& InAnimation) override { Animations.Remove(InAnimation); }

private:
	TReflected<FString> File;
	TReflected<FMD5Hash> FileHash;
	TReflected<float> FrameRate;

	TArray< TSharedRef< IDatasmithBaseAnimationElement > > Animations;
};

template< typename InterfaceType >
class FDatasmithBaseAnimationElementImpl : public FDatasmithElementImpl< InterfaceType >
{
public:
	explicit FDatasmithBaseAnimationElementImpl(const TCHAR* InName, EDatasmithElementType ChildType, EDatasmithElementAnimationSubType InSubtype = EDatasmithElementAnimationSubType::BaseAnimation);

	virtual bool IsSubType( const EDatasmithElementAnimationSubType AnimSubType ) const override { return FDatasmithElementImpl< InterfaceType >::IsSubTypeInternal( (uint64)AnimSubType ); }

	virtual void SetCompletionMode(EDatasmithCompletionMode InCompletionMode) override
	{
		CompletionMode = InCompletionMode;
	}

	virtual EDatasmithCompletionMode GetCompletionMode() const override
	{
		return CompletionMode;
	}

private:
	EDatasmithCompletionMode CompletionMode;
};

template< typename T >
inline FDatasmithBaseAnimationElementImpl<T>::FDatasmithBaseAnimationElementImpl(const TCHAR* InName, EDatasmithElementType ChildType, EDatasmithElementAnimationSubType InSubtype)
	: FDatasmithElementImpl<T>(InName, EDatasmithElementType::Animation | ChildType, (uint64)InSubtype)
	, CompletionMode(EDatasmithCompletionMode::ProjectDefault)
{
}

class FDatasmithTransformAnimationElementImpl : public FDatasmithBaseAnimationElementImpl< IDatasmithTransformAnimationElement >
{
public:
	explicit FDatasmithTransformAnimationElementImpl(const TCHAR* InName);

	virtual void AddFrame(EDatasmithTransformType TransformType, const FDatasmithTransformFrameInfo& FrameInfo) override { Frames[(uint8)TransformType].Add(FrameInfo); }
	virtual int32 GetFramesCount(EDatasmithTransformType TransformType) const override { return Frames[(uint8)TransformType].Num(); }

	virtual void SetCurveInterpMode(EDatasmithTransformType TransformType, EDatasmithCurveInterpMode CurveInterpMode) override
	{
		TransformCurveInterpMode[(uint32) TransformType] = CurveInterpMode;
	}

	virtual EDatasmithCurveInterpMode GetCurveInterpMode(EDatasmithTransformType TransformType) const override
	{
		return TransformCurveInterpMode[(uint32) TransformType];
	}

	virtual const FDatasmithTransformFrameInfo& GetFrame(EDatasmithTransformType TransformType, int32 Index) const override
	{
		return Frames[(uint8)TransformType].IsValidIndex(Index) ? Frames[(uint8)TransformType][Index] : FDatasmithTransformFrameInfo::InvalidFrameInfo;
	}

	virtual void RemoveFrame(EDatasmithTransformType TransformType, int32 Index) override
	{
		if (Frames[(uint8)TransformType].IsValidIndex(Index))
		{
			Frames[(uint8)TransformType].RemoveAt(Index);
		}
	}

	virtual EDatasmithTransformChannels GetEnabledTransformChannels() const override
	{
		return EnabledChannels;
	}

	virtual void SetEnabledTransformChannels(EDatasmithTransformChannels Channels) override
	{
		EnabledChannels = Channels;
	}

private:
	TArray< FDatasmithTransformFrameInfo > Frames[(uint8) EDatasmithTransformType::Count];
	EDatasmithCurveInterpMode TransformCurveInterpMode[(uint8) EDatasmithTransformType::Count];
	EDatasmithTransformChannels EnabledChannels;
};

class FDatasmithVisibilityAnimationElementImpl : public FDatasmithBaseAnimationElementImpl< IDatasmithVisibilityAnimationElement >
{
public:
	explicit FDatasmithVisibilityAnimationElementImpl(const TCHAR* InName);

	virtual void AddFrame(const FDatasmithVisibilityFrameInfo& FrameInfo) override { Frames.Add(FrameInfo); }
	virtual int32 GetFramesCount() const override { return Frames.Num(); }

	virtual void SetCurveInterpMode(EDatasmithCurveInterpMode InCurveInterpMode) override
	{
		CurveInterpMode = InCurveInterpMode;
	}

	virtual EDatasmithCurveInterpMode GetCurveInterpMode() const override
	{
		return CurveInterpMode;
	}

	virtual const FDatasmithVisibilityFrameInfo& GetFrame(int32 Index) const override
	{
		return Frames.IsValidIndex(Index) ? Frames[Index] : FDatasmithVisibilityFrameInfo::InvalidFrameInfo;
	}

	virtual void RemoveFrame(int32 Index) override
	{
		if (Frames.IsValidIndex(Index))
		{
			Frames.RemoveAt(Index);
		}
	}

	virtual void SetPropagateToChildren(bool bInPropagate) override
	{
		bPropagate = bInPropagate;
	}

	virtual bool GetPropagateToChildren() const override
	{
		return bPropagate;
	}

private:
	TArray<FDatasmithVisibilityFrameInfo> Frames;
	EDatasmithCurveInterpMode CurveInterpMode;
	bool bPropagate;
};

class FDatasmithSubsequenceAnimationElementImpl : public FDatasmithBaseAnimationElementImpl< IDatasmithSubsequenceAnimationElement >
{
public:
	explicit FDatasmithSubsequenceAnimationElementImpl(const TCHAR* InName);

	virtual FFrameNumber GetStartTime() const override
	{
		return StartTime;
	}

	virtual void SetStartTime(FFrameNumber InStartTime) override
	{
		StartTime = InStartTime;
	}

	virtual int32 GetDuration() const override
	{
		return Duration;
	}

	virtual void SetDuration(int32 InDuration) override
	{
		Duration = InDuration;
	}

	virtual float GetTimeScale() const override
	{
		return TimeScale;
	}

	virtual void SetTimeScale(float InTimeScale) override
	{
		TimeScale = InTimeScale;
	}

	virtual TWeakPtr<IDatasmithLevelSequenceElement> GetSubsequence() const override
	{
		return Subsequence;
	}

	virtual void SetSubsequence(TWeakPtr<IDatasmithLevelSequenceElement> InSubsequence) override
	{
		Subsequence = InSubsequence;
	}

private:
	FFrameNumber StartTime;
	int32 Duration;
	float TimeScale;
	TWeakPtr<IDatasmithLevelSequenceElement> Subsequence;
};
