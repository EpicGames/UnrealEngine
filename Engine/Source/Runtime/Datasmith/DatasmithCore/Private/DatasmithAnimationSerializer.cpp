// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithAnimationSerializer.h"
#include "DatasmithLocaleScope.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"
#include "Misc/FileHelper.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"

// General info fields
#define DSANIM_VERSION			TEXT("version")
#define DSANIM_VERSION_NUMBER	TEXT("0.2")
#define DSANIM_FRAMERATE		TEXT("fps")
#define DSANIM_ANIMATIONS		TEXT("animations")

// Animation fields
#define DSANIM_ACTORNAME		TEXT("actor")
#define DSANIM_ANIMTYPE			TEXT("type")

// Transform animation data fields
#define DSANIM_ANIMTYPE_TRANSFORM	TEXT("transform")
#define DSANIM_TRANSLATION		TEXT("trans")
#define DSANIM_ROTATION			TEXT("rot")
#define DSANIM_SCALE			TEXT("scl")

// Transform frame data fields
#define DSANIM_FRAME_NUMBER		TEXT("id")
#define DSANIM_TRANSFORM_X		TEXT("x")
#define DSANIM_TRANSFORM_Y		TEXT("y")
#define DSANIM_TRANSFORM_Z		TEXT("z")

// Visibility animation data fields
#define DSANIM_ANIMTYPE_VISIBILITY	TEXT("visibility")
#define DSANIM_PROPAGATE		TEXT("prop")

// Visibility frame data fields
#define DSANIM_VISIBILITY		TEXT("vis")

namespace DatasmithAnimationJsonSerializerImpl
{
	TSharedRef<FJsonObject> SerializeTransformFrame(const FDatasmithTransformFrameInfo& FrameInfo, FDatasmithTransformFrameInfo& PreviousFrame, const FDatasmithTransformFrameInfo& NextFrame, ETransformChannelComponents EnabledComponents)
	{
		TSharedRef<FJsonObject> Frame = MakeShared<FJsonObject>();
		Frame->SetNumberField(DSANIM_FRAME_NUMBER, FrameInfo.FrameNumber);

		// Serialize the frame transform only if the values have changed since the previous frame or represent a change with the following frame
		if (EnumHasAnyFlags(EnabledComponents, ETransformChannelComponents::X) &&
			(!FMath::IsNearlyEqual(FrameInfo.X, PreviousFrame.X, DOUBLE_KINDA_SMALL_NUMBER) ||
			(NextFrame.IsValid() && !FMath::IsNearlyEqual(FrameInfo.X, NextFrame.X, DOUBLE_KINDA_SMALL_NUMBER))))
		{
			Frame->SetNumberField(DSANIM_TRANSFORM_X, FrameInfo.X);
			PreviousFrame.X = FrameInfo.X;
		}
		if (EnumHasAnyFlags(EnabledComponents, ETransformChannelComponents::Y) &&
			(!FMath::IsNearlyEqual(FrameInfo.Y, PreviousFrame.Y, DOUBLE_KINDA_SMALL_NUMBER) ||
			(NextFrame.IsValid() && !FMath::IsNearlyEqual(FrameInfo.Y, NextFrame.Y, DOUBLE_KINDA_SMALL_NUMBER))))
		{
			Frame->SetNumberField(DSANIM_TRANSFORM_Y, FrameInfo.Y);
			PreviousFrame.Y = FrameInfo.Y;
		}
		if (EnumHasAnyFlags(EnabledComponents, ETransformChannelComponents::Z) &&
			(!FMath::IsNearlyEqual(FrameInfo.Z, PreviousFrame.Z, DOUBLE_KINDA_SMALL_NUMBER) ||
			(NextFrame.IsValid() && !FMath::IsNearlyEqual(FrameInfo.Z, NextFrame.Z, DOUBLE_KINDA_SMALL_NUMBER))))
		{
			Frame->SetNumberField(DSANIM_TRANSFORM_Z, FrameInfo.Z);
			PreviousFrame.Z = FrameInfo.Z;
		}

		return Frame;
	}

	TSharedRef<FJsonObject> SerializeVisibilityFrame( const FDatasmithVisibilityFrameInfo& FrameInfo, FDatasmithVisibilityFrameInfo& PreviousFrame, const FDatasmithVisibilityFrameInfo& NextFrame )
	{
		TSharedRef<FJsonObject> Frame = MakeShared<FJsonObject>();
		Frame->SetNumberField( DSANIM_FRAME_NUMBER, FrameInfo.FrameNumber );

		// Serialize the frame transform only if the values have changed since the previous frame or represent a change with the following frame
		if ( FrameInfo.bVisible != PreviousFrame.bVisible || ( NextFrame.IsValid() && FrameInfo.bVisible != NextFrame.bVisible ) )
		{
			Frame->SetBoolField( DSANIM_VISIBILITY, FrameInfo.bVisible );
			PreviousFrame.bVisible = FrameInfo.bVisible;
		}

		return Frame;
	}

	ETransformChannelComponents DeserializeTransformFrame(const TSharedRef<FJsonObject>& Frame, FDatasmithTransformFrameInfo& FrameInfo)
	{
		int32 FrameNumber;
		if (!Frame->TryGetNumberField(DSANIM_FRAME_NUMBER, FrameNumber))
		{
			FrameInfo = FDatasmithTransformFrameInfo::InvalidFrameInfo;
		}

		FrameInfo.FrameNumber = FrameNumber;

		ETransformChannelComponents ChannelsWithData = ETransformChannelComponents::None;

		double Value;
		if (Frame->TryGetNumberField(DSANIM_TRANSFORM_X, Value))
		{
			ChannelsWithData |= ETransformChannelComponents::X;
			FrameInfo.X = Value;
		}

		if (Frame->TryGetNumberField(DSANIM_TRANSFORM_Y, Value))
		{
			ChannelsWithData |= ETransformChannelComponents::Y;
			FrameInfo.Y = Value;
		}

		if (Frame->TryGetNumberField(DSANIM_TRANSFORM_Z, Value))
		{
			ChannelsWithData |= ETransformChannelComponents::Z;
			FrameInfo.Z = Value;
		}

		return ChannelsWithData;
	}

	bool DeserializeVisibilityFrame( const TSharedRef<FJsonObject>& Frame, FDatasmithVisibilityFrameInfo& FrameInfo )
	{
		int32 FrameNumber;
		if ( !Frame->TryGetNumberField( DSANIM_FRAME_NUMBER, FrameNumber ) )
		{
			FrameInfo = FDatasmithVisibilityFrameInfo::InvalidFrameInfo;
		}

		FrameInfo.FrameNumber = FrameNumber;

		bool bHasData = false;

		bool Value;
		if ( Frame->TryGetBoolField( DSANIM_VISIBILITY, Value ) )
		{
			bHasData = true;
			FrameInfo.bVisible = Value;
		}

		return bHasData;
	}

	TArray<TSharedPtr<FJsonValue>> SerializeTransformFrames(TSharedPtr<IDatasmithTransformAnimationElement> AnimationElement, EDatasmithTransformType TransformType, EDatasmithTransformChannels EnabledChannels)
	{
		TArray<TSharedPtr<FJsonValue>> Frames;
		int32 NumFrames = AnimationElement->GetFramesCount(TransformType);
		if (NumFrames > 0)
		{
			ETransformChannelComponents EnabledComponents = FDatasmithAnimationUtils::GetChannelTypeComponents(EnabledChannels, TransformType);
			if (EnabledComponents != ETransformChannelComponents::None)
			{
				FDatasmithTransformFrameInfo PreviousFrame(MAX_uint32, MAX_flt, MAX_flt, MAX_flt);
				for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
				{
					const FDatasmithTransformFrameInfo& FrameInfo = AnimationElement->GetFrame(TransformType, FrameIndex);
					if (FrameInfo.IsValid())
					{
						const FDatasmithTransformFrameInfo& NextFrameInfo = FrameIndex < NumFrames - 1 ? AnimationElement->GetFrame(TransformType, FrameIndex + 1) : FDatasmithTransformFrameInfo::InvalidFrameInfo;
						TSharedRef<FJsonObject> Frame = SerializeTransformFrame(FrameInfo, PreviousFrame, NextFrameInfo, EnabledComponents);

						// Add the frame only if there are some values in it
						if (Frame->HasField(DSANIM_TRANSFORM_X) || Frame->HasField(DSANIM_TRANSFORM_Y) || Frame->HasField(DSANIM_TRANSFORM_Z))
						{
							Frames.Add(MakeShared<FJsonValueObject>(Frame));
						}
					}
				}
			}
		}
		return Frames;
	}

	TArray<TSharedPtr<FJsonValue>> SerializeVisibilityFrames( TSharedPtr<IDatasmithVisibilityAnimationElement> AnimationElement )
	{
		TArray<TSharedPtr<FJsonValue>> Frames;
		int32 NumFrames = AnimationElement->GetFramesCount();
		if ( NumFrames > 0 )
		{
			FDatasmithVisibilityFrameInfo PreviousFrame( MAX_uint32, false );
			for ( int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex )
			{
				const FDatasmithVisibilityFrameInfo& FrameInfo = AnimationElement->GetFrame( FrameIndex );
				if ( FrameInfo.IsValid() )
				{
					const FDatasmithVisibilityFrameInfo& NextFrameInfo = FrameIndex < NumFrames - 1 ? AnimationElement->GetFrame( FrameIndex + 1 ) : FDatasmithVisibilityFrameInfo::InvalidFrameInfo;
					TSharedRef<FJsonObject> Frame = SerializeVisibilityFrame( FrameInfo, PreviousFrame, NextFrameInfo );

					// Add the frame only if there are some values in it
					if ( Frame->HasField( DSANIM_VISIBILITY ) )
					{
						Frames.Add( MakeShared<FJsonValueObject>( Frame ) );
					}
				}
			}
		}
		return Frames;
	}

	ETransformChannelComponents DeserializeTransformFrames(const TSharedRef<IDatasmithTransformAnimationElement>& AnimationElement, EDatasmithTransformType TransformType, const TArray<TSharedPtr<FJsonValue>>& Frames)
	{
		ETransformChannelComponents ChannelsWithData = ETransformChannelComponents::None;
		FDatasmithTransformFrameInfo FrameInfo(FDatasmithTransformFrameInfo::InvalidFrameInfo);

		for (const TSharedPtr<FJsonValue>& Frame : Frames)
		{
			ETransformChannelComponents ChannelsWidthDataInFrame = DeserializeTransformFrame(Frame->AsObject().ToSharedRef(), FrameInfo);
			if (FrameInfo.IsValid())
			{
				AnimationElement->AddFrame(TransformType, FrameInfo);
				ChannelsWithData |= ChannelsWidthDataInFrame;
			}
		}
		return ChannelsWithData;
	}

	bool DeserializeVisibilityFrames( const TSharedRef<IDatasmithVisibilityAnimationElement>& AnimationElement, const TArray<TSharedPtr<FJsonValue>>& Frames )
	{
		FDatasmithVisibilityFrameInfo FrameInfo( FDatasmithVisibilityFrameInfo::InvalidFrameInfo );

		bool bHasData = false;
		for ( const TSharedPtr<FJsonValue>& Frame : Frames )
		{
			bool bFrameHasData = DeserializeVisibilityFrame( Frame->AsObject().ToSharedRef(), FrameInfo );
			if ( FrameInfo.IsValid() )
			{
				AnimationElement->AddFrame( FrameInfo );
				bHasData |= bFrameHasData;
			}
		}
		return bHasData;
	}

	TSharedRef<FJsonObject> SerializeAnimation(const TSharedRef<IDatasmithBaseAnimationElement>& AnimationElement)
	{
		TSharedRef<FJsonObject> Animation = MakeShared<FJsonObject>();
		FString ActorName = AnimationElement->GetName();
		Animation->SetStringField(DSANIM_ACTORNAME, ActorName);

		if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
		{
			Animation->SetStringField(DSANIM_ANIMTYPE, DSANIM_ANIMTYPE_TRANSFORM);

			const TSharedRef< IDatasmithTransformAnimationElement > TransformAnimation = StaticCastSharedRef<IDatasmithTransformAnimationElement>(AnimationElement);
			EDatasmithTransformChannels EnabledChannels = TransformAnimation->GetEnabledTransformChannels();

			TArray<TSharedPtr<FJsonValue>> Frames = SerializeTransformFrames(TransformAnimation, EDatasmithTransformType::Translation, EnabledChannels);
			if (Frames.Num() > 0)
			{
				Animation->SetArrayField(DSANIM_TRANSLATION, Frames);
			}

			Frames = SerializeTransformFrames(TransformAnimation, EDatasmithTransformType::Rotation, EnabledChannels);
			if (Frames.Num() > 0)
			{
				Animation->SetArrayField(DSANIM_ROTATION, Frames);
			}

			Frames = SerializeTransformFrames(TransformAnimation, EDatasmithTransformType::Scale, EnabledChannels);
			if (Frames.Num() > 0)
			{
				Animation->SetArrayField(DSANIM_SCALE, Frames);
			}
		}
		else if ( AnimationElement->IsSubType( EDatasmithElementAnimationSubType::VisibilityAnimation ) )
		{
			Animation->SetStringField( DSANIM_ANIMTYPE, DSANIM_ANIMTYPE_VISIBILITY );

			const TSharedRef< IDatasmithVisibilityAnimationElement > Element = StaticCastSharedRef<IDatasmithVisibilityAnimationElement>( AnimationElement );

			TArray<TSharedPtr<FJsonValue>> Frames = SerializeVisibilityFrames( Element );
			if ( Frames.Num() > 0 )
			{
				Animation->SetArrayField( DSANIM_VISIBILITY, Frames );
			}

			Animation->SetBoolField( DSANIM_PROPAGATE, Element->GetPropagateToChildren() );
		}

		return Animation;
	}

	bool DeserializeAnimation(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TSharedPtr<FJsonObject>& Animation)
	{
		FString AnimType;
		if (!Animation->TryGetStringField(DSANIM_ANIMTYPE, AnimType))
		{
			return false;
		}

		FString ActorName;
		if (!Animation->TryGetStringField(DSANIM_ACTORNAME, ActorName))
		{
			return false;
		}

		if (AnimType == DSANIM_ANIMTYPE_TRANSFORM)
		{
			EDatasmithTransformChannels TransformChannels = EDatasmithTransformChannels::None;

			const TSharedRef< IDatasmithTransformAnimationElement > TransformAnimation = FDatasmithSceneFactory::CreateTransformAnimation(*ActorName);
			ETransformChannelComponents Components = ETransformChannelComponents::None;

			const TArray<TSharedPtr<FJsonValue>>* Frames;
			if (Animation->TryGetArrayField(DSANIM_TRANSLATION, Frames))
			{
				Components = DeserializeTransformFrames(TransformAnimation, EDatasmithTransformType::Translation, *Frames);
				TransformChannels |= FDatasmithAnimationUtils::SetChannelTypeComponents(Components, EDatasmithTransformType::Translation);
			}

			if (Animation->TryGetArrayField(DSANIM_ROTATION, Frames))
			{
				Components = DeserializeTransformFrames(TransformAnimation, EDatasmithTransformType::Rotation, *Frames);
				TransformChannels |= FDatasmithAnimationUtils::SetChannelTypeComponents(Components, EDatasmithTransformType::Rotation);
			}

			if (Animation->TryGetArrayField(DSANIM_SCALE, Frames))
			{
				Components = DeserializeTransformFrames(TransformAnimation, EDatasmithTransformType::Scale, *Frames);
				TransformChannels |= FDatasmithAnimationUtils::SetChannelTypeComponents(Components, EDatasmithTransformType::Scale);
			}

			if (TransformChannels != EDatasmithTransformChannels::None)
			{
				TransformAnimation->SetEnabledTransformChannels(TransformChannels);
				LevelSequence->AddAnimation(TransformAnimation);
			}

			return TransformChannels != EDatasmithTransformChannels::None;
		}
		else if ( AnimType == DSANIM_ANIMTYPE_VISIBILITY )
		{
			bool bHasData = false;
			const TSharedRef< IDatasmithVisibilityAnimationElement > Element = FDatasmithSceneFactory::CreateVisibilityAnimation( *ActorName );

			const TArray<TSharedPtr<FJsonValue>>* Frames;
			if ( Animation->TryGetArrayField( DSANIM_VISIBILITY, Frames ) )
			{
				bHasData = DeserializeVisibilityFrames( Element, *Frames );
			}

			bool bPropagate = false;
			if ( Animation->TryGetBoolField( DSANIM_PROPAGATE, bPropagate ) )
			{
				Element->SetPropagateToChildren( bPropagate );
			}

			if ( bHasData )
			{
				LevelSequence->AddAnimation( Element );
			}

			return bHasData;
		}

		return false;
	}

	bool SerializeLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TCHAR* FilePath, bool bDebugFormat)
	{
		TSharedRef<FJsonObject> DatasmithAnimation = MakeShareable(new FJsonObject);

		// Set the general fields for the Datasmith Animation
		DatasmithAnimation->SetStringField(DSANIM_VERSION, DSANIM_VERSION_NUMBER);
		DatasmithAnimation->SetNumberField(DSANIM_FRAMERATE, LevelSequence->GetFrameRate());

		// Serialize each animations in the scene
		TArray<TSharedPtr<FJsonValue>> Animations;
		int32 NumAnims = LevelSequence->GetAnimationsCount();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			const TSharedPtr<IDatasmithBaseAnimationElement>& AnimationElement = LevelSequence->GetAnimation(AnimIndex);
			if (AnimationElement.IsValid())
			{
				TSharedRef<FJsonObject> Animation = SerializeAnimation(AnimationElement.ToSharedRef());

				Animations.Add(MakeShareable(new FJsonValueObject(Animation)));
			}
		}

		DatasmithAnimation->SetArrayField(DSANIM_ANIMATIONS, Animations);

		bool bSuccess = false;
		FString JsonString;
		if (bDebugFormat)
		{
			// Human readable Json format
			auto Writer = TJsonWriterFactory<>::Create(&JsonString);
			bSuccess = FJsonSerializer::Serialize(DatasmithAnimation, Writer);
		}
		else
		{
			// Json without whitespaces
			auto Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonString);
			bSuccess = FJsonSerializer::Serialize(DatasmithAnimation, Writer);
		}

		if (bSuccess)
		{
			// Json spec mentions that default encoding is UTF-8 and must not add BOM
			bSuccess = FFileHelper::SaveStringToFile(JsonString, FilePath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
		}
		return bSuccess;
	}

	bool DeserializeLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TCHAR* FilePath)
	{
		FString JSonString;
		if (!FFileHelper::LoadFileToString(JSonString, FilePath))
		{
			return false;
		}

		auto Reader = TJsonReaderFactory<>::Create(JSonString);
		TSharedPtr<FJsonObject> JsonLevelSequence = MakeShared<FJsonObject>();
		if (!FJsonSerializer::Deserialize(Reader, JsonLevelSequence))
		{
			return false;
		}

		// Could do version checking, but for now just ensure that there's a version field
		FString Version;
		if (!JsonLevelSequence->TryGetStringField(DSANIM_VERSION, Version))
		{
			return false;
		}

		double FrameRate = 0.f;
		if (JsonLevelSequence->TryGetNumberField(DSANIM_FRAMERATE, FrameRate))
		{
			LevelSequence->SetFrameRate(FrameRate);
		}

		const TArray<TSharedPtr<FJsonValue>>* Animations;
		if (JsonLevelSequence->TryGetArrayField(DSANIM_ANIMATIONS, Animations))
		{
			for (TSharedPtr<FJsonValue> JsonAnimation : *Animations)
			{
				DeserializeAnimation(LevelSequence, JsonAnimation->AsObject());
			}
		}

		return LevelSequence->GetAnimationsCount() > 0;
	}
}

bool FDatasmithAnimationSerializer::Serialize(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TCHAR* FilePath, bool bDebugFormat)
{
	FDatasmithLocaleScope LocaleScope;
	return DatasmithAnimationJsonSerializerImpl::SerializeLevelSequence(LevelSequence, FilePath, bDebugFormat);
}

bool FDatasmithAnimationSerializer::Deserialize(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TCHAR* FilePath)
{
	FDatasmithLocaleScope LocaleScope;
	return DatasmithAnimationJsonSerializerImpl::DeserializeLevelSequence(LevelSequence, FilePath);
}

#undef DSANIM_VERSION
#undef DSANIM_VERSION_NUMBER
#undef DSANIM_FRAMERATE
#undef DSANIM_ANIMATIONS
#undef DSANIM_ACTORNAME
#undef DSANIM_ANIMTYPE
#undef DSANIM_ANIMTYPE_TRANSFORM
#undef DSANIM_TRANSLATION
#undef DSANIM_ROTATION
#undef DSANIM_SCALE
#undef DSANIM_FRAME_NUMBER
#undef DSANIM_TRANSFORM_X
#undef DSANIM_TRANSFORM_Y
#undef DSANIM_TRANSFORM_Z
#undef DSANIM_ANIMTYPE_VISIBILITY
#undef DSANIM_PROPAGATE
#undef DSANIM_VISIBILITY
