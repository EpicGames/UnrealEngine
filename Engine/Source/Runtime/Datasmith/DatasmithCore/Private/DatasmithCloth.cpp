// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCloth.h"

#include "Serialization/CustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithCloth)


enum UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") EDatasmithClothSerializationVersion
{
    EDCSV_Base = 0,
    EDCSV_WithPatternParameters = 1,
    EDCSV_WithSewingInfo = 2,

	// -----<new versions can be added before this line>-------------------------------------------------
    _EDCSV_Count,
    _EDCSV_Last = _EDCSV_Count - 1
};

struct UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FDatasmithClothSerializationVersion
{
    const static FGuid GUID;

private:
    FDatasmithClothSerializationVersion() = default;
};

const FGuid FDatasmithClothSerializationVersion::GUID(0x28B01036, 0x66B4498F, 0x99425ACA, 0xDB78A9B5);

PRAGMA_DISABLE_DEPRECATION_WARNINGS

// Register the custom version with core
UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
FCustomVersionRegistration GRegisterDatasmithClothCustomVersion(FDatasmithClothSerializationVersion::GUID, _EDCSV_Last, TEXT("DatasmithCloth"));


FArchive& operator<<(FArchive& Ar, FParameterData& ParameterData)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Ar.UsingCustomVersion(FDatasmithClothSerializationVersion::GUID);

	static_assert(TVariantSize_V<decltype(ParameterData.Data)> == 2, "Serialization code not synced with structure");
	static_assert(decltype(ParameterData.Data)::IndexOfType<TArray<float>>() == 0, "Serialization relies on this specific order");
	static_assert(decltype(ParameterData.Data)::IndexOfType<TArray<double>>() == 1, "Serialization relies on this specific order");

	Ar << ParameterData.Target;
	Ar << ParameterData.Data;

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FDatasmithClothPattern& Pattern)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Ar.UsingCustomVersion(FDatasmithClothSerializationVersion::GUID);
	const int32 ClothSerialVersion = Ar.CustomVer(FDatasmithClothSerializationVersion::GUID);

	Ar << Pattern.SimPosition;
	Ar << Pattern.SimRestPosition;
	Ar << Pattern.SimTriangleIndices;

	if (ClothSerialVersion >= EDCSV_WithPatternParameters)
	{
		Ar << Pattern.Parameters;
	}

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FDatasmithClothSewingInfo& Sewing)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Ar.UsingCustomVersion(FDatasmithClothSerializationVersion::GUID);

	Ar << Sewing.Seam0MeshIndices;
	Ar << Sewing.Seam1MeshIndices;
	Ar << Sewing.Seam0PanelIndex;
	Ar << Sewing.Seam1PanelIndex;

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FDatasmithClothPresetProperty& Property)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Ar.UsingCustomVersion(FDatasmithClothSerializationVersion::GUID);

	Ar << Property.Name;
	Ar << Property.Value;

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FDatasmithClothPresetPropertySet& PropertySet)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Ar.UsingCustomVersion(FDatasmithClothSerializationVersion::GUID);

	Ar << PropertySet.SetName;
	Ar << PropertySet.Properties;

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FDatasmithCloth& Cloth)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Ar.UsingCustomVersion(FDatasmithClothSerializationVersion::GUID);
	const int32 ClothSerialVersion = Ar.CustomVer(FDatasmithClothSerializationVersion::GUID);

	Ar << Cloth.Patterns;
	Ar << Cloth.PropertySets;

	if (ClothSerialVersion >= EDCSV_WithSewingInfo)
	{
		Ar << Cloth.Sewing;
	}

	return Ar;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
