// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithCore.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogDatasmith)

class FDatasmithCoreModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FDatasmithCoreModule, DatasmithCore);
