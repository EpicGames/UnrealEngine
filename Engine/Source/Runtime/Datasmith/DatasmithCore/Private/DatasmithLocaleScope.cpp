// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithLocaleScope.h"

#include "Containers/StringConv.h"

#include <clocale>


FDatasmithLocaleScope::FDatasmithLocaleScope(const TCHAR* InScopedLocale)
{
	if (const char* saved = std::setlocale(LC_ALL, nullptr))
	{
		SavedLocale = ANSI_TO_TCHAR(saved);
	}
	ScopedLocale = ANSI_TO_TCHAR(std::setlocale(LC_ALL, TCHAR_TO_ANSI(InScopedLocale)));
}

FDatasmithLocaleScope::~FDatasmithLocaleScope()
{
	std::setlocale(LC_ALL, TCHAR_TO_ANSI(*SavedLocale));
}

