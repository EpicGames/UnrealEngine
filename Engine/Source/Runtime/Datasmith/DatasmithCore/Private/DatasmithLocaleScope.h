// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"


/**
 * Setup a locale for a given scope, the previous locale is restored on scope end.
 */
struct FDatasmithLocaleScope
{
	// Set a custom locale within this object scope
	FDatasmithLocaleScope(const TCHAR* InScopedLocale = TEXT("C"));

	// restore the captured Locale
	~FDatasmithLocaleScope();

private:
	FString SavedLocale; // Locale previously used, captured, and to be restored
	FString ScopedLocale; // Locale in use within the scope
};

