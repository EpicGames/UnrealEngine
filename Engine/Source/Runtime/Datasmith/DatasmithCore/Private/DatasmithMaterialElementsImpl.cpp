// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaterialElementsImpl.h"


FDatasmithExpressionInputImpl::FDatasmithExpressionInputImpl( const TCHAR* InInputName )
	: FDatasmithElementImpl< IDatasmithExpressionInput >( InInputName, EDatasmithElementType::MaterialExpressionInput )
	, Expression()
	, OutputIndex( 0 )
{
	RegisterReferenceProxy( Expression, "Expression" );

	Store.RegisterParameter( OutputIndex, "OutputIndex" );
}

void FDatasmithExpressionInputImpl::SetExpression( IDatasmithMaterialExpression* InExpression )
{
	if ( InExpression )
	{
		switch ( InExpression->GetExpressionType() )
		{
		case EDatasmithMaterialExpressionType::ConstantBool:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionBoolImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::ConstantColor:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionColorImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::ConstantScalar:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionScalarImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::FlattenNormal:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionFlattenNormalImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::FunctionCall:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionFunctionCallImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::Generic:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionGenericImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::Texture:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionTextureImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::TextureCoordinate:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionTextureCoordinateImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::Custom:
			Expression.Edit() = static_cast<FDatasmithMaterialExpressionCustomImpl*>(InExpression)->AsShared();
			break;
		case EDatasmithMaterialExpressionType::None:
		default:
			check( false );
			break;
		}
	}
	else
	{
		Expression.Edit() = nullptr;
		OutputIndex = 0;
	}
}

void FDatasmithExpressionInputImpl::CustomSerialize(DirectLink::FSnapshotProxy& Ar)
{
	// [4.26.1 .. 4.27.0[ compatibility
	if (Ar.IsSaving())
	{
		// In 4.26, an ExpressionType info was used alongside the expression itself.
		// namely: TReflected<EDatasmithMaterialExpressionType, int32> ExpressionType;
		// This field was required. In order to be readable by 4.26, it is recreated here.
		// Without it, a 4.26 DirectLink receiver could crash on 4.27 data usage.
		const EDatasmithMaterialExpressionType ExpressionTypeEnum = Expression.View().IsValid() ? Expression.View()->GetExpressionType() : EDatasmithMaterialExpressionType::None;
		int32 ExpressionType = int32(ExpressionTypeEnum);
		Ar.TagSerialize("ExpressionType", ExpressionType);
	}
}


FDatasmithMaterialExpressionBoolImpl::FDatasmithMaterialExpressionBoolImpl()
	: FDatasmithExpressionParameterImpl( EDatasmithMaterialExpressionType::ConstantBool )
{
	Store.RegisterParameter( bValue, "bValue" );

	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "Out" ) ) );
}

FDatasmithMaterialExpressionColorImpl::FDatasmithMaterialExpressionColorImpl()
	: FDatasmithExpressionParameterImpl( EDatasmithMaterialExpressionType::ConstantColor )
{
	Store.RegisterParameter( LinearColor, "LinearColor" );

	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "RGB" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "R" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "G" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "B" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "A" ) ) );
}

FDatasmithMaterialExpressionScalarImpl::FDatasmithMaterialExpressionScalarImpl()
	: FDatasmithExpressionParameterImpl( EDatasmithMaterialExpressionType::ConstantScalar )
{
	Store.RegisterParameter( Scalar, "Scalar" );

	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "Out" ) ) );
}

FDatasmithMaterialExpressionTextureImpl::FDatasmithMaterialExpressionTextureImpl()
	: FDatasmithExpressionParameterImpl( EDatasmithMaterialExpressionType::Texture )
	, TextureCoordinate( MakeShared< FDatasmithExpressionInputImpl >( TEXT("Coordinates") ) )
{
	Store.RegisterParameter( TexturePathName, "TexturePathName" );
	RegisterReferenceProxy( TextureCoordinate, "TextureCoordinate" );

	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "RGB" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "R" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "G" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "B" ) ) );
	Outputs.Add( MakeShared< FDatasmithExpressionOutputImpl >( TEXT( "A" ) ) );
}

FDatasmithMaterialExpressionTextureCoordinateImpl::FDatasmithMaterialExpressionTextureCoordinateImpl()
	: FDatasmithMaterialExpressionImpl( EDatasmithMaterialExpressionType::TextureCoordinate )
	, CoordinateIndex( 0 )
	, UTiling( 1.f )
	, VTiling( 1.f )
{
	Store.RegisterParameter( CoordinateIndex, "CoordinateIndex" );
	Store.RegisterParameter( UTiling, "UTiling" );
	Store.RegisterParameter( VTiling, "VTiling" );
}

FDatasmithMaterialExpressionFlattenNormalImpl::FDatasmithMaterialExpressionFlattenNormalImpl()
	: FDatasmithMaterialExpressionImpl( EDatasmithMaterialExpressionType::FlattenNormal )
	, Normal( MakeShared< FDatasmithExpressionInputImpl >( TEXT("Normal") ) )
	, Flatness( MakeShared< FDatasmithExpressionInputImpl >( TEXT("Flatness") ) )
{
	RegisterReferenceProxy( Normal, "Normal" );
	RegisterReferenceProxy( Flatness, "Flatness" );

	Outputs.Add( MakeShared<FDatasmithExpressionOutputImpl>( TEXT( "RGB" ) ) );
}

TSharedPtr< IDatasmithKeyValueProperty > FDatasmithMaterialExpressionGenericImpl::NullPropertyPtr;

const TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMaterialExpressionGenericImpl::GetProperty( int32 InIndex ) const
{
	if ( Properties.IsValidIndex( InIndex ) )
	{
		return Properties[InIndex];
	}

	return NullPropertyPtr;
}

TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMaterialExpressionGenericImpl::GetProperty( int32 InIndex )
{
	if ( Properties.IsValidIndex( InIndex ) )
	{
		return Properties[InIndex];
	}

	return NullPropertyPtr;
}

const TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMaterialExpressionGenericImpl::GetPropertyByName( const TCHAR* InName ) const
{
	const TSharedPtr< IDatasmithKeyValueProperty >* FindResult = Properties.View().FindByPredicate( [&InName]( const TSharedPtr<IDatasmithKeyValueProperty>& CurrentKeyValue )
		{
			return FCString::Strcmp( CurrentKeyValue->GetName(), InName ) == 0;
		});

	return FindResult ? *FindResult : NullPropertyPtr;
}

TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMaterialExpressionGenericImpl::GetPropertyByName( const TCHAR* InName )
{
	TSharedPtr< IDatasmithKeyValueProperty >* FindResult = Properties.Edit().FindByPredicate( [&InName]( const TSharedPtr<IDatasmithKeyValueProperty>& CurrentKeyValue )
	{
		return FCString::Strcmp( CurrentKeyValue->GetName(), InName ) == 0;
	} );

	return FindResult ? *FindResult : NullPropertyPtr;
}

void FDatasmithMaterialExpressionGenericImpl::AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& InProperty )
{
	if ( !GetPropertyByName( InProperty->GetName() ) )
	{
		Properties.Add( InProperty );
	}
}

FDatasmithUEPbrMaterialElementImpl::FDatasmithUEPbrMaterialElementImpl( const TCHAR* InName )
	: FDatasmithBaseMaterialElementImpl( InName, EDatasmithElementType::UEPbrMaterial )
	, BaseColor(          MakeShared< FDatasmithExpressionInputImpl >( TEXT("BaseColor") ) )
	, Metallic(           MakeShared< FDatasmithExpressionInputImpl >( TEXT("Metallic") ) )
	, Specular(           MakeShared< FDatasmithExpressionInputImpl >( TEXT("Specular") ) )
	, Roughness(          MakeShared< FDatasmithExpressionInputImpl >( TEXT("Roughness") ) )
	, EmissiveColor(      MakeShared< FDatasmithExpressionInputImpl >( TEXT("EmissiveColor") ) )
	, Opacity(            MakeShared< FDatasmithExpressionInputImpl >( TEXT("Opacity") ) )
	, Normal(             MakeShared< FDatasmithExpressionInputImpl >( TEXT("Normal") ) )
	, WorldDisplacement(  MakeShared< FDatasmithExpressionInputImpl >( TEXT("WorldDisplacement") ) )
	, Refraction(         MakeShared< FDatasmithExpressionInputImpl >( TEXT("Refraction") ) )
	, AmbientOcclusion(   MakeShared< FDatasmithExpressionInputImpl >( TEXT("AmbientOcclusion") ) )
	, ClearCoat(		  MakeShared< FDatasmithExpressionInputImpl >( TEXT("ClearCoat") ) )
	, ClearCoatRoughness( MakeShared< FDatasmithExpressionInputImpl >( TEXT("ClearCoatRoughness") ) )
	, WorldPositionOffset( MakeShared< FDatasmithExpressionInputImpl >( TEXT("WorldPositionOffset") ) )
	, MaterialAttributes( MakeShared< FDatasmithExpressionInputImpl >( TEXT("MaterialAttributes") ) )
	, BlendMode(0)
	, bTwoSided( false )
	, bIsThinSurface(false)
	, bUseMaterialAttributes( false )
	, bMaterialFunctionOnly ( false )
	, OpacityMaskClipValue( 0.3333f )
	, TranslucencyLightingMode( 0 )
	, ShadingModel( EDatasmithShadingModel::DefaultLit )
{
	RegisterReferenceProxy( BaseColor, "BaseColor" );
	RegisterReferenceProxy( Metallic, "Metallic" );
	RegisterReferenceProxy( Specular, "Specular" );
	RegisterReferenceProxy( Roughness, "Roughness" );
	RegisterReferenceProxy( EmissiveColor, "EmissiveColor" );
	RegisterReferenceProxy( Opacity, "Opacity" );
	RegisterReferenceProxy( Normal, "Normal" );
	RegisterReferenceProxy( WorldDisplacement, "WorldDisplacement" );
	RegisterReferenceProxy( Refraction, "Refraction" );
	RegisterReferenceProxy( AmbientOcclusion, "AmbientOcclusion" );
	RegisterReferenceProxy( ClearCoat, "ClearCoat" );
	RegisterReferenceProxy( ClearCoatRoughness, "ClearCoatRoughness" );
	RegisterReferenceProxy( WorldPositionOffset, "WorldPositionOffset" );
	RegisterReferenceProxy( MaterialAttributes, "MaterialAttributes" );

	RegisterReferenceProxy( Expressions, "Expressions" );

	Store.RegisterParameter( BlendMode, "BlendMode" );
	Store.RegisterParameter( bTwoSided, "bTwoSided" );
	Store.RegisterParameter( bIsThinSurface, "bIsThinSurface");
	Store.RegisterParameter( bUseMaterialAttributes, "bUseMaterialAttributes" );
	Store.RegisterParameter( bMaterialFunctionOnly, "bMaterialFunctionOnly" );
	Store.RegisterParameter( OpacityMaskClipValue, "OpacityMaskClipValue" );

	Store.RegisterParameter( ParentLabel, "ParentLabel" );
	Store.RegisterParameter( ShadingModel, "ShadingModel" );
}

FMD5Hash FDatasmithUEPbrMaterialElementImpl::CalculateElementHash(bool bForce)
{
	if (ElementHash.IsValid() && !bForce)
	{
		return ElementHash;
	}

	FMD5 MD5;
	MD5.Update(reinterpret_cast<const uint8*>(&BlendMode), sizeof(BlendMode));
	MD5.Update(reinterpret_cast<const uint8*>(&bTwoSided), sizeof(bTwoSided));
	MD5.Update(reinterpret_cast<const uint8*>(&bIsThinSurface), sizeof(bIsThinSurface));
	MD5.Update(reinterpret_cast<const uint8*>(&bUseMaterialAttributes), sizeof(bUseMaterialAttributes));
	MD5.Update(reinterpret_cast<const uint8*>(&bMaterialFunctionOnly), sizeof(bMaterialFunctionOnly));
	MD5.Update(reinterpret_cast<const uint8*>(&OpacityMaskClipValue), sizeof(OpacityMaskClipValue));
	MD5.Update(reinterpret_cast<const uint8*>(&ShadingModel), sizeof(ShadingModel));

	const FString& NativeParentLabel = ParentLabel;
	if (!NativeParentLabel.IsEmpty())
	{
		MD5.Update(reinterpret_cast<const uint8*>(*NativeParentLabel), NativeParentLabel.Len() * sizeof(TCHAR));
	}

	TFunction<void(FDatasmithExpressionInputImpl&)> UpdateMD5 = [&](FDatasmithExpressionInputImpl& Input) -> void
	{
		const FMD5Hash& InputHashValue = Input.CalculateElementHash(bForce);
		MD5.Update(InputHashValue.GetBytes(), InputHashValue.GetSize());
	};

	UpdateMD5(*BaseColor.Edit());
	UpdateMD5(*Metallic.Edit());
	UpdateMD5(*Specular.Edit());
	UpdateMD5(*Roughness.Edit());
	UpdateMD5(*EmissiveColor.Edit());
	UpdateMD5(*Opacity.Edit());
	UpdateMD5(*Normal.Edit());
	UpdateMD5(*WorldDisplacement.Edit());
	UpdateMD5(*Refraction.Edit());
	UpdateMD5(*AmbientOcclusion.Edit());
	UpdateMD5(*ClearCoat.Edit());
	UpdateMD5(*ClearCoatRoughness.Edit());
	UpdateMD5(*WorldPositionOffset.Edit());
	UpdateMD5(*MaterialAttributes.Edit());

	ElementHash.Set(MD5);
	return ElementHash;
}

IDatasmithMaterialExpression* FDatasmithUEPbrMaterialElementImpl::GetExpression( int32 Index )
{
	return Expressions.IsValidIndex( Index ) ? Expressions[Index].Get() : nullptr;
}

int32 FDatasmithUEPbrMaterialElementImpl::GetExpressionIndex( const IDatasmithMaterialExpression* Expression ) const
{
	int32 ExpressionIndex = INDEX_NONE;

	for ( int32 Index = 0; Index < Expressions.Num(); ++Index )
	{
		IDatasmithMaterialExpression* CurrentElement = Expressions[Index].Get();
		if ( Expression == CurrentElement)
		{
			ExpressionIndex = Index;
			break;
		}
	}

	return ExpressionIndex;
}

IDatasmithMaterialExpression* FDatasmithUEPbrMaterialElementImpl::AddMaterialExpression( const EDatasmithMaterialExpressionType ExpressionType )
{
	TSharedPtr<IDatasmithMaterialExpression> Expression = nullptr;

	switch ( ExpressionType )
	{
	case EDatasmithMaterialExpressionType::ConstantBool:
		Expression = MakeShared< FDatasmithMaterialExpressionBoolImpl >();
		break;
	case EDatasmithMaterialExpressionType::ConstantColor:
		Expression = MakeShared< FDatasmithMaterialExpressionColorImpl >();
		break;
	case EDatasmithMaterialExpressionType::ConstantScalar:
		Expression = MakeShared< FDatasmithMaterialExpressionScalarImpl >();
		break;
	case EDatasmithMaterialExpressionType::FlattenNormal:
		Expression = MakeShared< FDatasmithMaterialExpressionFlattenNormalImpl >();
		break;
	case EDatasmithMaterialExpressionType::FunctionCall:
		Expression = MakeShared< FDatasmithMaterialExpressionFunctionCallImpl >();
		break;
	case EDatasmithMaterialExpressionType::Generic:
		Expression = MakeShared< FDatasmithMaterialExpressionGenericImpl >();
		break;
	case EDatasmithMaterialExpressionType::Texture:
		Expression = MakeShared< FDatasmithMaterialExpressionTextureImpl >();
		break;
	case EDatasmithMaterialExpressionType::TextureCoordinate:
		Expression = MakeShared< FDatasmithMaterialExpressionTextureCoordinateImpl >();
		break;
	case EDatasmithMaterialExpressionType::Custom:
		Expression = MakeShared < FDatasmithMaterialExpressionCustomImpl>();
		break;
	default:
		check( false );
		break;
	}

	Expressions.Add( Expression );

	return Expression.Get();
}

void FDatasmithUEPbrMaterialElementImpl::ResetExpressionGraph( bool bRemoveAllExpressions )
{
	if (bRemoveAllExpressions)
	{
		Expressions.Edit().Reset();
	}
	else
	{
		for ( const TSharedPtr<IDatasmithMaterialExpression>& MaterialExpression : Expressions.View() )
		{
			MaterialExpression->ResetExpression();
		}
	}

	//Reset material inputs.
	BaseColor.Edit()->SetExpression(nullptr);
	Metallic.Edit()->SetExpression(nullptr);
	Specular.Edit()->SetExpression(nullptr);
	Roughness.Edit()->SetExpression(nullptr);
	EmissiveColor.Edit()->SetExpression(nullptr);
	Opacity.Edit()->SetExpression(nullptr);
	Normal.Edit()->SetExpression(nullptr);
	WorldDisplacement.Edit()->SetExpression(nullptr);
	Refraction.Edit()->SetExpression(nullptr);
	AmbientOcclusion.Edit()->SetExpression(nullptr);
	ClearCoat.Edit()->SetExpression(nullptr);
	ClearCoatRoughness.Edit()->SetExpression(nullptr);
	WorldPositionOffset.Edit()->SetExpression(nullptr);
	MaterialAttributes.Edit()->SetExpression(nullptr);
}

const TCHAR* FDatasmithUEPbrMaterialElementImpl::GetParentLabel() const
{
	return ParentLabel.Get().IsEmpty() ? GetLabel() : *ParentLabel.Get();
}


void FDatasmithUEPbrMaterialElementImpl::CustomSerialize(class DirectLink::FSnapshotProxy& Ar)
{
	// [4.26.1 .. 4.27.0[ compatibility
	if (Ar.IsSaving())
	{
		// In 4.26, an ExpressionTypes array was used alongside the expressions array.
		// namely: TReflected<TArray<EDatasmithMaterialExpressionType>, TArray<int32>> FDatasmithUEPbrMaterialElementImpl::ExpressionTypes;
		// This field was required. In order to be readable by 4.26, that array is recreated here.
		// Without it, a 4.26 DirectLink receiver could crash on 4.27 data usage.
		TArray<int32> ExpressionTypes;
		for (const TSharedPtr<IDatasmithMaterialExpression>& Expression : Expressions.View() )
		{
			EDatasmithMaterialExpressionType ExpressionType = Expression.IsValid() ? Expression->GetExpressionType() : EDatasmithMaterialExpressionType::None;
			ExpressionTypes.Add(int32(ExpressionType));
		}
		Ar.TagSerialize("ExpressionTypes", ExpressionTypes);
	}
}


FDatasmithMaterialExpressionCustomImpl::FDatasmithMaterialExpressionCustomImpl() : FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionCustom >(EDatasmithMaterialExpressionType::Custom)
{
	RegisterReferenceProxy(Inputs, "Inputs");
	Store.RegisterParameter(Code, "Code");
	Store.RegisterParameter(Description, "Description");
	Store.RegisterParameter(OutputType, "OutputType");
	Store.RegisterParameter(IncludeFilePaths, "IncludeFilePaths");
	Store.RegisterParameter(Defines, "Defines");
	Store.RegisterParameter(ArgNames, "ArgNames");
}


IDatasmithExpressionInput* FDatasmithMaterialExpressionCustomImpl::GetInput(int32 Index)
{
	if (!ensure(Index >= 0))
	{
		return nullptr;
	}

	while (!Inputs.IsValidIndex(Index))
	{
		Inputs.Add(MakeShared< FDatasmithExpressionInputImpl >(*FString::FromInt(Inputs.Num())));
	}

	return Inputs[Index].Get();
}


void FDatasmithMaterialExpressionCustomImpl::SetArgumentName(int32 ArgIndex, const TCHAR* ArgName)
{
	if (!ensure(ArgIndex >= 0))
	{
		return;
	}

	TArray<FString>& Names = ArgNames;
	while (!Names.IsValidIndex(ArgIndex))
	{
		int32 CurrentIndex = Names.Num();
		Names.Add(FString::Printf(TEXT("Arg%d"), CurrentIndex));
	}
	Names[ArgIndex] = ArgName;
}

