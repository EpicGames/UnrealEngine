// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithMaterialElements.h"
#include "DatasmithSceneElementsImpl.h"

#include "Algo/Find.h"
#include "Containers/Array.h"
#include "Misc/SecureHash.h"

#define UE_API DATASMITHCORE_API


namespace FDatasmithExpressionUtils
{
	FORCEINLINE void UpdateHashWithString(FMD5& MD5, const FString& String)
	{
		if (!String.IsEmpty())
		{
			MD5.Update(reinterpret_cast<const uint8*>(*String), String.Len() * sizeof(TCHAR));
		}
	}

	FORCEINLINE void UpdateHashWithStrings(FMD5& MD5, const TArray<FString>& Strings)
	{
		for (const FString& String : Strings)
		{
			UpdateHashWithString(MD5, String);
		}
	}

	template< typename ReferenceType >
	void UpdateHashWithReference(FMD5& MD5, TDatasmithReferenceProxy<ReferenceType>& Reference)
	{
		if (Reference.View())
		{
			FMD5Hash ReferenceHash = Reference.Edit()->CalculateElementHash(true);
			MD5.Update(ReferenceHash.GetBytes(), ReferenceHash.GetSize());
		}
	}

	template< typename ClassType >
	void UpdateHashWithArray(FMD5& MD5, TDatasmithReferenceArrayProxy<ClassType>& Array)
	{
		for (TSharedPtr<ClassType>& Object : Array.Edit())
		{
			FMD5Hash ObjectHash = Object->CalculateElementHash(true);
			MD5.Update(ObjectHash.GetBytes(), ObjectHash.GetSize());
		}
	}
};

class FDatasmithExpressionInputImpl : public FDatasmithElementImpl < IDatasmithExpressionInput >
{
public:
	explicit FDatasmithExpressionInputImpl( const TCHAR* InInputName );
	virtual ~FDatasmithExpressionInputImpl() = default;

	virtual IDatasmithMaterialExpression* GetExpression() override { return Expression.Edit().Get(); }
	virtual const IDatasmithMaterialExpression* GetExpression() const override { return Expression.View().Get(); }
	virtual void SetExpression( IDatasmithMaterialExpression* InExpression ) override;

	virtual int32 GetOutputIndex() const override { return OutputIndex; }
	virtual void SetOutputIndex( int32 InOutputIndex ) override { OutputIndex = InOutputIndex; }

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FDatasmithExpressionUtils::UpdateHashWithReference(MD5, Expression);
		MD5.Update(reinterpret_cast<const uint8*>(&OutputIndex), sizeof(OutputIndex));

		ElementHash.Set(MD5);
		return ElementHash;
	}

	virtual void CustomSerialize(class DirectLink::FSnapshotProxy& Ar) override;

protected:
	TDatasmithReferenceProxy<IDatasmithMaterialExpression> Expression;
	TReflected<int32> OutputIndex;
};

class FDatasmithExpressionOutputImpl : public FDatasmithElementImpl < IDatasmithExpressionOutput >
{
public:
	explicit FDatasmithExpressionOutputImpl( const TCHAR* InOutputName )
		: FDatasmithElementImpl< IDatasmithExpressionOutput >( InOutputName, EDatasmithElementType::MaterialExpressionOutput )
	{}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FDatasmithExpressionUtils::UpdateHashWithString(MD5, Name);

		ElementHash.Set(MD5);
		return ElementHash;
	}
};

template< typename InterfaceType >
class FDatasmithMaterialExpressionImpl
	: public FDatasmithElementImpl< InterfaceType >
	, public TSharedFromThis< FDatasmithMaterialExpressionImpl< InterfaceType > >
{
public:
	explicit FDatasmithMaterialExpressionImpl( EDatasmithMaterialExpressionType InSubType );

	virtual ~FDatasmithMaterialExpressionImpl() = default;

	virtual EDatasmithMaterialExpressionType GetExpressionType() const override { return static_cast<EDatasmithMaterialExpressionType>(this->Subtype.Get()); }

	virtual bool IsSubType( const EDatasmithMaterialExpressionType ExpressionType ) const override
	{ return this->IsSubTypeInternal( (uint64)ExpressionType ); }

	virtual bool IsSubTypeInternal( uint64 InSubType ) const override { return InSubType == this->GetSubType(); } // as our subtype is not a bitfield

	virtual void ConnectExpression( IDatasmithExpressionInput& ExpressionInput ) override
	{
		ConnectExpression( ExpressionInput, GetDefaultOutputIndex() );
	}

	inline virtual void ConnectExpression( IDatasmithExpressionInput& ExpressionInput, int32 OutputIndex ) override;

	virtual int32 GetInputCount() const override { return 0; }
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) override { return nullptr; }
	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const override { return nullptr; }

	virtual int32 GetDefaultOutputIndex() const override { return DefaultOutputIndex; }
	virtual void SetDefaultOutputIndex( int32 InDefaultOutputIndex ) override { DefaultOutputIndex = InDefaultOutputIndex; }

	virtual void ResetExpression() override;

	virtual void ResetExpressionImpl() = 0;

protected:
	FMD5Hash ComputeHash()
	{
		FMD5 MD5;

		FDatasmithExpressionUtils::UpdateHashWithArray(MD5, Outputs);
		MD5.Update(reinterpret_cast<const uint8*>(&DefaultOutputIndex), sizeof(DefaultOutputIndex));

		FMD5Hash MD5Hash;
		MD5Hash.Set(MD5);

		return MD5Hash;
	}

protected:
	TDatasmithReferenceArrayProxy<FDatasmithExpressionOutputImpl> Outputs;

	TReflected<int32> DefaultOutputIndex;
};

template< typename InterfaceType >
class FDatasmithExpressionParameterImpl : public FDatasmithMaterialExpressionImpl< InterfaceType >
{
public:
	typedef FDatasmithMaterialExpressionImpl< InterfaceType > TBaseExpression;

	FDatasmithExpressionParameterImpl( EDatasmithMaterialExpressionType InSubType )
		: FDatasmithMaterialExpressionImpl< InterfaceType >( InSubType )
	{
		this->Store.RegisterParameter( GroupName, "GroupName" );
	}

	virtual const TCHAR* GetGroupName() const override { return *GroupName.Get(); }
	virtual void SetGroupName( const TCHAR* InGroupName ) override { GroupName = InGroupName; }

protected:
	FMD5Hash ComputeHash()
	{
		FMD5 MD5;

		FMD5Hash BaseHash = TBaseExpression::ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		FDatasmithExpressionUtils::UpdateHashWithString(MD5, GroupName);

		FMD5Hash MD5Hash;
		MD5Hash.Set(MD5);

		return MD5Hash;
	}

protected:
	TReflected<FString> GroupName;
};

class FDatasmithMaterialExpressionBoolImpl : public FDatasmithExpressionParameterImpl< IDatasmithMaterialExpressionBool >
{
public:
	FDatasmithMaterialExpressionBoolImpl();

	virtual bool& GetBool() override { return bValue; }
	virtual const bool& GetBool() const override { return bValue; }

	virtual void ResetExpressionImpl() override
	{
		GroupName = FString();
		bValue = false;
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		MD5.Update(reinterpret_cast<const uint8*>(&bValue), sizeof(bValue));

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TReflected<bool> bValue;
};

class FDatasmithMaterialExpressionColorImpl : public FDatasmithExpressionParameterImpl< IDatasmithMaterialExpressionColor >
{
public:
	FDatasmithMaterialExpressionColorImpl();

	virtual FLinearColor& GetColor() override { return LinearColor; }
	virtual const FLinearColor& GetColor() const override { return LinearColor; }

	virtual void ResetExpressionImpl() override
	{
		GroupName = FString();
		LinearColor = FLinearColor();
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		MD5.Update(reinterpret_cast<const uint8*>(&LinearColor), sizeof(LinearColor));

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TReflected<FLinearColor> LinearColor;
};

class FDatasmithMaterialExpressionScalarImpl : public FDatasmithExpressionParameterImpl< IDatasmithMaterialExpressionScalar >
{
public:
	FDatasmithMaterialExpressionScalarImpl();

	virtual float& GetScalar() override { return Scalar; }
	virtual const float& GetScalar() const override { return Scalar; }

	virtual void ResetExpressionImpl() override
	{
		GroupName = FString();
		Scalar = 0;
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		MD5.Update(reinterpret_cast<const uint8*>(&Scalar), sizeof(Scalar));

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TReflected<float> Scalar;
};

class FDatasmithMaterialExpressionTextureImpl : public FDatasmithExpressionParameterImpl< IDatasmithMaterialExpressionTexture >
{
public:
	FDatasmithMaterialExpressionTextureImpl();

	virtual const TCHAR* GetTexturePathName() const override { return *TexturePathName.Get(); }
	virtual void SetTexturePathName( const TCHAR* InTexturePathName ) { TexturePathName = InTexturePathName; }

	/**
	 * Inputs
	 */
	virtual IDatasmithExpressionInput& GetInputCoordinate() override { return *TextureCoordinate.Edit(); }
	virtual const IDatasmithExpressionInput& GetInputCoordinate() const override { return *TextureCoordinate.View(); }

	virtual int32 GetInputCount() const override { return 1; }
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) override { return TextureCoordinate.Edit().Get(); }
	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const override { return TextureCoordinate.View().Get(); }

	virtual void ResetExpressionImpl() override
	{
		GroupName = FString();
		TexturePathName = FString();
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		FDatasmithExpressionUtils::UpdateHashWithString(MD5, TexturePathName);

		FDatasmithExpressionUtils::UpdateHashWithReference(MD5, TextureCoordinate);

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TReflected< FString > TexturePathName;

	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > TextureCoordinate;

	/**
	 * Outputs:
	 * - RGB
	 * - R
	 * - G
	 * - B
	 * - A
	 */
};

class FDatasmithMaterialExpressionTextureCoordinateImpl : public FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionTextureCoordinate >
{
public:
	FDatasmithMaterialExpressionTextureCoordinateImpl();

	virtual int32 GetCoordinateIndex() const override { return CoordinateIndex; }
	virtual void SetCoordinateIndex( int32 InCoordinateIndex ) override { CoordinateIndex = InCoordinateIndex; }

	virtual float GetUTiling() const override { return UTiling; }
	virtual void SetUTiling( float InUTiling ) override { UTiling = InUTiling; }

	virtual float GetVTiling() const override { return VTiling;}
	virtual void SetVTiling( float InVTiling ) override { VTiling = InVTiling; }

	virtual void ResetExpressionImpl() override
	{
		CoordinateIndex = 0;
		UTiling = 1;
		VTiling = 1;
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		MD5.Update(reinterpret_cast<const uint8*>(&CoordinateIndex), sizeof(CoordinateIndex));
		MD5.Update(reinterpret_cast<const uint8*>(&UTiling), sizeof(UTiling));
		MD5.Update(reinterpret_cast<const uint8*>(&VTiling), sizeof(VTiling));

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TReflected< int32 > CoordinateIndex;
	TReflected< float > UTiling;
	TReflected< float > VTiling;
};

class FDatasmithMaterialExpressionFlattenNormalImpl : public FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionFlattenNormal >
{
public:
	FDatasmithMaterialExpressionFlattenNormalImpl();

	virtual IDatasmithExpressionInput& GetNormal() override { return *Normal.Edit(); }
	virtual const IDatasmithExpressionInput& GetNormal() const override { return *Normal.View(); }

	virtual IDatasmithExpressionInput& GetFlatness() override { return *Flatness.Edit(); }
	virtual const IDatasmithExpressionInput& GetFlatness() const override { return *Flatness.View(); }

	virtual int32 GetInputCount() const override { return 2; }
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) override { return Index == 0 ? Normal.Edit().Get() : Flatness.Edit().Get(); }
	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const override { return Index == 0 ? Normal.View().Get() : Flatness.View().Get(); }

	virtual void ResetExpressionImpl() override { /*Nothing to reset.*/ }

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		FDatasmithExpressionUtils::UpdateHashWithReference(MD5, Normal);
		FDatasmithExpressionUtils::UpdateHashWithReference(MD5, Flatness);

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Normal;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Flatness;
};

class FDatasmithMaterialExpressionGenericImpl : public FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionGeneric >
{
public:
	static TSharedPtr< IDatasmithKeyValueProperty > NullPropertyPtr;

	FDatasmithMaterialExpressionGenericImpl()
		: FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionGeneric >( EDatasmithMaterialExpressionType::Generic )
	{
		RegisterReferenceProxy( Inputs, "Inputs" );
		RegisterReferenceProxy( Properties, "Properties" );
		Store.RegisterParameter( ExpressionName, "ExpressionName" );
	}


	virtual void SetExpressionName( const TCHAR* InExpressionName ) override { ExpressionName = InExpressionName; }
	virtual const TCHAR* GetExpressionName() const override { return *ExpressionName.Get(); }

	int32 GetPropertiesCount() const override { return Properties.Num(); }

	const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty( int32 InIndex ) const override;
	TSharedPtr< IDatasmithKeyValueProperty >& GetProperty( int32 InIndex ) override;

	const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName( const TCHAR* InName ) const override;
	TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName( const TCHAR* InName ) override;

	void AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& InProperty ) override;

	virtual int32 GetInputCount() const override { return Inputs.Num(); }
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) override
	{
		while ( !Inputs.IsValidIndex( Index ) )
		{
			Inputs.Add( MakeShared< FDatasmithExpressionInputImpl >( *FString::FromInt( Inputs.Num() ) ) );
		}

		return Inputs[Index].Get();
	}

	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const override { return Inputs.IsValidIndex( Index ) ? Inputs[Index].Get() : nullptr; }

	virtual void ResetExpressionImpl() override
	{
		Inputs.Empty();
		ExpressionName = FString();
		Properties.Empty();
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		FDatasmithExpressionUtils::UpdateHashWithArray(MD5, Inputs);
		FDatasmithExpressionUtils::UpdateHashWithString(MD5, ExpressionName);
		FDatasmithExpressionUtils::UpdateHashWithArray(MD5, Properties);

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TDatasmithReferenceArrayProxy< FDatasmithExpressionInputImpl > Inputs;
	TReflected<FString> ExpressionName;

	TDatasmithReferenceArrayProxy< IDatasmithKeyValueProperty > Properties;
};

class FDatasmithMaterialExpressionFunctionCallImpl : public FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionFunctionCall >
{
public:
	FDatasmithMaterialExpressionFunctionCallImpl()
		: FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionFunctionCall >( EDatasmithMaterialExpressionType::FunctionCall )
	{
		RegisterReferenceProxy( Inputs, "Inputs" );
		Store.RegisterParameter( FunctionPathName, "FunctionPathName" );
	}

	virtual void SetFunctionPathName( const TCHAR* InFunctionPathName ) override { FunctionPathName = InFunctionPathName; }
	virtual const TCHAR* GetFunctionPathName() const override { return *FunctionPathName.Get(); }

	virtual int32 GetInputCount() const override { return Inputs.Num(); }
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) override
	{
		while (!Inputs.IsValidIndex( Index ))
		{
			Inputs.Add( MakeShared< FDatasmithExpressionInputImpl >( *FString::FromInt( Inputs.Num() ) ) );
		}

		return Inputs[Index].Get();
	}

	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const override { return Inputs.IsValidIndex( Index ) ? Inputs[Index].Get() : nullptr; }

	virtual void ResetExpressionImpl() override
	{
		Inputs.Empty();
		FunctionPathName = FString();
	}

	FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		FDatasmithExpressionUtils::UpdateHashWithArray(MD5, Inputs);
		FDatasmithExpressionUtils::UpdateHashWithString(MD5, FunctionPathName);

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TDatasmithReferenceArrayProxy< FDatasmithExpressionInputImpl > Inputs;
	TReflected<FString> FunctionPathName;
};

class FDatasmithMaterialExpressionCustomImpl : public FDatasmithMaterialExpressionImpl< IDatasmithMaterialExpressionCustom >
{
public:
	FDatasmithMaterialExpressionCustomImpl();

	virtual int32 GetInputCount() const override { return Inputs.Num(); }
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) override;
	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const override { return Inputs.IsValidIndex( Index ) ? Inputs[Index].Get() : nullptr; }

	virtual void SetCode(const TCHAR* InCode) override { Code = InCode; }
	virtual const TCHAR* GetCode() const override { return *Code.Get(); }

	virtual void SetDescription(const TCHAR* InDescription) override { Description = InDescription; }
	virtual const TCHAR* GetDescription() const override { return *Description.Get(); }

	virtual void SetOutputType(EDatasmithShaderDataType InOutputType) override { OutputType = InOutputType; }
	virtual EDatasmithShaderDataType GetOutputType() const override { return OutputType; }

	virtual int32 GetIncludeFilePathCount() const override { return IncludeFilePaths.Get().Num(); }
	virtual void AddIncludeFilePath(const TCHAR* Path) override { IncludeFilePaths.Get().Add(Path); }
	virtual const TCHAR* GetIncludeFilePath(int32 Index) const override { return IncludeFilePaths.Get().IsValidIndex(Index) ? *IncludeFilePaths.Get()[Index] : TEXT(""); }

	virtual int32 GetAdditionalDefineCount() const override { return Defines.Get().Num(); }
	virtual void AddAdditionalDefine(const TCHAR* Define) override { Defines.Get().Add(Define); }
	virtual const TCHAR* GetAdditionalDefine(int32 Index) const override { return Defines.Get().IsValidIndex(Index) ? *Defines.Get()[Index] : TEXT(""); }

	virtual int32 GetArgumentNameCount() const override { return ArgNames.Get().Num(); }
	virtual void SetArgumentName(int32 ArgIndex, const TCHAR* ArgName) override;
	virtual const TCHAR* GetArgumentName(int32 Index) const override { return ArgNames.Get().IsValidIndex(Index) ? *ArgNames.Get()[Index] : TEXT("");}

	virtual void ResetExpressionImpl() override
	{
		Code = FString();
		Description = FString();
		OutputType = EDatasmithShaderDataType::Float1;
		IncludeFilePaths.Get().Empty();
		Defines.Get().Empty();
		ArgNames.Get().Empty();
		Inputs.Empty();
	}

	virtual FMD5Hash CalculateElementHash(bool bForce) override
	{
		if (ElementHash.IsValid() && !bForce)
		{
			return ElementHash;
		}

		FMD5 MD5;

		FMD5Hash BaseHash = ComputeHash();
		MD5.Update(BaseHash.GetBytes(), BaseHash.GetSize());

		FDatasmithExpressionUtils::UpdateHashWithString(MD5, Code);
		FDatasmithExpressionUtils::UpdateHashWithString(MD5, Description);
		MD5.Update(reinterpret_cast<const uint8*>(&OutputType), sizeof(OutputType));
		FDatasmithExpressionUtils::UpdateHashWithStrings(MD5, IncludeFilePaths);
		FDatasmithExpressionUtils::UpdateHashWithStrings(MD5, Defines);
		FDatasmithExpressionUtils::UpdateHashWithStrings(MD5, ArgNames);
		FDatasmithExpressionUtils::UpdateHashWithArray(MD5, Inputs);

		ElementHash.Set(MD5);
		return ElementHash;
	}

protected:
	TReflected<FString> Code;
	TReflected<FString> Description;
	TReflected<EDatasmithShaderDataType, uint32> OutputType = EDatasmithShaderDataType::Float1;
	TReflected<TArray<FString>> IncludeFilePaths;
	TReflected<TArray<FString>> Defines;
	TReflected<TArray<FString>> ArgNames;
	TDatasmithReferenceArrayProxy< FDatasmithExpressionInputImpl > Inputs;
};

class FDatasmithUEPbrMaterialElementImpl : public FDatasmithBaseMaterialElementImpl< IDatasmithUEPbrMaterialElement >
{
public:
	UE_API explicit FDatasmithUEPbrMaterialElementImpl( const TCHAR* InName );
	virtual ~FDatasmithUEPbrMaterialElementImpl() = default;

	UE_API virtual FMD5Hash CalculateElementHash(bool bForce) override;

	virtual IDatasmithExpressionInput& GetBaseColor() override { return *BaseColor.Edit(); }
	virtual IDatasmithExpressionInput& GetMetallic() override { return *Metallic.Edit(); }
	virtual IDatasmithExpressionInput& GetSpecular() override { return *Specular.Edit(); }
	virtual IDatasmithExpressionInput& GetRoughness() override { return *Roughness.Edit(); }
	virtual IDatasmithExpressionInput& GetEmissiveColor() override { return *EmissiveColor.Edit(); }
	virtual IDatasmithExpressionInput& GetOpacity() override { return *Opacity.Edit(); }
	virtual IDatasmithExpressionInput& GetNormal() override { return *Normal.Edit(); }
	virtual IDatasmithExpressionInput& GetRefraction() override { return *Refraction.Edit(); }
	virtual IDatasmithExpressionInput& GetAmbientOcclusion() override { return *AmbientOcclusion.Edit(); }
	virtual IDatasmithExpressionInput& GetClearCoat() override { return *ClearCoat.Edit(); }
	virtual IDatasmithExpressionInput& GetClearCoatRoughness() override { return *ClearCoatRoughness.Edit(); }
	virtual IDatasmithExpressionInput& GetWorldPositionOffset() override { return *WorldPositionOffset.Edit(); }
	virtual IDatasmithExpressionInput& GetMaterialAttributes() override { return *MaterialAttributes.Edit(); }

	virtual int GetBlendMode() const override {return BlendMode; }
	virtual void SetBlendMode( int InBlendMode ) override { BlendMode = InBlendMode; }

	virtual bool GetTwoSided() const override {return bTwoSided; }
	virtual void SetTwoSided( bool bInTwoSided ) override { bTwoSided = bInTwoSided; }

	virtual bool GetIsThinSurface() const override { return bIsThinSurface; }
	virtual void SetIsThinSurface(bool bInIsThinSurface) override { bIsThinSurface = bInIsThinSurface; }

	virtual bool GetUseMaterialAttributes() const override{ return bUseMaterialAttributes; }
	virtual void SetUseMaterialAttributes( bool bInUseMaterialAttributes ) override { bUseMaterialAttributes = bInUseMaterialAttributes; }

	virtual bool GetMaterialFunctionOnly() const override { return bMaterialFunctionOnly; };
	virtual void SetMaterialFunctionOnly(bool bInMaterialFunctionOnly) override { bMaterialFunctionOnly = bInMaterialFunctionOnly; };

	virtual float GetOpacityMaskClipValue() const override { return OpacityMaskClipValue; }
	virtual void SetOpacityMaskClipValue(float InClipValue) override { OpacityMaskClipValue = InClipValue; }

	virtual int GetTranslucencyLightingMode() const { return TranslucencyLightingMode; }
	virtual void SetTranslucencyLightingMode(int InMode) { TranslucencyLightingMode = InMode; }

	virtual int32 GetExpressionsCount() const override { return Expressions.View().Num(); }
	UE_API virtual IDatasmithMaterialExpression* GetExpression( int32 Index ) override;
	UE_API virtual int32 GetExpressionIndex( const IDatasmithMaterialExpression* Expression ) const override;

	UE_API virtual IDatasmithMaterialExpression* AddMaterialExpression( const EDatasmithMaterialExpressionType ExpressionType ) override;

	UE_API virtual void ResetExpressionGraph( bool bRemoveAllExpressions ) override;

	virtual void SetParentLabel( const TCHAR* InParentLabel ) override { ParentLabel = InParentLabel; }
	UE_API virtual const TCHAR* GetParentLabel() const override;

	virtual void SetShadingModel( const EDatasmithShadingModel InShadingModel ) override { ShadingModel = InShadingModel; }
	virtual EDatasmithShadingModel GetShadingModel() const override { return ShadingModel; }

	UE_API virtual void CustomSerialize(class DirectLink::FSnapshotProxy& Ar) override;

protected:
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > BaseColor;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Metallic;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Specular;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Roughness;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > EmissiveColor;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Opacity;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Normal;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > WorldDisplacement;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > Refraction;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > AmbientOcclusion;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > ClearCoat;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > ClearCoatRoughness;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > WorldPositionOffset;
	TDatasmithReferenceProxy< FDatasmithExpressionInputImpl > MaterialAttributes;

	TDatasmithReferenceArrayProxy< IDatasmithMaterialExpression > Expressions;

	TReflected<int32> BlendMode;
	TReflected<bool> bTwoSided;
	TReflected<bool> bIsThinSurface;
	TReflected<bool> bUseMaterialAttributes;
	TReflected<bool> bMaterialFunctionOnly;
	TReflected<float> OpacityMaskClipValue;
	TReflected<int32> TranslucencyLightingMode;

	TReflected<FString> ParentLabel;
	TReflected<EDatasmithShadingModel, uint8> ShadingModel;
};

template< typename InterfaceType >
FDatasmithMaterialExpressionImpl< InterfaceType >::FDatasmithMaterialExpressionImpl( EDatasmithMaterialExpressionType InSubType )
	: FDatasmithElementImpl< InterfaceType >( nullptr, EDatasmithElementType::MaterialExpression, (uint64)InSubType )
	, DefaultOutputIndex( 0 )
{
	this->RegisterReferenceProxy( Outputs, "Outputs" );
	this->Store.RegisterParameter( DefaultOutputIndex, "DefaultOutputIndex" );
}

template< typename InterfaceType >
void FDatasmithMaterialExpressionImpl< InterfaceType >::ConnectExpression( IDatasmithExpressionInput& ExpressionInput, int32 InOutputIndex )
{
	while ( !Outputs.IsValidIndex( InOutputIndex ) && InOutputIndex >= 0 )
	{
		Outputs.Add( MakeShared<FDatasmithExpressionOutputImpl>( TEXT( "Ouput" ) ) );
	}

	int32 OutputIndex = Outputs.IsValidIndex( InOutputIndex ) ? InOutputIndex : INDEX_NONE;

	if ( OutputIndex != INDEX_NONE )
	{
		ExpressionInput.SetExpression( this );
		ExpressionInput.SetOutputIndex( OutputIndex );
	}
}

template< typename InterfaceType >
void FDatasmithMaterialExpressionImpl< InterfaceType >::ResetExpression()
{
	// Call the derived implementation first, it may remove unneeded inputs.
	ResetExpressionImpl();

	this->Label = FString();
	this->ElementHash = FMD5Hash();

	for ( int InputIndex = 0; InputIndex < GetInputCount(); ++InputIndex )
	{
		IDatasmithExpressionInput* Input = GetInput( InputIndex );
		Input->SetExpression( nullptr );
	}
}

#undef UE_API
