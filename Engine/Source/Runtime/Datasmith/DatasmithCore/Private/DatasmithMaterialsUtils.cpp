// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaterialsUtils.h"

#include "DatasmithMaterialElements.h"

namespace DatasmithMaterialsUtilsInternal
{
	void SetupUVEdit( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, IDatasmithExpressionInput& UVCoordinatesInput, const DatasmithMaterialsUtils::FUVEditParameters& UVParameters )
	{
		IDatasmithMaterialExpressionTextureCoordinate* TextureCoordinateExpression = nullptr;
		if ( UVParameters.ChannelIndex != 0)
		{
			TextureCoordinateExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTextureCoordinate >();
			TextureCoordinateExpression->SetCoordinateIndex( UVParameters.ChannelIndex );
		}

		// Convert degrees to "normalized" value where 1 is full rotation
		float WRotation = UVParameters.RotationAngle / 360.f;

		IDatasmithMaterialExpressionFunctionCall* UVEditExpression = nullptr;
		if ( !UVParameters.UVTiling.Equals( FVector2D::UnitVector ) || !UVParameters.UVOffset.IsNearlyZero() || !FMath::IsNearlyZero( WRotation ) || UVParameters.bMirrorU || UVParameters.bMirrorV )
		{
			UVEditExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
			UVEditExpression->SetFunctionPathName( TEXT("/DatasmithContent/Materials/UVEdit.UVEdit") );

			UVEditExpression->ConnectExpression( UVCoordinatesInput );

			// Mirror
			IDatasmithMaterialExpressionBool* MirrorUFlag = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionBool >();
			MirrorUFlag->SetName( TEXT("Mirror U") );
			MirrorUFlag->GetBool() = ( UVParameters.bMirrorU );

			MirrorUFlag->ConnectExpression( *UVEditExpression->GetInput(3) );

			IDatasmithMaterialExpressionBool* MirrorVFlag = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionBool >();
			MirrorVFlag->SetName( TEXT("Mirror V") );
			MirrorVFlag->GetBool() = ( UVParameters.bMirrorV );

			MirrorVFlag->ConnectExpression( *UVEditExpression->GetInput(4) );

			// Tiling
			IDatasmithMaterialExpressionColor* TilingValue = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
			TilingValue->SetName( TEXT("UV Tiling") );
			TilingValue->GetColor() = FLinearColor( UVParameters.UVTiling.X, UVParameters.UVTiling.Y, 0.f );

			TilingValue->ConnectExpression( *UVEditExpression->GetInput(2) );

			IDatasmithMaterialExpressionColor* OffsetValue = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
			OffsetValue->SetName( TEXT("UV Offset") );
			OffsetValue->GetColor() = FLinearColor( UVParameters.UVOffset.X, UVParameters.UVOffset.Y, 0.f );

			OffsetValue->ConnectExpression( *UVEditExpression->GetInput(7) );

			IDatasmithMaterialExpressionColor* TilingPivot = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
			TilingPivot->SetName( TEXT("Tiling Pivot") );
			TilingPivot->GetColor() = UVParameters.bIsUsingRealWorldScale && !MirrorUFlag->GetBool() ? FLinearColor( 0.5f, 0.5f, 0.f ) : FLinearColor( 0.f, 0.5f, 0.f );

			TilingPivot->ConnectExpression( *UVEditExpression->GetInput(1) );

			// Rotation
			if ( !FMath::IsNearlyZero( WRotation ) )
			{
				IDatasmithMaterialExpressionScalar* RotationValue = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				RotationValue->SetName( TEXT("W Rotation") );
				RotationValue->GetScalar() = WRotation;

				RotationValue->ConnectExpression( *UVEditExpression->GetInput(6) );

				IDatasmithMaterialExpressionColor* RotationPivot = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
				RotationPivot->SetName( TEXT("Rotation Pivot") );
				RotationPivot->GetColor() = UVParameters.bIsUsingRealWorldScale ? FLinearColor( 0.5f, 0.5f, 0.f ) : FLinearColor( UVParameters.RotationPivot );

				RotationPivot->ConnectExpression( *UVEditExpression->GetInput(5) );
			}

			// A texture coordinate is mandatory for the UV Edit function
			if ( !TextureCoordinateExpression )
			{
				TextureCoordinateExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTextureCoordinate >();
				TextureCoordinateExpression->SetCoordinateIndex( 0 );
			}
		}

		if ( TextureCoordinateExpression )
		{
			if ( UVEditExpression )
			{
				TextureCoordinateExpression->ConnectExpression( *UVEditExpression->GetInput( 0 ) );
			}
			else
			{
				TextureCoordinateExpression->ConnectExpression( UVCoordinatesInput );
			}
		}
	}
}

IDatasmithMaterialExpressionTexture* DatasmithMaterialsUtils::CreateTextureExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const TCHAR* ParameterName, const TCHAR* TextureMapPath, const FUVEditParameters& UVParameters )
{
	IDatasmithMaterialExpressionTexture* Expression = nullptr;
	if ( TextureMapPath )
	{
		Expression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTexture >();
		Expression->SetName( ParameterName );

		IDatasmithMaterialExpressionTexture* Texture = static_cast< IDatasmithMaterialExpressionTexture* >( Expression );
		Texture->SetTexturePathName( TextureMapPath );

		DatasmithMaterialsUtilsInternal::SetupUVEdit( MaterialElement, Texture->GetInputCoordinate(), UVParameters );
	}
	return Expression;
}

IDatasmithMaterialExpression* DatasmithMaterialsUtils::CreateWeightedMaterialExpression(const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const TCHAR* ParameterName, TOptional< FLinearColor > Color, TOptional< float > Scalar, IDatasmithMaterialExpression* Expression, float Weight, EDatasmithTextureMode TextureMode )
{
	if ( !Expression || !FMath::IsNearlyEqual( Weight, 1.f ) )
	{
		if ( Expression )
		{
			if ( TextureMode == EDatasmithTextureMode::Bump || TextureMode == EDatasmithTextureMode::Normal )
			{
				IDatasmithMaterialExpressionFunctionCall* FlattenNormal = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				FlattenNormal->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions01/Texturing/FlattenNormal") );

				Expression->ConnectExpression( *FlattenNormal->GetInput(0) );

				IDatasmithMaterialExpressionScalar* Flatness = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				Flatness->SetName( TEXT("Normal Flatness") );
				Flatness->GetScalar() = 1.f - Weight;

				Flatness->ConnectExpression( *FlattenNormal->GetInput(1) );

				Expression = FlattenNormal;
			}
			else
			{
				IDatasmithMaterialExpression* ValueExpression = nullptr;

				if ( Color )
				{
					IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
					ColorExpression->SetName( ParameterName );
					ColorExpression->GetColor() = Color.GetValue();

					ValueExpression = ColorExpression;
				}
				else if ( Scalar )
				{
					IDatasmithMaterialExpressionScalar* ScalarExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
					ScalarExpression->SetName( ParameterName );
					ScalarExpression->GetScalar() = Scalar.GetValue();

					ValueExpression = ScalarExpression;
				}

				IDatasmithMaterialExpressionGeneric* MapWeightLerp = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				MapWeightLerp->SetExpressionName( TEXT("LinearInterpolate") );

				IDatasmithMaterialExpressionScalar* MapWeight = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				MapWeight->SetName( TEXT("Map Weight") );
				MapWeight->GetScalar() = Weight;

				if ( ValueExpression )
				{
					ValueExpression->ConnectExpression( *MapWeightLerp->GetInput(0) );
				}

				Expression->ConnectExpression( *MapWeightLerp->GetInput(1) );
				MapWeight->ConnectExpression( *MapWeightLerp->GetInput(2) );

				Expression = MapWeightLerp;
			}
		}
		else
		{
			IDatasmithMaterialExpression* ValueExpression = nullptr;

			if ( Color )
			{
				IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
				ColorExpression->SetName( ParameterName );
				ColorExpression->GetColor() = Color.GetValue();

				ValueExpression = ColorExpression;
			}
			else if ( Scalar )
			{
				IDatasmithMaterialExpressionScalar* ScalarExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				ScalarExpression->SetName( ParameterName );
				ScalarExpression->GetScalar() = Scalar.GetValue();

				ValueExpression = ScalarExpression;
			}

			Expression = ValueExpression;
		}
	}

	return Expression;
}

FLinearColor DatasmithMaterialsUtils::TemperatureToColor(float Kelvin)
{
	float Temp = Kelvin / 100.0f;
	float Red;
	float Green;
	float Blue;

	if (Temp <= 66.0f)
	{
		Red = 255.0f;
		Green = Temp;
		Green = 99.4708025861f * FMath::Loge(Green) - 161.1195681661f;
		if (Temp <= 19.0f)
		{
			Blue = 0.0f;
		}
		else
		{
			Blue = Temp - 10.0f;
			Blue = 138.5177312231f * FMath::Loge(Blue) - 305.0447927307f;
		}
	}
	else
	{
		Red = Temp - 60.0f;
		Red = 329.698727446f * FMath::Pow(Red, -0.1332047592f);
		Green = Temp - 60.f;
		Green = 288.1221695283f * FMath::Pow(Green, -0.0755148492f);
		Blue = 255.0f;
	}

	FLinearColor Color;
	Color.R = Red / 255.0f;
	Color.G = Green / 255.0f;
	Color.B = Blue / 255.0f;

	return Color;
}

