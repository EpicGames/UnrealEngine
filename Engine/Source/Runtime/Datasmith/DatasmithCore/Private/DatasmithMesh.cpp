// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMesh.h"

#include "DatasmithUtils.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Math/Color.h"

namespace DatasmithMeshImpl
{

template<typename T>
static void UpdateMD5SimpleType(FMD5& MD5, const T& Value)
{
	static_assert(TIsPODType<T>::Value, "Simple type required");
	MD5.Update(reinterpret_cast<const uint8*>(&Value), sizeof(Value));
}

template<typename T>
static void UpdateMD5Array(FMD5& MD5, TArray<T> Value)
{
	static_assert(TIsPODType<T>::Value, "This function requires POD array");
	UpdateMD5SimpleType(MD5, Value.Num());
	if (!Value.IsEmpty())
	{
		MD5.Update(reinterpret_cast<const uint8*>(Value.GetData()), Value.GetTypeSize()*Value.Num());
	}
}

}


class FDatasmithMesh::FDatasmithMeshImpl
{
public:
	FDatasmithMeshImpl();

	void SetIdInUse( int32 Id );
	bool GetIdInUse( int32 Id ) const;
	int32 GetMaterialsCount() const { return IdsInUse.Num(); }

	FMD5Hash CalculateHash() const;

	FString Name;

	TArray< FVector3f > Vertices;
	TArray< uint32 > Indices;

	TArray< FVector3f > Normals;
	TArray< int32 > MaterialIndices;

	TArray< TArray< FVector2D > > UVs;
	TArray< TArray< int32 > > UVIndices;

	TArray< uint32 > FaceSmoothingMasks;

	TIndirectArray< FDatasmithMesh > LODs;

	TArray< FColor > IndicesColor;

	int32 LightmapUVChannel;

	FBox3f Extents;

private:
	TSet< int32 > IdsInUse;
};

FDatasmithMesh::FDatasmithMeshImpl::FDatasmithMeshImpl()
	: LightmapUVChannel(-1)
	, Extents( ForceInit )
{
}

void FDatasmithMesh::FDatasmithMeshImpl::SetIdInUse( int32 Id )
{
	IdsInUse.Add( Id );
}

bool FDatasmithMesh::FDatasmithMeshImpl::GetIdInUse( int32 Id ) const
{
	return IdsInUse.Contains(Id);
}

FMD5Hash FDatasmithMesh::FDatasmithMeshImpl::CalculateHash() const
{
	FMD5 MD5;

	using namespace DatasmithMeshImpl;

	UpdateMD5Array(MD5, Vertices);
	UpdateMD5Array(MD5, Indices);

	UpdateMD5Array(MD5, Normals);
	UpdateMD5Array(MD5, MaterialIndices);

	int32 UVChannelCount = UVs.Num();
	UpdateMD5SimpleType(MD5, UVChannelCount);
	for (const TArray<FVector2D>& UVChannel: UVs)
	{
		UpdateMD5Array(MD5, UVChannel);
	}

	check(UVChannelCount == UVIndices.Num()); // UV indices are per-channel
	for (const TArray<int32>& UVChannelIndices: UVIndices)
	{
		UpdateMD5Array(MD5, UVChannelIndices);
	}

	UpdateMD5Array(MD5, FaceSmoothingMasks);
	UpdateMD5Array(MD5, IndicesColor);
	UpdateMD5SimpleType(MD5, LightmapUVChannel);

	TArray<int32> IdsInUseArray = IdsInUse.Array();
	IdsInUseArray.Sort();
	UpdateMD5Array(MD5, IdsInUseArray);

	int32 LODCount = LODs.Num();
	UpdateMD5SimpleType(MD5, LODCount);
	for (const FDatasmithMesh& LODMesh : LODs)
	{
		FMD5Hash LODHash = LODMesh.CalculateHash();
		MD5.Update(LODHash.GetBytes(), LODHash.GetSize());
	}

	FMD5Hash Hash;
	Hash.Set(MD5);
	return Hash;
}

FDatasmithMesh::FDatasmithMesh()
	: Impl( new FDatasmithMeshImpl() )
{
}

FDatasmithMesh::~FDatasmithMesh()
{
	delete Impl;
}

FDatasmithMesh::FDatasmithMesh( const FDatasmithMesh& Other )
	: Impl( new FDatasmithMeshImpl( *Other.Impl ) )
{
}

FDatasmithMesh::FDatasmithMesh( FDatasmithMesh&& Other )
{
	Impl = Other.Impl;
	Other.Impl = nullptr;
}

FDatasmithMesh& FDatasmithMesh::operator=( const FDatasmithMesh& Other )
{
	delete Impl;
	Impl = new FDatasmithMeshImpl( *Other.Impl );

	return *this;
}

FDatasmithMesh& FDatasmithMesh::operator=( FDatasmithMesh&& Other )
{
	delete Impl;
	Impl = Other.Impl;
	Other.Impl = nullptr;

	return *this;
}

FMD5Hash FDatasmithMesh::CalculateHash() const
{
	return Impl->CalculateHash();
}


void FDatasmithMesh::SetName(const TCHAR* InName)
{
	Impl->Name = InName;
}

const TCHAR* FDatasmithMesh::GetName() const
{
	return *Impl->Name;
}

void FDatasmithMesh::SetFacesCount(int32 NumFaces)
{
	Impl->Indices.Empty(NumFaces * 3);
	Impl->Indices.Init(0, NumFaces * 3);

	Impl->MaterialIndices.Empty(NumFaces);
	Impl->MaterialIndices.Init(0, NumFaces);

	Impl->Normals.Empty(NumFaces * 3);
	Impl->Normals.Init( FVector3f::ZeroVector, NumFaces * 3 );

	Impl->FaceSmoothingMasks.Empty( NumFaces );
	Impl->FaceSmoothingMasks.AddZeroed( NumFaces );

	for ( TArray< int32 >& UVIndices : Impl->UVIndices )
	{
		UVIndices.Empty( GetFacesCount() * 3 );
		UVIndices.AddZeroed( GetFacesCount() * 3 );
	}
}

int32 FDatasmithMesh::GetFacesCount() const
{
	return Impl->Indices.Num() / 3;
}

void FDatasmithMesh::SetFace(int32 Index, int32 Vertex1, int32 Vertex2, int32 Vertex3, int32 MaterialId)
{
	const int32 IndicesOffset = Index * 3;

	if ( Impl->Indices.IsValidIndex( IndicesOffset ) )
	{
		Impl->Indices[IndicesOffset + 0] = Vertex1;
		Impl->Indices[IndicesOffset + 1] = Vertex2;
		Impl->Indices[IndicesOffset + 2] = Vertex3;
		Impl->MaterialIndices[Index] = MaterialId;

		Impl->SetIdInUse(MaterialId);
	}
}

void FDatasmithMesh::GetFace(int32 Index, int32& Vertex1, int32& Vertex2, int32& Vertex3, int32& MaterialId) const
{
	const int32 IndicesOffset = Index * 3;

	if ( Impl->Indices.IsValidIndex( IndicesOffset ) )
	{
		Vertex1 = Impl->Indices[IndicesOffset + 0];
		Vertex2 = Impl->Indices[IndicesOffset + 1];
		Vertex3 = Impl->Indices[IndicesOffset + 2];
		MaterialId = Impl->MaterialIndices[Index];
	}
}

int32 FDatasmithMesh::GetMaterialsCount() const
{
	return Impl->GetMaterialsCount();
}

bool FDatasmithMesh::IsMaterialIdUsed(int32 MaterialId) const
{
	return Impl->GetIdInUse( MaterialId );
}

void FDatasmithMesh::SetVerticesCount(int32 NumVerts)
{
	Impl->Vertices.Empty(NumVerts);
	Impl->Vertices.Init( FVector3f::ZeroVector, NumVerts );
}

int32 FDatasmithMesh::GetVerticesCount() const
{
	return Impl->Vertices.Num();
}

void FDatasmithMesh::SetVertex(int32 Index, float X, float Y, float Z)
{
	if ( Impl->Vertices.IsValidIndex( Index ) )
	{
		FVector3f Vertex(X, Y, Z);

		Impl->Vertices[Index] = Vertex;

		Impl->Extents += Vertex;
	}
}

FVector3f FDatasmithMesh::GetVertex(int32 Index) const
{
	if ( Impl->Vertices.IsValidIndex( Index ) )
	{
		return Impl->Vertices[Index];
	}
	else
	{
		return FVector3f::ZeroVector;
	}
}

void FDatasmithMesh::SetNormal(int32 Index, float X, float Y, float Z)
{
	if ( Impl->Normals.IsValidIndex( Index ) )
	{
		Impl->Normals[Index] = FVector3f(X, Y, Z).GetSafeNormal();
	}
}

FVector3f FDatasmithMesh::GetNormal(int32 Index) const
{
	if ( Impl->Normals.IsValidIndex( Index ) )
	{
		return Impl->Normals[Index];
	}
	else
	{
		return FVector3f::ZeroVector;
	}
}

void FDatasmithMesh::SetUVChannelsCount(int32 ChannelCount)
{
	Impl->UVs.Empty( ChannelCount );
	Impl->UVs.AddDefaulted( ChannelCount );

	Impl->UVIndices.Empty( ChannelCount );
	Impl->UVIndices.AddDefaulted( ChannelCount );

	for ( TArray< int32 >& UVIndices : Impl->UVIndices )
	{
		UVIndices.AddZeroed( GetFacesCount() * 3 );
	}
}

void FDatasmithMesh::AddUVChannel()
{
	Impl->UVs.AddDefaulted();
	Impl->UVIndices.AddDefaulted();

	Impl->UVIndices.Last().AddZeroed( GetFacesCount() * 3 );
}

void FDatasmithMesh::RemoveUVChannel()
{
	const int32 Index = Impl->UVs.Num() - 1;
	Impl->UVs.RemoveAt( Index, EAllowShrinking::No);
	Impl->UVIndices.RemoveAt( Index, EAllowShrinking::No);

}

int32 FDatasmithMesh::GetUVChannelsCount() const
{
	return Impl->UVs.Num();
}

void FDatasmithMesh::SetUVCount(int32 Channel, int32 NumVerts)
{
	if ( Impl->UVs.IsValidIndex( Channel ) )
	{
		Impl->UVs[ Channel ].Empty( NumVerts );
		Impl->UVs[ Channel ].AddDefaulted( NumVerts );
	}
}

int32 FDatasmithMesh::GetUVCount(int32 Channel) const
{
	if ( Impl->UVs.IsValidIndex( Channel ) )
	{
		return Impl->UVs[ Channel ].Num();
	}
	else
	{
		return 0;
	}
}

void FDatasmithMesh::SetUV(int32 Channel, int32 Index, double U, double V)
{
	if ( Impl->UVs.IsValidIndex( Channel ) )
	{
		TArray< FVector2D >& UVChannel = Impl->UVs[ Channel ];

		if ( UVChannel.IsValidIndex( Index ) )
		{
			UVChannel[ Index ].Set(U, V);
		}
	}
}

uint32 FDatasmithMesh::GetHashForUVChannel(int32 Channel) const
{
	if ( Impl->UVs.IsValidIndex( Channel ) && Impl->UVIndices.IsValidIndex( Channel ) )
	{
		const TArray< FVector2D >& UVChannel = Impl->UVs[Channel];
		uint32 Hash = GetTypeHash( UVChannel.Num() );
		const TArray< int32 >& UVIndices = Impl->UVIndices[Channel];

		Hash = HashCombine( Hash, GetTypeHash( UVIndices.Num() ) );

		// The mapping from the face wedges to the UV matter to the hash
		for ( const int32& Index : UVIndices )
		{
			Hash = HashCombine( Hash, GetTypeHash( UVChannel[Index] ) );
		}

		return Hash;
	}
	return 0;
}

FVector2D FDatasmithMesh::GetUV(int32 Channel, int32 Index) const
{
	if ( Impl->UVs.IsValidIndex( Channel ) )
	{
		const TArray< FVector2D >& UVChannel = Impl->UVs[ Channel ];

		if ( UVChannel.IsValidIndex( Index ) )
		{
			return UVChannel[ Index ];
		}
	}

	return FVector2D(ForceInit);
}

void FDatasmithMesh::SetFaceUV(int32 Index, int32 Channel, int32 Vertex1, int32 Vertex2, int32 Vertex3)
{
	const int32 IndicesOffset = Index * 3;

	if ( Impl->UVIndices.IsValidIndex( Channel ) )
	{
		Impl->UVIndices[Channel][ IndicesOffset + 0 ] = Vertex1;
		Impl->UVIndices[Channel][ IndicesOffset + 1 ] = Vertex2;
		Impl->UVIndices[Channel][ IndicesOffset + 2 ] = Vertex3;
	}
}

void FDatasmithMesh::GetFaceUV(int32 Index, int32 Channel, int32& Vertex1, int32& Vertex2, int32& Vertex3) const
{
	const int32 IndicesOffset = Index * 3;

	if ( Impl->UVIndices.IsValidIndex( Channel ) && Impl->UVIndices[Channel].IsValidIndex( IndicesOffset ) )
	{
		Vertex1 = Impl->UVIndices[Channel][ IndicesOffset + 0 ];
		Vertex2 = Impl->UVIndices[Channel][ IndicesOffset + 1 ];
		Vertex3 = Impl->UVIndices[Channel][ IndicesOffset + 2 ];
	}
}

int32 FDatasmithMesh::GetVertexColorCount() const
{
	return Impl->IndicesColor.Num();
}

void FDatasmithMesh::SetVertexColor(int32 Index, const FColor& Color)
{
	if (Impl->IndicesColor.Num() == 0)
	{
		const int32 NumVertexColor = Impl->Indices.Num();
		Impl->IndicesColor.Empty(NumVertexColor);
		Impl->IndicesColor.AddZeroed(NumVertexColor);
	}

	if ( Impl->IndicesColor.IsValidIndex( Index ) )
	{
		Impl->IndicesColor[Index] = Color;
	}
}

FColor FDatasmithMesh::GetVertexColor(int32 Index) const
{
	if ( Impl->IndicesColor.IsValidIndex( Index ) )
	{
		return Impl->IndicesColor[ Index ];
	}

	return FColor::White;
}

void FDatasmithMesh::SetFaceSmoothingMask(int32 Index, uint32 SmoothingMask)
{
	if ( Impl->FaceSmoothingMasks.IsValidIndex( Index ) )
	{
		Impl->FaceSmoothingMasks[ Index ] = SmoothingMask;
	}
}

uint32 FDatasmithMesh::GetFaceSmoothingMask(int32 Index) const
{
	if ( Impl->FaceSmoothingMasks.IsValidIndex( Index ) )
	{
		return Impl->FaceSmoothingMasks[ Index ];
	}

	return 0;
}

void FDatasmithMesh::AddLOD( const FDatasmithMesh& InLODMesh )
{
	Impl->LODs.Add( new FDatasmithMesh( InLODMesh ) );
}

void FDatasmithMesh::AddLOD( FDatasmithMesh&& InLODMesh )
{
	Impl->LODs.Add( new FDatasmithMesh( MoveTemp( InLODMesh ) ) );
}

int32 FDatasmithMesh::GetLODsCount() const
{
	return Impl->LODs.Num();
}

FDatasmithMesh* FDatasmithMesh::GetLOD( int32 Index )
{
	if (Impl->LODs.IsValidIndex(Index))
	{
		return &Impl->LODs[Index];
	}

	return nullptr;
}

const FDatasmithMesh* FDatasmithMesh::GetLOD( int32 Index ) const
{
	if (Impl->LODs.IsValidIndex(Index))
	{
		return &Impl->LODs[Index];
	}

	return nullptr;
}

void FDatasmithMesh::SetLightmapSourceUVChannel(int32 Channel)
{
	Impl->LightmapUVChannel = Channel;
}

int32 FDatasmithMesh::GetLightmapSourceUVChannel() const
{
	return Impl->LightmapUVChannel;
}

float FDatasmithMesh::ComputeArea() const
{
	int32 NumFaces = Impl->Indices.Num() / 3;

	float Area = 0.f;

	for (int32 i = 0; i < NumFaces; i++)
	{
		Area += FDatasmithUtils::AreaTriangle3D(
			Impl->Vertices[ Impl->Indices[3 * i + 0] ],
			Impl->Vertices[ Impl->Indices[3 * i + 1] ],
			Impl->Vertices[ Impl->Indices[3 * i + 2] ]);
	}

	return Area;
}

FBox3f FDatasmithMesh::GetExtents() const
{
	return Impl->Extents;
}

