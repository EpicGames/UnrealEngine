// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMeshSerialization.h"

#include "DatasmithMeshUObject.h"

#include "Compression/OodleDataCompression.h"
#include "HAL/FileManager.h"
#include "Misc/Compression.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UObject/Package.h"

FArchive& operator<<(FArchive& Ar, FDatasmithMeshModels& Models)
{
	Ar << Models.MeshName;
	Ar << Models.bIsCollisionMesh;
	Ar << Models.SourceModels;

	return Ar;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
FArchive& operator<<(FArchive& Ar, FDatasmithClothInfo& Info)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	uint8 Version = 0;
	Ar << Version;

	Ar << Info.Cloth;

	return Ar;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


namespace DatasmithMeshSerializationImpl
{
enum class ECompressionMethod
{
	ECM_ZLib  = 1,
	ECM_Gzip  = 2,
	ECM_LZ4   = 3,
	ECM_Oodle = 4,

	ECM_Default = ECM_Oodle,
};

FName NAME_Oodle("Oodle");
FName GetMethodName(ECompressionMethod MethodCode)
{
	switch (MethodCode)
	{
		case ECompressionMethod::ECM_ZLib: return NAME_Zlib;
		case ECompressionMethod::ECM_Gzip: return NAME_Gzip;
		case ECompressionMethod::ECM_LZ4:  return NAME_LZ4;
		case ECompressionMethod::ECM_Oodle:return NAME_Oodle;
		default: ensure(0); return NAME_None;
	}
}

bool CompressInline(TArray<uint8>& UncompressedData, ECompressionMethod Method)
{
	int32 UncompressedSize = UncompressedData.Num();
	FName MethodName = GetMethodName(Method);
	int32 CompressedSize = (Method != ECompressionMethod::ECM_Oodle)
		? FCompression::CompressMemoryBound(MethodName, UncompressedSize)
		: FOodleDataCompression::CompressedBufferSizeNeeded(UncompressedSize);

	TArray<uint8> CompressedData;
	int32 HeaderSize = 5;
	CompressedData.AddUninitialized(CompressedSize + HeaderSize);

	// header
	{
		FMemoryWriter Ar(CompressedData);
		uint8 MethodCode = uint8(Method);
		Ar << MethodCode;
		Ar << UncompressedSize;
		check(HeaderSize == Ar.Tell());
	}

	if (Method == ECompressionMethod::ECM_Oodle)
	{
		auto Compressor = FOodleDataCompression::ECompressor::Kraken;
		auto Level = FOodleDataCompression::ECompressionLevel::VeryFast;
		if (int64 ActualCompressedSize = FOodleDataCompression::CompressParallel(CompressedData.GetData() + HeaderSize, CompressedSize, UncompressedData.GetData(), UncompressedSize, Compressor, Level))
		{
			CompressedData.SetNum(ActualCompressedSize + HeaderSize, EAllowShrinking::Yes);
			UncompressedData = MoveTemp(CompressedData);

			return true;
		}
	}
	else
	{
		if (FCompression::CompressMemory(MethodName, CompressedData.GetData() + HeaderSize, CompressedSize, UncompressedData.GetData(), UncompressedSize))
		{

			CompressedData.SetNum(CompressedSize + HeaderSize, EAllowShrinking::Yes);
			UncompressedData = MoveTemp(CompressedData);

			return true;
		}
	}

	UE_LOGF(LogDatasmith, Warning, "Compression failed");
	return false;
}

bool DecompressInline(TArray<uint8>& CompressedData)
{
	ECompressionMethod Method;
	uint8* BufferStart = nullptr;
	int32 UncompressedSize = -1;
	int32 CompressedSize = CompressedData.Num();
	int32 HeaderSize = 0;
	{
		FMemoryReader Ar(CompressedData);
		uint8 MethodCode = 0;
		Ar << MethodCode;
		Method = ECompressionMethod(MethodCode);
		Ar << UncompressedSize;
		HeaderSize = Ar.Tell();
	}

	FName MethodName = GetMethodName(Method);
	if (MethodName == NAME_Oodle)
	{
		TArray<uint8> UncompressedData;
		UncompressedData.SetNumUninitialized(UncompressedSize);
		bool Ok = FOodleDataCompression::Decompress(UncompressedData.GetData(), UncompressedSize, CompressedData.GetData() + HeaderSize, CompressedData.Num() - HeaderSize);
		if (Ok)
		{
			CompressedData = MoveTemp(UncompressedData);
			return true;
		}
	}
	else if (MethodName != NAME_None)
	{
		TArray<uint8> UncompressedData;
		UncompressedData.SetNumUninitialized(UncompressedSize);
		bool Ok = FCompression::UncompressMemory(MethodName, UncompressedData.GetData(), UncompressedData.Num(), CompressedData.GetData() + HeaderSize, CompressedData.Num() - HeaderSize);
		if (Ok)
		{
			CompressedData = MoveTemp(UncompressedData);
			return true;
		}
	}
	UE_LOGF(LogDatasmith, Warning, "Decompression failed");
	return false;
}

} // ns DatasmithMeshSerializationImpl


FMD5Hash FDatasmithPackedMeshes::Serialize(FArchive& Ar, bool bCompressed)
{
	using namespace DatasmithMeshSerializationImpl;

	// structure          from
	// | Guard           | v0 | constant string, prevents loading of invalid buffers
	// | SerialVersion   | v0 | this structure versioning
	// | BufferType      | v0 | describe the payload
	// | UEVer           | v1 | global engine version
	// | LicenseeUEVer   | v1 |
	// | CustomVersions  | v0 | internal structures use them, we have to pass them along
	// | Mesh buffer     | v0 | may be compressed, the actual payload

	const TCHAR* StaticGuard = TEXT("FDatasmithPackedMeshes");
	FString Guard = Ar.IsLoading() ? TEXT("") : StaticGuard;
	Ar << Guard;
	if (!ensure(Guard == StaticGuard))
	{
		Ar.SetError();
		return {};
	}

	uint32 SerialVersion = 1;
	Ar << SerialVersion;

	enum EBufferType{ RawMeshDescription, CompressedMeshDescription };
	uint8 BufferType = bCompressed ? CompressedMeshDescription : RawMeshDescription;
	Ar << BufferType;

	FMD5Hash OutHash;
	if (Ar.IsLoading())
	{
		// versioning
		if (SerialVersion >= 1)
		{
			FPackageFileVersion LoadingUEVer;
			Ar << LoadingUEVer;
			Ar.SetUEVer(LoadingUEVer);

			int32 LoadingLicenseeUEVer = 0;
			Ar << LoadingLicenseeUEVer;
			Ar.SetLicenseeUEVer(LoadingLicenseeUEVer);
		}
		FCustomVersionContainer CustomVersions;
		CustomVersions.Serialize(Ar);

		// content
		TArray<uint8> Bytes;
		Ar << Bytes;
		if (BufferType == CompressedMeshDescription)
		{
			DecompressInline(Bytes);
		}

		FMemoryReader BytesReader(Bytes, true);
		BytesReader.SetLicenseeUEVer(Ar.LicenseeUEVer());
		BytesReader.SetUEVer(Ar.UEVer());
		BytesReader.SetCustomVersions(CustomVersions);
		BytesReader << Meshes;
	}
	else
	{
		if (SerialVersion >= 1)
		{
			FPackageFileVersion SavingUEVer = Ar.UEVer();
			Ar << SavingUEVer;

			int32 SavingLicenseeUEVer = Ar.LicenseeUEVer();
			Ar << SavingLicenseeUEVer;
		}

		TArray<uint8> Bytes;
		FMemoryWriter BytesWriter(Bytes, true);
		BytesWriter << Meshes;
		if (BufferType == CompressedMeshDescription)
		{
			CompressInline(Bytes, ECompressionMethod::ECM_Default);
		}

		// MeshDescriptions uses custom versioning,
		FCustomVersionContainer MeshesCustomVersions = BytesWriter.GetCustomVersions();
		MeshesCustomVersions.Serialize(Ar);

		Ar << Bytes;
		FMD5 Md5;
		Md5.Update(Bytes.GetData(), Bytes.Num());
		OutHash.Set(Md5);
	}
	return OutHash;
}

TOptional<FMeshDescription> ExtractToMeshDescription(FDatasmithMeshSourceModel& SourceModel)
{
	FRawMesh RawMesh;
	SourceModel.RawMeshBulkData.LoadRawMesh( RawMesh );

	if ( !RawMesh.IsValid() )
	{
		return {};
	}

	// RawMesh -> MeshDescription conversion requires an {mat_index: slot_name} map for its PolygonGroups.
	TMap<int32, FName> GroupNamePerGroupIndex;

	// There is no guaranty that incoming RawMesh.FaceMaterialIndices are sequential, but the conversion assumes so.
	// -> we remap materials identifiers to material indices
	// eg:
	//   incoming per-face mat identifier   5   5   1   1   1   99   99
	//   remapped per-face index            0   0   1   1   1   2    2
	//   per PolygonGroup FName:           "5" "5" "1" "1" "1" "99" "99"
	TSet<int32> MaterialIdentifiers;
	for (int32& MatIdentifier : RawMesh.FaceMaterialIndices)
	{
		bool bAlreadyIn = false;
		int32 IndexOfIdentifier = MaterialIdentifiers.Add(MatIdentifier, &bAlreadyIn).AsInteger();

		// identifier -> name association
		if (!bAlreadyIn)
		{
			FName MaterialSlotName = *FString::FromInt(MatIdentifier);
			GroupNamePerGroupIndex.Add(IndexOfIdentifier, MaterialSlotName);
		}

		// remap old identifier to material index
		MatIdentifier = IndexOfIdentifier;
	}

	FMeshDescription MeshDescription;
	FStaticMeshAttributes(MeshDescription).Register();

	// Do not compute normals and tangents during conversion since we have other operations to apply
	// on the mesh that might invalidate them anyway and we must also validate the mesh to detect
	// vertex positions containing NaN before doing computation as MikkTSpace crashes on NaN values.
	const bool bSkipNormalsAndTangents = true;
	FStaticMeshOperations::ConvertFromRawMesh(RawMesh, MeshDescription, GroupNamePerGroupIndex, bSkipNormalsAndTangents);
	return MeshDescription;
}

// Reads previous format of meshes (RawMesh based)
TArray<FDatasmithMeshModels> GetDatasmithMeshFromMeshPath_Legacy(FArchive* Archive, int32 LeagacyNumMeshesCount)
{
	TArray< FDatasmithMeshModels > Result;

	UDatasmithMesh* DatasmithMesh = nullptr;
	{
		// Make sure the new UDatasmithMesh object is not created while a garbage collection is performed
		FGCScopeGuard GCGuard;
		// Setting the RF_Standalone bitmask on the new UDatasmithMesh object, to make sure it is not garbage collected
		// while loading and processing the udsmesh file. This can happen on very big meshes (5M+ triangles)
		DatasmithMesh = NewObject< UDatasmithMesh >(GetTransientPackage(), NAME_None, RF_Standalone);
	}

	// Currently we only have 1 mesh per file. If there's a second mesh, it will be a CollisionMesh
	while ( LeagacyNumMeshesCount-- > 0)
	{
		TArray< uint8 > Bytes;
		*Archive << Bytes;

		FMemoryReader MemoryReader( Bytes, true );
		MemoryReader.ArIgnoreClassRef = false;
		MemoryReader.ArIgnoreArchetypeRef = false;
		MemoryReader.SetWantBinaryPropertySerialization(true);
		MemoryReader.SetUEVer(FPackageFileVersion::CreateUE4Version(EUnrealEngineObjectUE4Version::VER_UE4_AUTOMATIC_VERSION));
		DatasmithMesh->Serialize( MemoryReader );

		FDatasmithMeshModels& MeshInternal = Result.AddDefaulted_GetRef();
		MeshInternal.bIsCollisionMesh = DatasmithMesh->bIsCollisionMesh;
		for (FDatasmithMeshSourceModel& SourceModel : DatasmithMesh->SourceModels)
		{
			if (TOptional<FMeshDescription> OptionalMesh = ExtractToMeshDescription(SourceModel))
			{
				MeshInternal.SourceModels.Add(MoveTemp(*OptionalMesh));
			}
		}
	}

	// Tell the garbage collector DatasmithMesh can now be deleted.
	DatasmithMesh->ClearInternalFlags(EInternalObjectFlags::Async);
	DatasmithMesh->ClearFlags(RF_Standalone);
	return Result;
}

FDatasmithPackedMeshes GetDatasmithMeshFromFile(const FString& MeshPath)
{
	FDatasmithPackedMeshes Result;

	TUniquePtr<FArchive> Archive( IFileManager::Get().CreateFileReader(*MeshPath) );
	if ( !Archive.IsValid() )
	{
		UE_LOGF(LogDatasmith, Warning, "Cannot read file %ls", *MeshPath);
		return Result;
	}

	int32 LeagacyNumMeshesCount = 0;
	*Archive << LeagacyNumMeshesCount;

	if (LeagacyNumMeshesCount > 0)
	{
		Result.Meshes = GetDatasmithMeshFromMeshPath_Legacy(Archive.Get(), LeagacyNumMeshesCount);
	}
	else
	{
		Result.Serialize(*Archive);

		if (Archive->IsError())
		{
			Result = FDatasmithPackedMeshes{};
			UE_LOGF(LogDatasmith, Warning, "Failed to read meshes from %ls", *MeshPath);
		}
	}

	return Result;
}

FMD5Hash FDatasmithPackedCloths::Serialize(FArchive& Ar, bool bSaveCompressed)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	using namespace DatasmithMeshSerializationImpl;
	// structure          from
	// | Guard           | v0 | constant string, prevents loading of invalid buffers
	// | SerialVersion   | v0 | this structure versioning
	// | BufferType      | v0 | describe the payload
	// | UEVer           | v0 | global engine version
	// | LicenseeUEVer   | v0 |
	// | Payload buffer  | v0 | may be compressed

	const TCHAR* StaticGuard = TEXT("FDatasmithPackedCloths");
	FString Guard = Ar.IsLoading() ? TEXT("") : StaticGuard;
	Ar << Guard;
	if (!ensure(Guard == StaticGuard))
	{
		Ar.SetError();
		return {};
	}

	uint32 SerialVersion = 0;
	Ar << SerialVersion;

	enum EBufferType{ Cloths, CompressedCloth };
	uint8 BufferType = bSaveCompressed ? CompressedCloth : Cloths;
	Ar << BufferType;

	FMD5Hash OutHash;
	if (Ar.IsLoading())
	{
		FPackageFileVersion LoadingUEVer;
		Ar << LoadingUEVer;
		Ar.SetUEVer(LoadingUEVer);

		int32 LoadingLicenseeUEVer = 0;
		Ar << LoadingLicenseeUEVer;
		Ar.SetLicenseeUEVer(LoadingLicenseeUEVer);

		FCustomVersionContainer CustomVersions;
		CustomVersions.Serialize(Ar);

		// content
		TArray<uint8> Bytes;
		Ar << Bytes;
		if (BufferType == CompressedCloth)
		{
			DecompressInline(Bytes);
		}

		FMemoryReader BytesReader(Bytes, true);
		BytesReader.SetLicenseeUEVer(Ar.LicenseeUEVer());
		BytesReader.SetUEVer(Ar.UEVer());
		BytesReader.SetCustomVersions(CustomVersions);
		BytesReader << ClothInfos;
	}
	else
	{
		FPackageFileVersion SavingUEVer = Ar.UEVer();
		Ar << SavingUEVer;

		int32 SavingLicenseeUEVer = Ar.LicenseeUEVer();
		Ar << SavingLicenseeUEVer;

		{
			TArray<uint8> Bytes;
			FMemoryWriter BytesWriter(Bytes, true);
			BytesWriter << ClothInfos;
			if (BufferType == CompressedCloth)
			{
				CompressInline(Bytes, ECompressionMethod::ECM_Default);
			}

			FCustomVersionContainer MeshesCustomVersions = BytesWriter.GetCustomVersions();
			MeshesCustomVersions.Serialize(Ar);

			Ar << Bytes;

			FMD5 Md5;
			Md5.Update(Bytes.GetData(), Bytes.Num());
			OutHash.Set(Md5);
		}
	}

	return OutHash;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDatasmithPackedCloths GetDatasmithClothFromFile(const FString& Path)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileReader(*Path));
	if ( !Archive.IsValid() )
	{
		UE_LOGF(LogDatasmith, Warning, "Cannot read file %ls", *Path);
		return {};
	}

	FDatasmithPackedCloths Result;
	Result.Serialize(*Archive);

	if (Archive->IsError())
	{
		UE_LOGF(LogDatasmith, Warning, "Failed to read cloth from %ls", *Path);
		return {};
	}

	return Result;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

