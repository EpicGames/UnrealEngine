// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMeshUObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithMeshUObject)

void FDatasmithMeshSourceModel::SerializeBulkData(FArchive& Ar, UObject* Owner)
{
	RawMeshBulkData.Serialize( Ar, Owner );
}

void UDatasmithMesh::Serialize(FArchive& Ar)
{
	Super::Serialize( Ar );

	for ( FDatasmithMeshSourceModel& SourceModel : SourceModels )
	{
		SourceModel.SerializeBulkData( Ar, this );
	}
}
