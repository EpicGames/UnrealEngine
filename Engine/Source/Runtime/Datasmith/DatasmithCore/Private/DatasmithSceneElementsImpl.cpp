// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneElementsImpl.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"

bool IDatasmithShaderElement::bUseRealisticFresnel = true;
bool IDatasmithShaderElement::bDisableReflectionFresnel = false;

FDatasmithMeshElementImpl::FDatasmithMeshElementImpl(const TCHAR* InName)
	: FDatasmithElementImpl(InName, EDatasmithElementType::StaticMesh)
	, Area(0.f)
	, Width(0.f)
	, Height(0.f)
	, Depth(0.f)
	, LODCount(1)
	, LightmapCoordinateIndex(-1)
	, LightmapSourceUV(-1)
{
	RegisterReferenceProxy(MaterialSlots, "MaterialSlots");

	Store.RegisterParameter(File,                    "File"                    );
	Store.RegisterParameter(FileHash,                "FileHash"                );
	Store.RegisterParameter(Area,                    "Area"                    );
	Store.RegisterParameter(Width,                   "Width"                   );
	Store.RegisterParameter(Height,                  "Height"                  );
	Store.RegisterParameter(Depth,                   "Depth"                   );
	Store.RegisterParameter(LODCount,                "LODCount"                );
	Store.RegisterParameter(LightmapCoordinateIndex, "LightmapCoordinateIndex" );
	Store.RegisterParameter(LightmapSourceUV,        "LightmapSourceUV"        );
}

FMD5Hash FDatasmithMeshElementImpl::CalculateElementHash(bool bForce)
{
	if (ElementHash.IsValid() && !bForce)
	{
		return ElementHash;
	}
	FMD5 MD5;
	const FMD5Hash& FileHashValue = FileHash;
	MD5.Update(FileHashValue.GetBytes(), FileHashValue.GetSize());
	MD5.Update(reinterpret_cast<const uint8*>(&LightmapSourceUV), sizeof(LightmapSourceUV));
	MD5.Update(reinterpret_cast<const uint8*>(&LightmapCoordinateIndex), sizeof(LightmapCoordinateIndex));

	for (const TSharedPtr<IDatasmithMaterialIDElement>& MatID : MaterialSlots.View())
	{
		int32 ThisMaterialId = MatID->GetId();
		MD5.Update(reinterpret_cast<const uint8*>(&ThisMaterialId), sizeof(ThisMaterialId));
		MD5.Update(reinterpret_cast<const uint8*>(MatID->GetName()), TCString<TCHAR>::Strlen(MatID->GetName()) * sizeof(TCHAR));
	}
	ElementHash.Set(MD5);
	return ElementHash;
}

void FDatasmithMeshElementImpl::SetMaterial(const TCHAR* MaterialPathName, int32 SlotId)
{
	for (const TSharedPtr<IDatasmithMaterialIDElement>& Slot : MaterialSlots.View())
	{
		if (Slot->GetId() == SlotId)
		{
			Slot->SetName(MaterialPathName);
			return;
		}
	}
	TSharedPtr< IDatasmithMaterialIDElement > MaterialIDElement = FDatasmithSceneFactory::CreateMaterialId(MaterialPathName);
	MaterialIDElement->SetId(SlotId);
	MaterialSlots.Add(MoveTemp(MaterialIDElement));
}

const TCHAR* FDatasmithMeshElementImpl::GetMaterial(int32 SlotId) const
{
	for (const TSharedPtr<IDatasmithMaterialIDElement>& Slot : MaterialSlots.View())
	{
		if (Slot->GetId() == SlotId)
		{
			return Slot->GetName();
		}
	}
	return nullptr;
}

int32 FDatasmithMeshElementImpl::GetMaterialSlotCount() const
{
	return MaterialSlots.Num();
}

TSharedPtr<const IDatasmithMaterialIDElement> FDatasmithMeshElementImpl::GetMaterialSlotAt(int32 Index) const
{
	if (MaterialSlots.IsValidIndex(Index))
	{
		return MaterialSlots[Index];
	}
	const TSharedPtr<IDatasmithMaterialIDElement> InvalidMaterialID;
	return InvalidMaterialID;
}

TSharedPtr<IDatasmithMaterialIDElement> FDatasmithMeshElementImpl::GetMaterialSlotAt(int32 Index)
{
	if (MaterialSlots.IsValidIndex(Index))
	{
		return MaterialSlots[Index];
	}
	const TSharedPtr<IDatasmithMaterialIDElement> InvalidMaterialID;
	return InvalidMaterialID;
}

FDatasmithClothElementImpl::FDatasmithClothElementImpl(const TCHAR* InName)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: FDatasmithElementImpl(InName, EDatasmithElementType::Cloth)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	Store.RegisterParameter(File, "File");
}

TSharedPtr< IDatasmithKeyValueProperty > FDatasmithKeyValuePropertyImpl::NullPropertyPtr;

FDatasmithKeyValuePropertyImpl::FDatasmithKeyValuePropertyImpl(const TCHAR* InName)
	: FDatasmithElementImpl(InName, EDatasmithElementType::KeyValueProperty)
	, PropertyType(EDatasmithKeyValuePropertyType::String)
{
	FDatasmithKeyValuePropertyImpl::SetName(InName);
	Store.RegisterParameter(Value, "Value");
	Store.RegisterParameter(PropertyType, "PropertyType");
}

void FDatasmithKeyValuePropertyImpl::SetPropertyType( EDatasmithKeyValuePropertyType InType )
{
	PropertyType = InType;
	FormatValue();
}

void FDatasmithKeyValuePropertyImpl::SetValue(const TCHAR* InValue)
{
	Value = InValue;
	FormatValue();
}

void FDatasmithKeyValuePropertyImpl::FormatValue()
{
	FString Tmp = Value;
	if ( Tmp.Len() > 0 && (
		GetPropertyType() == EDatasmithKeyValuePropertyType::Vector ||
		GetPropertyType() == EDatasmithKeyValuePropertyType::Color ) )
	{
		if ( Tmp[0] != TEXT('(') )
		{
			Tmp.InsertAt( 0, TEXT("(") );
		}

		if ( Tmp[ Tmp.Len() - 1 ] != TEXT(')') )
		{
			Tmp += TEXT(")");
		}

		Tmp.ReplaceInline( TEXT(" "), TEXT(",") ); // FVector::ToString separates the arguments with a " " rather than with a ","
	}
	Value = Tmp;
}

FDatasmithMaterialIDElementImpl::FDatasmithMaterialIDElementImpl(const TCHAR* InName)
	: FDatasmithElementImpl( InName, EDatasmithElementType::MaterialId )
	, Id( 0 )
{
	FDatasmithMaterialIDElementImpl::SetName(InName); // no virtual call from ctr
	Store.RegisterParameter(Id, "Id");
}

FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::FDatasmithHierarchicalInstancedStaticMeshActorElementImpl(const TCHAR* InName)
	: FDatasmithMeshActorElementImpl< IDatasmithHierarchicalInstancedStaticMeshActorElement >(InName, EDatasmithElementType::HierarchicalInstanceStaticMesh)
{
	Store.RegisterParameter(Instances, "Instaces");
}

FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::~FDatasmithHierarchicalInstancedStaticMeshActorElementImpl()
{
}

int32 FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::GetInstancesCount() const
{
	return Instances.Get().Num();
}

void FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::ReserveSpaceForInstances(int32 NumIntances)
{
	Instances.Get().Reserve(NumIntances);
}

int32 FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::AddInstance(const FTransform& Transform)
{
	Instances.Get().Add(Transform);
	return Instances.Get().Num() - 1;
}

FTransform FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::GetInstance(int32 InstanceIndex) const
{
	if (Instances.Get().IsValidIndex(InstanceIndex))
	{
		return Instances.Get()[InstanceIndex];
	}
	return FTransform();
}

void FDatasmithHierarchicalInstancedStaticMeshActorElementImpl::RemoveInstance(int32 InstanceIndex)
{
	if (Instances.Get().IsValidIndex(InstanceIndex))
	{
		Instances.Get().RemoveAtSwap(InstanceIndex);
	}
}

FDatasmithPostProcessElementImpl::FDatasmithPostProcessElementImpl()
	: FDatasmithElementImpl( TEXT("unnamed"), EDatasmithElementType::PostProcess )
	, Temperature(6500.0f)
	, ColorFilter(FVector3f::ZeroVector)
	, Vignette(0.0f)
	, Dof(0.0f)
	, MotionBlur(0.0f)
	, Saturation(1.0f)
	, CameraISO(-1.f) // Negative means don't override
	, CameraShutterSpeed(-1.f)
	, Fstop(-1.f)
{
	Store.RegisterParameter(Temperature,        "Temperature"        );
	Store.RegisterParameter(Vignette,           "Vignette"           );
	Store.RegisterParameter(Dof,                "Dof"                );
	Store.RegisterParameter(MotionBlur,         "MotionBlur"         );
	Store.RegisterParameter(Saturation,         "Saturation"         );
	Store.RegisterParameter(ColorFilter,        "ColorFilter"        );
	Store.RegisterParameter(CameraISO,          "CameraISO"          );
	Store.RegisterParameter(CameraShutterSpeed, "CameraShutterSpeed" );
	Store.RegisterParameter(Fstop,              "Fstop"              );
}

FDatasmithPostProcessVolumeElementImpl::FDatasmithPostProcessVolumeElementImpl(const TCHAR* InName)
	: FDatasmithActorElementImpl( InName, EDatasmithElementType::PostProcessVolume )
	, Settings(MakeShared<FDatasmithPostProcessElementImpl>())
	, bEnabled(true)
	, bUnbound(true)
{
	RegisterReferenceProxy(Settings,  "Settings" );

	Store.RegisterParameter(bEnabled, "bEnabled" );
	Store.RegisterParameter(bUnbound, "bUnbound" );
}

FDatasmithCameraActorElementImpl::FDatasmithCameraActorElementImpl(const TCHAR* InName)
	: FDatasmithActorElementImpl(InName, EDatasmithElementType::Camera)
	, PostProcess(MakeShared<FDatasmithPostProcessElementImpl>())
	, SensorWidth(36.0f)
	, SensorAspectRatio(1.7777777f)
	, bEnableDepthOfField(true)
	, FocusDistance(1000.0f)
	, FStop(5.6f)
	, FocalLength(35.0f)
	, ActorName()
	, bLookAtAllowRoll(false)
{
	RegisterReferenceProxy(PostProcess, "PostProcess");

	Store.RegisterParameter(SensorWidth,         "SensorWidth"        );
	Store.RegisterParameter(SensorAspectRatio,   "SensorAspectRatio"  );
	Store.RegisterParameter(bEnableDepthOfField, "bEnableDepthOfField");
	Store.RegisterParameter(FocusDistance,       "FocusDistance"      );
	Store.RegisterParameter(FStop,               "FStop"              );
	Store.RegisterParameter(FocalLength,         "FocalLength"        );
	Store.RegisterParameter(ActorName,           "ActorName"          );
	Store.RegisterParameter(bLookAtAllowRoll,    "bLookAtAllowRoll"   );

	SetMobility(EDatasmithActorMobilityType::Movable);
}

float FDatasmithCameraActorElementImpl::GetSensorWidth() const
{
	return SensorWidth;
}

void FDatasmithCameraActorElementImpl::SetSensorWidth(float InSensorWidth)
{
	SensorWidth = InSensorWidth;
}

float FDatasmithCameraActorElementImpl::GetSensorAspectRatio() const
{
	return SensorAspectRatio;
}

void FDatasmithCameraActorElementImpl::SetSensorAspectRatio(float InSensorAspectRatio)
{
	SensorAspectRatio = InSensorAspectRatio;
}

float FDatasmithCameraActorElementImpl::GetFocusDistance() const
{
	return FocusDistance;
}

void FDatasmithCameraActorElementImpl::SetFocusDistance(float InFocusDistance)
{
	FocusDistance = InFocusDistance;
}

float FDatasmithCameraActorElementImpl::GetFStop() const
{
	return FStop;
}

void FDatasmithCameraActorElementImpl::SetFStop(float InFStop)
{
	FStop = InFStop;
}

float FDatasmithCameraActorElementImpl::GetFocalLength() const
{
	return FocalLength;
}

void FDatasmithCameraActorElementImpl::SetFocalLength(float InFocalLength)
{
	FocalLength = InFocalLength;
}

TSharedPtr< IDatasmithPostProcessElement >& FDatasmithCameraActorElementImpl::GetPostProcess()
{
	return PostProcess.Inner;
}

const TSharedPtr< IDatasmithPostProcessElement >& FDatasmithCameraActorElementImpl::GetPostProcess() const
{
	return PostProcess.Inner;
}

void FDatasmithCameraActorElementImpl::SetPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& InPostProcess)
{
	PostProcess.Inner = InPostProcess;
}

FDatasmithMaterialElementImpl::FDatasmithMaterialElementImpl(const TCHAR* InName)
	: FDatasmithBaseMaterialElementImpl(InName, EDatasmithElementType::Material)
{
}

bool FDatasmithMaterialElementImpl::IsSingleShaderMaterial() const
{
	return GetShadersCount() == 1;
}

bool FDatasmithMaterialElementImpl::IsClearCoatMaterial() const
{
	if (GetShadersCount() != 2)
	{
		return false;
	}

	if (GetShader(0)->GetBlendMode() != EDatasmithBlendMode::ClearCoat)
	{
		return false;
	}

	return true;
}

void FDatasmithMaterialElementImpl::AddShader( const TSharedPtr< IDatasmithShaderElement >& InShader )
{
	Shaders.Add(InShader);
}

int32 FDatasmithMaterialElementImpl::GetShadersCount() const
{
	return (int32)Shaders.Num();
}

TSharedPtr< IDatasmithShaderElement >& FDatasmithMaterialElementImpl::GetShader(int32 InIndex)
{
	return Shaders[InIndex];
}

const TSharedPtr< IDatasmithShaderElement >& FDatasmithMaterialElementImpl::GetShader(int32 InIndex) const
{
	return Shaders[InIndex];
}

FDatasmithMaterialIntanceElementImpl::FDatasmithMaterialIntanceElementImpl(const TCHAR* InName)
	: FDatasmithBaseMaterialElementImpl(InName, EDatasmithElementType::MaterialInstance)
	, MaterialType( EDatasmithReferenceMaterialType::Auto )
	, Quality( EDatasmithReferenceMaterialQuality::High )
{
	RegisterReferenceProxy(Properties, "Properties");

	Store.RegisterParameter(MaterialType,           "MaterialType"          );
	Store.RegisterParameter(Quality,                "Quality"               );
	Store.RegisterParameter(CustomMaterialPathName, "CustomMaterialPathName");
}

const TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMaterialIntanceElementImpl::GetProperty( int32 InIndex ) const
{
	return Properties.IsValidIndex( InIndex ) ? Properties[InIndex] : FDatasmithKeyValuePropertyImpl::NullPropertyPtr;
}

const TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMaterialIntanceElementImpl::GetPropertyByName( const TCHAR* InName ) const
{
	int32 Index = Properties.View().IndexOfByPredicate([InName](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
		return Property.IsValid() && FCString::Stricmp(Property->GetName(), InName) == 0;
	});
	return GetProperty(Index);
}

void FDatasmithMaterialIntanceElementImpl::AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& InProperty )
{
	if (!InProperty.IsValid())
	{
		return;
	}

	const TCHAR* InName = InProperty->GetName();
	auto elt = Properties.View().FindByPredicate([InName](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
		return Property.IsValid() && FCString::Stricmp(Property->GetName(), InName) == 0;
	});

	if ( elt == nullptr )
	{
		Properties.Add( InProperty );
	}
}

FMD5Hash FDatasmithMaterialIntanceElementImpl::CalculateElementHash(bool bForce)
{
	if (ElementHash.IsValid() && !bForce)
	{
		return ElementHash;
	}

	FMD5 MD5;
	MD5.Update(reinterpret_cast<const uint8*>(&MaterialType), sizeof(MaterialType));
	MD5.Update(reinterpret_cast<const uint8*>(&Quality), sizeof(Quality));

	const FString& CustomName = CustomMaterialPathName;
	if (!CustomName.IsEmpty())
	{
		MD5.Update(reinterpret_cast<const uint8*>(*CustomName), CustomName.Len() * sizeof(TCHAR));
	}

	for (const TSharedPtr<IDatasmithKeyValueProperty>& Property : Properties.View())
	{
		const TCHAR* PropertyName = Property->GetName();
		MD5.Update(reinterpret_cast<const uint8*>(PropertyName), TCString<TCHAR>::Strlen(PropertyName) * sizeof(TCHAR));
		const TCHAR* PropertyValue = Property->GetValue();
		MD5.Update(reinterpret_cast<const uint8*>(PropertyValue), TCString<TCHAR>::Strlen(PropertyValue) * sizeof(TCHAR));
		EDatasmithKeyValuePropertyType PropertyType = Property->GetPropertyType();
		MD5.Update(reinterpret_cast<const uint8*>(&PropertyType), sizeof(PropertyType));
	}

	ElementHash.Set(MD5);
	return ElementHash;
}

FDatasmithEnvironmentElementImpl::FDatasmithEnvironmentElementImpl(const TCHAR* InName)
	: FDatasmithLightActorElementImpl(InName, EDatasmithElementType::EnvironmentLight)
	, EnvironmentComp( MakeShared<FDatasmithCompositeTextureImpl>() )
	, bIsIlluminationMap(false)
{
	Store.RegisterParameter(bIsIlluminationMap, "bIsIlluminationMap" );
}

TSharedPtr<IDatasmithCompositeTexture>& FDatasmithEnvironmentElementImpl::GetEnvironmentComp()
{
	return EnvironmentComp;
}

const TSharedPtr<IDatasmithCompositeTexture>& FDatasmithEnvironmentElementImpl::GetEnvironmentComp() const
{
	return EnvironmentComp;
}

void FDatasmithEnvironmentElementImpl::SetEnvironmentComp(const TSharedPtr<IDatasmithCompositeTexture>& InEnvironmentComp)
{
	EnvironmentComp = InEnvironmentComp;
}

bool FDatasmithEnvironmentElementImpl::GetIsIlluminationMap() const
{
	return bIsIlluminationMap;
}

void FDatasmithEnvironmentElementImpl::SetIsIlluminationMap(bool bInIsIlluminationMap)
{
	bIsIlluminationMap = bInIsIlluminationMap;
}

FDatasmithTextureElementImpl::FDatasmithTextureElementImpl(const TCHAR* InName)
	: FDatasmithElementImpl( InName, EDatasmithElementType::Texture )
{
	TextureMode = EDatasmithTextureMode::Other;
	TextureFilter = EDatasmithTextureFilter::Default;
	TextureAddressX = EDatasmithTextureAddress::Wrap;
	TextureAddressY = EDatasmithTextureAddress::Wrap;
	bAllowResize = true; // only disabled for environment maps
	RGBCurve = -1.0;
	ColorSpace = EDatasmithColorSpace::Default;

	Data = nullptr;
	DataSize = 0;

	Store.RegisterParameter(File,            "File"           );
	Store.RegisterParameter(FileHash,        "FileHash"       );
	Store.RegisterParameter(RGBCurve,        "RGBCurve"       );
	Store.RegisterParameter(ColorSpace,      "ColorSpace"     );
	Store.RegisterParameter(TextureMode,     "TextureMode"    );
	Store.RegisterParameter(TextureFilter,   "TextureFilter"  );
	Store.RegisterParameter(TextureAddressX, "TextureAddressX");
	Store.RegisterParameter(TextureAddressY, "TextureAddressY");
	Store.RegisterParameter(bAllowResize,    "bAllowResize"   );
	// buffer ?
	Store.RegisterParameter(TextureFormat,   "TextureFormat"  );
}

FMD5Hash FDatasmithTextureElementImpl::CalculateElementHash(bool bForce)
{
	if (ElementHash.IsValid() && !bForce)
	{
		return ElementHash;
	}
	FMD5 MD5;
	// Update FileHash if it has not already been done
	if (!FileHash.Get().IsValid() && GetFile() != nullptr && FCString::Strlen(GetFile()) > 0)
	{
		FileHash = FMD5Hash::HashFile(GetFile());
	}
	const FMD5Hash& FileHashValue = FileHash.Get();
	MD5.Update(FileHashValue.GetBytes(), FileHashValue.GetSize());
	MD5.Update(reinterpret_cast<uint8*>(&RGBCurve), sizeof(RGBCurve));
	MD5.Update(reinterpret_cast<uint8*>(&TextureMode), sizeof(TextureMode));
	MD5.Update(reinterpret_cast<uint8*>(&TextureFilter), sizeof(TextureFilter));
	MD5.Update(reinterpret_cast<uint8*>(&TextureAddressX), sizeof(TextureAddressX));
	MD5.Update(reinterpret_cast<uint8*>(&TextureAddressY), sizeof(TextureAddressY));

	if (DataSize > 0 && Data != nullptr)
	{
		MD5.Update(Data, DataSize);
	}

	ElementHash.Set(MD5);
	return ElementHash;
}

const TCHAR* FDatasmithTextureElementImpl::GetFile() const
{
	return *(FString&)File;
}

void FDatasmithTextureElementImpl::SetFile(const TCHAR* InFile)
{
	File = InFile;
}

EDatasmithTextureMode FDatasmithTextureElementImpl::GetTextureMode() const
{
	return TextureMode;
}

void FDatasmithTextureElementImpl::SetData(const uint8* InData, uint32 InDataSize, EDatasmithTextureFormat InFormat)
{
	Data = InData;
	DataSize = InDataSize;
	TextureFormat = InFormat;
}

const uint8* FDatasmithTextureElementImpl::GetData(uint32& OutDataSize, EDatasmithTextureFormat& OutFormat) const
{
	OutDataSize = DataSize;
	OutFormat = TextureFormat;
	return Data;
}

void FDatasmithTextureElementImpl::SetTextureMode(EDatasmithTextureMode InMode)
{
	TextureMode = InMode;
}

EDatasmithTextureFilter FDatasmithTextureElementImpl::GetTextureFilter() const
{
	return TextureFilter;
}

void FDatasmithTextureElementImpl::SetTextureFilter(EDatasmithTextureFilter InFilter)
{
	TextureFilter = InFilter;
}

EDatasmithTextureAddress FDatasmithTextureElementImpl::GetTextureAddressX() const
{
	return TextureAddressX;
}

void FDatasmithTextureElementImpl::SetTextureAddressX(EDatasmithTextureAddress InMode)
{
	TextureAddressX = InMode;
}

EDatasmithTextureAddress FDatasmithTextureElementImpl::GetTextureAddressY() const
{
	return TextureAddressY;
}

void FDatasmithTextureElementImpl::SetTextureAddressY(EDatasmithTextureAddress InMode)
{
	TextureAddressY = InMode;
}

bool FDatasmithTextureElementImpl::GetAllowResize() const
{
	return bAllowResize;
}

void FDatasmithTextureElementImpl::SetAllowResize(bool bInAllowResize)
{
	bAllowResize = bInAllowResize;
}

float FDatasmithTextureElementImpl::GetRGBCurve() const
{
	return RGBCurve;
}

void FDatasmithTextureElementImpl::SetRGBCurve(float InRGBCurve)
{
	RGBCurve = InRGBCurve;
}

EDatasmithColorSpace FDatasmithTextureElementImpl::GetSRGB() const
{
	return ColorSpace;
}

void FDatasmithTextureElementImpl::SetSRGB(EDatasmithColorSpace Option)
{
	ColorSpace = Option;
}

FDatasmithShaderElementImpl::FDatasmithShaderElementImpl(const TCHAR* InName)
	: FDatasmithElementImpl( InName, EDatasmithElementType::Shader )
	, IOR(0.0)
	, IORk(0.0)
	, IORRefra(0.0)
	, BumpAmount(1.0)
	, bTwoSided(false)
	, DiffuseColor(FLinearColor(0.f, 0.f, 0.f))
	, DiffuseComp( new FDatasmithCompositeTextureImpl() )
	, ReflectanceColor(FLinearColor(0.f, 0.f, 0.f))
	, RefleComp( new FDatasmithCompositeTextureImpl() )
	, Roughness(0.01)
	, RoughnessComp( new FDatasmithCompositeTextureImpl() )
	, NormalComp( new FDatasmithCompositeTextureImpl() )
	, BumpComp( new FDatasmithCompositeTextureImpl() )
	, TransparencyColor(FLinearColor(0.f, 0.f, 0.f))
	, TransComp( new FDatasmithCompositeTextureImpl() )
	, MaskComp( new FDatasmithCompositeTextureImpl() )
	, Displace(0.0)
	, DisplaceSubDivision(0)
	, DisplaceComp( new FDatasmithCompositeTextureImpl() )
	, Metal(0.0)
	, MetalComp(new FDatasmithCompositeTextureImpl())
	, EmitColor(FLinearColor(0.f, 0.f, 0.f))
	, EmitTemperature(0)
	, EmitPower(0)
	, EmitComp( new FDatasmithCompositeTextureImpl() )
	, bLightOnly(false)
	, WeightColor(FLinearColor(0.f, 0.f, 0.f))
	, WeightComp(new FDatasmithCompositeTextureImpl())
	, WeightValue(1.0)
	, BlendMode(EDatasmithBlendMode::Alpha)
	, bIsStackedLayer(false)
	, ShaderUsage(EDatasmithShaderUsage::Surface)
	, bUseEmissiveForDynamicAreaLighting(false)
{
	FDatasmithShaderElementImpl::GetDiffuseComp()->SetBaseNames(DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, TEXT("unsupported"), DATASMITH_DIFFUSECOMPNAME);
	FDatasmithShaderElementImpl::GetRefleComp()->SetBaseNames(DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, TEXT("unsupported"), DATASMITH_REFLECOMPNAME);
	FDatasmithShaderElementImpl::GetRoughnessComp()->SetBaseNames(DATASMITH_ROUGHNESSTEXNAME, TEXT("unsupported"), DATASMITH_ROUGHNESSVALUENAME, DATASMITH_ROUGHNESSCOMPNAME);
	FDatasmithShaderElementImpl::GetNormalComp()->SetBaseNames(DATASMITH_NORMALTEXNAME, TEXT("unsupported"), DATASMITH_BUMPVALUENAME, DATASMITH_NORMALCOMPNAME);
	FDatasmithShaderElementImpl::GetBumpComp()->SetBaseNames(DATASMITH_BUMPTEXNAME, TEXT("unsupported"), DATASMITH_BUMPVALUENAME, DATASMITH_BUMPCOMPNAME);
	FDatasmithShaderElementImpl::GetTransComp()->SetBaseNames(DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, TEXT("unsupported"), DATASMITH_TRANSPCOMPNAME);
	FDatasmithShaderElementImpl::GetMaskComp()->SetBaseNames(DATASMITH_CLIPTEXNAME, TEXT("unsupported"), TEXT("unsupported"), DATASMITH_CLIPCOMPNAME);
	FDatasmithShaderElementImpl::GetMetalComp()->SetBaseNames(DATASMITH_METALTEXNAME, TEXT("unsupported"), DATASMITH_METALVALUENAME, DATASMITH_METALCOMPNAME);
	FDatasmithShaderElementImpl::GetEmitComp()->SetBaseNames(DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, TEXT("unsupported"), DATASMITH_EMITCOMPNAME);
	FDatasmithShaderElementImpl::GetWeightComp()->SetBaseNames(DATASMITH_WEIGHTTEXNAME, DATASMITH_WEIGHTCOLNAME, DATASMITH_WEIGHTVALUENAME, DATASMITH_WEIGHTCOMPNAME);
}

FDatasmithCompositeSurface::FDatasmithCompositeSurface(const TSharedPtr<IDatasmithCompositeTexture>& SubComp)
{
	ParamTextures = TEXT("");
	ParamSampler = FDatasmithTextureSampler();
	ParamSubComposite = SubComp;
	ParamColor = FLinearColor();
	bParamUseTexture = true;
}

FDatasmithCompositeSurface::FDatasmithCompositeSurface(const TCHAR* InTexture, FDatasmithTextureSampler InTexUV)
{
	ParamTextures = FDatasmithUtils::SanitizeObjectName(InTexture);
	ParamSampler = InTexUV;
	ParamSubComposite = FDatasmithSceneFactory::CreateCompositeTexture();
	ParamColor = FLinearColor();
	bParamUseTexture = true;
}

FDatasmithCompositeSurface::FDatasmithCompositeSurface(const FLinearColor& InColor)
{
	ParamTextures = TEXT("");
	ParamSampler = FDatasmithTextureSampler();
	ParamSubComposite = FDatasmithSceneFactory::CreateCompositeTexture();
	ParamColor = InColor;
	bParamUseTexture = false;
}

bool FDatasmithCompositeSurface::GetUseTexture() const
{
	return (bParamUseTexture == true && !ParamSubComposite->IsValid());
}

bool FDatasmithCompositeSurface::GetUseComposite() const
{
	return (bParamUseTexture == true && ParamSubComposite->IsValid());
}

bool FDatasmithCompositeSurface::GetUseColor() const
{
	return !bParamUseTexture;
}

FDatasmithTextureSampler& FDatasmithCompositeSurface::GetParamTextureSampler()
{
	return ParamSampler;
}

const TCHAR* FDatasmithCompositeSurface::GetParamTexture() const
{
	return *ParamTextures;
}

void FDatasmithCompositeSurface::SetParamTexture(const TCHAR* InTexture)
{
	ParamTextures = FDatasmithUtils::SanitizeObjectName(InTexture);
}

const FLinearColor& FDatasmithCompositeSurface::GetParamColor() const
{
	return ParamColor;
}

TSharedPtr<IDatasmithCompositeTexture>& FDatasmithCompositeSurface::GetParamSubComposite()
{
	return ParamSubComposite;
}

FDatasmithCompositeTextureImpl::FDatasmithCompositeTextureImpl()
{
	CompMode = EDatasmithCompMode::Regular;

	BaseTexName = DATASMITH_TEXTURENAME;
	BaseColName = DATASMITH_COLORNAME;
	BaseValName = DATASMITH_VALUE1NAME;
	BaseCompName = DATASMITH_TEXTURECOMPNAME;
}

bool FDatasmithCompositeTextureImpl::IsValid() const
{
	return ( ParamSurfaces.Num() != 0 || ParamVal1.Num() != 0 );
}

bool FDatasmithCompositeTextureImpl::GetUseTexture(int32 InIndex)
{
	ensure( ParamSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamSurfaces.IsValidIndex( InIndex ) )
	{
		return false;
	}

	return ParamSurfaces[InIndex].GetUseTexture();
}

const TCHAR* FDatasmithCompositeTextureImpl::GetParamTexture(int32 InIndex)
{
	ensure( ParamSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamSurfaces.IsValidIndex( InIndex ) )
	{
		return TEXT("");
	}

	return ParamSurfaces[InIndex].GetParamTexture();
}

void FDatasmithCompositeTextureImpl::SetParamTexture(int32 InIndex, const TCHAR* InTexture)
{
	if (ParamSurfaces.IsValidIndex(InIndex) )
	{
		ParamSurfaces[InIndex].SetParamTexture(InTexture);
	}
}

static FDatasmithTextureSampler DefaultTextureSampler;

FDatasmithTextureSampler& FDatasmithCompositeTextureImpl::GetParamTextureSampler(int32 InIndex)
{
	ensure(ParamSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamSurfaces.IsValidIndex( InIndex ) )
	{
		return DefaultTextureSampler;
	}

	return ParamSurfaces[InIndex].GetParamTextureSampler();
}

bool FDatasmithCompositeTextureImpl::GetUseColor(int32 InIndex)
{
	ensure(ParamSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamSurfaces.IsValidIndex( InIndex ) )
	{
		return true; // Fallback to using a color
	}

	return ParamSurfaces[InIndex].GetUseColor();
}

const FLinearColor& FDatasmithCompositeTextureImpl::GetParamColor(int32 InIndex)
{
	ensure(ParamSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamSurfaces.IsValidIndex( InIndex ) )
	{
		return FLinearColor::Black;
	}

	return ParamSurfaces[InIndex].GetParamColor();
}

bool FDatasmithCompositeTextureImpl::GetUseComposite(int32 InIndex)
{
	ensure(ParamSurfaces.IsValidIndex( InIndex ));
	if ( !ParamSurfaces.IsValidIndex( InIndex ))
	{
		return false;
	}

	return (ParamSurfaces[InIndex].GetUseComposite());
}

IDatasmithCompositeTexture::ParamVal FDatasmithCompositeTextureImpl::GetParamVal1(int32 InIndex) const
{
	ensure( ParamVal1.IsValidIndex( InIndex ) );
	if ( !ParamVal1.IsValidIndex( InIndex ) )
	{
		return ParamVal( 0, TEXT("") );
	}

	return ParamVal( ParamVal1[InIndex].Key, *ParamVal1[InIndex].Value );
}

IDatasmithCompositeTexture::ParamVal FDatasmithCompositeTextureImpl::GetParamVal2(int32 InIndex) const
{
	ensure( ParamVal2.IsValidIndex( InIndex ) );
	if ( !ParamVal2.IsValidIndex( InIndex ) )
	{
		return ParamVal( 0, TEXT("") );
	}

	return ParamVal( ParamVal2[InIndex].Key, *ParamVal2[InIndex].Value );
}

const TCHAR* FDatasmithCompositeTextureImpl::GetParamMask(int32 InIndex)
{
	ensure(ParamMaskSurfaces.IsValidIndex(InIndex));
	if (!ParamMaskSurfaces.IsValidIndex(InIndex))
	{
		return TEXT("");
	}

	return ParamMaskSurfaces[InIndex].GetParamTexture();
}

const FLinearColor& FDatasmithCompositeTextureImpl::GetParamMaskColor(int32 InIndex) const
{
	ensure(ParamMaskSurfaces.IsValidIndex(InIndex));
	if (!ParamMaskSurfaces.IsValidIndex(InIndex))
	{
		return FLinearColor::Black;
	}

	return ParamMaskSurfaces[InIndex].GetParamColor();
}

bool FDatasmithCompositeTextureImpl::GetMaskUseComposite(int32 InIndex) const
{
	ensure(ParamMaskSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamMaskSurfaces.IsValidIndex( InIndex ) )
	{
		return false;
	}

	return ParamMaskSurfaces[InIndex].GetUseComposite();
}

FDatasmithTextureSampler FDatasmithCompositeTextureImpl::GetParamMaskTextureSampler(int32 InIndex)
{
	ensure(ParamMaskSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamMaskSurfaces.IsValidIndex( InIndex ) )
	{
		return FDatasmithTextureSampler();
	}

	return ParamMaskSurfaces[InIndex].GetParamTextureSampler();
}

static TSharedPtr<IDatasmithCompositeTexture> InvalidCompositeTexture;

TSharedPtr<IDatasmithCompositeTexture>& FDatasmithCompositeTextureImpl::GetParamSubComposite(int32 InIndex)
{
	ensure(ParamSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamSurfaces.IsValidIndex( InIndex ) )
	{
		return InvalidCompositeTexture;
	}

	return ParamSurfaces[InIndex].GetParamSubComposite();
}

TSharedPtr<IDatasmithCompositeTexture>& FDatasmithCompositeTextureImpl::GetParamMaskSubComposite(int32 InIndex)
{
	ensure(ParamMaskSurfaces.IsValidIndex( InIndex ) );
	if ( !ParamMaskSurfaces.IsValidIndex( InIndex ) )
	{
		return InvalidCompositeTexture;
	}

	return ParamMaskSurfaces[InIndex].GetParamSubComposite();
}

void FDatasmithCompositeTextureImpl::SetBaseNames(const TCHAR* InTextureName, const TCHAR* InColorName, const TCHAR* InValueName, const TCHAR* InCompName)
{
	BaseTexName = InTextureName;
	BaseColName = InColorName;
	BaseValName = InValueName;
	BaseCompName = InCompName;
}

FDatasmithMetaDataElementImpl::FDatasmithMetaDataElementImpl(const TCHAR* InName)
	: FDatasmithElementImpl(InName, EDatasmithElementType::MetaData)
{
	RegisterReferenceProxy(AssociatedElement, "AssociatedElement");
	RegisterReferenceProxy(Properties, "Properties");
}

const TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMetaDataElementImpl::GetProperty(int32 Index) const
{
	return Properties.IsValidIndex(Index) ? Properties[Index] : FDatasmithKeyValuePropertyImpl::NullPropertyPtr;
}

const TSharedPtr< IDatasmithKeyValueProperty >& FDatasmithMetaDataElementImpl::GetPropertyByName( const TCHAR* InName ) const
{
	int32 Index = Properties.View().IndexOfByPredicate([InName](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
		return Property.IsValid() && FCString::Stricmp(Property->GetName(), InName) == 0;
	});
	return GetProperty(Index);
}

void FDatasmithMetaDataElementImpl::AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& InProperty )
{
	if (!InProperty.IsValid())
	{
		return;
	}
	const TCHAR* InName = InProperty->GetName();
	auto elt = Properties.View().FindByPredicate([InName](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
		return Property.IsValid() && FCString::Stricmp(Property->GetName(), InName) == 0;
	});

	if ( elt == nullptr )
	{
		Properties.Add( InProperty );
	}
}

void FDatasmithMetaDataElementImpl::RemoveProperty( const TSharedPtr<IDatasmithKeyValueProperty>& Property )
{
	Properties.Edit().Remove( Property );
}

void FDatasmithMetaDataElementImpl::ResetProperties()
{
	Properties.Edit().Reset();
}

FDatasmithDecalActorElementImpl::FDatasmithDecalActorElementImpl( const TCHAR* InName )
	: FDatasmithCustomActorElementImpl( InName, EDatasmithElementType::Decal )
{
	SetClassOrPathName(TEXT("DecalActor"));

	const TCHAR* SortOrderPropertyName = TEXT("DECAL_SORT_ORDER_PROP");
	const TCHAR* DimensionsPropertyName = TEXT("DECAL_DIMENSIONS_PROP");
	const TCHAR* MaterialPropertyName = TEXT("DECAL_MATERIAL_PROP");

	SortOrderPropertyIndex = AddPropertyInternal( SortOrderPropertyName, EDatasmithKeyValuePropertyType::Integer, TEXT("0") );
	DimensionsPropertyIndex = AddPropertyInternal( DimensionsPropertyName, EDatasmithKeyValuePropertyType::Vector, *FVector::ZeroVector.ToString() );
	MaterialPropertyIndex = AddPropertyInternal( MaterialPropertyName, EDatasmithKeyValuePropertyType::String, TEXT("") );
}

FVector FDatasmithDecalActorElementImpl::GetDimensions() const
{
	ensure(GetProperty( DimensionsPropertyIndex).IsValid() );

	FVector Dimensions;
	Dimensions.InitFromString( GetProperty( DimensionsPropertyIndex )->GetValue() );

	return Dimensions;
}

void FDatasmithDecalActorElementImpl::SetDimensions( const FVector& InDimensions )
{
	ensure( GetProperty( DimensionsPropertyIndex ).IsValid() );
	GetProperty(DimensionsPropertyIndex)->SetValue( *InDimensions.ToString() );
}

int32 FDatasmithDecalActorElementImpl::GetSortOrder() const
{
	ensure( GetProperty( SortOrderPropertyIndex ).IsValid() );
	return FCString::Atoi( GetProperty( SortOrderPropertyIndex )->GetValue() );
}

void FDatasmithDecalActorElementImpl::SetSortOrder( int32 InSortOrder )
{
	ensure( GetProperty( SortOrderPropertyIndex ).IsValid() );
	GetProperty( SortOrderPropertyIndex )->SetValue( *FString::FromInt( InSortOrder ) );
}

const TCHAR* FDatasmithDecalActorElementImpl::GetDecalMaterialPathName() const
{
	ensure( GetProperty( MaterialPropertyIndex ).IsValid() );
	return GetProperty( MaterialPropertyIndex )->GetValue();
}

void FDatasmithDecalActorElementImpl::SetDecalMaterialPathName( const TCHAR* InMaterialPathName )
{
	ensure( GetProperty( MaterialPropertyIndex ).IsValid() );
	GetProperty( MaterialPropertyIndex )->SetValue( InMaterialPathName );
}

FDatasmithSceneImpl::FDatasmithSceneImpl(const TCHAR * InName)
	: FDatasmithElementImpl(InName, EDatasmithElementType::Scene)
{
	RegisterReferenceProxy(Actors,           "Actors"           );
	RegisterReferenceProxy(Meshes,           "Meshes"           );
	RegisterReferenceProxy(Materials,        "Materials"        );
	RegisterReferenceProxy(Textures,         "Textures"         );
	RegisterReferenceProxy(MetaData,         "MetaData"         );
	RegisterReferenceProxy(LevelSequences,   "LevelSequences"   );
	RegisterReferenceProxy(LevelVariantSets, "LevelVariantSets" );
	RegisterReferenceProxy(PostProcess,      "PostProcess"      );

	Store.RegisterParameter(Hostname,           "Hostname"           );
	Store.RegisterParameter(ExporterVersion,    "ExporterVersion"    );
	Store.RegisterParameter(ExporterSDKVersion, "ExporterSDKVersion" );
	Store.RegisterParameter(ResourcePath,       "ResourcePath"       );
	Store.RegisterParameter(Vendor,             "Vendor"             );
	Store.RegisterParameter(ProductName,        "ProductName"        );
	Store.RegisterParameter(ProductVersion,     "ProductVersion"     );
	Store.RegisterParameter(UserID,             "UserID"             );
	Store.RegisterParameter(UserOS,             "UserOS"             );
	Store.RegisterParameter(Geolocation,        "Geolocation"        );
	Store.RegisterParameter(ExportDuration,     "ExportDuration"     );
	Store.RegisterParameter(bUseSky,            "bUseSky"            );
	FDatasmithSceneImpl::Reset();
}

void FDatasmithSceneImpl::Reset()
{
	Actors.Empty();
	Meshes.Empty();
	Materials.Empty();
	Textures.Empty();
	MetaData.Empty();
	LevelSequences.Empty();
	LevelVariantSets.Empty();
	PostProcess.Inner.Reset();
	ElementToMetaDataMap.Empty();

	Hostname = TEXT("");
	ExporterVersion = FDatasmithUtils::GetDatasmithFormatVersionAsString();
	ExporterSDKVersion = FDatasmithUtils::GetEnterpriseVersionAsString();
	Vendor = TEXT("");
	ProductName = TEXT("");
	ProductVersion = TEXT("");
	UserID = TEXT("");
	UserOS = TEXT("");
	Geolocation = FVector(TNumericLimits<double>::Max(), TNumericLimits<double>::Max(), TNumericLimits<double>::Max());  // max double indicates values are not set
	ResourcePath = TEXT("");

	ExportDuration = 0;

	bUseSky = false;
}

const TCHAR* FDatasmithSceneImpl::GetHost() const
{
	return *Hostname.Get();
}

void FDatasmithSceneImpl::SetHost(const TCHAR* InHostname)
{
	Hostname = InHostname;
}


TSharedPtr< IDatasmithMeshElement > FDatasmithSceneImpl::GetMesh(int32 InIndex)
{
	if ( Meshes.IsValidIndex( InIndex ) )
	{
		return Meshes[InIndex];
	}
	else
	{
		return TSharedPtr< IDatasmithMeshElement >();
	}
}

const TSharedPtr< IDatasmithMeshElement >& FDatasmithSceneImpl::GetMesh(int32 InIndex) const
{
	if ( Meshes.IsValidIndex( InIndex ) )
	{
		return Meshes[InIndex];
	}
	else
	{
		static const TSharedPtr< IDatasmithMeshElement > InvalidMeshElement;
		return InvalidMeshElement;
	}
}

void FDatasmithSceneImpl::RemoveMeshAt(int32 InIndex)
{
	if (Meshes.IsValidIndex(InIndex))
	{
		Meshes.RemoveAt(InIndex);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDatasmithSceneImpl::AddCloth(const TSharedPtr< IDatasmithClothElement >& InElement)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Clothes.Add(InElement);
}

int32 FDatasmithSceneImpl::GetClothesCount() const
{
	return Clothes.Num();
}

TSharedPtr< IDatasmithClothElement > FDatasmithSceneImpl::GetCloth(int32 InIndex)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	if ( Clothes.IsValidIndex( InIndex ) )
	{
		return Clothes[InIndex];
	}
	return {};
}

const TSharedPtr< IDatasmithClothElement >& FDatasmithSceneImpl::GetCloth(int32 InIndex) const  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	if ( Clothes.IsValidIndex( InIndex ) )
	{
		return Clothes[InIndex];
	}
	static TSharedPtr< IDatasmithClothElement > NullCloth;
	return NullCloth;
}

void FDatasmithSceneImpl::RemoveCloth(const TSharedPtr< IDatasmithClothElement >& InElement)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Clothes.Remove(InElement);
}

void FDatasmithSceneImpl::RemoveClothAt(int32 InIndex)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	if (Clothes.IsValidIndex(InIndex))
	{
		Clothes.RemoveAt(InIndex);
	}
}

void FDatasmithSceneImpl::EmptyClothes()  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	Clothes.Empty();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

static const TSharedPtr< IDatasmithMetaDataElement > InvalidMetaData;

TSharedPtr< IDatasmithMetaDataElement > FDatasmithSceneImpl::GetMetaData(int32 InIndex)
{
	if ( MetaData.IsValidIndex( InIndex ) )
	{
		return MetaData[ InIndex ];
	}
	else
	{
		return TSharedPtr< IDatasmithMetaDataElement >();
	}
}

const TSharedPtr< IDatasmithMetaDataElement >& FDatasmithSceneImpl::GetMetaData(int32 InIndex) const
{
	if ( MetaData.IsValidIndex( InIndex ) )
	{
		return MetaData[ InIndex ];
	}
	else
	{
		return InvalidMetaData;
	}
}

TSharedPtr< IDatasmithMetaDataElement > FDatasmithSceneImpl::GetMetaData(const TSharedPtr<IDatasmithElement>& Element)
{
	if (TSharedPtr< IDatasmithMetaDataElement >* MetaDataElement = GetElementToMetaDataCache().Find(Element))
	{
		return *MetaDataElement;
	}
	else
	{
		return TSharedPtr< IDatasmithMetaDataElement >();
	}
}

const TSharedPtr< IDatasmithMetaDataElement >& FDatasmithSceneImpl::GetMetaData(const TSharedPtr<IDatasmithElement>& Element) const
{
	if (const TSharedPtr< IDatasmithMetaDataElement >* MetaDataElement = GetElementToMetaDataCache().Find(Element))
	{
		return *MetaDataElement;
	}
	else
	{
		return InvalidMetaData;
	}
}

void FDatasmithSceneImpl::RemoveMetaData( const TSharedPtr<IDatasmithMetaDataElement>& Element )
{
	if ( Element )
	{
		GetElementToMetaDataCache().Remove( Element->GetAssociatedElement() );
		MetaData.Remove( Element );
	}
}

void FDatasmithSceneImpl::RemoveMetaDataAt(int32 InIndex)
{
	RemoveMetaData(GetMetaData(InIndex));
}

TMap< TSharedPtr< IDatasmithElement >, TSharedPtr< IDatasmithMetaDataElement> >& FDatasmithSceneImpl::GetElementToMetaDataCache() const
{
	if (ElementToMetaDataMap.Num() == 0)
	{
		for (const TSharedPtr<IDatasmithMetaDataElement>& MetaDataElement : MetaData.View())
		{
			ElementToMetaDataMap.Add(MetaDataElement->GetAssociatedElement(), MetaDataElement);
		}
	}

	return ElementToMetaDataMap;
}

TSharedPtr<IDatasmithLevelSequenceElement> FDatasmithSceneImpl::GetLevelSequence(int32 InIndex)
{
	return LevelSequences.IsValidIndex(InIndex) ? LevelSequences[InIndex] : nullptr;
}

const TSharedPtr<IDatasmithLevelSequenceElement>& FDatasmithSceneImpl::GetLevelSequence(int32 InIndex) const
{
	static TSharedPtr<IDatasmithLevelSequenceElement> InvalidLevelSequence;
	return LevelSequences.IsValidIndex(InIndex) ? LevelSequences[InIndex] : InvalidLevelSequence;
}

void FDatasmithSceneImpl::RemoveLevelSequenceAt(int32 InIndex)
{
	if (LevelSequences.IsValidIndex(InIndex))
	{
		LevelSequences.RemoveAt(InIndex);
	}
}

TSharedPtr<IDatasmithLevelVariantSetsElement> FDatasmithSceneImpl::GetLevelVariantSets(int32 InIndex)
{
	return LevelVariantSets.IsValidIndex(InIndex) ? LevelVariantSets[InIndex] : nullptr;
}

const TSharedPtr<IDatasmithLevelVariantSetsElement>& FDatasmithSceneImpl::GetLevelVariantSets(int32 InIndex) const
{
	static TSharedPtr<IDatasmithLevelVariantSetsElement> InvalidLevelVariantSets;
	return LevelVariantSets.IsValidIndex(InIndex) ? LevelVariantSets[InIndex] : InvalidLevelVariantSets;
}

void FDatasmithSceneImpl::RemoveLevelVariantSetsAt(int32 InIndex)
{
	if (LevelVariantSets.IsValidIndex(InIndex))
	{
		LevelVariantSets.RemoveAt(InIndex);
	}
}

namespace DatasmithSceneImplInternal
{
	template<typename ContainerType, typename SharedPtrElementType>
	void RemoveActor(FDatasmithSceneImpl* SceneImpl, ContainerType& ActorContainer, SharedPtrElementType& InActor, EDatasmithActorRemovalRule RemoveRule)
	{
		FDatasmithSceneUtils::TActorHierarchy FoundHierarchy;
		bool bFound = FDatasmithSceneUtils::FindActorHierarchy(SceneImpl, InActor, FoundHierarchy);
		if (bFound)
		{
			// If Actor is found, it is always added to FoundHierarchy
			// And if it is at the root, it will be the only item in FoundHierarchy
			if (FoundHierarchy.Num() == 1)
			{
				// The actor lives at the root
				if (RemoveRule == EDatasmithActorRemovalRule::KeepChildrenAndKeepRelativeTransform)
				{
					for (int32 ChildIndex = InActor->GetChildrenCount() - 1; ChildIndex >= 0; --ChildIndex)
					{
						const TSharedPtr< IDatasmithActorElement > Child = InActor->GetChild(ChildIndex);
						if (!Child->IsAComponent())
						{
							InActor->RemoveChild(Child);
							SceneImpl->AddActor(Child);
						}
					}
				}
				else
				{
					check(RemoveRule == EDatasmithActorRemovalRule::RemoveChildren);
				}

				ActorContainer.Remove(InActor);
			}
			else
			{
				// The actor lives as a child of another actor
				if (RemoveRule == EDatasmithActorRemovalRule::KeepChildrenAndKeepRelativeTransform)
				{
					for (int32 ChildIndex = InActor->GetChildrenCount() - 1; ChildIndex >= 0; --ChildIndex)
					{
						const TSharedPtr< IDatasmithActorElement > Child = InActor->GetChild(ChildIndex);
						if (!Child->IsAComponent())
						{
							InActor->RemoveChild(Child);
							FoundHierarchy.Last()->AddChild(Child);
						}
					}
				}
				else
				{
					check(RemoveRule == EDatasmithActorRemovalRule::RemoveChildren);
				}

				FoundHierarchy.Last()->RemoveChild(InActor);
			}
		}
	}
}

TSharedPtr<IDatasmithActorElement> FDatasmithSceneImpl::GetActor(int32 InIndex)
{
	return Actors.IsValidIndex(InIndex) ? Actors[InIndex] : nullptr;
}

const TSharedPtr<IDatasmithActorElement>& FDatasmithSceneImpl::GetActor(int32 InIndex) const
{
	static TSharedPtr<IDatasmithActorElement> InvalidActor;
	return Actors.IsValidIndex(InIndex) ? Actors[InIndex] : InvalidActor;
}

void FDatasmithSceneImpl::RemoveActor(const TSharedPtr< IDatasmithActorElement >& InActor, EDatasmithActorRemovalRule RemoveRule)
{
	DatasmithSceneImplInternal::RemoveActor(this, Actors, InActor, RemoveRule);
}

void FDatasmithSceneImpl::RemoveActorAt(int32 InIndex, EDatasmithActorRemovalRule RemoveRule)
{
	if (Actors.IsValidIndex(InIndex))
	{
		RemoveActor(Actors[InIndex], RemoveRule);
	}
}

TSharedPtr<IDatasmithBaseMaterialElement> FDatasmithSceneImpl::GetMaterial(int32 InIndex)
{
	return Materials.IsValidIndex(InIndex) ? Materials[InIndex] : nullptr;
}

const TSharedPtr<IDatasmithBaseMaterialElement>& FDatasmithSceneImpl::GetMaterial(int32 InIndex) const
{
	static TSharedPtr<IDatasmithBaseMaterialElement> InvalidBaseMaterial;
	return Materials.IsValidIndex(InIndex) ? Materials[InIndex] : InvalidBaseMaterial;
}

void FDatasmithSceneImpl::RemoveMaterialAt(int32 InIndex)
{
	if (Materials.IsValidIndex(InIndex))
	{
		Materials.RemoveAt(InIndex);
	}
}

TSharedPtr<IDatasmithTextureElement> FDatasmithSceneImpl::GetTexture(int32 InIndex)
{
	return Textures.IsValidIndex(InIndex) ? Textures[InIndex] : nullptr;
}

const TSharedPtr<IDatasmithTextureElement>& FDatasmithSceneImpl::GetTexture(int32 InIndex) const
{
	static TSharedPtr<IDatasmithTextureElement> InvalidTexture;
	return Textures.IsValidIndex(InIndex) ? Textures[InIndex] : InvalidTexture;
}

void FDatasmithSceneImpl::RemoveTextureAt(int32 InIndex)
{
	if (Textures.IsValidIndex(InIndex))
	{
		Textures.RemoveAt(InIndex);
	}
}

namespace DatasmithSceneImplInternal
{
	void AttachActorToSceneRoot(FDatasmithSceneImpl* SceneImpl, const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule, const FDatasmithSceneUtils::TActorHierarchy& FoundChildHierarchy)
	{
		// The child is already to the root?
		if (FoundChildHierarchy.Num() != 0)
		{
			if ( AttachmentRule == EDatasmithActorAttachmentRule::KeepRelativeTransform )
			{
				TSharedPtr< IDatasmithActorElement > DirectParent = FoundChildHierarchy.Last();

				FTransform ChildWorldTransform( Child->GetRotation(), Child->GetTranslation(), Child->GetScale() );
				FTransform ParentWorldTransform( DirectParent->GetRotation(), DirectParent->GetTranslation(), DirectParent->GetScale() );

				FTransform ChildRelativeTransform = ChildWorldTransform.GetRelativeTransform( ParentWorldTransform );

				Child->SetRotation( ChildRelativeTransform.GetRotation() );
				Child->SetTranslation( ChildRelativeTransform.GetTranslation() );
				Child->SetScale( ChildRelativeTransform.GetScale3D() );
			}

			FoundChildHierarchy.Last()->RemoveChild(Child);
			SceneImpl->AddActor(Child);
		}
	}
}

void FDatasmithSceneImpl::AttachActor(const TSharedPtr< IDatasmithActorElement >& NewParent, const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule)
{
	FDatasmithSceneUtils::TActorHierarchy FoundParentHierarchy;
	bool bNewParentFound = FDatasmithSceneUtils::FindActorHierarchy(this, NewParent, FoundParentHierarchy);
	FDatasmithSceneUtils::TActorHierarchy FoundChildHierarchy;
	bool bChildFound = FDatasmithSceneUtils::FindActorHierarchy(this, Child, FoundChildHierarchy);

	if (!bNewParentFound)
	{
		if (bChildFound)
		{
			// If the parent doesn't exist, move it at the root
			DatasmithSceneImplInternal::AttachActorToSceneRoot(this, Child, AttachmentRule, FoundChildHierarchy);
		}
		return;
	}

	if(!bChildFound)
	{
		// No one to attach
		return;
	}

	if (AttachmentRule == EDatasmithActorAttachmentRule::KeepRelativeTransform)
	{
		// Convert Child transform from world to relative, so that we end up at the same position relatively to NewParent
		if ( FoundChildHierarchy.Num() > 0 )
		{
			TSharedPtr< IDatasmithActorElement > DirectParent = FoundChildHierarchy.Last();

			FTransform ChildWorldTransform( Child->GetRotation(), Child->GetTranslation(), Child->GetScale() );
			FTransform ParentWorldTransform( DirectParent->GetRotation(), DirectParent->GetTranslation(), DirectParent->GetScale() );

			FTransform ChildRelativeTransform = ChildWorldTransform.GetRelativeTransform( ParentWorldTransform );;

			Child->SetRotation( ChildRelativeTransform.GetRotation() );
			Child->SetTranslation( ChildRelativeTransform.GetTranslation() );
			Child->SetScale( ChildRelativeTransform.GetScale3D() );
		}
	}

	if (FoundChildHierarchy.Num() == 0)
	{
		RemoveActor(Child, EDatasmithActorRemovalRule::RemoveChildren);
	}
	else
	{
		FoundChildHierarchy.Last()->RemoveChild(Child);
	}

	NewParent->AddChild(Child, AttachmentRule);
}

void FDatasmithSceneImpl::AttachActorToSceneRoot(const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule)
{
	FDatasmithSceneUtils::TActorHierarchy FoundChildHierarchy;
	bool bChildFound = FDatasmithSceneUtils::FindActorHierarchy(this, Child, FoundChildHierarchy);

	if (bChildFound)
	{
		DatasmithSceneImplInternal::AttachActorToSceneRoot(this, Child, AttachmentRule, FoundChildHierarchy);
	}
}

FDatasmithClothActorElementImpl::FDatasmithClothActorElementImpl(const TCHAR* InName)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: FDatasmithActorElementImpl<IDatasmithClothActorElement>(InName, EDatasmithElementType::ClothActor)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	Store.RegisterParameter(Cloth, "Cloth");
}
