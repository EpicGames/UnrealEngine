// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneGraphSharedState.h"
#include "DatasmithUtils.h"
#include "DirectLinkParameterStore.h"
#include "IDatasmithSceneElements.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/PackageName.h"
#include "Templates/SharedPointer.h"

#define UE_API DATASMITHCORE_API


template<typename T, typename S=T> using TReflected = DirectLink::TStoreKey<T, S>;


template< typename InterfaceType >
class FDatasmithElementImpl : public InterfaceType
{
public:
	FDatasmithElementImpl(const TCHAR* InName, EDatasmithElementType InType, uint64 InSubType = 0);
	virtual ~FDatasmithElementImpl() {}

	virtual bool IsA( EDatasmithElementType InType ) const override { return EnumHasAnyFlags( GetElementType(), InType); }

	virtual const TCHAR* GetName() const override { return *Name.Get(); }
	virtual void SetName(const TCHAR* InName) override { Name = FDatasmithUtils::SanitizeObjectName(InName); }

	virtual const TCHAR* GetLabel() const override { const FString& Tmp = Label.Get(); return Tmp.IsEmpty() ? GetName() : *Tmp; }
	virtual void SetLabel(const TCHAR* InLabel) override { Label = FDatasmithUtils::SanitizeObjectName(InLabel); }

	virtual FMD5Hash CalculateElementHash(bool) override { return ElementHash; }


	virtual TSharedPtr<DirectLink::FSceneGraphSharedState> MakeSharedState() const override { return MakeShared<FDatasmithSceneGraphSharedState>(); }
	virtual const DirectLink::FParameterStore& GetStore() const override { return Store; }
	virtual       DirectLink::FParameterStore& GetStore()       override { return Store; }

protected:
	virtual bool IsSubTypeInternal( uint64 InSubType ) const { return ( InSubType & GetSubType() ) != 0; }
	EDatasmithElementType GetElementType() const { return Type.Get(); }
	uint64 GetSubType() const { return Subtype.Get(); }

protected:
	FMD5Hash ElementHash;

	DirectLink::FParameterStore Store;
	TReflected<EDatasmithElementType, uint64> Type;
	TReflected<uint64> Subtype;
	TReflected<FString> Name;
	TReflected<FString> Label;
};


template< typename InterfaceType >
inline FDatasmithElementImpl< InterfaceType >::FDatasmithElementImpl(const TCHAR* InName, EDatasmithElementType InType, uint64 InSubType)
	: Type(InType)
	, Subtype(InSubType)
{
	FDatasmithElementImpl< InterfaceType >::SetName(InName);
	Store.RegisterParameter(Type,    "Type");
	Store.RegisterParameter(Subtype, "Subtype");
	Store.RegisterParameter(Name,    "Name");
	Store.RegisterParameter(Label,   "Label");
}

class FDatasmithKeyValuePropertyImpl : public FDatasmithElementImpl< IDatasmithKeyValueProperty >
{
public:
	FDatasmithKeyValuePropertyImpl(const TCHAR* InName);
	virtual void SetName(const TCHAR* InName) override { Name = InName; }

	EDatasmithKeyValuePropertyType GetPropertyType() const override { return PropertyType; }
	void SetPropertyType( EDatasmithKeyValuePropertyType InType ) override;

	const TCHAR* GetValue() const override { return *Value.Get(); }
	void SetValue( const TCHAR* InValue ) override;

	static TSharedPtr< IDatasmithKeyValueProperty > NullPropertyPtr;

protected:
	void FormatValue();

private:
	TReflected<EDatasmithKeyValuePropertyType, uint8> PropertyType;
	TReflected<FString> Value;
};


enum class EActorFlags : uint8
{
	IsAComponent       = 0x01,
	IsASelector        = 0x02, // Deprecated
	IsVisible          = 0x04,
};
ENUM_CLASS_FLAGS(EActorFlags);

#define UPDATE_BITFLAGS( Flags, bValue, EnumValue) Flags = bValue ? Flags | EnumValue : Flags & ~EnumValue

template< typename InterfaceType >
class FDatasmithActorElementImpl : public FDatasmithElementImpl< InterfaceType >, public TSharedFromThis< FDatasmithActorElementImpl< InterfaceType > >
{
public:
	using FDatasmithElementImpl< InterfaceType >::Store;
	FDatasmithActorElementImpl(const TCHAR* InName, EDatasmithElementType InType);

	virtual FVector GetTranslation() const override { return Translation.Get(); }
	virtual void SetTranslation(double InX, double InY, double InZ, bool bKeepChildrenRelative) override { SetTranslation( FVector( InX, InY, InZ ), bKeepChildrenRelative ); }
	virtual void SetTranslation(const FVector& Value, bool bKeepChildrenRelative) override
	{
		if (bKeepChildrenRelative)
		{
			ConvertChildsToRelative();
		}
		SetInternalTranslation(Value);
		if (bKeepChildrenRelative)
		{
			ConvertChildsToWorld();
		}
	}

	virtual FVector GetScale() const override { return Scale.Get(); }
	virtual void SetScale(double InX, double InY, double InZ, bool bKeepChildrenRelative) override { SetScale( FVector( InX, InY, InZ ), bKeepChildrenRelative ); }
	virtual void SetScale(const FVector& Value, bool bKeepChildrenRelative) override
	{
		if (bKeepChildrenRelative)
		{
			ConvertChildsToRelative();
		}
		SetInternalScale(Value);
		if (bKeepChildrenRelative)
		{
			ConvertChildsToWorld();
		}
	}

	virtual FQuat GetRotation() const override { return Rotation; }
	virtual void SetRotation(double InX, double InY, double InZ, double InW, bool bKeepChildrenRelative) override { SetRotation( FQuat( InX, InY, InZ, InW ), bKeepChildrenRelative ); }
	virtual void SetRotation(const FQuat& Value, bool bKeepChildrenRelative) override
	{
		if (bKeepChildrenRelative)
		{
			ConvertChildsToRelative();
		}
		SetInternalRotation(Value);
		if (bKeepChildrenRelative)
		{
			ConvertChildsToWorld();
		}
	}

	virtual FTransform GetRelativeTransform() const override;

	virtual const TCHAR* GetLayer() const override { return *(FString&)Layer; }
	virtual void SetLayer(const TCHAR* InLayer) override { Layer = InLayer; }

	virtual void AddTag(const TCHAR* InTag) override { Tags.Get().Add(InTag); }
	virtual void ResetTags() override { Tags.Get().Reset(); }
	virtual int32 GetTagsCount() const { return Tags.Get().Num(); }
	virtual const TCHAR* GetTag(int32 TagIndex) const override { return Tags.Get().IsValidIndex(TagIndex) ? *Tags.Get()[TagIndex] : nullptr; }

	virtual void AddChild(const TSharedPtr< IDatasmithActorElement >& InChild, EDatasmithActorAttachmentRule AttachementRule = EDatasmithActorAttachmentRule::KeepWorldTransform) override;
	virtual int32 GetChildrenCount() const override
	{
		return Children.Num();
	}

	/** Get the 'InIndex'th child of the actor  */
	virtual TSharedPtr< IDatasmithActorElement > GetChild(int32 InIndex) override
	{
		return Children.IsValidIndex(InIndex) ? Children[InIndex] : NullActorPtr;
	};

	virtual const TSharedPtr< IDatasmithActorElement >& GetChild(int32 InIndex) const override
	{
		return Children.IsValidIndex(InIndex) ? Children[InIndex] : NullActorPtr;
	};

	virtual void RemoveChild(const TSharedPtr< IDatasmithActorElement >& InChild) override
	{
		Children.Edit().Remove(InChild);
		static_cast< FDatasmithActorElementImpl* >( InChild.Get() )->Parent.Edit().Reset();
	}

	virtual const TSharedPtr< IDatasmithActorElement >& GetParentActor() const override
	{
		return Parent.View();
	}

	virtual void SetIsAComponent(bool Value) override { UPDATE_BITFLAGS(Flags, Value, EActorFlags::IsAComponent); }
	virtual bool IsAComponent() const override { return !!(Flags & EActorFlags::IsAComponent); }

	virtual void SetVisibility(bool bInVisibility) override { UPDATE_BITFLAGS(Flags, bInVisibility, EActorFlags::IsVisible); }
	virtual bool GetVisibility() const override { return !!(Flags & EActorFlags::IsVisible); }

	virtual void SetCastShadow(bool bInCastShadow) override { bCastShadow = bInCastShadow; }
	virtual bool GetCastShadow() const override { return bCastShadow; }

	virtual void SetMobility(EDatasmithActorMobilityType InMobility) override { Mobility = InMobility; }
	virtual EDatasmithActorMobilityType GetMobility() const override { return Mobility; }

protected:
	/** Converts all children's transforms to relative */
	void ConvertChildsToRelative();

	/** Converts all children's transforms to world */
	void ConvertChildsToWorld();

	void SetInternalRotation(const FQuat& Value) { Rotation = Value; }
	void SetInternalScale(const FVector& Value) { Scale = Value; }
	void SetInternalTranslation(const FVector& Value) { Translation = Value; }

private:
	static TSharedPtr<IDatasmithActorElement> NullActorPtr;

	TReflected<FVector> Translation;
	TReflected<FVector> Scale;
	TReflected<FQuat> Rotation;

	TReflected<FString> Layer;

	TReflected<TArray<FString>> Tags;

	TDatasmithReferenceArrayProxy<IDatasmithActorElement> Children;
	TDatasmithReferenceProxy<IDatasmithActorElement> Parent;

	TReflected<EActorFlags, uint8> Flags;
	TReflected<bool> bCastShadow;
	TReflected<EDatasmithActorMobilityType, uint8> Mobility;
};

template< typename InterfaceType >
TSharedPtr<IDatasmithActorElement> FDatasmithActorElementImpl< InterfaceType >::NullActorPtr;

template< typename T >
inline FDatasmithActorElementImpl<T>::FDatasmithActorElementImpl(const TCHAR* InName, EDatasmithElementType ChildType)
	: FDatasmithElementImpl<T>(InName, EDatasmithElementType::Actor | ChildType)
	, Translation(FVector::ZeroVector)
	, Scale(FVector::OneVector)
	, Rotation(FQuat::Identity)
	, Flags(EActorFlags::IsVisible)
	, bCastShadow(true)
	, Mobility(EDatasmithActorMobilityType::Static)
{
	this->RegisterReferenceProxy(Children, "Children");
	this->RegisterReferenceProxy(Parent,   "Parent"  );

	Store.RegisterParameter(Translation,  "Translation"  );
	Store.RegisterParameter(Scale,        "Scale"        );
	Store.RegisterParameter(Rotation,     "Rotation"     );
	Store.RegisterParameter(Layer,        "Layer"        );
	Store.RegisterParameter(Tags,         "Tags"         ); // reflect as low prio for directlink
	Store.RegisterParameter(Flags,        "Flags"        );
	Store.RegisterParameter(bCastShadow,  "CastShadow"   );
	Store.RegisterParameter(Mobility,     "Mobility"     );
}

template< typename T >
inline FTransform FDatasmithActorElementImpl<T>::GetRelativeTransform() const
{
	FTransform ActorTransform( GetRotation(), GetTranslation(), GetScale() );

	if ( Parent.Inner.IsValid() )
	{
		FTransform ParentTransform( Parent.Inner->GetRotation(), Parent.Inner->GetTranslation(), Parent.Inner->GetScale() );

		return ActorTransform.GetRelativeTransform( ParentTransform );
	}

	return ActorTransform;
}

template< typename T >
inline void FDatasmithActorElementImpl<T>::AddChild(const TSharedPtr< IDatasmithActorElement >& InChild, EDatasmithActorAttachmentRule AttachementRule)
{
	if ( AttachementRule == EDatasmithActorAttachmentRule::KeepRelativeTransform )
	{
		FTransform RelativeTransform( InChild->GetRotation(), InChild->GetTranslation(), InChild->GetScale() );
		FTransform ParentTransform( GetRotation(), GetTranslation(), GetScale() );

		FTransform WorldTransform = RelativeTransform * ParentTransform;

		InChild->SetRotation( WorldTransform.GetRotation() );
		InChild->SetTranslation( WorldTransform.GetTranslation() );
		InChild->SetScale( WorldTransform.GetScale3D() );
	}

	Children.Add(InChild);
	static_cast< FDatasmithActorElementImpl* >( InChild.Get() )->Parent.Inner = this->AsShared();
}

template< typename T >
inline void FDatasmithActorElementImpl<T>::ConvertChildsToRelative()
{
	FTransform ThisWorldTransform( GetRotation(), GetTranslation(), GetScale() );

	for ( const TSharedPtr< IDatasmithActorElement >& Child : Children.View() )
	{
		if ( !Child.IsValid() )
		{
			continue;
		}

		FDatasmithActorElementImpl* ChildImpl = static_cast< FDatasmithActorElementImpl* >( Child.Get() );
		ChildImpl->ConvertChildsToRelative(); // Depth first while we're still in world space

		FTransform ChildWorldTransform( Child->GetRotation(), Child->GetTranslation(), Child->GetScale() );

		FTransform ChildRelativeTransform = ChildWorldTransform.GetRelativeTransform( ThisWorldTransform );
		ChildImpl->SetInternalRotation(ChildRelativeTransform.GetRotation());
		ChildImpl->SetInternalTranslation(ChildRelativeTransform.GetTranslation());
		ChildImpl->SetInternalScale(ChildRelativeTransform.GetScale3D());
	}
}

template< typename T >
inline void FDatasmithActorElementImpl<T>::ConvertChildsToWorld()
{
	FTransform ThisWorldTransform( GetRotation(), GetTranslation(), GetScale() );

	for ( const TSharedPtr< IDatasmithActorElement >& Child : Children.View() )
	{
		if ( !Child.IsValid() )
		{
			continue;
		}

		FDatasmithActorElementImpl* ChildImpl = static_cast< FDatasmithActorElementImpl* >( Child.Get() );

		FTransform ChildRelativeTransform( Child->GetRotation(), Child->GetTranslation(), Child->GetScale() );

		FTransform ChildWorldTransform = ChildRelativeTransform * ThisWorldTransform;
		ChildImpl->SetInternalRotation(ChildWorldTransform.GetRotation());
		ChildImpl->SetInternalTranslation(ChildWorldTransform.GetTranslation());
		ChildImpl->SetInternalScale(ChildWorldTransform.GetScale3D());

		ChildImpl->ConvertChildsToWorld(); // Depth last now that we're in world space
	}
}

class FDatasmithMeshElementImpl : public FDatasmithElementImpl< IDatasmithMeshElement >
{
public:
	explicit FDatasmithMeshElementImpl(const TCHAR* InName);

	virtual FMD5Hash CalculateElementHash(bool bForce) override;

	virtual const TCHAR* GetFile() const override { return *(FString&)File; }
	virtual void SetFile(const TCHAR* InFile) override { File = InFile; };

	virtual FMD5Hash GetFileHash() const override { return FileHash; }
	virtual void SetFileHash(FMD5Hash Hash) override { FileHash = Hash; }

	virtual void SetDimensions(float InArea, float InWidth, float InHeight, float InDepth) override { Area = InArea; Width = InWidth; Height = InHeight; Depth = InDepth;};
	virtual FVector3f GetDimensions() const override { return { Width, Height, Depth }; }

	virtual float GetArea() const override { return Area; }
	virtual float GetWidth() const override { return Width; }
	virtual float GetHeight() const override { return Height; }
	virtual float GetDepth() const override { return Depth; }

	virtual int32 GetLightmapCoordinateIndex() const { return LightmapCoordinateIndex; }
	virtual void SetLightmapCoordinateIndex(int32 UVChannel) { LightmapCoordinateIndex = UVChannel;  }

	virtual int32 GetLightmapSourceUV() const override { return LightmapSourceUV; }
	virtual void SetLightmapSourceUV( int32 UVChannel ) override { LightmapSourceUV = UVChannel; }

	virtual void SetMaterial(const TCHAR* MaterialPathName, int32 SlotId) override;
	virtual const TCHAR* GetMaterial(int32 SlotId) const override;

	virtual int32 GetMaterialSlotCount() const override;
	virtual TSharedPtr<const IDatasmithMaterialIDElement> GetMaterialSlotAt(int32 Index) const override;
	virtual TSharedPtr<IDatasmithMaterialIDElement> GetMaterialSlotAt(int32 Index) override;

protected:
	virtual int32 GetLODCount() const override { return LODCount; }
	virtual void SetLODCount(int32 Count) override { LODCount = Count; }

private:
	TReflected<FString>  File;
	TReflected<FMD5Hash> FileHash;
	TReflected<float>    Area;
	TReflected<float>    Width;
	TReflected<float>    Height;
	TReflected<float>    Depth;
	TReflected<int32>    LODCount;
	TReflected<int32>    LightmapCoordinateIndex;
	TReflected<int32>    LightmapSourceUV;
	TDatasmithReferenceArrayProxy<IDatasmithMaterialIDElement> MaterialSlots;
};

/*
 * Experimental Element that describes a cloth asset.
 */
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FDatasmithClothElementImpl
PRAGMA_DISABLE_DEPRECATION_WARNINGS
 : public FDatasmithElementImpl< IDatasmithClothElement >
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	explicit FDatasmithClothElementImpl(const TCHAR* InName);

public:
	virtual const TCHAR* GetFile() const override { return *(FString&)File; }
	virtual void SetFile(const TCHAR* InFile) override { File = InFile; }

private:
	TReflected<FString> File;
};


class FDatasmithMaterialIDElementImpl : public FDatasmithElementImpl< IDatasmithMaterialIDElement >
{
public:
	explicit FDatasmithMaterialIDElementImpl(const TCHAR* InName);
	virtual void SetName(const TCHAR* InName) override
	{
		FString Unsanitized = InName;
		Name = FPackageName::IsValidObjectPath(Unsanitized) ? Unsanitized : FDatasmithUtils::SanitizeObjectName(Unsanitized);
	}
	virtual int32 GetId() const override { return Id; }
	virtual void SetId(int32 InId) override { Id = InId; }

private:
	TReflected<int32> Id;
};

template< typename InterfaceType = IDatasmithMeshActorElement >
class FDatasmithMeshActorElementImpl : public FDatasmithActorElementImpl< InterfaceType >
{
public:
	using FDatasmithElementImpl< InterfaceType >::Store;

	explicit FDatasmithMeshActorElementImpl(const TCHAR* InName);

	virtual void AddMaterialOverride(const TCHAR* InMaterialName, int32 Id) override;
	virtual void AddMaterialOverride(const TSharedPtr< IDatasmithMaterialIDElement >& Material) override;

	virtual int32 GetMaterialOverridesCount() const override;
	virtual TSharedPtr<IDatasmithMaterialIDElement> GetMaterialOverride(int32 i) override;
	virtual TSharedPtr<const IDatasmithMaterialIDElement> GetMaterialOverride(int32 i) const override;
	virtual void RemoveMaterialOverride(const TSharedPtr< IDatasmithMaterialIDElement >& Material) override;
	virtual void ResetMaterialOverrides() override;

	virtual const TCHAR* GetStaticMeshPathName() const override;
	virtual void SetStaticMeshPathName(const TCHAR* InStaticMeshName) override;

protected:
	explicit FDatasmithMeshActorElementImpl(const TCHAR* InName, EDatasmithElementType ElementType);

private:
	TReflected<FString> StaticMeshPathName;
	TDatasmithReferenceArrayProxy<IDatasmithMaterialIDElement> Materials;
};

template < typename InterfaceType >
FDatasmithMeshActorElementImpl< InterfaceType >::FDatasmithMeshActorElementImpl(const TCHAR* InName)
	: FDatasmithMeshActorElementImpl< InterfaceType >(InName, EDatasmithElementType::None)
{}

template < typename InterfaceType >
FDatasmithMeshActorElementImpl< InterfaceType >::FDatasmithMeshActorElementImpl(const TCHAR* InName, EDatasmithElementType ElementType)
	: FDatasmithActorElementImpl< InterfaceType >(InName, EDatasmithElementType::StaticMeshActor | ElementType)
{
	this->RegisterReferenceProxy(Materials, "Materials");
	Store.RegisterParameter(StaticMeshPathName, "StaticMeshPathName");
}

template < typename InterfaceType >
void FDatasmithMeshActorElementImpl< InterfaceType >::AddMaterialOverride(const TCHAR* InMaterialName, int32 Id)
{
	FString MaterialName = FDatasmithUtils::SanitizeObjectName(InMaterialName);

	for (const TSharedPtr< IDatasmithMaterialIDElement >& Material : Materials.View())
	{
		if (FString(Material->GetName()) == MaterialName && Material->GetId() == Id)
		{
			return;
		}
	}

	TSharedPtr< IDatasmithMaterialIDElement > MaterialIDElement = FDatasmithSceneFactory::CreateMaterialId(*MaterialName);
	MaterialIDElement->SetId(Id);
	Materials.Add(MaterialIDElement);
}

template < typename InterfaceType >
void FDatasmithMeshActorElementImpl< InterfaceType >::AddMaterialOverride(const TSharedPtr< IDatasmithMaterialIDElement >& Material)
{
	Materials.Add(Material);
}

template < typename InterfaceType >
int32 FDatasmithMeshActorElementImpl< InterfaceType >::GetMaterialOverridesCount() const
{
	return (int32)Materials.Num();
}

template < typename InterfaceType >
TSharedPtr<IDatasmithMaterialIDElement> FDatasmithMeshActorElementImpl< InterfaceType >::GetMaterialOverride(int32 i)
{
	if (Materials.IsValidIndex(i))
	{
		return Materials[i];
	}
	const TSharedPtr<IDatasmithMaterialIDElement> InvalidMaterialID;
	return InvalidMaterialID;
}

template < typename InterfaceType >
TSharedPtr<const IDatasmithMaterialIDElement> FDatasmithMeshActorElementImpl< InterfaceType >::GetMaterialOverride(int32 i) const
{
	if (Materials.IsValidIndex(i))
	{
		return Materials[i];
	}
	const TSharedPtr<IDatasmithMaterialIDElement> InvalidMaterialID;
	return InvalidMaterialID;
}

template < typename InterfaceType >
void FDatasmithMeshActorElementImpl< InterfaceType >::RemoveMaterialOverride(const TSharedPtr< IDatasmithMaterialIDElement >& Material)
{
	Materials.Remove(Material);
}

template < typename InterfaceType >
void FDatasmithMeshActorElementImpl< InterfaceType >::ResetMaterialOverrides()
{
	Materials.Edit().Reset();
}

template < typename InterfaceType >
const TCHAR* FDatasmithMeshActorElementImpl< InterfaceType >::GetStaticMeshPathName() const
{
	return *(FString&)StaticMeshPathName;
}

template < typename InterfaceType >
void FDatasmithMeshActorElementImpl< InterfaceType >::SetStaticMeshPathName(const TCHAR* InStaticMeshName)
{
	StaticMeshPathName = InStaticMeshName;
}

class FDatasmithHierarchicalInstancedStaticMeshActorElementImpl : public FDatasmithMeshActorElementImpl< IDatasmithHierarchicalInstancedStaticMeshActorElement >
{
public:
	explicit FDatasmithHierarchicalInstancedStaticMeshActorElementImpl(const TCHAR* InName);

	virtual ~FDatasmithHierarchicalInstancedStaticMeshActorElementImpl();

	virtual int32 GetInstancesCount() const override;
	virtual void ReserveSpaceForInstances(int32 NumIntances) override;
	virtual int32 AddInstance(const FTransform& Transform) override;
	virtual FTransform GetInstance(int32 InstanceIndex) const override;
	virtual void RemoveInstance(int32 InstanceIndex) override;

private:
	TReflected<TArray<FTransform>> Instances;
};

enum class LightActorFlags : uint8
{
	LightNone             = 0x00,
	LightEnabled          = 0x01,
	LightUseTemperature   = 0x02,
	LightUseIes           = 0x04,
	LightUseIesBrightness = 0x08,
};
ENUM_CLASS_FLAGS(LightActorFlags);

template< typename InterfaceType = IDatasmithLightActorElement >
class FDatasmithLightActorElementImpl : public FDatasmithActorElementImpl< InterfaceType >
{
public:
	using FDatasmithElementImpl< InterfaceType >::Store;
	virtual bool IsEnabled() const override	{ return !!(LightFLags & LightActorFlags::LightEnabled); }
	virtual void SetEnabled(bool bInIsEnabled) override {  UPDATE_BITFLAGS( LightFLags, bInIsEnabled, LightActorFlags::LightEnabled); }

	virtual double GetIntensity() const override { return Intensity; }
	virtual void SetIntensity(double InIntensity) override { Intensity = InIntensity; }

	virtual FLinearColor GetColor() const override { return FLinearColor(Color); }
	virtual void SetColor(FLinearColor InColor) override { Color = FVector(InColor.R, InColor.G, InColor.B); }

	virtual double GetTemperature() const override { return Temperature; }
	virtual void SetTemperature(double InTemperature) override { Temperature = InTemperature; }

	virtual bool GetUseTemperature() const override { return !!(LightFLags & LightActorFlags::LightUseTemperature); }
	virtual void SetUseTemperature(bool bInUseTemperature) override { UPDATE_BITFLAGS( LightFLags, bInUseTemperature, LightActorFlags::LightUseTemperature); }

	virtual const TCHAR* GetIesFile() const override { return *(FString&)IesFile;	}
	virtual void SetIesFile(const TCHAR* InIesFile) override { IesFile = InIesFile;	}

	virtual bool GetUseIes() const override { return !!(LightFLags & LightActorFlags::LightUseIes); }
	virtual void SetUseIes(bool bInUseIes) override { UPDATE_BITFLAGS( LightFLags, bInUseIes, LightActorFlags::LightUseIes); }

	virtual double GetIesBrightnessScale() const override { return IesBrightnessScale; }
	virtual void SetIesBrightnessScale(double InIesBrightnessScale) override { IesBrightnessScale = InIesBrightnessScale; }

	virtual bool GetUseIesBrightness() const override { return !!(LightFLags & LightActorFlags::LightUseIesBrightness); }
	virtual void SetUseIesBrightness(bool bInUseIesBrightness) override { UPDATE_BITFLAGS( LightFLags, bInUseIesBrightness, LightActorFlags::LightUseIesBrightness); }

	virtual FQuat GetIesRotation() const override { return IesRotation; }
	virtual void SetIesRotation(const FQuat& InIesRotation) override { IesRotation = InIesRotation; }

	TSharedPtr< IDatasmithMaterialIDElement >& GetLightFunctionMaterial() override	{ return LightFunctionMaterial.Inner; }

	void SetLightFunctionMaterial(const TSharedPtr< IDatasmithMaterialIDElement >& InMaterial) override { LightFunctionMaterial = InMaterial; }

	void SetLightFunctionMaterial(const TCHAR* InMaterialName) override
	{
		FString MaterialName = FDatasmithUtils::SanitizeObjectName(InMaterialName);
		LightFunctionMaterial.Inner = FDatasmithSceneFactory::CreateMaterialId(*MaterialName);
	}

	const TCHAR* GetIesTexturePathName() const override	{ return *(FString&)IesTexturePathName; }
	void SetIesTexturePathName(const TCHAR* InTextureName) override { IesTexturePathName = InTextureName; }

protected:
	explicit FDatasmithLightActorElementImpl(const TCHAR* InName, EDatasmithElementType ChildType);

private:
	TReflected<double> Intensity;

	TReflected<FVector> Color;

	TReflected<double> Temperature;

	TReflected<FString> IesFile;

	TReflected<FString> IesTexturePathName;

	TReflected<double> IesBrightnessScale;

	TReflected<FQuat> IesRotation;

	TReflected<LightActorFlags, uint8> LightFLags;

	TDatasmithReferenceProxy< IDatasmithMaterialIDElement > LightFunctionMaterial;
};

template<typename InterfaceType>
FDatasmithLightActorElementImpl<InterfaceType>::FDatasmithLightActorElementImpl(const TCHAR* InName, EDatasmithElementType ChildType)
	: FDatasmithActorElementImpl< InterfaceType >( InName, EDatasmithElementType::Light | ChildType )
	, Intensity(1.0)
	, Color(FVector::OneVector)
	, Temperature(6500.0)
	, IesBrightnessScale(1.0)
	, IesRotation(FQuat::Identity)
	, LightFLags(LightActorFlags::LightEnabled)
{
	this->RegisterReferenceProxy(LightFunctionMaterial, "LightFunctionMaterial" );

	Store.RegisterParameter(Intensity,          "Intensity"           );
	Store.RegisterParameter(Color,              "Color"               );
	Store.RegisterParameter(Temperature,        "Temperature"         );
	Store.RegisterParameter(IesFile,            "IesFile"             );
	Store.RegisterParameter(IesTexturePathName, "IesTexturePathName"  );
	Store.RegisterParameter(IesBrightnessScale, "IesBrightnessScale"  );
	Store.RegisterParameter(IesRotation,        "IesRotation"         );
	Store.RegisterParameter(LightFLags,         "LightFLags"          );
}

template< typename InterfaceType = IDatasmithPointLightElement >
class FDatasmithPointLightElementImpl : public FDatasmithLightActorElementImpl< InterfaceType >
{
	using FDatasmithElementImpl< InterfaceType >::Store;

public:
	explicit FDatasmithPointLightElementImpl(const TCHAR* InName)
		: FDatasmithPointLightElementImpl( InName, EDatasmithElementType::None )
	{
	}

	virtual void SetIntensityUnits(EDatasmithLightUnits InUnits) { Units = InUnits; }
	virtual EDatasmithLightUnits GetIntensityUnits() const { return (EDatasmithLightUnits)Units; }

	virtual float GetSourceRadius() const override { return SourceRadius; }
	virtual void SetSourceRadius(float InSourceRadius) override { SourceRadius = InSourceRadius; }

	virtual float GetSourceLength() const override { return SourceLength; }
	virtual void SetSourceLength(float InSourceLength) override { SourceLength = InSourceLength;}

	virtual float GetAttenuationRadius() const override	{ return AttenuationRadius;	}
	virtual void SetAttenuationRadius(float InAttenuationRadius) override {	AttenuationRadius = InAttenuationRadius; }

protected:
	explicit FDatasmithPointLightElementImpl(const TCHAR* InName, EDatasmithElementType ChildType)
		: FDatasmithLightActorElementImpl< InterfaceType >( InName, EDatasmithElementType::PointLight | ChildType )
		, Units(EDatasmithLightUnits::Unitless)
		, SourceRadius(-1)
		, SourceLength(-1)
		, AttenuationRadius(-1)
	{
		Store.RegisterParameter(Units,             "Units"             );
		Store.RegisterParameter(SourceRadius,      "SourceRadius"      );
		Store.RegisterParameter(SourceLength,      "SourceLength"      );
		Store.RegisterParameter(AttenuationRadius, "AttenuationRadius" );
	}

private:
	TReflected<EDatasmithLightUnits, uint8> Units;
	TReflected<float> SourceRadius;
	TReflected<float> SourceLength;
	TReflected<float> AttenuationRadius;
};

template< typename InterfaceType = IDatasmithSpotLightElement >
class FDatasmithSpotLightElementImpl : public FDatasmithPointLightElementImpl< InterfaceType >
{
public:
	using FDatasmithElementImpl< InterfaceType >::Store;
	explicit FDatasmithSpotLightElementImpl(const TCHAR* InName)
		: FDatasmithSpotLightElementImpl( InName, EDatasmithElementType::None )
	{
	}

	virtual float GetInnerConeAngle() const override
	{
		return InnerConeAngle;
	}

	virtual void SetInnerConeAngle(float InInnerConeAngle) override
	{
		InnerConeAngle = InInnerConeAngle;
	}

	virtual float GetOuterConeAngle() const override
	{
		return OuterConeAngle;
	}

	virtual void SetOuterConeAngle(float InOuterConeAngle) override
	{
		OuterConeAngle = InOuterConeAngle;
	}

protected:
	explicit FDatasmithSpotLightElementImpl(const TCHAR* InName, EDatasmithElementType ChildType)
		: FDatasmithPointLightElementImpl< InterfaceType >( InName, EDatasmithElementType::SpotLight | ChildType )
		, InnerConeAngle(45.f)
		, OuterConeAngle(60.f)
	{
		Store.RegisterParameter(InnerConeAngle,	"InnerConeAngle" );
		Store.RegisterParameter(OuterConeAngle,	"OuterConeAngle" );
	}

private:
	TReflected<float> InnerConeAngle;
	TReflected<float> OuterConeAngle;
};

class FDatasmithDirectionalLightElementImpl : public FDatasmithLightActorElementImpl< IDatasmithDirectionalLightElement >
{
public:
	explicit FDatasmithDirectionalLightElementImpl(const TCHAR* InName)
		: FDatasmithLightActorElementImpl< IDatasmithDirectionalLightElement >( InName, EDatasmithElementType::DirectionalLight )
	{
	}
};

class FDatasmithAreaLightElementImpl : public FDatasmithSpotLightElementImpl< IDatasmithAreaLightElement >
{
public:
	explicit FDatasmithAreaLightElementImpl(const TCHAR* InName)
		: FDatasmithSpotLightElementImpl< IDatasmithAreaLightElement >( InName, EDatasmithElementType::AreaLight )
		, LightShape( EDatasmithLightShape::Rectangle )
		, LightType( EDatasmithAreaLightType::Point )
		, Width( 0.f )
		, Length( 0.f )
	{
		Store.RegisterParameter(LightShape, "LightShape" );
		Store.RegisterParameter(LightType,  "LightType"  );
		Store.RegisterParameter(Width,      "Width"      );
		Store.RegisterParameter(Length,     "Length"     );
	}

	virtual EDatasmithLightShape GetLightShape() const override { return LightShape; }
	virtual void SetLightShape(EDatasmithLightShape InShape) override { LightShape = InShape; }

	virtual EDatasmithAreaLightType GetLightType() const override { return LightType; }
	virtual void SetLightType(EDatasmithAreaLightType InLightType) override { LightType = InLightType; }

	virtual void SetWidth(float InWidth) override { Width = InWidth; }
	virtual float GetWidth() const override { return Width; }

	virtual void SetLength(float InLength) override { Length = InLength; }
	virtual float GetLength() const override { return Length; }

private:
	TReflected<EDatasmithLightShape, uint8> LightShape;
	TReflected<EDatasmithAreaLightType, uint8> LightType;
	TReflected<float> Width;
	TReflected<float> Length;
};

class FDatasmithLightmassPortalElementImpl : public FDatasmithPointLightElementImpl< IDatasmithLightmassPortalElement >
{
public:
	explicit FDatasmithLightmassPortalElementImpl(const TCHAR* InName)
		: FDatasmithPointLightElementImpl< IDatasmithLightmassPortalElement >( InName, EDatasmithElementType::LightmassPortal )
	{
	}
};

class FDatasmithPostProcessElementImpl : public FDatasmithElementImpl< IDatasmithPostProcessElement >
{
public:
	FDatasmithPostProcessElementImpl();

	virtual float GetTemperature() const override { return Temperature; }
	virtual void SetTemperature(float InTemperature) override { Temperature = InTemperature; }

	virtual FLinearColor GetColorFilter() const override { return ColorFilter; }
	virtual void SetColorFilter(FLinearColor InColorFilter) override { ColorFilter = InColorFilter; }

	virtual float GetVignette() const override { return Vignette; }
	virtual void SetVignette(float InVignette) override { Vignette = InVignette; }

	virtual float GetDof() const override { return Dof; }
	virtual void SetDof(float InDof) override { Dof = InDof; }

	virtual float GetMotionBlur() const override { return MotionBlur; }
	virtual void SetMotionBlur(float InMotionBlur) override { MotionBlur = InMotionBlur; }

	virtual float GetSaturation() const override { return Saturation; }
	virtual void SetSaturation(float InSaturation) override { Saturation = InSaturation; }

	virtual float GetCameraISO() const override { return CameraISO; }
	virtual void SetCameraISO(float InCameraISO) override { CameraISO = InCameraISO; }

	virtual float GetCameraShutterSpeed() const override { return CameraShutterSpeed; }
	virtual void SetCameraShutterSpeed(float InCameraShutterSpeed) override { CameraShutterSpeed = InCameraShutterSpeed; }

	virtual float GetDepthOfFieldFstop() const override { return Fstop; }
	virtual void SetDepthOfFieldFstop( float InFstop ) override { Fstop = InFstop; }

private:
	TReflected<float> Temperature;
	TReflected<FLinearColor> ColorFilter;
	TReflected<float> Vignette;
	TReflected<float> Dof;
	TReflected<float> MotionBlur;
	TReflected<float> Saturation;
	TReflected<float> CameraISO;
	TReflected<float> CameraShutterSpeed;
	TReflected<float> Fstop;
};

class FDatasmithPostProcessVolumeElementImpl : public FDatasmithActorElementImpl< IDatasmithPostProcessVolumeElement >
{
public:
	FDatasmithPostProcessVolumeElementImpl( const TCHAR* InName );

	virtual TSharedRef< IDatasmithPostProcessElement > GetSettings() const override { return Settings.Inner.ToSharedRef(); }
	virtual void SetSettings(const TSharedRef< IDatasmithPostProcessElement >& InSettings) override { Settings.Inner = InSettings; }

	virtual bool GetEnabled() const { return bEnabled; }
	virtual void SetEnabled( bool bInEnabled ) { bEnabled = bInEnabled; }

	virtual bool GetUnbound() const override { return bUnbound; }
	virtual void SetUnbound( bool bInUnbound) override { bUnbound = bInUnbound; }

private:
	TDatasmithReferenceProxy<IDatasmithPostProcessElement> Settings;

	TReflected<bool> bEnabled;
	TReflected<bool> bUnbound;
};

class FDatasmithCameraActorElementImpl : public FDatasmithActorElementImpl< IDatasmithCameraActorElement >
{
public:
	explicit FDatasmithCameraActorElementImpl(const TCHAR* InName);

	virtual float GetSensorWidth() const override;
	virtual void SetSensorWidth(float InSensorWidth) override;

	virtual float GetSensorAspectRatio() const override;
	virtual void SetSensorAspectRatio(float InSensorAspectRatio) override;

	virtual bool GetEnableDepthOfField() const override { return bEnableDepthOfField; }
	virtual void SetEnableDepthOfField(bool bInEnableDepthOfField) override { bEnableDepthOfField = bInEnableDepthOfField; }

	virtual float GetFocusDistance() const override;
	virtual void SetFocusDistance(float InFocusDistance) override;

	virtual float GetFStop() const override;
	virtual void SetFStop(float InFStop) override;

	virtual float GetFocalLength() const override;
	virtual void SetFocalLength(float InFocalLength) override;

	virtual TSharedPtr< IDatasmithPostProcessElement >& GetPostProcess() override;
	virtual const TSharedPtr< IDatasmithPostProcessElement >& GetPostProcess() const override;
	virtual void SetPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& InPostProcess) override;

	virtual const TCHAR* GetLookAtActor() const override { return *(FString&)ActorName; }
	virtual void SetLookAtActor(const TCHAR* InActorName) override { ActorName = InActorName; }

	virtual bool GetLookAtAllowRoll() const override { return bLookAtAllowRoll; }
	virtual void SetLookAtAllowRoll(bool bAllow) override { bLookAtAllowRoll = bAllow; }

private:
	TDatasmithReferenceProxy<IDatasmithPostProcessElement> PostProcess;

	TReflected<float>   SensorWidth;
	TReflected<float>   SensorAspectRatio;
	TReflected<bool>    bEnableDepthOfField;
	TReflected<float>   FocusDistance;
	TReflected<float>   FStop;
	TReflected<float>   FocalLength;
	TReflected<FString> ActorName;
	TReflected<bool>    bLookAtAllowRoll;
};

template< typename InterfaceType = IDatasmithCustomActorElement >
class FDatasmithCustomActorElementImpl : public FDatasmithActorElementImpl< InterfaceType >
{
public:
	using FDatasmithElementImpl< InterfaceType >::Store;

	explicit FDatasmithCustomActorElementImpl(const TCHAR* InName, EDatasmithElementType InChildType = EDatasmithElementType::None)
		: FDatasmithActorElementImpl< InterfaceType >(InName, EDatasmithElementType::CustomActor | InChildType)
	{
		this->RegisterReferenceProxy(Properties, "Properties");
		Store.RegisterParameter(ClassOrPathName, "ClassOrPathName");
	}

	/** The class name or path to the blueprint to instantiate. */
	virtual const TCHAR* GetClassOrPathName() const override { return *(FString&)ClassOrPathName; }
	virtual void SetClassOrPathName( const TCHAR* InClassOrPathName ) override { ClassOrPathName = InClassOrPathName; }

	/** Get the total amount of properties in this actor */
	virtual int32 GetPropertiesCount() const override { return Properties.Num(); }

	/** Get the property i-th of this actor */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 Index) const override
	{
		return Properties.IsValidIndex(Index) ? Properties[Index] : FDatasmithKeyValuePropertyImpl::NullPropertyPtr;
	}

	/** Get a property by its name if it exists */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName(const TCHAR* InName) const override
	{
		const int32 Index = Properties.View().IndexOfByPredicate([InName](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
			return Property.IsValid() && FCString::Stricmp(Property->GetName(), InName) == 0;
			});
		return GetProperty(Index);
	}

	/** Add a property to this actor */
	virtual void AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& InProperty ) override
	{
		if (!InProperty.IsValid())
		{
			return;
		}

		const TCHAR* InName = InProperty->GetName();
		const int32 Index = Properties.View().IndexOfByPredicate([InName](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
			return Property.IsValid() && FCString::Stricmp(Property->GetName(), InName) == 0;
			});

		if (Index == INDEX_NONE)
		{
			Properties.Add(InProperty);
		}
	}

	/** Removes a property from this actor, doesn't preserve ordering */
	virtual void RemoveProperty( const TSharedPtr< IDatasmithKeyValueProperty >& Property ) override { Properties.Edit().RemoveSingleSwap( Property ); }

protected:
	/** Add a property to this actor */
	int32 AddPropertyInternal(const TCHAR* InKey, EDatasmithKeyValuePropertyType InType, const TCHAR* InValue)
	{
		const int32 Index = Properties.View().IndexOfByPredicate([InKey](const TSharedPtr<IDatasmithKeyValueProperty>& Property){
			return Property.IsValid() && FCString::Stricmp(Property->GetName(), InKey) == 0;
			});

		if (Index == INDEX_NONE)
		{
			TSharedPtr<IDatasmithKeyValueProperty> Property = MakeShared<FDatasmithKeyValuePropertyImpl>(InKey);
			Property->SetPropertyType(InType);
			Property->SetValue(InValue);

			return Properties.Add( Property );
		}

		return INDEX_NONE;
	}

private:
	TReflected<FString> ClassOrPathName;

	TDatasmithReferenceArrayProxy<IDatasmithKeyValueProperty> Properties;
};

class FDatasmithLandscapeElementImpl final : public FDatasmithActorElementImpl< IDatasmithLandscapeElement >
{
public:
	explicit FDatasmithLandscapeElementImpl(const TCHAR* InName)
		: FDatasmithActorElementImpl(InName, EDatasmithElementType::Landscape)
	{
		SetScale( 100.f, 100.f, 100.f, true );

		RegisterReferenceProxy(Material,  "Material"  );
		RegisterReferenceProxy(Heightmap, "Heightmap" );

		Store.RegisterParameter(HeightmapFilePath, "HeightmapFilePath" );
		Store.RegisterParameter(MaterialPathName,  "MaterialPathName"  );
	}

	virtual void SetHeightmap( const TCHAR* InFilePath ) override { HeightmapFilePath = InFilePath; }
	virtual const TCHAR* GetHeightmap() const override { return *(FString&)HeightmapFilePath; }

	virtual void SetMaterial( const TCHAR* InMaterialPathName ) override { MaterialPathName = InMaterialPathName; }
	virtual const TCHAR* GetMaterial() const override { return *(FString&)MaterialPathName; }

private:
	TDatasmithReferenceProxy<IDatasmithBaseMaterialElement> Material;
	TDatasmithReferenceProxy<IDatasmithTextureElement> Heightmap;

	TReflected<FString> HeightmapFilePath;
	TReflected<FString> MaterialPathName;
};


PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FDatasmithClothActorElementImpl
	: public FDatasmithActorElementImpl<IDatasmithClothActorElement>
{
public:
	FDatasmithClothActorElementImpl(const TCHAR* InName);

	virtual void SetCloth(const TCHAR* InCloth) override { Cloth = InCloth; }
	virtual const TCHAR* GetCloth() const override { return *(FString&)Cloth; }

private:
	TReflected<FString> Cloth;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS


class FDatasmithEnvironmentElementImpl : public FDatasmithLightActorElementImpl< IDatasmithEnvironmentElement >
{
public:
	explicit FDatasmithEnvironmentElementImpl(const TCHAR* InName);

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetEnvironmentComp() override;
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetEnvironmentComp() const override;

	virtual void SetEnvironmentComp(const TSharedPtr<IDatasmithCompositeTexture>& InEnvironmentComp) override;
	virtual bool GetIsIlluminationMap() const override;
	virtual void SetIsIlluminationMap(bool bInIsIlluminationMap) override;

private:
	TSharedPtr<IDatasmithCompositeTexture> EnvironmentComp;
	TReflected<bool> bIsIlluminationMap;
};

class FDatasmithTextureElementImpl : public FDatasmithElementImpl< IDatasmithTextureElement >
{
public:
	explicit FDatasmithTextureElementImpl(const TCHAR* InName);

	virtual FMD5Hash CalculateElementHash(bool bForce) override;

	virtual const TCHAR* GetFile() const override;
	virtual void SetFile(const TCHAR* InFile) override;

	virtual void SetData(const uint8* InData, uint32 InDataSize, EDatasmithTextureFormat InFormat) override;
	virtual const uint8* GetData(uint32& OutDataSize, EDatasmithTextureFormat& OutFormat) const override;

	virtual FMD5Hash GetFileHash() const override { return FileHash; }
	virtual void SetFileHash(FMD5Hash Hash) override { FileHash = Hash; }

	virtual EDatasmithTextureMode GetTextureMode() const override;
	virtual void SetTextureMode(EDatasmithTextureMode InMode) override;

	virtual EDatasmithTextureFilter GetTextureFilter() const override;
	virtual void SetTextureFilter(EDatasmithTextureFilter InFilter) override;

	virtual EDatasmithTextureAddress GetTextureAddressX() const override;
	virtual void SetTextureAddressX(EDatasmithTextureAddress InMode) override;

	virtual EDatasmithTextureAddress GetTextureAddressY() const override;
	virtual void SetTextureAddressY(EDatasmithTextureAddress InMode) override;

	virtual bool GetAllowResize() const override;
	virtual void SetAllowResize(bool bInAllowResize) override;

	virtual float GetRGBCurve() const override;
	virtual void SetRGBCurve(float InRGBCurve) override;

	virtual EDatasmithColorSpace GetSRGB() const override;
	virtual void SetSRGB(EDatasmithColorSpace Option) override;

private:
	TReflected<FString> File;
	TReflected<FMD5Hash> FileHash;
	TReflected<float> RGBCurve;
	TReflected<EDatasmithColorSpace, uint8> ColorSpace;
	TReflected<EDatasmithTextureMode, uint8> TextureMode;
	TReflected<EDatasmithTextureFilter, uint8> TextureFilter;
	TReflected<EDatasmithTextureAddress, uint8> TextureAddressX;
	TReflected<EDatasmithTextureAddress, uint8> TextureAddressY;
	TReflected<bool> bAllowResize;

	// #ue_directlink_reflect buffer: should not be separated in 2 properties.
	const uint8* Data;
	uint32 DataSize;

	TReflected<EDatasmithTextureFormat, uint8> TextureFormat;
};

class FDatasmithShaderElementImpl : public FDatasmithElementImpl< IDatasmithShaderElement >
{
public:
	explicit FDatasmithShaderElementImpl(const TCHAR* InName);

	virtual double GetIOR() const override { return IOR; }
	virtual void SetIOR(double InValue) override { IOR = InValue; }

	virtual double GetIORk() const override { return IORk; }
	virtual void SetIORk(double InValue) override { IORk = InValue; }

	virtual double GetIORRefra() const override { return IORRefra; }
	virtual void SetIORRefra(double Value) override { IORRefra = Value; }

	virtual double GetBumpAmount() const override { return BumpAmount; }
	virtual void SetBumpAmount(double InValue) override { BumpAmount = InValue; }

	virtual bool GetTwoSided() const override { return bTwoSided; }
	virtual void SetTwoSided(bool InValue) override { bTwoSided = InValue; }

	virtual FLinearColor GetDiffuseColor() const override { return DiffuseColor; }
	virtual void SetDiffuseColor(FLinearColor InValue) override { DiffuseColor = InValue; }

	virtual const TCHAR* GetDiffuseTexture() const override { return *DiffuseTexture; }
	virtual void SetDiffuseTexture(const TCHAR* InValue) override { DiffuseTexture = InValue; }

	virtual FDatasmithTextureSampler GetDiffTextureSampler() const override { return DiffSampler; }
	virtual void SetDiffTextureSampler(FDatasmithTextureSampler InValue) override { DiffSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetDiffuseComp() override { return DiffuseComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetDiffuseComp() const override { return DiffuseComp; }
	virtual void SetDiffuseComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { DiffuseComp = InValue; }

	virtual FLinearColor GetReflectanceColor() const override { return ReflectanceColor; }
	virtual void SetReflectanceColor(FLinearColor InValue) override { ReflectanceColor = InValue; }

	virtual const TCHAR* GetReflectanceTexture() const override { return *ReflectanceTexture; }
	virtual void SetReflectanceTexture(const TCHAR* InValue) override { ReflectanceTexture = InValue; }

	virtual FDatasmithTextureSampler GetRefleTextureSampler() const override { return RefleSampler; }
	virtual void SetRefleTextureSampler(FDatasmithTextureSampler InValue) override { RefleSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetRefleComp() override { return RefleComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetRefleComp() const override { return RefleComp; }
	virtual void SetRefleComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { RefleComp = InValue; }

	virtual double GetRoughness() const override { return Roughness; }
	virtual void SetRoughness(double InValue) override { Roughness = InValue; }

	virtual const TCHAR* GetRoughnessTexture() const override { return *RoughnessTexture; }
	virtual void SetRoughnessTexture(const TCHAR* InValue) override { RoughnessTexture = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetRoughnessComp() override { return RoughnessComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetRoughnessComp() const override { return RoughnessComp; }
	virtual void SetRoughnessComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { RoughnessComp = InValue; }

	virtual FDatasmithTextureSampler GetRoughTextureSampler() const override { return RoughSampler; }
	virtual void SetRoughTextureSampler(FDatasmithTextureSampler InValue) override { RoughSampler = InValue; }

	virtual const TCHAR* GetNormalTexture() const override { return *NormalTexture; }
	virtual void SetNormalTexture(const TCHAR* InValue) override { NormalTexture = InValue; }

	virtual FDatasmithTextureSampler GetNormalTextureSampler() const override { return NormalSampler; }
	virtual void SetNormalTextureSampler(FDatasmithTextureSampler InValue) override { NormalSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetNormalComp() override { return NormalComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetNormalComp() const override { return NormalComp; }
	virtual void SetNormalComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { NormalComp = InValue; }

	virtual const TCHAR* GetBumpTexture() const override { return *BumpTexture; }
	virtual void SetBumpTexture(const TCHAR* Value) override { BumpTexture = Value; }

	virtual FDatasmithTextureSampler GetBumpTextureSampler() const override { return BumpSampler; }
	virtual void SetBumpTextureSampler(FDatasmithTextureSampler InValue) override { BumpSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetBumpComp() override { return BumpComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetBumpComp() const override { return BumpComp; }
	virtual void SetBumpComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { BumpComp = InValue; }

	virtual FLinearColor GetTransparencyColor() const override { return TransparencyColor; }
	virtual void SetTransparencyColor(FLinearColor InValue) override { TransparencyColor = InValue; }

	virtual const TCHAR* GetTransparencyTexture() const override { return *TransparencyTexture; }
	virtual void SetTransparencyTexture(const TCHAR* InValue) override { TransparencyTexture = InValue; }

	virtual FDatasmithTextureSampler GetTransTextureSampler() const override { return TransSampler; }
	virtual void SetTransTextureSampler(FDatasmithTextureSampler InValue) override { TransSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetTransComp() override { return TransComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetTransComp() const override { return TransComp; }
	virtual void SetTransComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { TransComp = InValue; }

	virtual const TCHAR* GetMaskTexture() const override { return *MaskTexture; }
	virtual void SetMaskTexture(const TCHAR* InValue) override { MaskTexture = InValue; }

	virtual FDatasmithTextureSampler GetMaskTextureSampler() const override { return MaskSampler; }
	virtual void SetMaskTextureSampler(FDatasmithTextureSampler InValue) override { MaskSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetMaskComp() override { return MaskComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetMaskComp() const override { return MaskComp; }
	virtual void SetMaskComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { MaskComp = InValue; }

	virtual double GetMetal() const override { return Metal; }
	virtual void SetMetal(double InValue) override { Metal = InValue; }

	virtual const TCHAR* GetMetalTexture() const override { return *MetalTexture; }
	virtual void SetMetalTexture(const TCHAR* InValue) override { MetalTexture = InValue; }

	virtual FDatasmithTextureSampler GetMetalTextureSampler() const override { return MetalSampler; }
	virtual void SetMetalTextureSampler(FDatasmithTextureSampler InValue) override { MetalSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetMetalComp() override { return MetalComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetMetalComp() const override { return MetalComp; }
	virtual void SetMetalComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) override { MetalComp = Value; }

	virtual const TCHAR* GetEmitTexture() const override { return *EmitTexture; }
	virtual void SetEmitTexture(const TCHAR* InValue) override { EmitTexture = InValue; }

	virtual FDatasmithTextureSampler GetEmitTextureSampler() const override { return EmitSampler; }
	virtual void SetEmitTextureSampler(FDatasmithTextureSampler InValue) override { EmitSampler = InValue; }

	virtual FLinearColor GetEmitColor() const override { return EmitColor; }
	virtual void SetEmitColor(FLinearColor InValue) override { EmitColor = InValue; }

	virtual double GetEmitTemperature() const override { return EmitTemperature; }
	virtual void SetEmitTemperature(double InValue) override { EmitTemperature = InValue; }

	virtual double GetEmitPower() const override { return EmitPower; }
	virtual void SetEmitPower(double InValue) override { EmitPower = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetEmitComp() override { return EmitComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetEmitComp() const override { return EmitComp; }
	virtual void SetEmitComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { EmitComp = InValue; }

	virtual bool GetLightOnly() const override { return bLightOnly; }
	virtual void SetLightOnly(bool InValue) override { bLightOnly = InValue; }

	virtual FLinearColor GetWeightColor() const override { return WeightColor; }
	virtual void SetWeightColor(FLinearColor InValue) override { WeightColor = InValue; }

	virtual const TCHAR* GetWeightTexture() const override { return *WeightTexture; }
	virtual void SetWeightTexture(const TCHAR* InValue) override { WeightTexture = InValue; }

	virtual FDatasmithTextureSampler GetWeightTextureSampler() const override { return WeightSampler; }
	virtual void SetWeightTextureSampler(FDatasmithTextureSampler InValue) override { WeightSampler = InValue; }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetWeightComp() override { return WeightComp; }
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetWeightComp() const override { return WeightComp; }
	virtual void SetWeightComp(const TSharedPtr<IDatasmithCompositeTexture>& InValue) override { WeightComp = InValue; }

	virtual double GetWeightValue() const override { return WeightValue; }
	virtual void SetWeightValue(double InValue) override { WeightValue = InValue; }

	virtual EDatasmithBlendMode GetBlendMode() const override { return BlendMode; }
	virtual void SetBlendMode(EDatasmithBlendMode InValue) override { BlendMode = InValue; }

	virtual bool GetIsStackedLayer() const override { return bIsStackedLayer; }
	virtual void SetIsStackedLayer(bool InValue) override { bIsStackedLayer = InValue; }

	virtual const EDatasmithShaderUsage GetShaderUsage() const override { return ShaderUsage; }
	virtual void SetShaderUsage(EDatasmithShaderUsage InShaderUsage) override { ShaderUsage = InShaderUsage; };

	virtual const bool GetUseEmissiveForDynamicAreaLighting() const override { return bUseEmissiveForDynamicAreaLighting; }
	virtual void SetUseEmissiveForDynamicAreaLighting(bool InUseEmissiveForDynamicAreaLighting) override {	bUseEmissiveForDynamicAreaLighting = InUseEmissiveForDynamicAreaLighting; };

private:// #ue_directlink_reflect ???
	double IOR;
	double IORk;
	double IORRefra;

	double BumpAmount;
	bool bTwoSided;

	FLinearColor DiffuseColor;
	FString DiffuseTexture;
	FDatasmithTextureSampler DiffSampler;
	TSharedPtr<IDatasmithCompositeTexture> DiffuseComp;

	FLinearColor ReflectanceColor;
	FString ReflectanceTexture;
	FDatasmithTextureSampler RefleSampler;
	TSharedPtr<IDatasmithCompositeTexture> RefleComp;

	double Roughness;
	FString RoughnessTexture;
	FDatasmithTextureSampler RoughSampler;
	TSharedPtr<IDatasmithCompositeTexture> RoughnessComp;

	FString NormalTexture;
	FDatasmithTextureSampler NormalSampler;
	TSharedPtr<IDatasmithCompositeTexture> NormalComp;

	FString BumpTexture;
	FDatasmithTextureSampler BumpSampler;
	TSharedPtr<IDatasmithCompositeTexture> BumpComp;

	FLinearColor TransparencyColor;
	FString TransparencyTexture;
	FDatasmithTextureSampler TransSampler;
	TSharedPtr<IDatasmithCompositeTexture> TransComp;

	FString MaskTexture;
	FDatasmithTextureSampler MaskSampler;
	TSharedPtr<IDatasmithCompositeTexture> MaskComp;

	FString DisplaceTexture;
	FDatasmithTextureSampler DisplaceSampler;
	double Displace;
	double DisplaceSubDivision;
	TSharedPtr<IDatasmithCompositeTexture> DisplaceComp;

	double Metal;
	FString MetalTexture;
	FDatasmithTextureSampler MetalSampler;
	TSharedPtr<IDatasmithCompositeTexture> MetalComp;

	FString EmitTexture;
	FDatasmithTextureSampler EmitSampler;
	FLinearColor EmitColor;
	double EmitTemperature;
	double EmitPower;
	TSharedPtr<IDatasmithCompositeTexture> EmitComp;

	bool bLightOnly;

	FLinearColor WeightColor;
	FString WeightTexture;
	FDatasmithTextureSampler WeightSampler;
	TSharedPtr<IDatasmithCompositeTexture> WeightComp;
	double WeightValue;

	EDatasmithBlendMode BlendMode;
	bool bIsStackedLayer;

	EDatasmithShaderUsage ShaderUsage;
	bool bUseEmissiveForDynamicAreaLighting;
};

template< typename InterfaceType >
class FDatasmithBaseMaterialElementImpl : public FDatasmithElementImpl< InterfaceType >
{
public:
	explicit FDatasmithBaseMaterialElementImpl(const TCHAR* InName, EDatasmithElementType ChildType);
};

template< typename T >
inline FDatasmithBaseMaterialElementImpl<T>::FDatasmithBaseMaterialElementImpl(const TCHAR* InName, EDatasmithElementType ChildType)
	: FDatasmithElementImpl<T>(InName, EDatasmithElementType::BaseMaterial | ChildType)
{
}

class FDatasmithMaterialElementImpl : public FDatasmithBaseMaterialElementImpl< IDatasmithMaterialElement >
{
public:
	explicit FDatasmithMaterialElementImpl(const TCHAR* InName);

	virtual bool IsSingleShaderMaterial() const override;
	virtual bool IsClearCoatMaterial() const override;

	virtual void AddShader(const TSharedPtr< IDatasmithShaderElement >& InShader) override;

	virtual int32 GetShadersCount() const override;
	virtual TSharedPtr< IDatasmithShaderElement >& GetShader(int32 InIndex) override;
	virtual const TSharedPtr< IDatasmithShaderElement >& GetShader(int32 InIndex) const override;

private:// #ue_directlink_reflect
	TArray< TSharedPtr< IDatasmithShaderElement > > Shaders;
};

class FDatasmithMaterialIntanceElementImpl : public FDatasmithBaseMaterialElementImpl< IDatasmithMaterialInstanceElement >
{
public:
	FDatasmithMaterialIntanceElementImpl(const TCHAR* InName);

	virtual EDatasmithReferenceMaterialType GetMaterialType() const override { return MaterialType; }
	virtual void SetMaterialType( EDatasmithReferenceMaterialType InType ) override { MaterialType = InType; }

	virtual EDatasmithReferenceMaterialQuality GetQuality() const override { return Quality; }
	virtual void SetQuality( EDatasmithReferenceMaterialQuality InQuality ) override { Quality = InQuality; }

	virtual const TCHAR* GetCustomMaterialPathName() const override { return *(FString&)CustomMaterialPathName; }
	virtual void SetCustomMaterialPathName( const TCHAR* InPathName ) override { CustomMaterialPathName = InPathName; }

	virtual int32 GetPropertiesCount() const override { return Properties.Num(); }
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty( int32 InIndex ) const override;
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName( const TCHAR* InName ) const override;
	virtual void AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& InProperty ) override;

	virtual FMD5Hash CalculateElementHash(bool bForce) override;

private:
	TDatasmithReferenceArrayProxy<IDatasmithKeyValueProperty> Properties;

	TReflected<EDatasmithReferenceMaterialType, uint8> MaterialType;
	TReflected<EDatasmithReferenceMaterialQuality, uint8> Quality;

	TReflected<FString> CustomMaterialPathName;
};

class FDatasmithDecalMaterialElementImpl : public FDatasmithBaseMaterialElementImpl< IDatasmithDecalMaterialElement >
{
public:
	FDatasmithDecalMaterialElementImpl(const TCHAR* InName)
		: FDatasmithBaseMaterialElementImpl(InName, EDatasmithElementType::DecalMaterial)
	{
		Store.RegisterParameter(DiffuseTexturePathName, "DiffuseTexturePathName");
		Store.RegisterParameter(NormalTexturePathName,  "NormalTexturePathName");
	}

	virtual const TCHAR* GetDiffuseTexturePathName() const override { return *(FString&)DiffuseTexturePathName; }
	virtual void SetDiffuseTexturePathName( const TCHAR* InPathName ) override { DiffuseTexturePathName = InPathName; }

	virtual const TCHAR* GetNormalTexturePathName() const override { return *(FString&)NormalTexturePathName; }
	virtual void SetNormalTexturePathName( const TCHAR* InPathName ) override { NormalTexturePathName = InPathName; }

private:
	TReflected<FString> DiffuseTexturePathName;
	TReflected<FString> NormalTexturePathName;
};

class FDatasmithCompositeSurface
{
public:
	FDatasmithCompositeSurface(const TSharedPtr<IDatasmithCompositeTexture>& SubComp);
	FDatasmithCompositeSurface(const TCHAR* InTexture, FDatasmithTextureSampler InTexUV);
	FDatasmithCompositeSurface(const FLinearColor& InColor);

	bool GetUseTexture() const;
	bool GetUseColor() const;
	bool GetUseComposite() const;

	FDatasmithTextureSampler& GetParamTextureSampler();
	const TCHAR* GetParamTexture() const;
	void SetParamTexture(const TCHAR* InTexture);
	const FLinearColor& GetParamColor() const;
	TSharedPtr<IDatasmithCompositeTexture>& GetParamSubComposite();

private:// #ue_directlink_reflect
	FDatasmithTextureSampler ParamSampler;
	FString ParamTextures;
	FLinearColor ParamColor;
	TSharedPtr<IDatasmithCompositeTexture> ParamSubComposite;
	bool bParamUseTexture;
};

class FDatasmithCompositeTextureImpl : public IDatasmithCompositeTexture
{
public:
	FDatasmithCompositeTextureImpl();

	virtual bool IsValid() const override;

	virtual EDatasmithCompMode GetMode() const override { return CompMode; }
	virtual void SetMode(EDatasmithCompMode InMode) override { CompMode = InMode; }
	virtual int32 GetParamSurfacesCount() const override { return ParamSurfaces.Num(); }

	virtual bool GetUseTexture(int32 InIndex) override;

	virtual const TCHAR* GetParamTexture(int32 InIndex) override;
	virtual void SetParamTexture(int32 InIndex, const TCHAR* InTexture) override;

	virtual FDatasmithTextureSampler& GetParamTextureSampler(int32 InIndex) override;

	virtual bool GetUseColor(int32 InIndex) override;
	virtual const FLinearColor& GetParamColor(int32 InIndex) override;

	virtual bool GetUseComposite(int32 InIndex) override;

	virtual int32 GetParamVal1Count() const override { return ParamVal1.Num(); }
	virtual ParamVal GetParamVal1(int32 InIndex) const override;
	virtual void AddParamVal1(ParamVal InParamVal) override { ParamVal1.Add( ParamValImpl( InParamVal.Key, InParamVal.Value ) ); }

	virtual int32 GetParamVal2Count() const override { return ParamVal2.Num(); }
	virtual ParamVal GetParamVal2(int32 InIndex) const override;
	virtual void AddParamVal2(ParamVal InParamVal) override { ParamVal2.Add( ParamValImpl( InParamVal.Key, InParamVal.Value ) ); }

	virtual int32 GetParamMaskSurfacesCount() const override { return ParamMaskSurfaces.Num(); }
	virtual const TCHAR* GetParamMask(int32 InIndex) override;
	virtual const FLinearColor& GetParamMaskColor(int32 i) const override;
	virtual bool GetMaskUseComposite(int32 InIndex) const override;
	virtual void AddMaskSurface(const TCHAR* InMask, const FDatasmithTextureSampler InMaskSampler) override { ParamMaskSurfaces.Add( FDatasmithCompositeSurface( InMask, InMaskSampler )); }
	virtual void AddMaskSurface(const FLinearColor& InColor) override { ParamMaskSurfaces.Add( FDatasmithCompositeSurface( InColor ) ); }

	virtual FDatasmithTextureSampler GetParamMaskTextureSampler(int32 InIndex) override;

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetParamSubComposite(int32 InIndex) override;
	virtual void AddSurface(const TSharedPtr<IDatasmithCompositeTexture>& SubComp) override { ParamSurfaces.Add( FDatasmithCompositeSurface( SubComp ) ); }

	virtual TSharedPtr<IDatasmithCompositeTexture>& GetParamMaskSubComposite(int32 InIndex) override;
	virtual void AddMaskSurface(const TSharedPtr<IDatasmithCompositeTexture>& InMaskSubComp) override { ParamMaskSurfaces.Add( FDatasmithCompositeSurface( InMaskSubComp ) ); }

	virtual const TCHAR* GetBaseTextureName() const override { return *BaseTexName; }
	virtual const TCHAR* GetBaseColName() const override { return *BaseColName; }
	virtual const TCHAR* GetBaseValName() const override { return *BaseValName; }
	virtual const TCHAR* GetBaseCompName() const override { return *BaseCompName; }

	virtual void SetBaseNames(const TCHAR* InTextureName, const TCHAR* InColorName, const TCHAR* InValueName, const TCHAR* InCompName) override;

	virtual void AddSurface(const TCHAR* InTexture, FDatasmithTextureSampler InTexUV) override { ParamSurfaces.Add( FDatasmithCompositeSurface( InTexture, InTexUV )); }
	virtual void AddSurface(const FLinearColor& InColor) override { ParamSurfaces.Add( FDatasmithCompositeSurface( InColor )); }
	virtual void ClearSurface() override
	{
		ParamSurfaces.Empty();
	}

private:
	TArray<FDatasmithCompositeSurface> ParamSurfaces;
	TArray<FDatasmithCompositeSurface> ParamMaskSurfaces;

	typedef TPair<float, FString> ParamValImpl;
	TArray<ParamValImpl> ParamVal1;
	TArray<ParamValImpl> ParamVal2;

	EDatasmithCompMode CompMode;

	// used for single material
	FString BaseTexName;
	FString BaseColName;
	FString BaseValName;
	FString BaseCompName;
};

class FDatasmithMetaDataElementImpl : public FDatasmithElementImpl< IDatasmithMetaDataElement >
{
public:
	UE_API explicit FDatasmithMetaDataElementImpl(const TCHAR* InName);

	virtual const TSharedPtr< IDatasmithElement >& GetAssociatedElement() const override { return AssociatedElement.Inner; }
	virtual void SetAssociatedElement(const TSharedPtr< IDatasmithElement >& Element) { AssociatedElement.Inner = Element; }

	virtual int32 GetPropertiesCount() const override { return Properties.Num(); }

	UE_API virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 i) const override;

	UE_API virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName( const TCHAR* InName ) const override;

	UE_API virtual void AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& Property ) override;

	UE_API virtual void RemoveProperty( const TSharedPtr<IDatasmithKeyValueProperty>& Property ) override;

	UE_API virtual void ResetProperties() override;

private:
	TDatasmithReferenceProxy<IDatasmithElement> AssociatedElement;
	TDatasmithReferenceArrayProxy<IDatasmithKeyValueProperty> Properties;
};

class FDatasmithDecalActorElementImpl : public FDatasmithCustomActorElementImpl< IDatasmithDecalActorElement >
{
public:
	UE_API explicit FDatasmithDecalActorElementImpl( const TCHAR* InName );

	UE_API virtual FVector GetDimensions() const;
	UE_API virtual void SetDimensions( const FVector& InDimensions );

	UE_API virtual const TCHAR* GetDecalMaterialPathName() const;
	UE_API virtual void SetDecalMaterialPathName( const TCHAR* InMaterialPathName );

	UE_API virtual int32 GetSortOrder() const;
	UE_API virtual void SetSortOrder( int32 InSortOrder );

private:
	int32 SortOrderPropertyIndex;
	int32 DimensionsPropertyIndex;
	int32 MaterialPropertyIndex;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For Clothes - UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
class FDatasmithSceneImpl : public FDatasmithElementImpl< IDatasmithScene >
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	UE_API explicit FDatasmithSceneImpl(const TCHAR* InName);

	UE_API virtual void Reset() override;

	UE_API virtual const TCHAR* GetHost() const override;
	UE_API virtual void SetHost(const TCHAR* InHostname) override;

	virtual const TCHAR* GetExporterVersion() const override { return *(FString&)ExporterVersion; }
	virtual void SetExporterVersion(const TCHAR* InVersion) override { ExporterVersion = InVersion; }

	virtual const TCHAR* GetExporterSDKVersion() const override { return *(FString&)ExporterSDKVersion; }
	virtual void SetExporterSDKVersion(const TCHAR* InVersion) override { ExporterSDKVersion = InVersion; }

	virtual const TCHAR* GetVendor() const override	{ return *(FString&)Vendor; }
	virtual void SetVendor(const TCHAR* InVendor) override { Vendor = InVendor; }

	virtual const TCHAR* GetProductName() const override { return *(FString&)ProductName; }
	virtual void SetProductName(const TCHAR* InProductName) override { ProductName = InProductName; }

	virtual const TCHAR* GetProductVersion() const override	{ return *(FString&)ProductVersion; }
	virtual void SetProductVersion(const TCHAR* InProductVersion) override { ProductVersion = InProductVersion; }

	virtual const TCHAR* GetResourcePath() const override	{ return *(FString&)ResourcePath; }
	virtual void SetResourcePath(const TCHAR* InResourcePath) override { ResourcePath = InResourcePath; }

	virtual const TCHAR* GetUserID() const override { return *(FString&)UserID; }
	virtual void SetUserID(const TCHAR* InUserID) override { UserID = InUserID; }

	virtual const TCHAR* GetUserOS() const override { return *(FString&)UserOS; }
	virtual void SetUserOS(const TCHAR* InUserOS) override { UserOS = InUserOS; }

	virtual FVector GetGeolocation() const override { return Geolocation; }
	virtual void SetGeolocationLatitude(double InLatitude) override {  Geolocation.Get().X = InLatitude;}
	virtual void SetGeolocationLongitude(double InLongitude) override {  Geolocation.Get().Y = InLongitude; }
	virtual void SetGeolocationElevation(double InElevation) override {  Geolocation.Get().Z = InElevation; }

	virtual int32 GetExportDuration() const override { return ExportDuration; }
	virtual void SetExportDuration(int32 InExportDuration) override { ExportDuration = InExportDuration; }

	virtual void AddMesh(const TSharedPtr< IDatasmithMeshElement >& InMesh) override { Meshes.Add(InMesh); }
	virtual int32 GetMeshesCount() const override { return Meshes.Num(); }
	UE_API virtual TSharedPtr< IDatasmithMeshElement > GetMesh(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithMeshElement >& GetMesh(int32 InIndex) const override;
	virtual void RemoveMesh(const TSharedPtr< IDatasmithMeshElement >& InMesh) override { Meshes.Remove(InMesh); }
	UE_API virtual void RemoveMeshAt(int32 InIndex) override;
	virtual void EmptyMeshes() override { Meshes.Empty(); }

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual void AddCloth(const TSharedPtr< IDatasmithClothElement >& InElement) override;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual int32 GetClothesCount() const override;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual TSharedPtr< IDatasmithClothElement > GetCloth(int32 InIndex) override;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual const TSharedPtr< IDatasmithClothElement >& GetCloth(int32 InIndex) const override;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual void RemoveCloth(const TSharedPtr< IDatasmithClothElement >& InElement) override;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual void RemoveClothAt(int32 InIndex) override;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UE_API virtual void EmptyClothes() override;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void AddActor(const TSharedPtr< IDatasmithActorElement >& InActor) override { Actors.Add(InActor);  }
	virtual int32 GetActorsCount() const override { return Actors.Num(); }
	UE_API virtual TSharedPtr< IDatasmithActorElement > GetActor(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithActorElement >& GetActor(int32 InIndex) const override;
	UE_API virtual void RemoveActor(const TSharedPtr< IDatasmithActorElement >& InActor, EDatasmithActorRemovalRule RemoveRule) override;
	UE_API virtual void RemoveActorAt(int32 InIndex, EDatasmithActorRemovalRule RemoveRule) override;

	virtual void AddMaterial(const TSharedPtr< IDatasmithBaseMaterialElement >& InMaterial) override { Materials.Add(InMaterial); }
	virtual int32 GetMaterialsCount() const override { return Materials.Num(); }
	UE_API virtual TSharedPtr< IDatasmithBaseMaterialElement > GetMaterial(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithBaseMaterialElement >& GetMaterial(int32 InIndex) const override;
	virtual void RemoveMaterial(const TSharedPtr< IDatasmithBaseMaterialElement >& InMaterial) override { Materials.Remove(InMaterial); }
	UE_API virtual void RemoveMaterialAt(int32 InIndex) override;
	virtual void EmptyMaterials() override { Materials.Empty(); }

	virtual void AddTexture(const TSharedPtr< IDatasmithTextureElement >& InTexture) override { Textures.Add(InTexture); }
	virtual int32 GetTexturesCount() const override { return Textures.Num(); }
	UE_API virtual TSharedPtr< IDatasmithTextureElement > GetTexture(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithTextureElement >& GetTexture(int32 InIndex) const override;
	virtual void RemoveTexture(const TSharedPtr< IDatasmithTextureElement >& InTexture) override { Textures.Remove(InTexture); }
	UE_API virtual void RemoveTextureAt(int32 InIndex) override;
	virtual void EmptyTextures() override { Textures.Empty(); }

	virtual void SetPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& InPostProcess) override { PostProcess.Inner = InPostProcess; }
	virtual TSharedPtr< IDatasmithPostProcessElement > GetPostProcess() override { return PostProcess.Inner; }
	virtual const TSharedPtr< IDatasmithPostProcessElement >& GetPostProcess() const override { return PostProcess.Inner; }

	virtual void SetUsePhysicalSky(bool bInUsePhysicalSky) override { bUseSky = bInUsePhysicalSky; }
	virtual bool GetUsePhysicalSky() const override { return bUseSky; }

	virtual void AddMetaData(const TSharedPtr< IDatasmithMetaDataElement >& InMetaData) override { MetaData.Add(InMetaData); GetElementToMetaDataCache().Add(InMetaData->GetAssociatedElement(), InMetaData); }
	virtual int32 GetMetaDataCount() const override { return MetaData.Num(); }
	UE_API virtual TSharedPtr< IDatasmithMetaDataElement > GetMetaData(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithMetaDataElement >& GetMetaData(int32 InIndex) const override;
	UE_API virtual TSharedPtr< IDatasmithMetaDataElement > GetMetaData(const TSharedPtr<IDatasmithElement>& Element) override;
	UE_API virtual const TSharedPtr< IDatasmithMetaDataElement >& GetMetaData(const TSharedPtr<IDatasmithElement>& Element) const override;
	UE_API virtual void RemoveMetaData( const TSharedPtr<IDatasmithMetaDataElement>& Element ) override;
	UE_API virtual void RemoveMetaDataAt(int32 InIndex) override;

	virtual void AddLevelSequence(const TSharedRef< IDatasmithLevelSequenceElement >& InSequence) override { LevelSequences.Add(InSequence);  }
	virtual int32 GetLevelSequencesCount() const override { return LevelSequences.Num(); }
	UE_API virtual TSharedPtr< IDatasmithLevelSequenceElement > GetLevelSequence(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithLevelSequenceElement >& GetLevelSequence(int32 InIndex) const override;
	virtual void RemoveLevelSequence(const TSharedRef< IDatasmithLevelSequenceElement >& InSequence) override { LevelSequences.Remove(InSequence); }
	UE_API virtual void RemoveLevelSequenceAt(int32 InIndex) override;

	virtual void AddLevelVariantSets(const TSharedPtr< IDatasmithLevelVariantSetsElement >& InLevelVariantSets) override { LevelVariantSets.Add(InLevelVariantSets);  }
	virtual int32 GetLevelVariantSetsCount() const override { return LevelVariantSets.Num(); }
	UE_API virtual TSharedPtr< IDatasmithLevelVariantSetsElement > GetLevelVariantSets(int32 InIndex) override;
	UE_API virtual const TSharedPtr< IDatasmithLevelVariantSetsElement >& GetLevelVariantSets(int32 InIndex) const override;
	virtual void RemoveLevelVariantSets(const TSharedPtr< IDatasmithLevelVariantSetsElement >& InLevelVariantSets) override { LevelVariantSets.Remove(InLevelVariantSets); }
	UE_API virtual void RemoveLevelVariantSetsAt(int32 InIndex) override;

	UE_API virtual void AttachActor(const TSharedPtr< IDatasmithActorElement >& NewParent, const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule) override;
	UE_API virtual void AttachActorToSceneRoot(const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule) override;

private:

	TDatasmithReferenceArrayProxy<IDatasmithActorElement>            Actors;
	TDatasmithReferenceArrayProxy<IDatasmithMeshElement>             Meshes;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	TDatasmithReferenceArrayProxy<IDatasmithClothElement>            Clothes;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	TDatasmithReferenceArrayProxy<IDatasmithBaseMaterialElement>     Materials;
	TDatasmithReferenceArrayProxy<IDatasmithTextureElement>          Textures;
	TDatasmithReferenceArrayProxy<IDatasmithMetaDataElement>         MetaData;
	TDatasmithReferenceArrayProxy<IDatasmithLevelSequenceElement>    LevelSequences;
	TDatasmithReferenceArrayProxy<IDatasmithLevelVariantSetsElement> LevelVariantSets;
	TDatasmithReferenceProxy<IDatasmithPostProcessElement>           PostProcess;

	TReflected<FString> Hostname;
	TReflected<FString> ExporterVersion;
	TReflected<FString> ExporterSDKVersion;
	TReflected<FString> Vendor;
	TReflected<FString> ProductName;
	TReflected<FString> ProductVersion;
	TReflected<FString> UserID;
	TReflected<FString> UserOS;
	TReflected<FVector> Geolocation;
	TReflected<FString> ResourcePath;

	TReflected<uint32> ExportDuration;

	TReflected<bool> bUseSky;

	// Internal cache for faster metadata access per-element, should be accessed via GetMetaDataCache(), do not use directly.
	mutable TMap< TSharedPtr< IDatasmithElement >, TSharedPtr< IDatasmithMetaDataElement> > ElementToMetaDataMap;
	UE_API TMap< TSharedPtr< IDatasmithElement >, TSharedPtr< IDatasmithMetaDataElement> >& GetElementToMetaDataCache() const;
};

#undef UE_API
