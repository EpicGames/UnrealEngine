// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithSceneFactory.h"

#include "DatasmithAnimationElementsImpl.h"
#include "DatasmithMaterialElementsImpl.h"
#include "DatasmithSceneElementsImpl.h"
#include "DatasmithVariantElementsImpl.h"

// enable a warning if not all cases values are covered by a switch statement
#if defined(_MSC_VER)
#pragma warning(push)
#pragma warning (default: 4062)
#endif

TSharedPtr< IDatasmithElement > FDatasmithSceneFactory::CreateElement( EDatasmithElementType InType, const TCHAR* InName )
{
	constexpr uint64 DefaultSubType = 0;
	return CreateElement( InType, DefaultSubType, InName );
}

TSharedPtr< IDatasmithElement > FDatasmithSceneFactory::CreateElement( EDatasmithElementType InType, uint64 InSubType, const TCHAR* InName )
{
	switch ( InType )
	{
	// Abstract types
	case EDatasmithElementType::None:
	case EDatasmithElementType::Light:
	case EDatasmithElementType::BaseMaterial:
		ensure( false );
		break;
	case EDatasmithElementType::Actor:
		return CreateActor( InName );
	case EDatasmithElementType::Animation:
		switch ( static_cast< EDatasmithElementAnimationSubType >( InSubType ) )
		{
		case EDatasmithElementAnimationSubType::SubsequenceAnimation:
			return CreateSubsequenceAnimation( InName );
		case EDatasmithElementAnimationSubType::TransformAnimation:
			return CreateTransformAnimation( InName );
		case EDatasmithElementAnimationSubType::VisibilityAnimation:
			return CreateVisibilityAnimation( InName );
		case EDatasmithElementAnimationSubType::BaseAnimation:
			ensure( false );
			break;
		}
	case EDatasmithElementType::StaticMesh:
		return CreateMesh( InName );
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	case EDatasmithElementType::Cloth:  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
		return CreateCloth( InName );
	case EDatasmithElementType::ClothActor:  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
		return CreateClothActor( InName );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	case EDatasmithElementType::StaticMeshActor:
		return CreateMeshActor( InName );
	case EDatasmithElementType::PointLight:
		return CreatePointLight( InName );
	case EDatasmithElementType::SpotLight:
		return CreateSpotLight( InName );
	case EDatasmithElementType::DirectionalLight:
		return CreateDirectionalLight( InName );
	case EDatasmithElementType::AreaLight:
		return CreateAreaLight( InName );
	case EDatasmithElementType::LightmassPortal:
		return CreateLightmassPortal( InName );
	case EDatasmithElementType::EnvironmentLight:
		return CreateEnvironment( InName );
	case EDatasmithElementType::Camera:
		return CreateCameraActor( InName );
	case EDatasmithElementType::Shader:
		return CreateShader( InName );
	case EDatasmithElementType::Material:
		return CreateMaterial( InName );
	case EDatasmithElementType::MaterialInstance:
		return CreateMaterialInstance( InName );
	case EDatasmithElementType::UEPbrMaterial:
		return CreateUEPbrMaterial( InName );
	case EDatasmithElementType::MaterialExpression:
		return CreateMaterialExpression( static_cast< EDatasmithMaterialExpressionType >( InSubType ) );
	case EDatasmithElementType::MaterialExpressionInput:
		return CreateExpressionInput( InName );
	case EDatasmithElementType::MaterialExpressionOutput:
		return CreateExpressionOutput( InName );
	case EDatasmithElementType::KeyValueProperty:
		return CreateKeyValueProperty( InName );
	case EDatasmithElementType::Texture:
		return CreateTexture( InName );
	case EDatasmithElementType::MaterialId:
		return CreateMaterialId( InName );
	case EDatasmithElementType::PostProcess:
		return CreatePostProcess();
	case EDatasmithElementType::PostProcessVolume:
		return CreatePostProcessVolume( InName );
	case EDatasmithElementType::Scene:
		return CreateScene( InName );
	case EDatasmithElementType::MetaData:
		return CreateMetaData( InName );
	case EDatasmithElementType::CustomActor:
		return CreateCustomActor( InName );
	case EDatasmithElementType::HierarchicalInstanceStaticMesh:
		return CreateHierarchicalInstanceStaticMeshActor( InName );
	case EDatasmithElementType::Decal:
		return CreateDecalActor( InName );
	case EDatasmithElementType::DecalMaterial:
		return CreateDecalMaterial( InName );
	case EDatasmithElementType::LevelSequence:
		return FDatasmithSceneFactory::CreateLevelSequence( InName );
	case EDatasmithElementType::Landscape:
		return FDatasmithSceneFactory::CreateLandscape( InName );
	case EDatasmithElementType::Variant:
		switch( static_cast< EDatasmithElementVariantSubType > ( InSubType ) )
		{
		case EDatasmithElementVariantSubType::ActorBinding:
			return CreateActorBinding();
		case EDatasmithElementVariantSubType::LevelVariantSets:
			return CreateLevelVariantSets( InName );
		case EDatasmithElementVariantSubType::ObjectPropertyCapture:
			return CreateObjectPropertyCapture();
		case EDatasmithElementVariantSubType::PropertyCapture:
			return CreatePropertyCapture();
		case EDatasmithElementVariantSubType::Variant:
			return CreateVariant( InName );
		case EDatasmithElementVariantSubType::VariantSet:
			return CreateVariantSet( InName );
		case EDatasmithElementVariantSubType::None:
			ensure( false );
			break;
		}
	}

	return TSharedPtr< IDatasmithElement >();
}

TSharedRef< IDatasmithActorElement > FDatasmithSceneFactory::CreateActor( const TCHAR* InName )
{
	return MakeShared< FDatasmithActorElementImpl< IDatasmithActorElement > >( InName, EDatasmithElementType::None );
}

TSharedRef< IDatasmithCameraActorElement > FDatasmithSceneFactory::CreateCameraActor( const TCHAR* InName )
{
	return MakeShared< FDatasmithCameraActorElementImpl >( InName );
}

TSharedRef< IDatasmithCompositeTexture > FDatasmithSceneFactory::CreateCompositeTexture()
{
	return MakeShared< FDatasmithCompositeTextureImpl >();
}

TSharedRef< IDatasmithCustomActorElement > FDatasmithSceneFactory::CreateCustomActor( const TCHAR* InName )
{
	return MakeShared< FDatasmithCustomActorElementImpl<IDatasmithCustomActorElement> >( InName );
}

TSharedRef< IDatasmithLandscapeElement > FDatasmithSceneFactory::CreateLandscape( const TCHAR* InName )
{
	return MakeShared< FDatasmithLandscapeElementImpl >( InName );
}

TSharedRef< IDatasmithPostProcessVolumeElement > FDatasmithSceneFactory::CreatePostProcessVolume( const TCHAR* InName )
{
	return MakeShared< FDatasmithPostProcessVolumeElementImpl >( InName );
}

TSharedRef< IDatasmithEnvironmentElement > FDatasmithSceneFactory::CreateEnvironment( const TCHAR* InName )
{
	return MakeShared< FDatasmithEnvironmentElementImpl >( InName );
}

TSharedRef< IDatasmithPointLightElement > FDatasmithSceneFactory::CreatePointLight( const TCHAR* InName )
{
	return MakeShared< FDatasmithPointLightElementImpl<> >( InName );
}

TSharedRef< IDatasmithSpotLightElement > FDatasmithSceneFactory::CreateSpotLight( const TCHAR* InName )
{
	return MakeShared< FDatasmithSpotLightElementImpl<> >( InName );
}

TSharedRef< IDatasmithDirectionalLightElement > FDatasmithSceneFactory::CreateDirectionalLight( const TCHAR* InName )
{
	return MakeShared< FDatasmithDirectionalLightElementImpl >( InName );
}

TSharedRef< IDatasmithAreaLightElement > FDatasmithSceneFactory::CreateAreaLight( const TCHAR* InName )
{
	return MakeShared< FDatasmithAreaLightElementImpl >( InName );
}

TSharedRef< IDatasmithLightmassPortalElement > FDatasmithSceneFactory::CreateLightmassPortal( const TCHAR* InName )
{
	return MakeShared< FDatasmithLightmassPortalElementImpl >( InName );
}

TSharedRef< IDatasmithKeyValueProperty > FDatasmithSceneFactory::CreateKeyValueProperty( const TCHAR* InName )
{
	return MakeShared< FDatasmithKeyValuePropertyImpl >( InName );
}

TSharedRef< IDatasmithMeshElement > FDatasmithSceneFactory::CreateMesh( const TCHAR* InName )
{
	return MakeShared< FDatasmithMeshElementImpl >( InName );
}

TSharedRef< IDatasmithMeshActorElement > FDatasmithSceneFactory::CreateMeshActor( const TCHAR* InName )
{
	return MakeShared< FDatasmithMeshActorElementImpl<> >( InName );
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedRef< IDatasmithClothElement > FDatasmithSceneFactory::CreateCloth(const TCHAR* InName)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	return MakeShared< FDatasmithClothElementImpl >( InName );
}

TSharedRef< IDatasmithClothActorElement > FDatasmithSceneFactory::CreateClothActor( const TCHAR* InName )  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	return MakeShared< FDatasmithClothActorElementImpl >( InName );
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement > FDatasmithSceneFactory::CreateHierarchicalInstanceStaticMeshActor(const TCHAR* InName)
{
	return MakeShared< FDatasmithHierarchicalInstancedStaticMeshActorElementImpl >( InName );
}

TSharedRef< IDatasmithMaterialElement > FDatasmithSceneFactory::CreateMaterial( const TCHAR* InName )
{
	return MakeShared< FDatasmithMaterialElementImpl >( InName );
}

TSharedRef< IDatasmithMaterialInstanceElement > FDatasmithSceneFactory::CreateMaterialInstance( const TCHAR* InName )
{
	return MakeShared< FDatasmithMaterialIntanceElementImpl >( InName );
}

TSharedRef< IDatasmithUEPbrMaterialElement > FDatasmithSceneFactory::CreateUEPbrMaterial( const TCHAR* InName )
{
	return MakeShared< FDatasmithUEPbrMaterialElementImpl >( InName );
}

TSharedPtr< IDatasmithMaterialExpression > FDatasmithSceneFactory::CreateMaterialExpression( EDatasmithMaterialExpressionType MaterialExpression )
{
	TSharedPtr<IDatasmithMaterialExpression> Expression;

	switch ( MaterialExpression )
	{
	case EDatasmithMaterialExpressionType::ConstantBool:
		Expression = MakeShared<FDatasmithMaterialExpressionBoolImpl>();
		break;
	case EDatasmithMaterialExpressionType::ConstantColor:
		Expression = MakeShared<FDatasmithMaterialExpressionColorImpl>();
		break;
	case EDatasmithMaterialExpressionType::ConstantScalar:
		Expression = MakeShared<FDatasmithMaterialExpressionScalarImpl>();
		break;
	case EDatasmithMaterialExpressionType::FlattenNormal:
		Expression = MakeShared<FDatasmithMaterialExpressionFlattenNormalImpl>();
		break;
	case EDatasmithMaterialExpressionType::FunctionCall:
		Expression = MakeShared<FDatasmithMaterialExpressionFunctionCallImpl>();
		break;
	case EDatasmithMaterialExpressionType::Generic:
		Expression = MakeShared<FDatasmithMaterialExpressionGenericImpl>();
		break;
	case EDatasmithMaterialExpressionType::Texture:
		Expression = MakeShared<FDatasmithMaterialExpressionTextureImpl>();
		break;
	case EDatasmithMaterialExpressionType::TextureCoordinate:
		Expression = MakeShared<FDatasmithMaterialExpressionTextureCoordinateImpl>();
		break;
	case EDatasmithMaterialExpressionType::Custom:
		Expression = MakeShared<FDatasmithMaterialExpressionCustomImpl>();
		break;
	case EDatasmithMaterialExpressionType::None:
		check( false );
		break;
	}

	return Expression;
}

TSharedRef< IDatasmithExpressionInput > FDatasmithSceneFactory::CreateExpressionInput( const TCHAR* InName )
{
	return MakeShared< FDatasmithExpressionInputImpl >( InName );
}

TSharedRef< IDatasmithExpressionOutput > FDatasmithSceneFactory::CreateExpressionOutput( const TCHAR* InName )
{
	return MakeShared< FDatasmithExpressionOutputImpl >( InName );
}

TSharedRef< IDatasmithMetaDataElement > FDatasmithSceneFactory::CreateMetaData( const TCHAR* InName )
{
	return MakeShared< FDatasmithMetaDataElementImpl >( InName );
}

TSharedRef< IDatasmithMaterialIDElement > FDatasmithSceneFactory::CreateMaterialId( const TCHAR* InName )
{
	return MakeShared< FDatasmithMaterialIDElementImpl >( InName );
}

TSharedRef< IDatasmithPostProcessElement > FDatasmithSceneFactory::CreatePostProcess()
{
	return MakeShared< FDatasmithPostProcessElementImpl >();
}

TSharedRef< IDatasmithShaderElement > FDatasmithSceneFactory::CreateShader( const TCHAR* InName )
{
	return MakeShared< FDatasmithShaderElementImpl >( InName );
}

TSharedRef< IDatasmithTextureElement > FDatasmithSceneFactory::CreateTexture( const TCHAR* InName )
{
	return MakeShared< FDatasmithTextureElementImpl >( InName );
}

TSharedRef< IDatasmithLevelSequenceElement > FDatasmithSceneFactory::CreateLevelSequence( const TCHAR* InName )
{
	return MakeShared< FDatasmithLevelSequenceElementImpl >( InName );
}

TSharedRef< IDatasmithTransformAnimationElement > FDatasmithSceneFactory::CreateTransformAnimation( const TCHAR* InName )
{
	return MakeShared< FDatasmithTransformAnimationElementImpl >( InName );
}

TSharedRef< IDatasmithVisibilityAnimationElement > FDatasmithSceneFactory::CreateVisibilityAnimation( const TCHAR* InName )
{
	return MakeShared< FDatasmithVisibilityAnimationElementImpl >( InName );
}

TSharedRef< IDatasmithSubsequenceAnimationElement > FDatasmithSceneFactory::CreateSubsequenceAnimation( const TCHAR* InName )
{
	return MakeShared< FDatasmithSubsequenceAnimationElementImpl >( InName );
}

TSharedRef< IDatasmithLevelVariantSetsElement > FDatasmithSceneFactory::CreateLevelVariantSets(const TCHAR* InName)
{
	return MakeShared< FDatasmithLevelVariantSetsElementImpl >( InName );
}

TSharedRef< IDatasmithVariantSetElement > FDatasmithSceneFactory::CreateVariantSet(const TCHAR* InName)
{
	return MakeShared< FDatasmithVariantSetElementImpl >( InName );
}

TSharedRef< IDatasmithVariantElement > FDatasmithSceneFactory::CreateVariant(const TCHAR* InName)
{
	return MakeShared< FDatasmithVariantElementImpl >( InName );
}

TSharedRef< IDatasmithActorBindingElement > FDatasmithSceneFactory::CreateActorBinding()
{
	return MakeShared< FDatasmithActorBindingElementImpl >();
}

TSharedRef< IDatasmithPropertyCaptureElement > FDatasmithSceneFactory::CreatePropertyCapture()
{
	return MakeShared< FDatasmithPropertyCaptureElementImpl >();
}

TSharedRef< IDatasmithObjectPropertyCaptureElement > FDatasmithSceneFactory::CreateObjectPropertyCapture()
{
	return MakeShared< FDatasmithObjectPropertyCaptureElementImpl >();
}

TSharedRef<IDatasmithDecalActorElement> FDatasmithSceneFactory::CreateDecalActor(const TCHAR* InName)
{
	return MakeShared< FDatasmithDecalActorElementImpl >( InName );
}

TSharedRef<IDatasmithDecalMaterialElement> FDatasmithSceneFactory::CreateDecalMaterial(const TCHAR* InName)
{
	return MakeShared< FDatasmithDecalMaterialElementImpl >( InName );
}

TSharedRef< IDatasmithScene > FDatasmithSceneFactory::CreateScene( const TCHAR* InName )
{
	return MakeShared< FDatasmithSceneImpl >( InName );
}

TSharedRef< IDatasmithScene > FDatasmithSceneFactory::DuplicateScene( const TSharedRef< IDatasmithScene >& InScene )
{
	return MakeShared< FDatasmithSceneImpl >( StaticCastSharedRef< FDatasmithSceneImpl >( InScene ).Get() );
}

#if defined(_MSC_VER)
#pragma warning(pop)
#endif
