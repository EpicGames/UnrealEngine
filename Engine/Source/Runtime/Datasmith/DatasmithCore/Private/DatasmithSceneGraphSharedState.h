// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DirectLinkCommon.h"
#include "DirectLinkLog.h"
#include "DirectLinkSceneGraphNode.h"

#include "Algo/Transform.h"

class FDatasmithSceneGraphSharedState : public DirectLink::FSceneGraphSharedState
{
public:
	FDatasmithSceneGraphSharedState() = default;

	FDatasmithSceneGraphSharedState(const DirectLink::FSceneIdentifier& InSceneId)
	{
		SceneId = InSceneId;
	}
};


template<typename T>
class TDatasmithReferenceProxy : public DirectLink::IReferenceProxy
{
	// this kind of proxy could use type flags to allow/prevent null references, etc
public:
	TDatasmithReferenceProxy(const TSharedPtr<T>& Reference = nullptr)
		: Inner(Reference)
	{}

	virtual int32 Num() const override { return 1; }

	virtual DirectLink::ISceneGraphNode* GetNode(int32 Index) const override
	{
		return Inner.Get();
	}

	virtual void SetNodes(DirectLink::IReferenceResolutionProvider& ResolutionProvider, const TArray<DirectLink::FSceneGraphId>& NodeIds) override
	{
		Inner = nullptr;
		if (NodeIds.Num() >= 1)
		{
			if (NodeIds.Num() > 1)
			{
				UE_LOGF(LogDirectLink, Error, "Bad argument type: several references provided in a simple ref");
				return;
			}

			TSharedPtr<DirectLink::ISceneGraphNode> Node = ResolutionProvider.AsSharedPtr(NodeIds[0]);
			check(Node.IsValid());
			Inner = StaticCastSharedPtr<T>(Node);
		}
	}

	const TSharedPtr<T>& View() const { return Inner; }
	      TSharedPtr<T>& Edit()       { return Inner; }

public:
	TSharedPtr<T> Inner;
};


template<typename T>
class TDatasmithReferenceArrayProxy : public DirectLink::IReferenceProxy
{
	using SizeType = typename TArray<TSharedPtr<T>>::SizeType;

public:
	virtual int32 Num() const override
	{
		return Inner.Num();
	}

	virtual DirectLink::ISceneGraphNode* GetNode(int32 Index) const override
	{
		return Inner[Index].Get();
	}

	virtual void SetNodes(DirectLink::IReferenceResolutionProvider& ResolutionProvider, const TArray<DirectLink::FSceneGraphId>& NodeIds) override
	{
		Inner.Reset(NodeIds.Num());
		Algo::Transform(NodeIds, Inner, [&](DirectLink::FSceneGraphId NodeId)
		{
			// #ue_directlink_quality validate type through property "type" ?
			TSharedPtr<DirectLink::ISceneGraphNode> Node = ResolutionProvider.AsSharedPtr(NodeId);
			check(Node.IsValid());
			return StaticCastSharedPtr<T>(Node);
		});
	}


	// simpler access to the inner array...
	const TSharedPtr<T>& operator[](int32 Index) const { return Inner[Index]; }
	TSharedPtr<T>& operator[](int32 Index) { return Inner[Index]; }
	bool IsValidIndex(SizeType Index) const { return Inner.IsValidIndex(Index); }
	SizeType Add(const TSharedPtr<T>& Element) { return Inner.Add(Element); }
	SizeType Add(TSharedPtr<T>&& Element) { return Inner.Add(MoveTemp(Element)); }
	SizeType Remove(const TSharedPtr<T>& Item) { return Inner.Remove(Item); }
	void RemoveAt(int32 Index) { Inner.RemoveAt(Index); }
	void Empty() { return Inner.Empty(); }

	const TArray<TSharedPtr<T>>& View() const { return Inner; }
	      TArray<TSharedPtr<T>>& Edit()       { return Inner; }

private:
	TArray<TSharedPtr<T>> Inner;
};