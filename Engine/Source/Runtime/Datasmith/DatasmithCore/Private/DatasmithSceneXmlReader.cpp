// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneXmlReader.h"

#include "DatasmithCore.h"
#include "DatasmithDefinitions.h"
#include "DatasmithLocaleScope.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"

#include "Algo/Find.h"
#include "Containers/ArrayView.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "XmlParser.h"


FDatasmithSceneXmlReader::FDatasmithSceneXmlReader() = default;
FDatasmithSceneXmlReader::~FDatasmithSceneXmlReader() = default;


const TCHAR* ActorTagsView[] = {
	DATASMITH_ACTORNAME,
	DATASMITH_ACTORHIERARCHICALINSTANCEDMESHNAME,
	DATASMITH_ACTORMESHNAME,
	DATASMITH_CAMERANAME,
	DATASMITH_CLOTHACTORNAME,  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	DATASMITH_CUSTOMACTORNAME,
	DATASMITH_DECALACTORNAME,
	DATASMITH_LANDSCAPENAME,
	DATASMITH_LIGHTNAME,
	DATASMITH_POSTPROCESSVOLUME,
};

template<>
float FDatasmithSceneXmlReader::ValueFromString< float >( const FString& InString ) const
{
	return FCString::Atof( *InString );
}

template<>
double FDatasmithSceneXmlReader::ValueFromString< double >( const FString& InString ) const
{
	return FCString::Atod( *InString );
}

template<>
FVector FDatasmithSceneXmlReader::ValueFromString< FVector >( const FString& InString ) const
{
	FVector Value;
	Value.InitFromString( InString );

	return Value;
}

template<>
FColor FDatasmithSceneXmlReader::ValueFromString< FColor >( const FString& InString ) const
{
	FColor Value;
	Value.InitFromString( InString );

	return Value;
}

template<>
FLinearColor FDatasmithSceneXmlReader::ValueFromString< FLinearColor >( const FString& InString ) const
{
	FLinearColor Value;
	Value.InitFromString( InString );

	return Value;
}

template<>
int32 FDatasmithSceneXmlReader::ValueFromString< int32 >( const FString& InString ) const
{
	return FCString::Atoi( *InString );
}

template<>
bool FDatasmithSceneXmlReader::ValueFromString< bool >( const FString& InString ) const
{
	return InString.ToBool();
}


void FDatasmithSceneXmlReader::PatchUpVersion(TSharedRef< IDatasmithScene >& OutScene) const
{
	//@todo parse version string in proper version object
	// Handle legacy behavior, when materials from the first actor using a mesh applied its materials to it.
	// Materials on meshes appeared in 0.19
	if (ValueFromString<float>(OutScene->GetExporterVersion()) < 0.19f)
	{
		TArray<TSharedPtr< IDatasmithMeshActorElement>> MeshActors = FDatasmithSceneUtils::GetAllMeshActorsFromScene(OutScene);
		TMap<FString, TSharedPtr< IDatasmithMeshActorElement>> ActorUsingMeshMap;
		for (const TSharedPtr< IDatasmithMeshActorElement >& MeshActor : MeshActors)
		{
			if (MeshActor.IsValid() && !ActorUsingMeshMap.Contains(MeshActor->GetStaticMeshPathName()))
			{
				ActorUsingMeshMap.Add(MeshActor->GetStaticMeshPathName(), MeshActor);
			}
		}

		for (int32 MeshIndex = 0; MeshIndex < OutScene->GetMeshesCount(); ++MeshIndex)
		{
			if (OutScene->GetMesh(MeshIndex)->GetMaterialSlotCount() == 0 && ActorUsingMeshMap.Contains(OutScene->GetMesh(MeshIndex)->GetName()))
			{
				TSharedPtr< IDatasmithMeshActorElement> MeshActor = ActorUsingMeshMap[OutScene->GetMesh(MeshIndex)->GetName()];
				for (int i = 0; i < MeshActor->GetMaterialOverridesCount(); ++i)
				{
					OutScene->GetMesh(MeshIndex)->SetMaterial(MeshActor->GetMaterialOverride(i)->GetName(), MeshActor->GetMaterialOverride(i)->GetId() == -1 ? 0 : MeshActor->GetMaterialOverride(i)->GetId());
				}
			}
		}
	}
}

[[nodiscard]] FString FDatasmithSceneXmlReader::UnsanitizeXMLText(const FString& InString) const
{
	FString OutString = InString;
	OutString.ReplaceInline( TEXT("&apos;"), TEXT("'")  );
	OutString.ReplaceInline( TEXT("&quot;"), TEXT("\"") );
	OutString.ReplaceInline( TEXT("&gt;"),   TEXT(">")  );
	OutString.ReplaceInline( TEXT("&lt;"),   TEXT("<")  );
	OutString.ReplaceInline( TEXT("&amp;"),  TEXT("&")  );
	return OutString;
}

FVector FDatasmithSceneXmlReader::VectorFromNode(FXmlNode* InNode, const TCHAR* XName, const TCHAR* YName, const TCHAR* ZName) const
{
	return {
		ValueFromString<double>(InNode->GetAttribute(XName)),
		ValueFromString<double>(InNode->GetAttribute(YName)),
		ValueFromString<double>(InNode->GetAttribute(ZName))
	};
}

FQuat FDatasmithSceneXmlReader::QuatFromHexString(const FString& HexString) const
{
	if (HexString.Len() / 2 >= 4 * sizeof(double))
	{
		double Double[4];
		FString::ToHexBlob( HexString, (uint8*)Double, sizeof(Double) );
		return FQuat( Double[0], Double[1], Double[2], Double[3] );
	}
	else if (HexString.Len() / 2 >= 4 * sizeof(float))
	{
		float Floats[4];
		FString::ToHexBlob( HexString, (uint8*)Floats, sizeof(Floats) );
		return FQuat( Floats[0], Floats[1], Floats[2], Floats[3] );
	}

	return FQuat::Identity;
}

FQuat FDatasmithSceneXmlReader::QuatFromNode(FXmlNode* InNode) const
{
	FString RotationBlob = InNode->GetAttribute(TEXT("qhex64"));
	if (RotationBlob.IsEmpty())
	{
		RotationBlob = InNode->GetAttribute(TEXT("qhex"));
	}

	if (!RotationBlob.IsEmpty())
	{
		return QuatFromHexString(RotationBlob);
	}

	return FQuat(
		ValueFromString<double>(InNode->GetAttribute(TEXT("qx"))),
		ValueFromString<double>(InNode->GetAttribute(TEXT("qy"))),
		ValueFromString<double>(InNode->GetAttribute(TEXT("qz"))),
		ValueFromString<double>(InNode->GetAttribute(TEXT("qw")))
	);
}

FTransform FDatasmithSceneXmlReader::ParseTransform(FXmlNode* InNode) const
{
	return {
		QuatFromNode(InNode),
		VectorFromNode(InNode, TEXT("tx"), TEXT("ty"), TEXT("tz")),
		VectorFromNode(InNode, TEXT("sx"), TEXT("sy"), TEXT("sz"))
	};
}

void FDatasmithSceneXmlReader::ParseTransform(FXmlNode* InNode, TSharedPtr< IDatasmithActorElement >& OutElement) const
{
	FTransform Tmp = ParseTransform(InNode);
	OutElement->SetTranslation(Tmp.GetTranslation());
	OutElement->SetRotation(Tmp.GetRotation());
	OutElement->SetScale(Tmp.GetScale3D());
}

void FDatasmithSceneXmlReader::ParseElement(FXmlNode* InNode, TSharedRef<IDatasmithElement> OutElement) const
{
	OutElement->SetLabel( *InNode->GetAttribute( TEXT("label") ) );
}

void FDatasmithSceneXmlReader::ParseLevelSequence(FXmlNode* InNode, const TSharedRef<IDatasmithLevelSequenceElement>& OutElement) const
{
	const TArray<FXmlNode*>& ChildrenNodes = InNode->GetChildrenNodes();
	for (int j = 0; j < ChildrenNodes.Num(); ++j)
	{
		if (ChildrenNodes[j]->GetTag() == TEXT("file"))
		{
			OutElement->SetFile( *ChildrenNodes[j]->GetAttribute(TEXT("path")) );
		}
		else if (ChildrenNodes[j]->GetTag() == DATASMITH_HASH)
		{
			FMD5Hash Hash;
			LexFromString(Hash, *ChildrenNodes[j]->GetAttribute(TEXT("value")));
			OutElement->SetFileHash(Hash);
		}
	}
}

void FDatasmithSceneXmlReader::ParseLevelVariantSets( FXmlNode* InNode, const TSharedRef<IDatasmithLevelVariantSetsElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const
{
	for ( FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( !ChildNode )
		{
			continue;
		}

		if ( ChildNode->GetTag() == DATASMITH_VARIANTSETNAME )
		{
			FString ElementName = ChildNode->GetAttribute( TEXT( "name" ) );
			TSharedRef< IDatasmithVariantSetElement > VarSetElement = FDatasmithSceneFactory::CreateVariantSet( *ElementName );

			ParseVariantSet( ChildNode, VarSetElement, Actors, Objects );
			OutElement->AddVariantSet( VarSetElement );
		}
	}
}

void FDatasmithSceneXmlReader::ParseVariantSet( FXmlNode* InNode, const TSharedRef<IDatasmithVariantSetElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const
{
	if ( !InNode )
	{
		return;
	}

	for ( FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( !ChildNode )
		{
			continue;
		}

		if ( ChildNode->GetTag() == DATASMITH_VARIANTNAME )
		{
			FString ElementName = ChildNode->GetAttribute( TEXT( "name" ) );
			TSharedRef< IDatasmithVariantElement > VarElement = FDatasmithSceneFactory::CreateVariant( *ElementName );

			ParseVariant( ChildNode, VarElement, Actors, Objects );
			OutElement->AddVariant( VarElement );
		}
	}
}

void FDatasmithSceneXmlReader::ParseVariant( FXmlNode* InNode, const TSharedRef<IDatasmithVariantElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const
{
	if ( !InNode )
	{
		return;
	}

	for ( FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( !ChildNode )
		{
			continue;
		}

		if ( ChildNode->GetTag() == DATASMITH_ACTORBINDINGNAME )
		{
			TSharedRef< IDatasmithActorBindingElement > BindingElement = FDatasmithSceneFactory::CreateActorBinding();

			FString ActorName = ChildNode->GetAttribute( TEXT( "actor" ) );
			const TSharedPtr<IDatasmithActorElement>* FoundBoundActor = Actors.Find( ActorName );
			if ( FoundBoundActor && ( *FoundBoundActor ).IsValid() )
			{
				BindingElement->SetActor( *FoundBoundActor );
			}
			else
			{
				UE_LOGF( LogDatasmith, Warning, "Missing actor referenced in variant %ls", OutElement->GetName() );
			}

			ParseActorBinding( ChildNode, BindingElement, Objects );
			OutElement->AddActorBinding( BindingElement );
		}
	}
}

void FDatasmithSceneXmlReader::ParseActorBinding( FXmlNode* InNode, const TSharedRef<IDatasmithActorBindingElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const
{
	if ( !InNode )
	{
		return;
	}

	for ( FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( !ChildNode )
		{
			continue;
		}

		if ( ChildNode->GetTag() == DATASMITH_PROPERTYCAPTURENAME )
		{
			TSharedRef< IDatasmithPropertyCaptureElement > PropertyElement = FDatasmithSceneFactory::CreatePropertyCapture();

			ParsePropertyCapture( ChildNode, PropertyElement );
			OutElement->AddPropertyCapture( PropertyElement );
		}
		else if ( ChildNode->GetTag() == DATASMITH_OBJECTPROPERTYCAPTURENAME )
		{
			TSharedRef< IDatasmithObjectPropertyCaptureElement > ObjectPropertyElement = FDatasmithSceneFactory::CreateObjectPropertyCapture();

			ParseObjectPropertyCapture( ChildNode, ObjectPropertyElement, Objects );
			OutElement->AddPropertyCapture( ObjectPropertyElement );
		}
	}
}

void FDatasmithSceneXmlReader::ParsePropertyCapture( FXmlNode* InNode, const TSharedRef<IDatasmithPropertyCaptureElement>& OutElement ) const
{
	if ( !InNode )
	{
		return;
	}

	FString PropertyPath = InNode->GetAttribute( TEXT( "path" ) );
	EDatasmithPropertyCategory Category = static_cast< EDatasmithPropertyCategory >( ValueFromString<int32>( InNode->GetAttribute( TEXT( "category" ) ) ) );
	FString RecordedDataHex = InNode->GetAttribute( TEXT( "data" ) );

	for ( TCHAR Char : RecordedDataHex )
	{
		if ( !CheckTCharIsHex( Char ) )
		{
			UE_LOGF( LogDatasmith, Warning, "Invalid recorded data '%ls' for captured property with path '%ls' and category '%d'", *RecordedDataHex, *PropertyPath, int(Category) );
			RecordedDataHex = FString();
			break;
		}
	}
	if ( RecordedDataHex.Len() % 2 != 0 )
	{
		UE_LOGF( LogDatasmith, Warning, "Invalid recorded data '%ls' for captured property with path '%ls' and category '%d'", *RecordedDataHex, *PropertyPath, int(Category) );
		RecordedDataHex = FString();
	}

	int32 NumBytes = RecordedDataHex.Len() / 2; // Two characters for each byte
	TArray<uint8> RecordedBytes;
	RecordedBytes.SetNumZeroed( NumBytes );

	int32 NumRead = HexToBytes( RecordedDataHex, RecordedBytes.GetData() );
	if ( NumRead != NumBytes )
	{
		UE_LOGF( LogDatasmith, Warning, "Invalid recorded data '%ls' for captured property with path '%ls' and category '%d'", *RecordedDataHex, *PropertyPath, int(Category) );
		RecordedBytes.SetNum( 0, EAllowShrinking::No );
		RecordedBytes.SetNumZeroed( NumBytes );
	}

	OutElement->SetPropertyPath( PropertyPath );
	OutElement->SetCategory( Category );
	OutElement->SetRecordedData( RecordedBytes.GetData(), NumBytes );
}

void FDatasmithSceneXmlReader::ParseObjectPropertyCapture( FXmlNode* InNode, const TSharedRef<IDatasmithObjectPropertyCaptureElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const
{
	if ( !InNode )
	{
		return;
	}

	FString PropertyPath = InNode->GetAttribute( TEXT( "path" ) );
	EDatasmithPropertyCategory Category = static_cast< EDatasmithPropertyCategory >( ValueFromString<int32>( InNode->GetAttribute( TEXT( "category" ) ) ) );
	FString ObjectName = InNode->GetAttribute( TEXT( "object" ) );

	OutElement->SetPropertyPath( PropertyPath );
	OutElement->SetCategory( Category );

	const TSharedPtr<IDatasmithElement>* FoundObject = Objects.Find( ObjectName );
	if ( FoundObject && ( *FoundObject ).IsValid() )
	{
		OutElement->SetRecordedObject( *FoundObject );
	}
	else
	{
		UE_LOGF( LogDatasmith, Warning, "Missing object '%ls' referenced by captured property with path '%ls' and category '%d'", OutElement->GetName(), *PropertyPath, int(Category) );
	}
}

void FDatasmithSceneXmlReader::ParseMesh(FXmlNode* InNode, TSharedPtr<IDatasmithMeshElement>& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	const TArray<FXmlNode*>& MeshNodes = InNode->GetChildrenNodes();
	for (int j = 0; j < MeshNodes.Num(); j++)
	{
		if (MeshNodes[j]->GetTag() == TEXT("file"))
		{
			OutElement->SetFile(*UnsanitizeXMLText(MeshNodes[j]->GetAttribute(TEXT("path"))));
		}
		else if (MeshNodes[j]->GetTag() == TEXT("Size"))
		{
			float Area = ValueFromString<float>(MeshNodes[j]->GetAttribute(TEXT("a")));
			float Width = ValueFromString<float>(MeshNodes[j]->GetAttribute(TEXT("x")));
			float Depth = ValueFromString<float>(MeshNodes[j]->GetAttribute(TEXT("y")));
			float Height = ValueFromString<float>(MeshNodes[j]->GetAttribute(TEXT("z")));
			OutElement->SetDimensions(Area, Width, Height, Depth);
		}
		else if (MeshNodes[j]->GetTag() == DATASMITH_LIGHTMAPCOORDINATEINDEX)
		{
			OutElement->SetLightmapCoordinateIndex(ValueFromString<int32>(MeshNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if (MeshNodes[j]->GetTag() == DATASMITH_LIGHTMAPUVSOURCE)
		{
			OutElement->SetLightmapSourceUV(ValueFromString<int32>(MeshNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if (MeshNodes[j]->GetTag() == DATASMITH_HASH)
		{
			FMD5Hash Hash;
			LexFromString(Hash, *MeshNodes[j]->GetAttribute(TEXT("value")));
			OutElement->SetFileHash(Hash);
		}
		else if (MeshNodes[j]->GetTag() == DATASMITH_MATERIAL)
		{
			OutElement->SetMaterial(*MeshNodes[j]->GetAttribute(TEXT("name")), ValueFromString<int32>(MeshNodes[j]->GetAttribute(TEXT("id"))));
		}

	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDatasmithSceneXmlReader::ParseCloth(FXmlNode* InNode, TSharedPtr<IDatasmithClothElement>& OutElement) const  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	for (const FXmlNode* Node : InNode->GetChildrenNodes())
	{
		if (Node->GetTag() == TEXT("file"))
		{
			OutElement->SetFile(*UnsanitizeXMLText(Node->GetAttribute(TEXT("path"))));
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDatasmithSceneXmlReader::ParseTextureElement(FXmlNode* InNode, TSharedPtr<IDatasmithTextureElement>& OutElement) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithSceneXmlReader::ParseTextureElement);
	ParseElement( InNode, OutElement.ToSharedRef() );

	FString StrValue = InNode->GetAttribute(TEXT("rgbcurve"));
	if (!StrValue.IsEmpty())
	{
		OutElement->SetRGBCurve(ValueFromString<float>(StrValue));
	}

	StrValue = InNode->GetAttribute(TEXT("srgb"));
	if (!StrValue.IsEmpty())
	{
		OutElement->SetSRGB((EDatasmithColorSpace)ValueFromString<int32>(StrValue));
	}

	StrValue = InNode->GetAttribute(TEXT("texturemode"));
	if (!StrValue.IsEmpty())
	{
		OutElement->SetTextureMode((EDatasmithTextureMode)ValueFromString<int32>(StrValue));
	}

	StrValue = InNode->GetAttribute(TEXT("texturefilter"));
	if (!StrValue.IsEmpty())
	{
		OutElement->SetTextureFilter((EDatasmithTextureFilter)ValueFromString<int32>(StrValue));
	}

	StrValue = InNode->GetAttribute(TEXT("textureaddressx"));
	if (!StrValue.IsEmpty())
	{
		OutElement->SetTextureAddressX((EDatasmithTextureAddress)ValueFromString<int32>(StrValue));
	}

	StrValue = InNode->GetAttribute(TEXT("textureaddressy"));
	if (!StrValue.IsEmpty())
	{
		OutElement->SetTextureAddressY((EDatasmithTextureAddress)ValueFromString<int32>(StrValue));
	}
	OutElement->SetFile(*InNode->GetAttribute(TEXT("file")));

	const TArray<FXmlNode*>& TexNode = InNode->GetChildrenNodes();
	for (int i = 0; i < TexNode.Num(); ++i)
	{
		if (TexNode[i]->GetTag() == DATASMITH_HASH)
		{
			FMD5Hash Hash;
			LexFromString(Hash, *TexNode[i]->GetAttribute(TEXT("value")));
			OutElement->SetFileHash(Hash);
		}
	}
}

void FDatasmithSceneXmlReader::ParseTexture(FXmlNode* InNode, FString& OutTextureFilename, FDatasmithTextureSampler& OutTextureSampler) const
{
	OutTextureFilename = InNode->GetAttribute(TEXT("tex"));
	OutTextureSampler.ScaleX = ValueFromString<float>(InNode->GetAttribute(TEXT("sx")));
	OutTextureSampler.ScaleY = ValueFromString<float>(InNode->GetAttribute(TEXT("sy")));
	OutTextureSampler.OffsetX = ValueFromString<float>(InNode->GetAttribute(TEXT("ox")));
	OutTextureSampler.OffsetY = ValueFromString<float>(InNode->GetAttribute(TEXT("oy")));
	OutTextureSampler.MirrorX = ValueFromString<int32>(InNode->GetAttribute(TEXT("mx")));
	OutTextureSampler.MirrorY = ValueFromString<int32>(InNode->GetAttribute(TEXT("my")));
	OutTextureSampler.Rotation = ValueFromString<float>(InNode->GetAttribute(TEXT("rot")));
	OutTextureSampler.Multiplier = ValueFromString<float>(InNode->GetAttribute(TEXT("mul")));
	OutTextureSampler.OutputChannel = ValueFromString<int32>(InNode->GetAttribute(TEXT("channel")));
	OutTextureSampler.CoordinateIndex = ValueFromString<int32>(InNode->GetAttribute(TEXT("coordinate")));

	if (ValueFromString<int32>(InNode->GetAttribute(TEXT("inv"))) == 1)
	{
		OutTextureSampler.bInvert = true;
	}

	if (ValueFromString<int32>(InNode->GetAttribute(TEXT("cropped"))) == 1)
	{
		OutTextureSampler.bCroppedTexture = true;
	}
}


void FDatasmithSceneXmlReader::ParsePostProcess(FXmlNode *InNode, const TSharedPtr< IDatasmithPostProcessElement >& Element) const
{
	ParseElement( InNode, Element.ToSharedRef() );

	const TArray<FXmlNode*>& CompNodes = InNode->GetChildrenNodes();

	for (int j = 0; j < CompNodes.Num(); j++)
	{
		if (CompNodes[j]->GetTag() == DATASMITH_POSTPRODUCTIONTEMP)
		{
			Element->SetTemperature(ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_POSTPRODUCTIONVIGNETTE)
		{
			Element->SetVignette(ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_POSTPRODUCTIONCOLOR)
		{
			FLinearColor Color;
			ParseColor(CompNodes[j], Color);
			Element->SetColorFilter(Color);
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_POSTPRODUCTIONSATURATION)
		{
			Element->SetSaturation(ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_POSTPRODUCTIONCAMERAISO)
		{
			Element->SetCameraISO(ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_POSTPRODUCTIONSHUTTERSPEED)
		{
			Element->SetCameraShutterSpeed(ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))));
		}
		else if ( CompNodes[j]->GetTag() == DATASMITH_FSTOP )
		{
			Element->SetDepthOfFieldFstop( ValueFromString<float>( CompNodes[j]->GetAttribute( TEXT("value") ) ) );
		}
	}
}

void FDatasmithSceneXmlReader::ParsePostProcessVolume(FXmlNode* InNode, const TSharedRef<IDatasmithPostProcessVolumeElement>& Element) const
{
	ParseElement( InNode, Element );

	for ( FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( ChildNode->GetTag() == DATASMITH_POSTPRODUCTIONNAME )
		{
			ParsePostProcess(ChildNode, Element->GetSettings());
		}
		else if ( ChildNode->GetTag() == DATASMITH_ENABLED )
		{
			Element->SetEnabled( ValueFromString< bool >( ChildNode->GetAttribute( TEXT("value") ) ) );
		}
		else if ( ChildNode->GetTag() == DATASMITH_POSTPROCESSVOLUME_UNBOUND )
		{
			Element->SetUnbound( ValueFromString< bool >( ChildNode->GetAttribute( TEXT("value") ) ) );
		}
	}
}

void FDatasmithSceneXmlReader::ParseColor(FXmlNode* InNode, FLinearColor& OutColor) const
{
	OutColor = FLinearColor(
		ValueFromString<float>(InNode->GetAttribute(TEXT("R"))),
		ValueFromString<float>(InNode->GetAttribute(TEXT("G"))),
		ValueFromString<float>(InNode->GetAttribute(TEXT("B")))
	);
}

void FDatasmithSceneXmlReader::ParseComp(FXmlNode* InNode, TSharedPtr< IDatasmithCompositeTexture >& OutCompTexture, bool bInIsNormal) const
{
	OutCompTexture->SetMode( (EDatasmithCompMode) ValueFromString<int32>(InNode->GetAttribute(TEXT("mode"))) );

	const TArray<FXmlNode*>& CompNodes = InNode->GetChildrenNodes();

	FString FileName;
	FDatasmithTextureSampler TextureSampler;
	FLinearColor Color;

	for (int j = 0; j < CompNodes.Num(); j++)
	{
		if (CompNodes[j]->GetTag() == DATASMITH_TEXTURENAME)
		{
			ParseTexture(CompNodes[j], FileName, TextureSampler);
			OutCompTexture->AddSurface( *FileName, TextureSampler );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_COLORNAME)
		{
			ParseColor(CompNodes[j], Color);
			OutCompTexture->AddSurface( Color );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_TEXTURECOMPNAME)
		{
			TSharedPtr< IDatasmithCompositeTexture > SubCompTex = FDatasmithSceneFactory::CreateCompositeTexture();
			ParseComp(CompNodes[j], SubCompTex, bInIsNormal);
			OutCompTexture->AddSurface( SubCompTex );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_MASKNAME)
		{
			ParseTexture(CompNodes[j], FileName, TextureSampler);
			OutCompTexture->AddMaskSurface( *FileName, TextureSampler );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_MASKCOLOR)
		{
			ParseColor(CompNodes[j], Color);
			OutCompTexture->AddMaskSurface( Color );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_MASKCOMPNAME)
		{
			TSharedPtr< IDatasmithCompositeTexture > SubCompTex = FDatasmithSceneFactory::CreateCompositeTexture();
			ParseComp(CompNodes[j], SubCompTex, bInIsNormal);
			OutCompTexture->AddMaskSurface( SubCompTex );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_VALUE1NAME)
		{
			OutCompTexture->AddParamVal1( IDatasmithCompositeTexture::ParamVal( ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))), TEXT("") ) );
		}
		else if (CompNodes[j]->GetTag() == DATASMITH_VALUE2NAME)
		{
			OutCompTexture->AddParamVal2( IDatasmithCompositeTexture::ParamVal( ValueFromString<float>(CompNodes[j]->GetAttribute(TEXT("value"))), TEXT("") ) );
		}
	}
}

void FDatasmithSceneXmlReader::ParseActor(FXmlNode* InNode, TSharedPtr<IDatasmithActorElement>& InOutElement, TSharedRef< IDatasmithScene > Scene, TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors) const
{
	if (InNode->GetTag() == DATASMITH_ACTORMESHNAME)
	{
		TSharedPtr< IDatasmithMeshActorElement> MeshElement = FDatasmithSceneFactory::CreateMeshActor(*InNode->GetAttribute(TEXT("name")));
		ParseMeshActor(InNode, MeshElement, Scene);
		InOutElement = MeshElement;
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if (InNode->GetTag() == DATASMITH_CLOTHACTORNAME)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		TSharedPtr< IDatasmithClothActorElement> Element = FDatasmithSceneFactory::CreateClothActor(*InNode->GetAttribute(TEXT("name")));
		ParseClothActor(InNode, Element, Scene);
		InOutElement = Element;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else if (InNode->GetTag() == DATASMITH_LIGHTNAME)
	{
		TSharedPtr< IDatasmithLightActorElement > LightElement;
		ParseLight(InNode, LightElement);
		InOutElement = LightElement;
	}
	else if (InNode->GetTag() == DATASMITH_CAMERANAME)
	{
		TSharedPtr< IDatasmithCameraActorElement > CameraElement = FDatasmithSceneFactory::CreateCameraActor(*InNode->GetAttribute(TEXT("name")));
		ParseCamera(InNode, CameraElement);
		InOutElement = CameraElement;
	}
	else if (InNode->GetTag() == DATASMITH_DECALACTORNAME)
	{
		TSharedPtr< IDatasmithDecalActorElement > DecalActorElement = FDatasmithSceneFactory::CreateDecalActor(*InNode->GetAttribute(TEXT("name")));
		TSharedPtr< IDatasmithCustomActorElement > CustomActorElement = StaticCastSharedPtr< IDatasmithCustomActorElement >( DecalActorElement );
		ParseCustomActor(InNode, CustomActorElement);
		InOutElement = DecalActorElement;
	}
	else if (InNode->GetTag() == DATASMITH_CUSTOMACTORNAME)
	{
		TSharedPtr< IDatasmithCustomActorElement > CustomActorElement = FDatasmithSceneFactory::CreateCustomActor(*InNode->GetAttribute(TEXT("name")));
		ParseCustomActor(InNode, CustomActorElement);
		InOutElement = CustomActorElement;
	}
	else if (InNode->GetTag() == DATASMITH_LANDSCAPENAME)
	{
		TSharedRef< IDatasmithLandscapeElement > LandscapeElement = FDatasmithSceneFactory::CreateLandscape(*InNode->GetAttribute(TEXT("name")));
		ParseLandscape(InNode, LandscapeElement);
		InOutElement = LandscapeElement;
	}
	else if (InNode->GetTag() == DATASMITH_POSTPROCESSVOLUME)
	{
		TSharedRef< IDatasmithPostProcessVolumeElement > PostElement = FDatasmithSceneFactory::CreatePostProcessVolume(*InNode->GetAttribute(TEXT("name")));
		ParsePostProcessVolume(InNode, PostElement);
		InOutElement = PostElement;
	}
	else if (InNode->GetTag() == DATASMITH_ACTORNAME)
	{
		TSharedPtr< IDatasmithActorElement > ActorElement = FDatasmithSceneFactory::CreateActor(*InNode->GetAttribute(TEXT("name")));
		ParseElement( InNode, ActorElement.ToSharedRef() );
		InOutElement = ActorElement;
	}
	else if (InNode->GetTag() == DATASMITH_ACTORHIERARCHICALINSTANCEDMESHNAME)
	{
		TSharedPtr< IDatasmithHierarchicalInstancedStaticMeshActorElement > HierarchicalInstancesStaticMeshElement = FDatasmithSceneFactory::CreateHierarchicalInstanceStaticMeshActor(*InNode->GetAttribute(TEXT("name")));
		ParseHierarchicalInstancedStaticMeshActor(InNode, HierarchicalInstancesStaticMeshElement, Scene);
		InOutElement = HierarchicalInstancesStaticMeshElement;
	}

	if (!InOutElement.IsValid())
	{
		return;
	}

	// Make sure that the InOutElement name is unique. It should be but if it isn't we need to force it to prevent issues down the road.
	FString ActorName = InOutElement->GetName();

	if ( Actors.Contains( ActorName ) )
	{
		FString Prefix = ActorName;
		int32 NameIdx = 1;

		// Update the actor name until we find one that doesn't already exist
		while ( Actors.Contains( ActorName ) )
		{
			++NameIdx;
			ActorName = FString::Printf( TEXT("%s%d"), *Prefix, NameIdx );
		}

		InOutElement->SetName( *ActorName );
	}

	Actors.Add( ActorName, InOutElement );

	InOutElement->SetLayer(*InNode->GetAttribute(TEXT("layer")));

	InOutElement->SetIsAComponent( ValueFromString<bool>( InNode->GetAttribute(TEXT("component"))) );

	FString VisibleAtribute = InNode->GetAttribute(TEXT("visible"));
	InOutElement->SetVisibility(VisibleAtribute.IsEmpty() ? true : ValueFromString<bool>(VisibleAtribute));

	FString CastShadowAtribute = InNode->GetAttribute(TEXT("castshadow"));
	InOutElement->SetCastShadow(CastShadowAtribute.IsEmpty() ? true : ValueFromString<bool>(CastShadowAtribute));

	FString MobilityString = InNode->GetAttribute(TEXT("mobility"));
	TArrayView< const TCHAR* > MobilityEnumStrings(DatasmithActorMobilityTypeStrings);
	int32 MobilityIndexOfEnumValue = MobilityEnumStrings.IndexOfByPredicate([&MobilityString](const TCHAR* Value)
		{
			return MobilityString == Value;
		});

	if (MobilityIndexOfEnumValue != INDEX_NONE)
	{
		InOutElement->SetMobility((EDatasmithActorMobilityType)MobilityIndexOfEnumValue);
	}

	for (FXmlNode* ChildNode : InNode->GetChildrenNodes())
	{
		if (ChildNode->GetTag() == TEXT("transform"))
		{
			ParseTransform(ChildNode, InOutElement);
		}
		else if (ChildNode->GetTag() == TEXT("tag"))
		{
			InOutElement->AddTag(*ChildNode->GetAttribute(TEXT("value")));
		}
		else if (ChildNode->GetTag() == TEXT("children"))
		{
			// Recursively parse the children, can be of any supported actor type
			for (FXmlNode* ChildActorNode : ChildNode->GetChildrenNodes())
			{
				for ( const TCHAR* ActorTag : ActorTagsView )
				{
					if ( ChildActorNode->GetTag() == ActorTag )
					{
						TSharedPtr< IDatasmithActorElement > ChildActorElement;
						ParseActor(ChildActorNode, ChildActorElement, Scene, Actors );
						if (ChildActorElement.IsValid())
						{
							InOutElement->AddChild(ChildActorElement);
						}

						break;
					}
				}
			}
		}
	}
}

void FDatasmithSceneXmlReader::ParseMeshActor(FXmlNode* InNode, TSharedPtr<IDatasmithMeshActorElement>& OutElement, TSharedRef< IDatasmithScene > Scene) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	for (FXmlNode* ChildNode : InNode->GetChildrenNodes())
	{
		if (ChildNode->GetTag() == TEXT("mesh"))
		{
			int32 StaticMeshIndex = -1;

			if ( !ChildNode->GetAttribute(TEXT("index")).IsEmpty() )
			{
				StaticMeshIndex = ValueFromString<int32>(ChildNode->GetAttribute(TEXT("index")));
			}

			if ( StaticMeshIndex < 0 )
			{
				OutElement->SetStaticMeshPathName( *ChildNode->GetAttribute(TEXT("name")) );
			}
			else
			{
				TSharedPtr< IDatasmithMeshElement > MeshElement = Scene->GetMesh( StaticMeshIndex );

				if ( MeshElement.IsValid() )
				{
					OutElement->SetStaticMeshPathName( MeshElement->GetName() );
				}
			}
		}
		else if (ChildNode->GetTag() == DATASMITH_MATERIAL)
		{
			TSharedPtr< IDatasmithMaterialIDElement > MatElement = FDatasmithSceneFactory::CreateMaterialId(*ChildNode->GetAttribute(TEXT("name")));
			MatElement->SetId(ValueFromString<int32>(ChildNode->GetAttribute(TEXT("id"))));
			OutElement->AddMaterialOverride(MatElement);
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDatasmithSceneXmlReader::ParseClothActor(FXmlNode* InNode, TSharedPtr<IDatasmithClothActorElement>& OutElement, TSharedRef< IDatasmithScene > Scene) const  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	ParseElement(InNode, OutElement.ToSharedRef());

	for (FXmlNode* ChildNode : InNode->GetChildrenNodes())
	{
		if (ChildNode->GetTag() == TEXT("Cloth"))
		{
			OutElement->SetCloth(*ChildNode->GetAttribute(TEXT("name")));
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDatasmithSceneXmlReader::ParseHierarchicalInstancedStaticMeshActor(FXmlNode* InNode, TSharedPtr<IDatasmithHierarchicalInstancedStaticMeshActorElement>& OutElement, TSharedRef< IDatasmithScene > Scene) const
{
	TSharedPtr<IDatasmithMeshActorElement> OutElementAsMeshActor = OutElement;
	ParseMeshActor(InNode, OutElementAsMeshActor, Scene);

	for (FXmlNode* ChildNode : InNode->GetChildrenNodes())
	{
		if (ChildNode->GetTag() == TEXT("Instances"))
		{
			OutElement->ReserveSpaceForInstances( ValueFromString<int32>( ChildNode->GetAttribute( TEXT("count") ) ) );

			for (FXmlNode* InstanceTransformNode : ChildNode->GetChildrenNodes())
			{
				OutElement->AddInstance(ParseTransform(InstanceTransformNode));
			}
			break;
		}
	}
}

void FDatasmithSceneXmlReader::ParseLight(FXmlNode* InNode, TSharedPtr<IDatasmithLightActorElement>& OutElement) const
{
	FString LightTypeValue = InNode->GetAttribute( TEXT("type") );

	EDatasmithElementType LightType = EDatasmithElementType::SpotLight;

	if ( LightTypeValue == DATASMITH_POINTLIGHTNAME )
	{
		LightType = EDatasmithElementType::PointLight;
	}
	else if ( LightTypeValue == DATASMITH_SPOTLIGHTNAME )
	{
		LightType = EDatasmithElementType::SpotLight;
	}
	else if ( LightTypeValue == DATASMITH_DIRECTLIGHTNAME )
	{
		LightType = EDatasmithElementType::DirectionalLight;
	}
	else if ( LightTypeValue == DATASMITH_AREALIGHTNAME )
	{
		LightType = EDatasmithElementType::AreaLight;
	}
	else if ( LightTypeValue == DATASMITH_PORTALLIGHTNAME )
	{
		LightType = EDatasmithElementType::LightmassPortal;
	}

	TSharedPtr< IDatasmithElement > Element = FDatasmithSceneFactory::CreateElement( LightType, *InNode->GetAttribute( TEXT( "name" ) ));

	if( !Element.IsValid() || !Element->IsA( EDatasmithElementType::Light ) )
	{
		return;
	}

	OutElement = StaticCastSharedPtr< IDatasmithLightActorElement >( Element );
	ParseElement( InNode, OutElement.ToSharedRef() );

	OutElement->SetEnabled( ValueFromString<bool>( InNode->GetAttribute(TEXT("enabled")) ) );

	for ( FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if (ChildNode->GetTag() == DATASMITH_LIGHTCOLORNAME)
		{
			OutElement->SetUseTemperature(ValueFromString<bool>(ChildNode->GetAttribute(DATASMITH_LIGHTUSETEMPNAME)));
			OutElement->SetTemperature(ValueFromString<double>(ChildNode->GetAttribute(DATASMITH_LIGHTTEMPNAME)));

			// Make sure color info is available
			if ( !ChildNode->GetAttribute(TEXT("R")).IsEmpty() )
			{
				FLinearColor Color;
				ParseColor(ChildNode, Color);
				OutElement->SetColor(Color);
			}
		}
		else if (ChildNode->GetTag() == DATASMITH_LIGHTIESNAME)
		{
			OutElement->SetUseIes(true);
			OutElement->SetIesFile( *ChildNode->GetAttribute(TEXT("file")) );
		}
		else if (ChildNode->GetTag() == DATASMITH_LIGHTIESTEXTURENAME)
		{
			OutElement->SetUseIes(true);
			OutElement->SetIesTexturePathName( *ChildNode->GetAttribute(TEXT("name")) );
		}
		else if (ChildNode->GetTag() == DATASMITH_LIGHTIESBRIGHTNAME)
		{
			OutElement->SetIesBrightnessScale(ValueFromString<double>(ChildNode->GetAttribute(TEXT("scale"))));
			if(OutElement->GetIesBrightnessScale() > 0.0)
			{
				OutElement->SetUseIesBrightness(true);
			}
			else
			{
				OutElement->SetUseIesBrightness(false);
				OutElement->SetIesBrightnessScale(1.0);
			}
		}
		else if (ChildNode->GetTag() == DATASMITH_LIGHTIESROTATION)
		{
			OutElement->SetIesRotation( QuatFromNode( ChildNode ) );
		}
		else if (ChildNode->GetTag() == DATASMITH_LIGHTINTENSITYNAME)
		{
			OutElement->SetIntensity( ValueFromString<double>(ChildNode->GetAttribute(TEXT("value"))) );
		}
		else if (ChildNode->GetTag() == DATASMITH_LIGHTMATERIAL)
		{
			OutElement->SetLightFunctionMaterial(*ChildNode->GetAttribute(TEXT("name")));
		}

		if ( OutElement->IsA( EDatasmithElementType::PointLight ) )
		{
			TSharedRef< IDatasmithPointLightElement > PointLightElement = StaticCastSharedRef< IDatasmithPointLightElement >( OutElement.ToSharedRef() );

			if (ChildNode->GetTag() == DATASMITH_LIGHTATTENUATIONRADIUSNAME)
			{
				PointLightElement->SetAttenuationRadius( ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))) );
			}
			else if (ChildNode->GetTag() == DATASMITH_LIGHTSOURCESIZENAME)
			{
				PointLightElement->SetSourceRadius( ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))) );
			}
			else if (ChildNode->GetTag() == DATASMITH_LIGHTSOURCELENGTHNAME)
			{
				PointLightElement->SetSourceLength( ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))) );
			}
			else if (ChildNode->GetTag() == DATASMITH_LIGHTINTENSITYUNITSNAME)
			{
				FString LightUnitsValue = ChildNode->GetAttribute( TEXT("value") );

				if ( LightUnitsValue == TEXT("Candelas") )
				{
					PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );
				}
				else if ( LightUnitsValue == TEXT("Lumens") )
				{
					PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
				}
				else
				{
					PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Unitless );
				}
			}
		}

		if ( OutElement->IsA( EDatasmithElementType::SpotLight ) )
		{
			TSharedRef< IDatasmithSpotLightElement > SpotLightElement = StaticCastSharedRef< IDatasmithSpotLightElement >( OutElement.ToSharedRef() );

			if (ChildNode->GetTag() == DATASMITH_LIGHTINNERRADIUSNAME)
			{
				SpotLightElement->SetInnerConeAngle( ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))) );
			}
			else if (ChildNode->GetTag() == DATASMITH_LIGHTOUTERRADIUSNAME)
			{
				SpotLightElement->SetOuterConeAngle( ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))) );
			}
		}

		if ( OutElement->IsA( EDatasmithElementType::AreaLight ) )
		{
			TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( OutElement.ToSharedRef() );

			if ( ChildNode->GetTag() == DATASMITH_AREALIGHTSHAPE )
			{
				FString ShapeType = *ChildNode->GetAttribute( TEXT("type") );

				TArrayView< const TCHAR* > ShapeTypeEnumStrings( DatasmithAreaLightShapeStrings );
				int32 ShapeTypeIndexOfEnumValue = ShapeTypeEnumStrings.IndexOfByPredicate( [ TypeString = ShapeType ]( const TCHAR* Value )
				{
					return TypeString == Value;
				} );

				if ( ShapeTypeIndexOfEnumValue != INDEX_NONE )
				{
					AreaLightElement->SetLightShape( (EDatasmithLightShape)ShapeTypeIndexOfEnumValue );
				}

				AreaLightElement->SetWidth( ValueFromString<float>(ChildNode->GetAttribute(TEXT("width"))) );
				AreaLightElement->SetLength( ValueFromString<float>(ChildNode->GetAttribute(TEXT("length"))) );

				FString AreaLightType = ChildNode->GetAttribute( DATASMITH_AREALIGHTTYPE );

				if ( AreaLightType.IsEmpty() )
				{
					AreaLightType = ChildNode->GetAttribute( DATASMITH_AREALIGHTDISTRIBUTION ); // Used to be called light distribution
				}

				TArrayView< const TCHAR* > LightTypeEnumStrings( DatasmithAreaLightTypeStrings );
				int32 LightTypeIndexOfEnumValue = LightTypeEnumStrings.IndexOfByPredicate( [ TypeString = AreaLightType ]( const TCHAR* Value )
				{
					return TypeString == Value;
				} );

				if ( LightTypeIndexOfEnumValue != INDEX_NONE )
				{
					AreaLightElement->SetLightType( (EDatasmithAreaLightType)LightTypeIndexOfEnumValue );
				}
			}
		}
	}
}

void FDatasmithSceneXmlReader::ParseCamera(FXmlNode* InNode, TSharedPtr<IDatasmithCameraActorElement>& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	for (FXmlNode* ChildNode : InNode->GetChildrenNodes())
	{
		if (ChildNode->GetTag() == DATASMITH_SENSORWIDTH)
		{
			OutElement->SetSensorWidth(ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))));
		}
		else if (ChildNode->GetTag() == DATASMITH_SENSORASPECT)
		{
			OutElement->SetSensorAspectRatio(ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))));
		}
		else if (ChildNode->GetTag() == DATASMITH_DEPTHOFFIELD)
		{
			OutElement->SetEnableDepthOfField( ValueFromString< bool >( ChildNode->GetAttribute(TEXT("enabled")) ) );
		}
		else if (ChildNode->GetTag() == DATASMITH_FOCUSDISTANCE)
		{
			OutElement->SetFocusDistance(ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))));
		}
		else if (ChildNode->GetTag() == DATASMITH_FSTOP)
		{
			OutElement->SetFStop(ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))));
		}
		else if (ChildNode->GetTag() == DATASMITH_FOCALLENGTH)
		{
			OutElement->SetFocalLength(ValueFromString<float>(ChildNode->GetAttribute(TEXT("value"))));
		}
		else if (ChildNode->GetTag() == DATASMITH_POSTPRODUCTIONNAME)
		{
			ParsePostProcess(ChildNode, OutElement->GetPostProcess());
		}
		else if (ChildNode->GetTag() == DATASMITH_LOOKAT)
		{
			OutElement->SetLookAtActor( *ChildNode->GetAttribute(DATASMITH_ACTORNAME) );
		}
		else if (ChildNode->GetTag() == DATASMITH_LOOKATROLL)
		{
			OutElement->SetLookAtAllowRoll( ValueFromString< bool >( ChildNode->GetAttribute(TEXT("enabled")) ) );
		}
	}
}

bool FDatasmithSceneXmlReader::LoadFromFile(const FString & InFilename)
{
	FString FileBuffer;
	if ( !FFileHelper::LoadFileToString( FileBuffer, *InFilename ) )
	{
		return false;
	}

	ProjectPath = FPaths::GetPath(InFilename);

	return LoadFromBuffer( FileBuffer );
}

bool FDatasmithSceneXmlReader::LoadFromBuffer(const FString& XmlBuffer)
{
	XmlFile = MakeUnique< FXmlFile >(XmlBuffer, EConstructMethod::ConstructFromBuffer); // Don't use FXmlFile to load the file for now because it fails to properly convert from UTF-8. FFileHelper::LoadFileToString does the conversion.

	const FXmlNode* SceneNode = XmlFile->GetRootNode();
	if (SceneNode == NULL)
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::FromString( TEXT("Invalid Datasmith File") ));
		XmlFile.Reset();
		return false;
	}

	if (SceneNode->GetTag() != TEXT("DatasmithUnrealScene"))
	{
		FText DialogTitle = FText::FromString( TEXT("Error parsing file") );
		FMessageDialog::Open(EAppMsgType::Ok, FText::FromString( SceneNode->GetTag() ), DialogTitle);
		XmlFile.Reset();
	}

	return true;
}

bool FDatasmithSceneXmlReader::ParseFile(const FString& InFilename, TSharedRef< IDatasmithScene >& OutScene, bool bInAppend)
{
	if ( !LoadFromFile(InFilename) )
	{
		return false;
	}

	return ParseXmlFile(OutScene, bInAppend);
}

bool FDatasmithSceneXmlReader::ParseBuffer(const FString& XmlBuffer, TSharedRef< IDatasmithScene >& OutScene, bool bInAppend)
{
	if ( !LoadFromBuffer(XmlBuffer) )
	{
		return false;
	}

	return ParseXmlFile( OutScene, bInAppend );
}

bool FDatasmithSceneXmlReader::ParseXmlFile(TSharedRef< IDatasmithScene >& OutScene, bool bInAppend)
{
	if (!XmlFile.IsValid())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithSceneXmlReader::ParseXmlFile);

	FDatasmithLocaleScope CLocaleScope;

	if (bInAppend == false)
	{
		OutScene->Reset();
	}

	OutScene->SetExporterSDKVersion( TEXT("N/A") ); // We're expecting to read the SDK Version from the XML file. If it's not available, put "N/A"

	TMap< FString, TSharedPtr<IDatasmithActorElement> > Actors;
	TMap< FString, TSharedPtr<IDatasmithElement> > Objects; // Our variants' property captures may reference objects. Usually materials, but can be actors/textures/etc.

	const TArray<FXmlNode*>& Nodes = XmlFile->GetRootNode()->GetChildrenNodes();

	for (int i = 0; i < Nodes.Num(); i++)
	{
		// HOST
		if (Nodes[i]->GetTag() == DATASMITH_HOSTNAME)
		{
			OutScene->SetHost(*UnsanitizeXMLText(Nodes[i]->GetContent()));
		}
		// VERSION
		else if (Nodes[i]->GetTag() == DATASMITH_EXPORTERVERSION)
		{
			OutScene->SetExporterVersion(*Nodes[i]->GetContent());
		}
		// SDK VERSION
		else if (Nodes[i]->GetTag() == DATASMITH_EXPORTERSDKVERSION)
		{
			OutScene->SetExporterSDKVersion(*Nodes[i]->GetContent());
		}
		// RESOURCE PATH
		else if (Nodes[i]->GetTag() == DATASMITH_RESOURCEPATH)
		{
			OutScene->SetResourcePath(*Nodes[i]->GetContent());
		}
		// APPLICATION INFO
		else if (Nodes[i]->GetTag() == DATASMITH_APPLICATION)
		{
			OutScene->SetVendor(*UnsanitizeXMLText(Nodes[i]->GetAttribute(DATASMITH_VENDOR)));
			OutScene->SetProductName(*UnsanitizeXMLText(Nodes[i]->GetAttribute(DATASMITH_PRODUCTNAME)));
			OutScene->SetProductVersion(*UnsanitizeXMLText(Nodes[i]->GetAttribute(DATASMITH_PRODUCTVERSION)));
		}
		// USER INFO
		else if (Nodes[i]->GetTag() == DATASMITH_USER)
		{
			OutScene->SetUserID(*Nodes[i]->GetAttribute(DATASMITH_USERID));
			OutScene->SetUserOS(*Nodes[i]->GetAttribute(DATASMITH_USEROS));
		}
		else if (Nodes[i]->GetTag() == DATASMITH_GEOLOCATION)
		{
			FString LatitudeStr = Nodes[i]->GetAttribute(DATASMITH_GEOLOCATION_LATITUDE);
			if (!LatitudeStr.IsEmpty())
			{
				OutScene->SetGeolocationLatitude(ValueFromString<double>(LatitudeStr));
			}

			FString LongitudeStr = Nodes[i]->GetAttribute(DATASMITH_GEOLOCATION_LONGITUDE);
			if (!LongitudeStr.IsEmpty())
			{
				OutScene->SetGeolocationLongitude(ValueFromString<double>(LongitudeStr));
			}

			FString ElevationStr = Nodes[i]->GetAttribute(DATASMITH_GEOLOCATION_ELEVATION);
			if (!ElevationStr.IsEmpty())
			{
				OutScene->SetGeolocationElevation(ValueFromString<double>(ElevationStr));
			}
		}
		// STATIC MESHES
		else if (Nodes[i]->GetTag() == DATASMITH_STATICMESHNAME)
		{
			FString ElementName = Nodes[i]->GetAttribute(TEXT("name"));
			TSharedPtr< IDatasmithMeshElement > Element = FDatasmithSceneFactory::CreateMesh(*ElementName);

			ParseMesh( Nodes[i], Element );

			OutScene->AddMesh(Element);

			Objects.Add( Element->GetName(), Element );
		}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// CLOTHES
		else if (Nodes[i]->GetTag() == DATASMITH_CLOTH)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
		{
			FString ElementName = Nodes[i]->GetAttribute(TEXT("name"));
			TSharedPtr< IDatasmithClothElement > Element = FDatasmithSceneFactory::CreateCloth(*ElementName);

			ParseCloth( Nodes[i], Element );

			OutScene->AddCloth(Element);

// 			Objects.Add( Element->GetName(), Element ); // #ue_ds_cloth_todo cloths referencable by other elements
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// LEVEL SEQUENCES
		else if (Nodes[i]->GetTag() == DATASMITH_LEVELSEQUENCENAME)
		{
			FString ElementName = Nodes[i]->GetAttribute(TEXT("name"));
			TSharedRef< IDatasmithLevelSequenceElement > Element = FDatasmithSceneFactory::CreateLevelSequence(*ElementName);

			ParseLevelSequence( Nodes[i], Element );

			OutScene->AddLevelSequence(Element);
		}
		// LEVEL VARIANT SETS
		else if (Nodes[i]->GetTag() == DATASMITH_LEVELVARIANTSETSNAME)
		{
			FString ElementName = Nodes[i]->GetAttribute(TEXT("name"));
			TSharedRef< IDatasmithLevelVariantSetsElement > Element = FDatasmithSceneFactory::CreateLevelVariantSets(*ElementName);

			ParseLevelVariantSets( Nodes[i], Element, Actors, Objects );

			OutScene->AddLevelVariantSets(Element);
		}
		// TEXTURES
		else if (Nodes[i]->GetTag() == DATASMITH_TEXTURENAME)
		{
			TSharedPtr< IDatasmithTextureElement > Element = FDatasmithSceneFactory::CreateTexture(*Nodes[i]->GetAttribute(TEXT("name")));
			ParseTextureElement(Nodes[i], Element);

			FString TextureName(Element->GetName());
			int32 TexturesCount = OutScene->GetTexturesCount();
			bool bIsDuplicate = false;
			for (int32 t = 0; t < TexturesCount; t++)
			{
				const TSharedPtr< IDatasmithTextureElement >& TextureElement = OutScene->GetTexture(t);
				if (TextureName == TextureElement->GetName())
				{
					bIsDuplicate = true;
					break;
				}
			}

			if (bIsDuplicate == false)
			{
				OutScene->AddTexture(Element);

				Objects.Add(Element->GetName(), Element);
			}
		}
		// ENVIRONMENTS
		else if (Nodes[i]->GetTag() == DATASMITH_ENVIRONMENTNAME)
		{
			TSharedPtr< IDatasmithEnvironmentElement > Element = FDatasmithSceneFactory::CreateEnvironment(*Nodes[i]->GetAttribute(TEXT("name")));

			ParseElement( Nodes[i], Element.ToSharedRef() );

			const TArray<FXmlNode*>& EleNodes = Nodes[i]->GetChildrenNodes();
			for (int j = 0; j < EleNodes.Num(); j++)
			{
				if (EleNodes[j]->GetTag() == DATASMITH_TEXTURENAME)
				{
					FString TextureFile;
					FDatasmithTextureSampler TextureSampler;
					ParseTexture(EleNodes[j], TextureFile, TextureSampler);

					Element->GetEnvironmentComp()->AddSurface( *TextureFile, TextureSampler );
				}
				else if (EleNodes[j]->GetTag() == DATASMITH_ENVILLUMINATIONMAP)
				{
					Element->SetIsIlluminationMap(ValueFromString<bool>(EleNodes[j]->GetAttribute(TEXT("enabled"))));
				}
			}

			if( Element->GetEnvironmentComp()->GetParamSurfacesCount() != 0 && !FString(Element->GetEnvironmentComp()->GetParamTexture(0)).IsEmpty() )
			{
				OutScene->AddActor(Element);

				Objects.Add( Element->GetName(), Element );
			}
		}
		// SKY
		else if (Nodes[i]->GetTag() == DATASMITH_PHYSICALSKYNAME)
		{
			OutScene->SetUsePhysicalSky( ValueFromString<bool>(Nodes[i]->GetAttribute(TEXT("enabled"))));
		}
		// POSTPROCESS
		else if (Nodes[i]->GetTag() == DATASMITH_POSTPRODUCTIONNAME)
		{
			TSharedPtr< IDatasmithPostProcessElement > PostProcess = FDatasmithSceneFactory::CreatePostProcess();
			ParsePostProcess(Nodes[i], PostProcess);
			OutScene->SetPostProcess(PostProcess);
		}
		// MATERIALS
		else if (Nodes[i]->GetTag() == DATASMITH_MATERIALNAME)
		{
			TSharedPtr< IDatasmithMaterialElement > Material = FDatasmithSceneFactory::CreateMaterial(*Nodes[i]->GetAttribute(TEXT("name")));

			ParseMaterial(Nodes[i], Material);
			OutScene->AddMaterial(Material);

			Objects.Add( Material->GetName(), Material );
		}
		// MATERIAL INSTANCES
		// Support legacy udatasmith files which have the banned word
		else if (Nodes[i]->GetTag().Equals(TEXT("Mas" "terMaterial")) || Nodes[i]->GetTag() == DATASMITH_MATERIALINSTANCENAME)
		{
			TSharedPtr< IDatasmithMaterialInstanceElement > ReferenceMaterial = FDatasmithSceneFactory::CreateMaterialInstance(*Nodes[i]->GetAttribute(TEXT("name")));

			ParseMaterialInstance(Nodes[i], ReferenceMaterial);
			OutScene->AddMaterial(ReferenceMaterial);

			Objects.Add( ReferenceMaterial->GetName(), ReferenceMaterial );
		}
		// DECAL MATERIALS
		else if (Nodes[i]->GetTag() == DATASMITH_DECALMATERIALNAME)
		{
			TSharedPtr< IDatasmithDecalMaterialElement > DecalMaterial = FDatasmithSceneFactory::CreateDecalMaterial(*Nodes[i]->GetAttribute(TEXT("name")));

			ParseDecalMaterial(Nodes[i], DecalMaterial);
			OutScene->AddMaterial(DecalMaterial);

			Objects.Add( DecalMaterial->GetName(), DecalMaterial );
		}
		// UEPBR MATERIALS
		else if (Nodes[i]->GetTag() == DATASMITH_UEPBRMATERIALNAME)
		{
			TSharedPtr< IDatasmithUEPbrMaterialElement > Material = FDatasmithSceneFactory::CreateUEPbrMaterial(*Nodes[i]->GetAttribute(TEXT("name")));

			ParseUEPbrMaterial(Nodes[i], Material);
			OutScene->AddMaterial(Material);

			Objects.Add( Material->GetName(), Material );
		}
		// METADATA
		else if (Nodes[i]->GetTag() == DATASMITH_METADATANAME)
		{
			TSharedPtr< IDatasmithMetaDataElement > MetaData = FDatasmithSceneFactory::CreateMetaData(*Nodes[i]->GetAttribute(TEXT("name")));
			ParseMetaData(Nodes[i], MetaData, OutScene, Actors );
			OutScene->AddMetaData(MetaData);
		}
		// EXPORT STATS
		else if (Nodes[i]->GetTag() == DATASMITH_EXPORT)
		{
			OutScene->SetExportDuration(ValueFromString<int32>(Nodes[i]->GetAttribute(DATASMITH_EXPORTDURATION)));
		}
		// ACTORS
		else
		{
			for ( const TCHAR* ActorTag : ActorTagsView )
			{
				if ( Nodes[i]->GetTag() == ActorTag )
				{
					TSharedPtr< IDatasmithActorElement > ActorElement;
					ParseActor(Nodes[i], ActorElement, OutScene, Actors );
					if ( ActorElement.IsValid() )
					{
						OutScene->AddActor(ActorElement);

						Objects.Add( ActorElement->GetName(), ActorElement );
					}

					break;
				}
			}
		}
	}

	PatchUpVersion(OutScene);

	FDatasmithSceneUtils::CleanUpScene(OutScene);

	return true;
}

void FDatasmithSceneXmlReader::ParseMaterial(FXmlNode* InNode, TSharedPtr< IDatasmithMaterialElement >& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	const TArray<FXmlNode*>& MaterialNodes = InNode->GetChildrenNodes();
	for (int m = 0; m < MaterialNodes.Num(); m++)
	{
		if (MaterialNodes[m]->GetTag() == DATASMITH_SHADERNAME)
		{
			TSharedPtr< IDatasmithShaderElement > ShaderElement = FDatasmithSceneFactory::CreateShader(*MaterialNodes[m]->GetAttribute(TEXT("name")));

			const TArray<FXmlNode*>& ShaderNodes = MaterialNodes[m]->GetChildrenNodes();
			for (int j = 0; j < ShaderNodes.Num(); j++)
			{
				FString Texture;
				FDatasmithTextureSampler TextureSampler;
				FLinearColor Color;
				const FString& NodeTag = ShaderNodes[j]->GetTag();
				if (NodeTag == DATASMITH_DIFFUSETEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetDiffuseTexture(*Texture);
					ShaderElement->SetDiffTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_DIFFUSECOLNAME)
				{
					ParseColor(ShaderNodes[j], Color);
					ShaderElement->SetDiffuseColor(Color);
				}
				else if (NodeTag == DATASMITH_DIFFUSECOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetDiffuseComp());
				}
				else if (NodeTag == DATASMITH_REFLETEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetReflectanceTexture(*Texture);
					ShaderElement->SetRefleTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_REFLECOLNAME)
				{
					ParseColor(ShaderNodes[j], Color);
					ShaderElement->SetReflectanceColor(Color);
				}
				else if (NodeTag == DATASMITH_REFLECOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetRefleComp());
				}
				else if (NodeTag == DATASMITH_ROUGHNESSTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetRoughnessTexture(*Texture);
					ShaderElement->SetRoughTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_ROUGHNESSVALUENAME)
				{
					ShaderElement->SetRoughness(fmax(0.02, ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value")))));
				}
				else if (NodeTag == DATASMITH_ROUGHNESSCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetRoughnessComp());
				}
				else if (NodeTag == DATASMITH_BUMPVALUENAME)
				{
					ShaderElement->SetBumpAmount(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_BUMPTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetBumpTexture(*Texture);
					ShaderElement->SetBumpTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_NORMALTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetNormalTexture(*Texture);
					ShaderElement->SetNormalTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_NORMALCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetNormalComp());
				}
				else if (NodeTag == DATASMITH_TRANSPTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetTransparencyTexture(*Texture);
					ShaderElement->SetTransTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_TRANSPCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetTransComp());
				}
				else if (NodeTag == DATASMITH_CLIPTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetMaskTexture(*Texture);
					ShaderElement->SetMaskTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_CLIPCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetMaskComp());
				}
				else if (NodeTag == DATASMITH_TRANSPCOLNAME)
				{
					ParseColor(ShaderNodes[j], Color);
					ShaderElement->SetTransparencyColor(Color);
				}
				else if (NodeTag == DATASMITH_IORVALUENAME)
				{
					ShaderElement->SetIOR(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_IORKVALUENAME)
				{
					ShaderElement->SetIORk(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_REFRAIORVALUENAME)
				{
					ShaderElement->SetIORRefra(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_TWOSIDEDVALUENAME)
				{
					ShaderElement->SetTwoSided(ValueFromString<bool>(ShaderNodes[j]->GetAttribute(TEXT("enabled"))));
				}
				else if (NodeTag == DATASMITH_METALTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetMetalTexture(*Texture);
					ShaderElement->SetMetalTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_METALVALUENAME)
				{
					ShaderElement->SetMetal(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_METALCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetMetalComp());
				}
				else if (NodeTag == DATASMITH_EMITTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetEmitTexture(*Texture);
					ShaderElement->SetEmitTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_EMITVALUENAME)
				{
					ShaderElement->SetEmitPower(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_EMITCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetEmitComp());
				}
				else if (NodeTag == DATASMITH_EMITTEMPNAME)
				{
					ShaderElement->SetEmitTemperature(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_EMITCOLNAME)
				{
					ParseColor(ShaderNodes[j], Color);
					ShaderElement->SetEmitColor(Color);
				}
				else if (NodeTag == DATASMITH_EMITONLYVALUENAME)
				{
					ShaderElement->SetLightOnly(ValueFromString<bool>(ShaderNodes[j]->GetAttribute(TEXT("enabled"))));
				}
				else if (NodeTag == DATASMITH_WEIGHTTEXNAME)
				{
					ParseTexture(ShaderNodes[j], Texture, TextureSampler);
					ShaderElement->SetWeightTexture(*Texture);
					ShaderElement->SetWeightTextureSampler(TextureSampler);
				}
				else if (NodeTag == DATASMITH_WEIGHTCOLNAME)
				{
					ParseColor(ShaderNodes[j], Color);
					ShaderElement->SetWeightColor(Color);
				}
				else if (NodeTag == DATASMITH_WEIGHTCOMPNAME)
				{
					ParseComp(ShaderNodes[j], ShaderElement->GetWeightComp());
				}
				else if (NodeTag == DATASMITH_WEIGHTVALUENAME)
				{
					ShaderElement->SetWeightValue(ValueFromString<double>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_BLENDMODE)
				{
					ShaderElement->SetBlendMode((EDatasmithBlendMode)ValueFromString<int32>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
				else if (NodeTag == DATASMITH_STACKLAYER)
				{
					ShaderElement->SetIsStackedLayer(ValueFromString<bool>(ShaderNodes[j]->GetAttribute(TEXT("enabled"))));
				}
				else if (NodeTag == DATASMITH_DYNAMICEMISSIVE)
				{
					ShaderElement->SetUseEmissiveForDynamicAreaLighting(ValueFromString<bool>(ShaderNodes[j]->GetAttribute(TEXT("enabled"))));
				}
				else if (NodeTag == DATASMITH_SHADERUSAGE)
				{
					ShaderElement->SetShaderUsage((EDatasmithShaderUsage)ValueFromString<int32>(ShaderNodes[j]->GetAttribute(TEXT("value"))));
				}
			}

			if (FCString::Strlen(ShaderElement->GetReflectanceTexture()) == 0 && ShaderElement->GetReflectanceColor().IsAlmostBlack() && ShaderElement->GetRefleComp().IsValid() == false &&
				FCString::Strlen(ShaderElement->GetMetalTexture()) == 0 && ShaderElement->GetMetal() <= 0.0 && ShaderElement->GetMetalComp().IsValid() == false)
			{
				ShaderElement->SetReflectanceColor(FLinearColor(0.07f, 0.07f, 0.07f));
				ShaderElement->SetRoughness(0.7);
			}
			OutElement->AddShader(ShaderElement);
		}
	}
}

void FDatasmithSceneXmlReader::ParseMaterialInstance(FXmlNode* InNode, TSharedPtr< IDatasmithMaterialInstanceElement >& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	for ( const FXmlAttribute& Attribute : InNode->GetAttributes() )
	{
		if (Attribute.GetTag() == DATASMITH_MATERIALINSTANCETYPE)
		{
			EDatasmithReferenceMaterialType MaterialType = (EDatasmithReferenceMaterialType)FMath::Clamp( ValueFromString<int32>( Attribute.GetValue() ), 0, (int32)EDatasmithReferenceMaterialType::Count - 1 );

			OutElement->SetMaterialType( MaterialType );
		}
		else if (Attribute.GetTag() == DATASMITH_MATERIALINSTANCEQUALITY)
		{
			EDatasmithReferenceMaterialQuality Quality = (EDatasmithReferenceMaterialQuality)FMath::Clamp( ValueFromString<int32>( Attribute.GetValue() ), 0, (int32)EDatasmithReferenceMaterialQuality::Count - 1 );
			OutElement->SetQuality( Quality );
		}
		else if (Attribute.GetTag() == DATASMITH_MATERIALINSTANCEPATHNAME)
		{
			OutElement->SetCustomMaterialPathName( *Attribute.GetValue() );
		}
	}

	ParseKeyValueProperties( InNode, *OutElement );
}

void FDatasmithSceneXmlReader::ParseDecalMaterial(FXmlNode* InNode, TSharedPtr< IDatasmithDecalMaterialElement >& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	for ( const FXmlNode* PropNode : InNode->GetChildrenNodes() )
	{
		if (PropNode->GetTag() == DATASMITH_DIFFUSETEXNAME)
		{
			FString PathName = PropNode->GetAttribute( TEXT("PathName") );
			OutElement->SetDiffuseTexturePathName( *PathName );
		}
		else if (PropNode->GetTag() == DATASMITH_NORMALTEXNAME)
		{
			FString PathName = PropNode->GetAttribute( TEXT("PathName") );
			OutElement->SetNormalTexturePathName( *PathName );
		}
	}
}


template< typename ExpressionInputType >
void FDatasmithSceneXmlReader::ParseExpressionInput(const FXmlNode* InNode, TSharedPtr< IDatasmithUEPbrMaterialElement >& OutElement, ExpressionInputType& ExpressionInput) const
{
	if ( !InNode )
	{
		return;
	}

	FString ExpressionIndexAttribute = InNode->GetAttribute( TEXT("expression") );

	if ( !ExpressionIndexAttribute.IsEmpty() )
	{
		// Before 4.23 Expressions were serialized as <0 expression="5" OutputIndex="0"/>
		// From 4.23 Expressions are serialized as <Input Name="0" expression="5" OutputIndex="0"/>
		// So if the Name is used and that backward compatibility is desired, the Node Tag can be used instead of the "Name" Attribute.

		int32 ExpressionIndex = ValueFromString<int32>( ExpressionIndexAttribute );

		int32 OutputIndex = ValueFromString<int32>( InNode->GetAttribute( TEXT("OutputIndex") ) );

		IDatasmithMaterialExpression* Expression = OutElement->GetExpression( ExpressionIndex );

		if ( Expression )
		{
			Expression->ConnectExpression( ExpressionInput, OutputIndex );
		}
	}
}

void FDatasmithSceneXmlReader::ParseUEPbrMaterial(FXmlNode* InNode, TSharedPtr< IDatasmithUEPbrMaterialElement >& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	OutElement->SetParentLabel( *InNode->GetAttribute( DATASMITH_PARENTMATERIALLABEL ) );

	FString ExpressionsTag = "Expressions";

	FXmlNode* const* ExpressionsNode = Algo::FindByPredicate( InNode->GetChildrenNodes(), [ ExpressionsTag ]( const FXmlNode* Node ) -> bool
	{
		return Node->GetTag() == ExpressionsTag;
	});

	if ( ExpressionsNode )
	{
		// Create all the material expressions
		for ( const FXmlNode* ChildNode : (*ExpressionsNode)->GetChildrenNodes() )
		{
			if ( ChildNode->GetTag() == TEXT("Texture") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Texture );

				if ( Expression )
				{
					Expression->SetName( *ChildNode->GetAttribute( TEXT("Name") ) );
					IDatasmithMaterialExpressionTexture* TextureExpression = static_cast< IDatasmithMaterialExpressionTexture* >( Expression );
					TextureExpression->SetTexturePathName( *ChildNode->GetAttribute( TEXT("PathName") ) );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("TextureCoordinate") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::TextureCoordinate );

				if ( Expression )
				{
					IDatasmithMaterialExpressionTextureCoordinate* TextureCoordinateExpression = static_cast< IDatasmithMaterialExpressionTextureCoordinate* >( Expression );
					TextureCoordinateExpression->SetCoordinateIndex( ValueFromString< int32 >( ChildNode->GetAttribute( TEXT("Index") ) ) );
					TextureCoordinateExpression->SetUTiling( ValueFromString< float >( ChildNode->GetAttribute( TEXT("UTiling") ) ) );
					TextureCoordinateExpression->SetVTiling( ValueFromString< float >( ChildNode->GetAttribute( TEXT("VTiling") ) ) );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("FlattenNormal") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::FlattenNormal );

				if ( Expression )
				{
					IDatasmithMaterialExpressionFlattenNormal* FlattenNormal = static_cast< IDatasmithMaterialExpressionFlattenNormal* >( Expression );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("Bool") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantBool );

				if ( Expression )
				{
					Expression->SetName( *ChildNode->GetAttribute( TEXT("Name") ) );

					IDatasmithMaterialExpressionBool* ConstantBool = static_cast< IDatasmithMaterialExpressionBool* >( Expression );

					ConstantBool->GetBool() = ValueFromString< bool >( ChildNode->GetAttribute( TEXT("Constant") ) );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("Color") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantColor );

				if ( Expression )
				{
					Expression->SetName( *ChildNode->GetAttribute( TEXT("Name") ) );

					IDatasmithMaterialExpressionColor* ConstantColor = static_cast< IDatasmithMaterialExpressionColor* >( Expression );

					ConstantColor->GetColor() = ValueFromString< FLinearColor >( ChildNode->GetAttribute( TEXT("Constant") ) );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("Scalar") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar );

				if ( Expression )
				{
					Expression->SetName( *ChildNode->GetAttribute( TEXT("Name") ) );

					IDatasmithMaterialExpressionScalar* ConstantScalar = static_cast< IDatasmithMaterialExpressionScalar* >( Expression );

					ConstantScalar->GetScalar() = ValueFromString< float >( ChildNode->GetAttribute( TEXT("Constant") ) );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("FunctionCall") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::FunctionCall );

				if ( Expression )
				{
					IDatasmithMaterialExpressionFunctionCall* FunctionCall = static_cast< IDatasmithMaterialExpressionFunctionCall* >( Expression );
					FunctionCall->SetFunctionPathName( *ChildNode->GetAttribute( TEXT("Function") ) );
				}
			}
			else if ( ChildNode->GetTag() == TEXT("Custom") )
			{
				if ( IDatasmithMaterialExpressionCustom* Expression = OutElement->AddMaterialExpression<IDatasmithMaterialExpressionCustom>() )
				{
					for (const FXmlNode* CustomChildNode : ChildNode->GetChildrenNodes())
					{
						if (CustomChildNode == nullptr)
						{
							continue;
						}

						if (CustomChildNode->GetTag() == TEXT("Code"))
						{
							const FString& SanitizedContent = CustomChildNode->GetContent();
							const FString& Content = UnsanitizeXMLText(SanitizedContent);
							Expression->SetCode(*Content);
						}
						else if (CustomChildNode->GetTag() == TEXT("Include"))
						{
							const FString& Path = UnsanitizeXMLText(CustomChildNode->GetAttribute(TEXT("path")));
							Expression->AddIncludeFilePath(*Path);
						}
						else if (CustomChildNode->GetTag() == TEXT("Define"))
						{
							const FString& Value = UnsanitizeXMLText(CustomChildNode->GetAttribute(TEXT("value")));
							Expression->AddAdditionalDefine(*Value);
						}
						else if (CustomChildNode->GetTag() == TEXT("Arg"))
						{
							int32 Index = ValueFromString<int32>(CustomChildNode->GetAttribute(TEXT("index")));
							const FString& Value = UnsanitizeXMLText(CustomChildNode->GetAttribute(TEXT("name")));
							Expression->SetArgumentName(Index, *Value);
						}
					}

					const FString& Description = ChildNode->GetAttribute( TEXT("Description") );
					Expression->SetDescription( *Description );

					int32 OutputType = ValueFromString< int32 >( ChildNode->GetAttribute( TEXT("OutputType") ) );;
					Expression->SetOutputType(EDatasmithShaderDataType(OutputType));
				}
			}
			else
			{
				IDatasmithMaterialExpression* Expression = OutElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic );

				if ( Expression )
				{
					IDatasmithMaterialExpressionGeneric* GenericExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( Expression );
					GenericExpression->SetName( *ChildNode->GetAttribute( TEXT("Name") ) );

					GenericExpression->SetExpressionName( *ChildNode->GetTag() );
					ParseKeyValueProperties( ChildNode, *GenericExpression );
				}
			}
		}

		// Connect the material expressions
		int32 ExpressionIndex = 0;
		for ( const FXmlNode* ChildNode : (*ExpressionsNode)->GetChildrenNodes() )
		{
			if ( ChildNode->GetTag() == TEXT("FlattenNormal") )
			{
				IDatasmithMaterialExpression* Expression = OutElement->GetExpression( ExpressionIndex );

				if ( Expression )
				{
					IDatasmithMaterialExpressionFlattenNormal* FlattenNormal = static_cast< IDatasmithMaterialExpressionFlattenNormal* >( Expression );

					{
						FXmlNode* const* NormalNode = Algo::FindByPredicate( ChildNode->GetChildrenNodes(), [InputName = FlattenNormal->GetNormal().GetName()]( FXmlNode* Node ) -> bool
						{
							return Node->GetTag() == InputName;
						} );
						FXmlNode* const* FlatnessNode = Algo::FindByPredicate( ChildNode->GetChildrenNodes(), [InputName = FlattenNormal->GetFlatness().GetName()]( FXmlNode* Node ) -> bool
						{
							return Node->GetTag() == InputName;
						} );

						ParseExpressionInput( *NormalNode, OutElement, FlattenNormal->GetNormal() );
						ParseExpressionInput( *NormalNode, OutElement, FlattenNormal->GetFlatness() );
					}
				}
			}
			else // Generic
			{
				IDatasmithMaterialExpression* Expression = OutElement->GetExpression( ExpressionIndex );

				if ( Expression )
				{
					IDatasmithMaterialExpressionGeneric* GenericExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( Expression );

					for ( const FXmlNode* InputChildNode : ChildNode->GetChildrenNodes() )
					{
						const FString& NameAttribute = InputChildNode->GetAttribute(TEXT("Name"));
						int32 InputIndex = ValueFromString< int32 >( NameAttribute.IsEmpty() ? InputChildNode->GetTag() : NameAttribute );

						if (IDatasmithExpressionInput* Input = GenericExpression->GetInput( InputIndex ))
						{
							ParseExpressionInput( InputChildNode, OutElement, *Input );
						}
					}
				}
			}

			++ExpressionIndex;
		}
	}

	const TArray<FXmlNode*>& ChildrenNodes = InNode->GetChildrenNodes();

	auto TryConnectMaterialInput = [&](IDatasmithExpressionInput& Input)
	{
		const TCHAR* InputName = Input.GetName();
		for (FXmlNode* XmlNode : ChildrenNodes)
		{
			if (XmlNode && (XmlNode->GetAttribute(TEXT("Name")) == InputName || XmlNode->GetTag() == InputName ))
			{
				ParseExpressionInput( XmlNode, OutElement, Input );
				return;
			}
		}
	};

	TryConnectMaterialInput(OutElement->GetBaseColor());
	TryConnectMaterialInput(OutElement->GetMetallic());
	TryConnectMaterialInput(OutElement->GetSpecular());
	TryConnectMaterialInput(OutElement->GetRoughness());
	TryConnectMaterialInput(OutElement->GetEmissiveColor());
	TryConnectMaterialInput(OutElement->GetOpacity());
	TryConnectMaterialInput(OutElement->GetNormal());
	TryConnectMaterialInput(OutElement->GetRefraction());
	TryConnectMaterialInput(OutElement->GetAmbientOcclusion());
	TryConnectMaterialInput(OutElement->GetClearCoat());
	TryConnectMaterialInput(OutElement->GetClearCoatRoughness());
	TryConnectMaterialInput(OutElement->GetWorldPositionOffset());
	TryConnectMaterialInput(OutElement->GetMaterialAttributes());

	for ( const FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( ChildNode->GetTag() == DATASMITH_USEMATERIALATTRIBUTESNAME )
		{
			OutElement->SetUseMaterialAttributes( ValueFromString< bool >( ChildNode->GetAttribute( TEXT("enabled") ) ) );
		}
		else if ( ChildNode->GetTag() == DATASMITH_TWOSIDEDVALUENAME )
		{
			OutElement->SetTwoSided( ValueFromString< bool >( ChildNode->GetAttribute( TEXT("enabled") ) ) );
		}
		else if ( ChildNode->GetTag() == DATASMITH_BLENDMODE )
		{
			OutElement->SetBlendMode( ValueFromString< int >( ChildNode->GetAttribute( TEXT("value") ) ) );
		}
		else if ( ChildNode->GetTag() == DATASMITH_OPACITYMASKCLIPVALUE )
		{
			OutElement->SetOpacityMaskClipValue( ValueFromString< float >( ChildNode->GetAttribute( TEXT("value") ) ) );
		}
		else if (ChildNode->GetTag() == DATASMITH_TRANSLUCENCYLIGHTINGMODE)
		{
			OutElement->SetTranslucencyLightingMode(ValueFromString< int >(ChildNode->GetAttribute(TEXT("value"))));
		}
		else if ( ChildNode->GetTag() == DATASMITH_FUNCTIONLYVALUENAME )
		{
			OutElement->SetMaterialFunctionOnly( ValueFromString< bool >( ChildNode->GetAttribute( TEXT("enabled") ) ) );
		}
		else if ( ChildNode->GetTag() == DATASMITH_SHADINGMODEL )
		{
			TArrayView< const TCHAR* > EnumStrings( DatasmithShadingModelStrings );
			int32 IndexOfEnumValue = EnumStrings.IndexOfByPredicate( [ TypeString = ChildNode->GetAttribute(TEXT("value")) ]( const TCHAR* Value )
			{
				return TypeString == Value;
			} );

			if ( IndexOfEnumValue != INDEX_NONE )
			{
				OutElement->SetShadingModel( (EDatasmithShadingModel)IndexOfEnumValue );
			}
		}
	}
}

void FDatasmithSceneXmlReader::ParseCustomActor(FXmlNode* InNode, TSharedPtr< IDatasmithCustomActorElement >& OutElement) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	for ( const FXmlAttribute& Attribute : InNode->GetAttributes() )
	{
		if (Attribute.GetTag() == DATASMITH_CUSTOMACTORPATHNAME)
		{
			OutElement->SetClassOrPathName( *Attribute.GetValue() );
		}
	}

	ParseKeyValueProperties( InNode, *OutElement );
}

namespace MetaDataElementUtils
{
	void FinalizeMetaData(const TSharedRef< IDatasmithScene >& InScene, TSharedPtr<IDatasmithMetaDataElement>& OutElement, const FString& ReferenceString, TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors)
	{
		// Retrieve the associated element, which is saved as "ReferenceType.ReferenceName"
		FString ReferenceType;
		FString ReferenceName;
		ReferenceString.Split(TEXT("."), &ReferenceType, &ReferenceName);

		TSharedPtr<IDatasmithElement> AssociatedElement;
		if (ReferenceType == TEXT("Actor"))
		{
			TSharedPtr<IDatasmithActorElement> * Actor = Actors.Find(ReferenceName);
			if (Actor != nullptr)
			{
				AssociatedElement = *Actor;
			}
			else
			{
				UE_LOGF(LogDatasmith, Warning, "Missing actor referenced in metadata %ls", *ReferenceName);
			}
		}
		else if (ReferenceType == TEXT("Texture"))
		{
			int32 NumElements = InScene->GetTexturesCount();
			for (int32 i = 0; i < NumElements; ++i)
			{
				const TSharedPtr<IDatasmithElement>& CurrentElement = InScene->GetTexture(i);
				if (FCString::Strcmp(CurrentElement->GetName(), *ReferenceName) == 0)
				{
					AssociatedElement = CurrentElement;
					break;
				}
			}
		}
		else if (ReferenceType == TEXT("Material"))
		{
			int32 NumElements = InScene->GetMaterialsCount();
			for (int32 i = 0; i < NumElements; ++i)
			{
				const TSharedPtr<IDatasmithElement>& CurrentElement = InScene->GetMaterial(i);
				if (FCString::Strcmp(CurrentElement->GetName(), *ReferenceName) == 0)
				{
					AssociatedElement = CurrentElement;
					break;
				}
			}
		}
		else if (ReferenceType == TEXT("StaticMesh"))
		{
			int32 NumElements = InScene->GetMeshesCount();
			for (int32 i = 0; i < NumElements; ++i)
			{
				const TSharedPtr<IDatasmithElement>& CurrentElement = InScene->GetMesh(i);
				if (FCString::Strcmp(CurrentElement->GetName(), *ReferenceName) == 0)
				{
					AssociatedElement = CurrentElement;
					break;
				}
			}
		}
		else
		{
			ensure(false);
		}

		OutElement->SetAssociatedElement(AssociatedElement);
	}
}

void FDatasmithSceneXmlReader::ParseMetaData(FXmlNode* InNode, TSharedPtr< IDatasmithMetaDataElement >& OutElement, const TSharedRef< IDatasmithScene >& InScene, TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors) const
{
	ParseElement( InNode, OutElement.ToSharedRef() );

	ParseKeyValueProperties(InNode, *OutElement);

	FString ReferenceString = InNode->GetAttribute(DATASMITH_REFERENCENAME);
	MetaDataElementUtils::FinalizeMetaData(InScene, OutElement, ReferenceString, Actors );
}

void FDatasmithSceneXmlReader::ParseLandscape(FXmlNode* InNode, TSharedRef< IDatasmithLandscapeElement >& OutElement) const
{
	ParseElement( InNode, OutElement );

	for ( const FXmlNode* ChildNode : InNode->GetChildrenNodes() )
	{
		if ( ChildNode->GetTag() == DATASMITH_HEIGHTMAPNAME )
		{
			OutElement->SetHeightmap( *ChildNode->GetAttribute( TEXT("value") ) );
		}
		else if ( ChildNode->GetTag() == DATASMITH_MATERIAL )
		{
			OutElement->SetMaterial( *ChildNode->GetAttribute( DATASMITH_PATHNAME ) );
		}
	}
}

template< typename ElementType >
void FDatasmithSceneXmlReader::ParseKeyValueProperties(const FXmlNode* InNode, ElementType& OutElement) const
{
	for ( const FXmlNode* PropNode : InNode->GetChildrenNodes() )
	{
		bool bAddProperty = false;
		const FString& PropertyName = PropNode->GetAttribute( TEXT("name") );
		TSharedPtr< IDatasmithKeyValueProperty > KeyValueProperty = OutElement.GetPropertyByName( *PropertyName );
		if (!KeyValueProperty.IsValid())
		{
			KeyValueProperty = FDatasmithSceneFactory::CreateKeyValueProperty( *PropertyName );
			bAddProperty = true;
		}

		TArrayView< const TCHAR* > EnumStrings( KeyValuePropertyTypeStrings );
		int32 IndexOfEnumValue = EnumStrings.IndexOfByPredicate( [ TypeString = PropNode->GetAttribute(TEXT("type")) ]( const TCHAR* Value )
		{
			return TypeString == Value;
		} );

		if ( IndexOfEnumValue != INDEX_NONE )
		{
			KeyValueProperty->SetPropertyType( (EDatasmithKeyValuePropertyType)IndexOfEnumValue );

			KeyValueProperty->SetValue( *PropNode->GetAttribute(TEXT("val")) );
			if (bAddProperty)
			{
				OutElement.AddProperty( KeyValueProperty );
			}
		}
	}
}
