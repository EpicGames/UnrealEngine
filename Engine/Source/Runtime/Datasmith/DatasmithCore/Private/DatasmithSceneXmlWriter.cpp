// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneXmlWriter.h"

#include "DatasmithAnimationElements.h"
#include "DatasmithDefinitions.h"
#include "DatasmithLocaleScope.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithUtils.h"
#include "DatasmithVariantElements.h"
#include "IDatasmithSceneElements.h"

#include "Math/Quat.h"


class FDatasmithSceneXmlWriterImpl
{
public:
	static FString GetLabelAndLayer(const TSharedPtr<IDatasmithActorElement>& ActorElement);

	static void WriteTransform(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent);
	static void WriteTransform(const FTransform& Transform, FArchive& Archive, int32 Indent);
	static FString QuatToHexString(FQuat Value);

	static void AppendIndent(FString& XmlString, int32 Indent);
	static void WriteIndent(FArchive& Archive, int32 Indent);
	static void WriteBool(FArchive& Archive, int32 Indent, const TCHAR* Prefix, bool bValue);
	static void WriteRGB(FArchive& Archive, int32 Indent, const TCHAR* Prefix, FLinearColor Color);
	static void WriteValue(FArchive& Archive, int32 Indent, const TCHAR* Prefix, float Value, bool bForceWriteAlways = false, FString Desc = FString());
	static void WriteValue(FArchive& Archive, int32 Indent, const TCHAR* Prefix, double Value, bool bForceWriteAlways, FString Desc = FString()) = delete;
	static void AppendXMLChild(FString& XmlString, int32 Indent, const TCHAR* Tag, const TCHAR* AttributeName, const TCHAR* AttributeValue);

	static void WriteTexture(FArchive& Archive, int32 Indent, const TCHAR* Prefix, const TCHAR* Name, FDatasmithTextureSampler UV);
	static void WriteMeshElement(const TSharedPtr< IDatasmithMeshElement >& MeshElement, FArchive& Archive, int32 Indent);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static void WriteClothElement(const TSharedPtr< IDatasmithClothElement >& ClothElement, FArchive& Archive, int32 Indent);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	static void WriteLevelSequenceElement( const TSharedRef< IDatasmithLevelSequenceElement>& SequenceElement, FArchive& Archive, int32 Indent );

	static void WriteLevelVariantSetsElement( const TSharedRef< IDatasmithLevelVariantSetsElement >& LevelVariantSetsElement, FArchive& Archive, int32 Indent );
	static void WriteVariantSetElement( const TSharedRef< IDatasmithVariantSetElement >& VariantSetElement, FArchive& Archive, int32 Indent );
	static void WriteVariantElement( const TSharedRef< IDatasmithVariantElement >& VariantElement, FArchive& Archive, int32 Indent );
	static void WriteActorBindingElement( const TSharedRef< IDatasmithActorBindingElement >& ActorBindingElement, FArchive& Archive, int32 Indent );
	static void WritePropertyCaptureElement( const TSharedRef< IDatasmithPropertyCaptureElement >& PropertyCaptureElement, FArchive& Archive, int32 Indent );
	static void WriteObjectPropertyCaptureElement( const TSharedRef< IDatasmithObjectPropertyCaptureElement >& PropertyCaptureElement, FArchive& Archive, int32 Indent );

	static void WriteActorElement(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent);
	static void WriteBaseActorElement(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent);
	static void WriteActorTags(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent);
	static void WriteActorChildren(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent);
	static void WriteMeshActorElement(const TSharedPtr< IDatasmithMeshActorElement >& MeshActorElement, FArchive& Archive, int32 Indent);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static void WriteClothActorElement(const TSharedPtr< IDatasmithClothActorElement >& ClothActorElement, FArchive& Archive, int32 Indent);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Write the start of the actor element (Open the xml element and add the essential child elements for the mesh actor)
	static void WriteBeginOfMeshActorElement(const TSharedPtr<IDatasmithMeshActorElement>& MeshActorElement, const FString& ElementTypeString, FArchive& Archive, int32 Indent);
	// Write the end of an actor element (Close the xml element)
	static void WriteEndOfMeshActorElement(const FString& ElementTypeString, FArchive& Archive, int32 Indent);

	static void WriteHierarchicalInstancedMeshElement(const TSharedPtr< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedMeshElement, FArchive& Archive, int32 Indent);
	static void WriteCustomActorElement(const TSharedPtr< IDatasmithCustomActorElement >& CustomActorElement, FArchive& Archive, int32 Indent, const TCHAR* = DATASMITH_CUSTOMACTORNAME);
	static void WriteLandscapeActorElement(const TSharedPtr< IDatasmithLandscapeElement >& LandscapeActorElement, FArchive& Archive, int32 Indent);

	static void WriteMetaDataElement(const TSharedPtr< IDatasmithMetaDataElement >& MetaDataElement, FArchive& Archive, int32 Indent);

	static void WriteLightActorElement(const TSharedPtr< IDatasmithLightActorElement >& LightActorElement, FArchive& Archive, int32 Indent);
	static void WriteSpotLightElement(const TSharedRef< IDatasmithSpotLightElement >& SpotLightElement, FArchive& Archive, int32 Indent);
	static void WritePointLightElement(const TSharedRef< IDatasmithPointLightElement >& PointLightElement, FArchive& Archive, int32 Indent);
	static void WriteAreaLightElement(const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FArchive& Archive, int32 Indent);

	static void WritePostProcessElement(const TSharedPtr< IDatasmithPostProcessElement >& PostElement, FArchive& Archive, int32 Indent);
	static void WritePostProcessVolumeElement(const TSharedPtr< IDatasmithPostProcessVolumeElement >& PostElement, FArchive& Archive, int32 Indent);
	static void WriteCameraActorElement(const TSharedPtr< IDatasmithCameraActorElement >& CameraActorElement, FArchive& Archive, int32 Indent);
	static void WriteShaderElement(const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FArchive& Archive, int32 Indent);
	static void WriteCompTex(const TSharedPtr< IDatasmithCompositeTexture >& Comp, FArchive& Archive, int32 Indent);
	static void WriteEnvironmentElement(const TSharedPtr< IDatasmithEnvironmentElement >& EnvironmentElement, FArchive& Archive, int32 Indent);
	static void WriteTextureElement(const TSharedPtr< IDatasmithTextureElement >& TextureElement, FArchive& Archive, int32 Indent);

	static void WriteMaterialElement( TSharedPtr< IDatasmithMaterialElement >& MaterialElement, FArchive& Archive, int32 Indent);
	static void WriteMaterialInstanceElement(TSharedPtr< IDatasmithMaterialInstanceElement >& MaterialInstanceElement, FArchive& Archive, int32 Indent);
	static void WriteDecalMaterialElement(TSharedPtr< IDatasmithDecalMaterialElement >& DecalMaterialElement, FArchive& Archive, int32 Indent);

	static void WriteUEPbrMaterialElement( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, FArchive& Archive, int32 Indent );
	static void WriteUEPbrMaterialExpressions( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, FArchive& Archive, int32 Indent );

	static void WriteUEPbrMaterialExpressionInput( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithExpressionInput& ExpressionInput, FArchive& Archive, int32 Indent );

	static void AppendMaterialExpressionAttributes( const IDatasmithMaterialExpression& Expression, FString& XmlString );
	static void WriteMaterialExpressionFlattenNormal( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionFlattenNormal& FlattenNormal, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionTexture( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionTexture& TextureExpression, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionTextureCoordinate( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionTextureCoordinate& TextureCoordinateExpression, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionBool( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionBool& BoolExpression, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionColor( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionColor& ColorExpression, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionScalar( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionScalar& ScalarExpression, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionGeneric( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionGeneric& GenericExpression, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionFunctionCall( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionFunctionCall& FunctionCall, FArchive& Archive, int32 Indent );
	static void WriteMaterialExpressionCustom(const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionCustom& CustomExpression, FArchive& Archive, int32 Indent);

	template< typename ElementType >
	static void WriteKeyValueProperties(const ElementType& Element, FArchive& Archive, int32 Indent);

	static void SerializeToArchive(FArchive& Archive, const TCHAR* Value)
	{
		FTCHARToUTF8 UTF8String( Value );
		Archive.Serialize( (ANSICHAR*)UTF8String.Get(), UTF8String.Length() );
	}

	static void SerializeToArchive(FArchive& Archive, const FString& Value)
	{
		SerializeToArchive( Archive, *Value );
	}

	static FString SanitizeXMLText(FString InString);
	static FString CompModeToText(EDatasmithCompMode Mode);
};

FString FDatasmithSceneXmlWriterImpl::GetLabelAndLayer(const TSharedPtr<IDatasmithActorElement>& ActorElement)
{
	FString LabelAndLayer;

	if (!FString(ActorElement->GetLabel()).IsEmpty())
	{
		LabelAndLayer += TEXT(" label=\"") + SanitizeXMLText(ActorElement->GetLabel()) + TEXT("\"");
	}

	if (!FString(ActorElement->GetLayer()).IsEmpty())
	{
		LabelAndLayer += TEXT(" layer=\"") + SanitizeXMLText(ActorElement->GetLayer()) + TEXT("\"");
	}

	if (!ActorElement->GetVisibility())
	{
		LabelAndLayer += TEXT(" visible=\"false\"");
	}

	if (!ActorElement->GetCastShadow())
	{
		LabelAndLayer += TEXT(" castshadow=\"false\"");
	}

	if (ActorElement->IsAComponent())
	{
		LabelAndLayer += TEXT(" component=\"true\"");
	}

	ensure((int32)ActorElement->GetMobility() < UE_ARRAY_COUNT(DatasmithActorMobilityTypeStrings));
	FString MobilityString = DatasmithActorMobilityTypeStrings[(int32)ActorElement->GetMobility()];

	LabelAndLayer += TEXT(" mobility=\"") + MobilityString + TEXT("\"");

	return LabelAndLayer;
}

void FDatasmithSceneXmlWriterImpl::AppendIndent(FString& XmlString, int32 Indent)
{
	for (int32 i = 0; i < Indent; i++)
	{
		XmlString += TEXT("\t");
	}
}

void FDatasmithSceneXmlWriterImpl::WriteIndent(FArchive& Archive, int32 Indent)
{
	FString XmlString;
	AppendIndent( XmlString, Indent );
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteBool(FArchive& Archive, int32 Indent, const TCHAR* Prefix, bool bValue)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(Prefix) + FString::Printf( TEXT(" enabled=\"%u\"/>"), bValue ? 1 : 0 ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteRGB(FArchive& Archive, int32 Indent, const TCHAR* Prefix, FLinearColor Color)
{
	if (Color.R >= 0.f || Color.G >= 0.f || Color.B >= 0.f)
	{
		WriteIndent(Archive, Indent);
		Color.R = FMath::Pow(Color.R, 2.2f);
		Color.G = FMath::Pow(Color.G, 2.2f);
		Color.B = FMath::Pow(Color.B, 2.2f);

		if (Color.R > 0.0f && Color.R < 0.001f)
		{
			Color.R = 0.001f;
		}

		if (Color.G > 0.0f && Color.G < 0.001f)
		{
			Color.G = 0.001f;
		}

		if (Color.B > 0.0f && Color.B < 0.001f)
		{
			Color.B = 0.001f;
		}

		FString XmlString = TEXT("<") + FString(Prefix) + TEXT(" R=\"") + FString::SanitizeFloat( Color.R ) + TEXT("\" G=\"") +
			FString::SanitizeFloat( Color.G ) + TEXT("\" B=\"") + FString::SanitizeFloat( Color.B ) + TEXT("\"/>") + LINE_TERMINATOR;

		SerializeToArchive( Archive, XmlString );
	}
}

void FDatasmithSceneXmlWriterImpl::WriteValue(FArchive& Archive, int32 Indent, const TCHAR* Prefix, float Value, bool bForceWriteAlways, FString Desc)
{
	if (Value != 0 || bForceWriteAlways == true)
	{
		WriteIndent(Archive, Indent);

		FString XmlString = TEXT("<") + FString(Prefix) + TEXT(" value=\"") + FString::SanitizeFloat( Value );

		if ( !Desc.IsEmpty() )
		{
			XmlString += TEXT("\" desc=\"") + Desc;
		}

		XmlString += FString( "\"/>" ) + LINE_TERMINATOR;
		SerializeToArchive( Archive, XmlString );
	}
}

void FDatasmithSceneXmlWriterImpl::AppendXMLChild(FString& XmlString, int32 Indent, const TCHAR* Tag, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	AppendIndent(XmlString, Indent);
	XmlString += FString::Printf( TEXT("<%s %s=\"%s\""), Tag, AttributeName, AttributeValue );
	XmlString += FString( "/>" ) + LINE_TERMINATOR;
}

void FDatasmithSceneXmlWriterImpl::WriteTexture(FArchive& Archive, int32 Indent, const TCHAR* Prefix, const TCHAR* Name, FDatasmithTextureSampler UV)
{
	WriteIndent(Archive, Indent);

	FString BaseName(Name);
	if ( !BaseName.EndsWith( TEXT("_Tex") ) && !BaseName.EndsWith( TEXT("_Norm") ) )
	{
		BaseName += TEXT("_Tex");
	}

	FString XmlString = TEXT("<") + FString(Prefix) + TEXT(" tex=\"") + SanitizeXMLText( FDatasmithUtils::SanitizeFileName( *BaseName ) ) + FString( "\" " );
	SerializeToArchive( Archive, XmlString );

	XmlString = TEXT("coordinate=\"") + FString::FromInt( UV.CoordinateIndex ) + TEXT("\" ");
	XmlString += TEXT("sx=\"") + FString::SanitizeFloat( UV.ScaleX ) + TEXT("\" ");
	XmlString += TEXT("sy=\"") + FString::SanitizeFloat( UV.ScaleY ) + TEXT("\" ");
	XmlString += TEXT("ox=\"") + FString::SanitizeFloat( UV.OffsetX ) + TEXT("\" ");
	XmlString += TEXT("oy=\"") + FString::SanitizeFloat( UV.OffsetY ) + TEXT("\" ");
	XmlString += TEXT("mx=\"") + FString::FromInt( UV.MirrorX ) + TEXT("\" ");
	XmlString += TEXT("my=\"") + FString::FromInt( UV.MirrorY ) + TEXT("\" ");
	XmlString += TEXT("rot=\"") + FString::SanitizeFloat( UV.Rotation ) + TEXT("\" ");
	XmlString += TEXT("mul=\"") + FString::SanitizeFloat( UV.Multiplier ) + TEXT("\" ");
	XmlString += TEXT("channel=\"") + FString::FromInt( UV.OutputChannel ) + TEXT("\" ");
	XmlString += FString::Printf( TEXT("inv=\"%u\" "), UV.bInvert ? 1 : 0 );
	XmlString += FString::Printf(TEXT("cropped=\"%u\""), UV.bCroppedTexture ? 1 : 0);
	XmlString += FString( "/>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteTransform(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = FString( "<Transform " ) +
		TEXT("tx=\"") + FString::SanitizeFloat( ActorElement->GetTranslation().X ) +
		TEXT("\" ty=\"") + FString::SanitizeFloat( ActorElement->GetTranslation().Y ) +
		TEXT("\" tz=\"") + FString::SanitizeFloat( ActorElement->GetTranslation().Z ) + TEXT("\" ");

	XmlString += TEXT("sx=\"") + FString::SanitizeFloat( ActorElement->GetScale().X ) +
		TEXT("\" sy=\"") + FString::SanitizeFloat( ActorElement->GetScale().Y ) +
		TEXT("\" sz=\"") + FString::SanitizeFloat( ActorElement->GetScale().Z ) + TEXT("\" ");

	XmlString += TEXT("qx=\"") + FString::SanitizeFloat(ActorElement->GetRotation().X) +
		TEXT("\" qy=\"") + FString::SanitizeFloat(ActorElement->GetRotation().Y) +
		TEXT("\" qz=\"") + FString::SanitizeFloat(ActorElement->GetRotation().Z) +
		TEXT("\" qw=\"") + FString::SanitizeFloat(ActorElement->GetRotation().W) + TEXT("\" ");

	XmlString += QuatToHexString(ActorElement->GetRotation()) + TEXT("/>") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteTransform(const FTransform& Transform, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FVector Location = Transform.GetLocation();

	FString XmlString = FString("<Transform ") +
		TEXT("tx=\"") + FString::SanitizeFloat(Location.X) +
		TEXT("\" ty=\"") + FString::SanitizeFloat(Location.Y) +
		TEXT("\" tz=\"") + FString::SanitizeFloat(Location.Z) + TEXT("\" ");

	FVector Scale = Transform.GetScale3D();

	XmlString += TEXT("sx=\"") + FString::SanitizeFloat(Scale.X) +
		TEXT("\" sy=\"") + FString::SanitizeFloat(Scale.Y) +
		TEXT("\" sz=\"") + FString::SanitizeFloat(Scale.Z) + TEXT("\" ");

	FQuat Quaternion = Transform.GetRotation();

	XmlString += TEXT("qx=\"") + FString::SanitizeFloat(Quaternion.X) +
		TEXT("\" qy=\"") + FString::SanitizeFloat(Quaternion.Y) +
		TEXT("\" qz=\"") + FString::SanitizeFloat(Quaternion.Z) +
		TEXT("\" qw=\"") + FString::SanitizeFloat(Quaternion.W) + TEXT("\" ");

	XmlString += QuatToHexString(Quaternion) + TEXT("/>") + LINE_TERMINATOR;

	SerializeToArchive(Archive, XmlString);
}

FString FDatasmithSceneXmlWriterImpl::QuatToHexString(FQuat Value)
{
	double Tmp[4] = {Value.X, Value.Y, Value.Z, Value.W};
	FString Result = TEXT("qhex64=\"") + FString::FromHexBlob((uint8*)Tmp, sizeof(Tmp)) + TEXT("\"");
	return Result;
}

void FDatasmithSceneXmlWriterImpl::WriteMeshElement(const TSharedPtr< IDatasmithMeshElement >& MeshElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_STATICMESHNAME) + TEXT(" name=\"") + SanitizeXMLText( MeshElement->GetName() ) + TEXT("\"");
	XmlString += TEXT(" label=\"") + SanitizeXMLText( MeshElement->GetLabel() ) + TEXT("\"");
	XmlString += FString( ">" ) + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);
	XmlString = TEXT("<file path=\"") + SanitizeXMLText(MeshElement->GetFile()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	if (MeshElement->GetArea() > 0)
	{
		WriteIndent(Archive, Indent+1);
		XmlString = FString("<Size ") +
			TEXT("a=\"") + FString::SanitizeFloat(MeshElement->GetArea()) +
			TEXT("\" x=\"") + FString::SanitizeFloat(MeshElement->GetWidth()) +
			TEXT("\" y=\"") + FString::SanitizeFloat(MeshElement->GetDepth()) +
			TEXT("\" z=\"") + FString::SanitizeFloat(MeshElement->GetHeight()) + TEXT("\"/>") + LINE_TERMINATOR;
		SerializeToArchive(Archive, XmlString);
	}

	WriteIndent(Archive, Indent + 1);
	XmlString = TEXT("<") + FString(DATASMITH_LIGHTMAPCOORDINATEINDEX) + TEXT(" value=\"") + FString::FromInt(MeshElement->GetLightmapCoordinateIndex()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);
	XmlString = TEXT("<") + FString(DATASMITH_LIGHTMAPUVSOURCE) + TEXT(" value=\"") + FString::FromInt( MeshElement->GetLightmapSourceUV() ) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);
	XmlString = TEXT("<") + FString(DATASMITH_HASH) + TEXT(" value=\"") + LexToString(MeshElement->GetFileHash()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	for (int i = 0; i < MeshElement->GetMaterialSlotCount(); ++i)
	{
		WriteIndent(Archive, Indent + 1);
		XmlString = TEXT("<") + FString(DATASMITH_MATERIAL) + FString::Printf(TEXT(" id=\"%d\" name=\""), MeshElement->GetMaterialSlotAt(i)->GetId()) + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(MeshElement->GetMaterialSlotAt(i)->GetName())) + TEXT("\"/>") + LINE_TERMINATOR;
		SerializeToArchive(Archive, XmlString);
	}

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</") + FString(DATASMITH_STATICMESHNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDatasmithSceneXmlWriterImpl::WriteClothElement(const TSharedPtr<IDatasmithClothElement>& Element, FArchive& Archive, int32 Indent)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	WriteIndent(Archive, Indent);
	FString XmlString = TEXT("<") DATASMITH_CLOTH;
	XmlString += TEXT(" name=\"") + SanitizeXMLText( Element->GetName() ) + TEXT("\"");
	XmlString += TEXT(" label=\"") + SanitizeXMLText( Element->GetLabel() ) + TEXT("\"");
	XmlString += TEXT(">") LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);
	SerializeToArchive(Archive, TEXT("<file path=\"") + SanitizeXMLText(Element->GetFile()) + TEXT("\"/>") LINE_TERMINATOR);

	WriteIndent(Archive, Indent);
	SerializeToArchive(Archive, TEXT("</") DATASMITH_CLOTH TEXT(">") LINE_TERMINATOR);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDatasmithSceneXmlWriterImpl::WriteLevelSequenceElement(const TSharedRef< IDatasmithLevelSequenceElement>& SequenceElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_LEVELSEQUENCENAME) + TEXT(" name=\"") + SanitizeXMLText( SequenceElement->GetName() ) + TEXT("\"");
	XmlString += FString( ">" ) + LINE_TERMINATOR;

	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);

	XmlString = TEXT("<file path=\"") + SanitizeXMLText(SequenceElement->GetFile()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);

	XmlString = TEXT("<") + FString(DATASMITH_HASH) + TEXT(" value=\"") + LexToString(SequenceElement->GetFileHash()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_LEVELSEQUENCENAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);
}

void FDatasmithSceneXmlWriterImpl::WriteLevelVariantSetsElement( const TSharedRef< IDatasmithLevelVariantSetsElement >& LevelVariantSetsElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = TEXT( "<" ) + FString( DATASMITH_LEVELVARIANTSETSNAME ) + TEXT( " name=\"" ) + SanitizeXMLText( LevelVariantSetsElement->GetName() ) + TEXT( "\"" );
	XmlString += FString( ">" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	for ( int32 VariantSetIndex = 0; VariantSetIndex < LevelVariantSetsElement->GetVariantSetsCount(); ++VariantSetIndex )
	{
		TSharedPtr<IDatasmithVariantSetElement> VariantSetElement = LevelVariantSetsElement->GetVariantSet( VariantSetIndex );
		if ( VariantSetElement.IsValid() )
		{
			WriteVariantSetElement(VariantSetElement.ToSharedRef(), Archive, Indent + 1);
		}
	}

	WriteIndent( Archive, Indent );

	XmlString = TEXT( "</" ) + FString( DATASMITH_LEVELVARIANTSETSNAME ) + TEXT( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteVariantSetElement( const TSharedRef< IDatasmithVariantSetElement >& VariantSetElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = TEXT( "<" ) + FString( DATASMITH_VARIANTSETNAME ) + TEXT( " name=\"" ) + SanitizeXMLText( VariantSetElement->GetName() ) + TEXT( "\"" );
	XmlString += FString( ">" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	for ( int32 VariantIndex = 0; VariantIndex < VariantSetElement->GetVariantsCount(); ++VariantIndex )
	{
		TSharedPtr<IDatasmithVariantElement> VariantElement = VariantSetElement->GetVariant( VariantIndex );
		if ( VariantElement.IsValid() )
		{
			WriteVariantElement( VariantElement.ToSharedRef(), Archive, Indent + 1 );
		}
	}

	WriteIndent( Archive, Indent );

	XmlString = TEXT( "</" ) + FString( DATASMITH_VARIANTSETNAME ) + TEXT( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteVariantElement( const TSharedRef< IDatasmithVariantElement >& VariantElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = TEXT( "<" ) + FString( DATASMITH_VARIANTNAME ) + TEXT( " name=\"" ) + SanitizeXMLText( VariantElement->GetName() ) + TEXT( "\"" );
	XmlString += FString( ">" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	for ( int32 BindingIndex = 0; BindingIndex < VariantElement->GetActorBindingsCount(); ++BindingIndex )
	{
		TSharedPtr<IDatasmithActorBindingElement> BindingElement = VariantElement->GetActorBinding( BindingIndex );
		if ( BindingElement.IsValid() )
		{
			WriteActorBindingElement( BindingElement.ToSharedRef(), Archive, Indent + 1 );
		}
	}

	WriteIndent( Archive, Indent );

	XmlString = TEXT( "</" ) + FString( DATASMITH_VARIANTNAME ) + TEXT( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteActorBindingElement( const TSharedRef< IDatasmithActorBindingElement >& ActorBindingElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString ActorName;
	TSharedPtr<IDatasmithActorElement> Actor = ActorBindingElement->GetActor();
	if ( Actor.IsValid() )
	{
		ActorName = Actor->GetName();
	}

	FString XmlString = TEXT( "<" ) + FString( DATASMITH_ACTORBINDINGNAME ) + TEXT( " actor=\"" ) + SanitizeXMLText( ActorName ) + TEXT( "\"" );
	XmlString += FString( ">" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	for ( int32 PropertyIndex = 0; PropertyIndex < ActorBindingElement->GetPropertyCapturesCount(); ++PropertyIndex )
	{
		TSharedPtr<IDatasmithBasePropertyCaptureElement> BasePropertyElement = ActorBindingElement->GetPropertyCapture( PropertyIndex );

		if ( BasePropertyElement->IsSubType( EDatasmithElementVariantSubType::PropertyCapture ) )
		{
			TSharedPtr<IDatasmithPropertyCaptureElement> PropertyElement = StaticCastSharedPtr< IDatasmithPropertyCaptureElement >( BasePropertyElement );
			if ( PropertyElement.IsValid() )
			{
				WritePropertyCaptureElement( PropertyElement.ToSharedRef(), Archive, Indent + 1 );
			}
		}
		else if ( BasePropertyElement->IsSubType( EDatasmithElementVariantSubType::ObjectPropertyCapture ) )
		{
			TSharedPtr<IDatasmithObjectPropertyCaptureElement> ObjectPropertyElement = StaticCastSharedPtr< IDatasmithObjectPropertyCaptureElement >( BasePropertyElement );
			if ( ObjectPropertyElement.IsValid() )
			{
				WriteObjectPropertyCaptureElement( ObjectPropertyElement.ToSharedRef(), Archive, Indent + 1 );
			}
		}
	}

	WriteIndent( Archive, Indent );

	XmlString = TEXT( "</" ) + FString( DATASMITH_ACTORBINDINGNAME ) + TEXT( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WritePropertyCaptureElement( const TSharedRef< IDatasmithPropertyCaptureElement >& PropertyCaptureElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	const FString& PropertyPath = PropertyCaptureElement->GetPropertyPath();
	FString PropertyCategory = LexToString( static_cast< uint8 >( PropertyCaptureElement->GetCategory() ) );

	const TArray<uint8>& DataBytes = PropertyCaptureElement->GetRecordedData();
	FString RecordedData = BytesToHex( DataBytes.GetData(), DataBytes.Num() );  // Use hex here to prevent XML injection

	FString XmlString =
		TEXT( "<" ) + FString( DATASMITH_PROPERTYCAPTURENAME ) +
			TEXT( " path=\"" ) + SanitizeXMLText( PropertyPath ) + TEXT( "\"" ) +
			TEXT( " category=\"" ) + SanitizeXMLText( PropertyCategory ) + TEXT( "\"" ) +
			TEXT( " data=\"" ) + SanitizeXMLText( RecordedData ) + TEXT( "\"" ) +
		TEXT( "/>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteObjectPropertyCaptureElement( const TSharedRef< IDatasmithObjectPropertyCaptureElement >& PropertyCaptureElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	const FString& PropertyPath = PropertyCaptureElement->GetPropertyPath();
	FString PropertyCategory = LexToString( static_cast< uint8 >( PropertyCaptureElement->GetCategory() ) );

	FString Object;
	if ( TSharedPtr<IDatasmithElement> PinnedElement = PropertyCaptureElement->GetRecordedObject().Pin() )
	{
		Object = PinnedElement->GetName();
	}

	FString XmlString =
		TEXT( "<" ) + FString( DATASMITH_OBJECTPROPERTYCAPTURENAME ) +
			TEXT( " path=\"" ) + SanitizeXMLText( PropertyPath ) + TEXT( "\"" ) +
			TEXT( " category=\"" ) + SanitizeXMLText( PropertyCategory ) + TEXT( "\"" ) +
			TEXT( " object=\"" ) + SanitizeXMLText( Object ) + TEXT( "\"" ) +
		TEXT( "/>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteActorElement(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent)
{
	if (ActorElement->IsA(EDatasmithElementType::HierarchicalInstanceStaticMesh))
	{
		WriteHierarchicalInstancedMeshElement(StaticCastSharedPtr< IDatasmithHierarchicalInstancedStaticMeshActorElement >(ActorElement), Archive, Indent);
	}
	else if ( ActorElement->IsA( EDatasmithElementType::StaticMeshActor ) )
	{
		WriteMeshActorElement(StaticCastSharedPtr< IDatasmithMeshActorElement >(ActorElement), Archive, Indent);
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if ( ActorElement->IsA( EDatasmithElementType::ClothActor ) )  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		WriteClothActorElement(StaticCastSharedPtr< IDatasmithClothActorElement >(ActorElement), Archive, Indent);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else if ( ActorElement->IsA( EDatasmithElementType::Camera ) )
	{
		WriteCameraActorElement(StaticCastSharedPtr< IDatasmithCameraActorElement >(ActorElement), Archive, Indent);
	}
	else if ( ActorElement->IsA( EDatasmithElementType::Light ) )
	{
		WriteLightActorElement(StaticCastSharedPtr< IDatasmithLightActorElement >(ActorElement), Archive, Indent);
	}
	else if ( ActorElement->IsA( EDatasmithElementType::Decal ) )
	{
		WriteCustomActorElement(StaticCastSharedPtr< IDatasmithCustomActorElement >(ActorElement), Archive, Indent, DATASMITH_DECALACTORNAME);
	}
	else if ( ActorElement->IsA( EDatasmithElementType::CustomActor ) )
	{
		WriteCustomActorElement(StaticCastSharedPtr< IDatasmithCustomActorElement >(ActorElement), Archive, Indent);
	}
	else if ( ActorElement->IsA( EDatasmithElementType::Landscape ) )
	{
		WriteLandscapeActorElement(StaticCastSharedPtr< IDatasmithLandscapeElement >(ActorElement), Archive, Indent);
	}
	else if ( ActorElement->IsA( EDatasmithElementType::PostProcessVolume ) )
	{
		WritePostProcessVolumeElement(StaticCastSharedPtr< IDatasmithPostProcessVolumeElement >(ActorElement), Archive, Indent);
	}
	else
	{
		WriteBaseActorElement(ActorElement, Archive, Indent);
	}
}

void FDatasmithSceneXmlWriterImpl::WriteBaseActorElement(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_ACTORNAME) + TEXT(" name=\"") + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(ActorElement->GetName())) + TEXT("\"");

	XmlString += GetLabelAndLayer(ActorElement) + FString(">") + LINE_TERMINATOR;

	SerializeToArchive(Archive, XmlString);

	WriteTransform(ActorElement, Archive, Indent + 1);

	WriteActorTags(ActorElement, Archive, Indent);

	WriteActorChildren(ActorElement, Archive, Indent);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_ACTORNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);
}

void FDatasmithSceneXmlWriterImpl::WriteActorTags(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent)
{
	int32 TagCount = ActorElement->GetTagsCount();
	FString XmlString;

	for (int32 i = 0; i < TagCount; i++)
	{
		WriteIndent(Archive, Indent + 1);
		XmlString = FString::Printf(TEXT("<tag value=\"%s\" />"), *SanitizeXMLText(ActorElement->GetTag(i))) + LINE_TERMINATOR;
		SerializeToArchive(Archive, XmlString);
	}
}

void FDatasmithSceneXmlWriterImpl::WriteActorChildren(const TSharedPtr< IDatasmithActorElement >& ActorElement, FArchive& Archive, int32 Indent)
{
	FString XmlString;
	if (ActorElement->GetChildrenCount() > 0)
	{
		// Compatibility note: in 4.26 and previous version, this 'visible' attribute was mandatory (because it's value defaulted to false).
		// In 4.27+, the 'visible' attribute moved on the actor tag itself.
		if (!ActorElement->GetVisibility())
		{
			XmlString = TEXT("<children visible=\"false\">");
		}
		else
		{
			XmlString = TEXT("<children visible=\"true\">");
		}
		XmlString += LINE_TERMINATOR;

		WriteIndent(Archive, Indent + 1);
		SerializeToArchive(Archive, XmlString);

		int32 ChildrenCount = ActorElement->GetChildrenCount();
		for (int32 i = 0; i < ChildrenCount; i++)
		{
			WriteActorElement(ActorElement->GetChild(i), Archive, Indent + 2);
		}

		XmlString = FString("</children>") + LINE_TERMINATOR;

		WriteIndent(Archive, Indent + 1);
		SerializeToArchive(Archive, XmlString);
	}
}

void FDatasmithSceneXmlWriterImpl::WriteMeshActorElement(const TSharedPtr< IDatasmithMeshActorElement >& MeshActorElement, FArchive& Archive, int32 Indent)
{
	const FString ElementTypeString(DATASMITH_ACTORMESHNAME);

	WriteBeginOfMeshActorElement(MeshActorElement, ElementTypeString, Archive, Indent);
	WriteActorChildren(MeshActorElement, Archive, Indent);
	WriteEndOfMeshActorElement(ElementTypeString, Archive, Indent);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDatasmithSceneXmlWriterImpl::WriteClothActorElement(const TSharedPtr< IDatasmithClothActorElement >& Element, FArchive& Archive, int32 Indent)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	WriteIndent(Archive, Indent);
	SerializeToArchive(Archive, FString(TEXT("<") DATASMITH_CLOTHACTORNAME TEXT(" name=\"")) + SanitizeXMLText(Element->GetName()) + TEXT("\""));
	SerializeToArchive(Archive, GetLabelAndLayer(Element) + TEXT(">") LINE_TERMINATOR);

	WriteIndent(Archive, Indent + 1);
	SerializeToArchive(Archive, TEXT("<Cloth name=\"") + SanitizeXMLText(Element->GetCloth()) + TEXT("\"/>") LINE_TERMINATOR);

	WriteIndent(Archive, Indent);
	SerializeToArchive(Archive, TEXT("</") DATASMITH_CLOTHACTORNAME TEXT(">") LINE_TERMINATOR);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDatasmithSceneXmlWriterImpl::WriteBeginOfMeshActorElement(const TSharedPtr<IDatasmithMeshActorElement>& MeshActorElement, const FString& ElementTypeString, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + ElementTypeString + TEXT(" name=\"") + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(MeshActorElement->GetName())) + TEXT("\"");

	XmlString += GetLabelAndLayer(MeshActorElement);

	XmlString += FString(">") + LINE_TERMINATOR;

	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent + 1);

	XmlString = FString::Printf(TEXT("<mesh"));

	if (!FString(MeshActorElement->GetStaticMeshPathName()).IsEmpty())
	{
		XmlString += TEXT(" name=\"") + SanitizeXMLText(MeshActorElement->GetStaticMeshPathName()) + TEXT("\"");
	}
	XmlString += FString("/>") + LINE_TERMINATOR;

	SerializeToArchive(Archive, XmlString);

	for (int32 i = 0; i < MeshActorElement->GetMaterialOverridesCount(); ++i)
	{
		const TSharedPtr< IDatasmithMaterialIDElement >& Material = MeshActorElement->GetMaterialOverride(i);

		WriteIndent(Archive, Indent + 1);

		XmlString = FString::Printf(TEXT("<material id=\"%d\" name=\""), Material->GetId()) + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(Material->GetName())) + TEXT("\"/>") + LINE_TERMINATOR;

		SerializeToArchive(Archive, XmlString);
	}

	WriteTransform(MeshActorElement, Archive, Indent + 1);

	WriteActorTags(MeshActorElement, Archive, Indent);
}

void FDatasmithSceneXmlWriterImpl::WriteEndOfMeshActorElement(const FString& ElementTypeString, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("</") + ElementTypeString + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);
}

void FDatasmithSceneXmlWriterImpl::WriteHierarchicalInstancedMeshElement(const TSharedPtr< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedMeshElement, FArchive& Archive, int32 Indent)
{
	const FString&  ElementTypeString(DATASMITH_ACTORHIERARCHICALINSTANCEDMESHNAME);

	WriteBeginOfMeshActorElement(HierarchicalInstancedMeshElement, ElementTypeString, Archive, Indent);

	Indent++;
	WriteIndent(Archive, Indent);
	int32 InstancesCount = HierarchicalInstancedMeshElement->GetInstancesCount();
	FString XmlString = TEXT("<Instances count=\"") + FString::FromInt(InstancesCount) + TEXT("\">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	Indent++;
	for (int32 i = 0; i < InstancesCount; i++)
	{
		WriteTransform(HierarchicalInstancedMeshElement->GetInstance(i), Archive, Indent);
	}
	Indent--;

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</Instances>") LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	Indent--;

	WriteActorChildren(HierarchicalInstancedMeshElement, Archive, Indent);

	WriteEndOfMeshActorElement(ElementTypeString, Archive, Indent);
}

void FDatasmithSceneXmlWriterImpl::WriteCustomActorElement(const TSharedPtr< IDatasmithCustomActorElement >& CustomActorElement, FArchive& Archive, int32 Indent, const TCHAR* XmlNodeName)
{
	WriteIndent(Archive, Indent);

	FString XmlString = "<";
	XmlString += FString(XmlNodeName) + TEXT(" name=\"") + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(CustomActorElement->GetName())) + TEXT("\" ");
	XmlString += FString(DATASMITH_CUSTOMACTORPATHNAME) + TEXT("=\"") + SanitizeXMLText( CustomActorElement->GetClassOrPathName() ) + TEXT("\" ");

	XmlString += GetLabelAndLayer( CustomActorElement );

	XmlString += FString(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteTransform( CustomActorElement, Archive, Indent + 1 );

	WriteKeyValueProperties( *CustomActorElement, Archive, Indent );

	WriteActorTags(CustomActorElement, Archive, Indent);

	WriteActorChildren(CustomActorElement, Archive, Indent);

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</") + FString(XmlNodeName) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteLandscapeActorElement(const TSharedPtr< IDatasmithLandscapeElement >& LandscapeActorElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = "<";
	XmlString += FString(DATASMITH_LANDSCAPENAME) + TEXT(" name=\"") + SanitizeXMLText( LandscapeActorElement->GetName() ) + TEXT("\" ");

	XmlString += GetLabelAndLayer( LandscapeActorElement );

	XmlString += FString(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteTransform( LandscapeActorElement, Archive, Indent + 1 );

	XmlString.Reset();
	AppendXMLChild( XmlString, Indent + 1, DATASMITH_HEIGHTMAPNAME, TEXT("value"), LandscapeActorElement->GetHeightmap() );
	AppendXMLChild( XmlString, Indent + 1, DATASMITH_MATERIAL, DATASMITH_PATHNAME, LandscapeActorElement->GetMaterial() );

	SerializeToArchive( Archive, XmlString );

	WriteActorTags(LandscapeActorElement, Archive, Indent);

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</") + FString(DATASMITH_LANDSCAPENAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteLightActorElement(const TSharedPtr< IDatasmithLightActorElement >& LightActorElement, FArchive& Archive, int32 Indent)
{
	if ( LightActorElement->IsA( EDatasmithElementType::EnvironmentLight ) )
	{
		TSharedRef< IDatasmithEnvironmentElement > EnvironmentElement = StaticCastSharedRef< IDatasmithEnvironmentElement >( LightActorElement.ToSharedRef() );
		WriteEnvironmentElement( EnvironmentElement, Archive, Indent );

		return;
	}

	FString TypeString;
	if ( LightActorElement->IsA( EDatasmithElementType::DirectionalLight ) )
	{
		TypeString = DATASMITH_DIRECTLIGHTNAME;
	}
	else if ( LightActorElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		TypeString = DATASMITH_AREALIGHTNAME;
	}
	else if ( LightActorElement->IsA( EDatasmithElementType::LightmassPortal ) )
	{
		TypeString = DATASMITH_PORTALLIGHTNAME;
	}
	else if ( LightActorElement->IsA( EDatasmithElementType::SpotLight ) )
	{
		TypeString = DATASMITH_SPOTLIGHTNAME;
	}
	else if ( LightActorElement->IsA( EDatasmithElementType::PointLight ) )
	{
		TypeString = DATASMITH_POINTLIGHTNAME;
	}

	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_LIGHTNAME) + TEXT(" name=\"") + SanitizeXMLText( LightActorElement->GetName() );
	XmlString += TEXT("\" type=\"") + TypeString + FString::Printf(TEXT("\" enabled=\"%d\""), LightActorElement->IsEnabled() ? 1 : 0);

	XmlString += GetLabelAndLayer(LightActorElement) + TEXT(">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	WriteTransform(LightActorElement, Archive, Indent + 1);

	WriteIndent(Archive, Indent + 1);

	XmlString = TEXT("<") + FString(DATASMITH_LIGHTCOLORNAME);

	XmlString += TEXT(" ") + FString(DATASMITH_LIGHTUSETEMPNAME) + FString::Printf( TEXT("=\"%d\""), LightActorElement->GetUseTemperature() ? 1 : 0) + TEXT(" ");
	XmlString += FString(DATASMITH_LIGHTTEMPNAME) + TEXT("=\"") + FString::SanitizeFloat( LightActorElement->GetTemperature() ) + TEXT("\"");

	XmlString += TEXT(" R=\"") + FString::SanitizeFloat( LightActorElement->GetColor().R ) +
		TEXT("\" G=\"") + FString::SanitizeFloat( LightActorElement->GetColor().G ) +
		TEXT("\" B=\"") + FString::SanitizeFloat( LightActorElement->GetColor().B );

	XmlString += FString("\"/>") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	if (LightActorElement->GetUseIes())
	{
		if(FCString::Strlen(LightActorElement->GetIesFile()) > 0)
		{
			WriteIndent(Archive, Indent + 1);

			XmlString = TEXT("<") + FString(DATASMITH_LIGHTIESNAME) + TEXT(" file=\"") + SanitizeXMLText(LightActorElement->GetIesFile()) + TEXT("\"/>") + LINE_TERMINATOR;
			SerializeToArchive( Archive, XmlString );
		}

		if(FCString::Strlen(LightActorElement->GetIesTexturePathName()) > 0)
		{
			WriteIndent(Archive, Indent + 1);

			XmlString = TEXT("<") + FString(DATASMITH_LIGHTIESTEXTURENAME) + TEXT(" name=\"") + SanitizeXMLText(LightActorElement->GetIesTexturePathName()) + TEXT("\"/>") + LINE_TERMINATOR;
			SerializeToArchive( Archive, XmlString );
		}

		if (LightActorElement->GetUseIesBrightness())
		{
			WriteIndent(Archive, Indent + 1);

			XmlString = TEXT("<") + FString(DATASMITH_LIGHTIESBRIGHTNAME) + TEXT(" scale=\"") + FString::SanitizeFloat( LightActorElement->GetIesBrightnessScale() ) + TEXT("\"/>") + LINE_TERMINATOR;
			SerializeToArchive( Archive, XmlString );
		}
		else
		{
			WriteIndent(Archive, Indent + 1);

			XmlString = TEXT("<") + FString(DATASMITH_LIGHTIESBRIGHTNAME) + TEXT(" scale=\"-1\"/>") + LINE_TERMINATOR;
			SerializeToArchive( Archive, XmlString );
		}
	}

	if ( LightActorElement->GetIesRotation() != FQuat::Identity )
	{
		auto QuatToHexStringFloat = [](const FQuat& Value) -> FString
		{
			float Tmp[4] = {(float)Value.X, (float)Value.Y, (float)Value.Z, (float)Value.W};
			FString Result = TEXT(" qhex=\"") + FString::FromHexBlob((uint8*)Tmp, sizeof(Tmp)) + TEXT("\"");
			return Result;
		};

		WriteIndent(Archive, Indent + 1);
		XmlString = TEXT("<") DATASMITH_LIGHTIESROTATION TEXT(" ")
			+ QuatToHexString( LightActorElement->GetIesRotation() )
			+ QuatToHexStringFloat( LightActorElement->GetIesRotation() ) // required for 5.1 -> 5.0.0 flow, as 5.0.0 cannot parse a qhex64 attribute. Should be fixed for 5.0.1.
			+ TEXT("/>") + LINE_TERMINATOR;
		SerializeToArchive( Archive, XmlString );
	}

	WriteIndent(Archive, Indent + 1);

	XmlString = TEXT("<") + FString(DATASMITH_LIGHTINTENSITYNAME) + TEXT(" value=\"") + FString::SanitizeFloat( LightActorElement->GetIntensity() ) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	if (LightActorElement->GetLightFunctionMaterial().IsValid())
	{
		WriteIndent(Archive, Indent + 1);
		XmlString = TEXT("<") + FString(DATASMITH_LIGHTMATERIAL) + TEXT(" name=\"") +
			SanitizeXMLText( LightActorElement->GetLightFunctionMaterial()->GetName() ) +
			TEXT("\"/>") + LINE_TERMINATOR;
		SerializeToArchive( Archive, XmlString );
	}

	if ( LightActorElement->IsA( EDatasmithElementType::PointLight ) )
	{
		WritePointLightElement( StaticCastSharedRef< IDatasmithPointLightElement >( LightActorElement.ToSharedRef() ), Archive, Indent );
	}

	if ( LightActorElement->IsA( EDatasmithElementType::SpotLight ) )
	{
		WriteSpotLightElement( StaticCastSharedRef< IDatasmithSpotLightElement >( LightActorElement.ToSharedRef() ), Archive, Indent );
	}

	if ( LightActorElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		WriteAreaLightElement( StaticCastSharedRef< IDatasmithAreaLightElement >( LightActorElement.ToSharedRef() ), Archive, Indent );
	}

	WriteActorTags(LightActorElement, Archive, Indent);

	WriteActorChildren(LightActorElement, Archive, Indent);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_LIGHTNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WritePointLightElement(const TSharedRef< IDatasmithPointLightElement >& PointLightElement, FArchive& Archive, int32 Indent)
{
	if (PointLightElement->GetSourceRadius() >= 0.f)
	{
		WriteValue( Archive, Indent + 1, DATASMITH_LIGHTSOURCESIZENAME, PointLightElement->GetSourceRadius() );
	}

	if (PointLightElement->GetSourceLength() >= 0.f)
	{
		WriteValue( Archive, Indent + 1, DATASMITH_LIGHTSOURCELENGTHNAME, PointLightElement->GetSourceLength() );
	}

	if (PointLightElement->GetAttenuationRadius() >= 0.f)
	{
		WriteValue( Archive, Indent + 1, DATASMITH_LIGHTATTENUATIONRADIUSNAME, PointLightElement->GetAttenuationRadius() );
	}

	FString UnitsString;
	switch ( PointLightElement->GetIntensityUnits() )
	{
	case EDatasmithLightUnits::Candelas :
		UnitsString = TEXT("Candelas");
		break;
	case EDatasmithLightUnits::Lumens :
		UnitsString = TEXT("Lumens");
		break;
	default:
		UnitsString = TEXT("Unitless");
		break;
	}

	WriteIndent(Archive, Indent + 1);

	FString XmlString = TEXT("<") + FString(DATASMITH_LIGHTINTENSITYUNITSNAME) + TEXT(" value=\"") + UnitsString + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteSpotLightElement(const TSharedRef< IDatasmithSpotLightElement >& SpotLightElement, FArchive& Archive, int32 Indent)
{
	WriteValue(Archive, Indent + 1, DATASMITH_LIGHTINNERRADIUSNAME, SpotLightElement->GetInnerConeAngle());
	WriteValue(Archive, Indent + 1, DATASMITH_LIGHTOUTERRADIUSNAME, SpotLightElement->GetOuterConeAngle());
}

void FDatasmithSceneXmlWriterImpl::WriteAreaLightElement(const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent + 1);

	check( (int32)AreaLightElement->GetLightShape() < UE_ARRAY_COUNT( DatasmithAreaLightShapeStrings ) );
	FString ShapeTypeString = DatasmithAreaLightShapeStrings[ (int32)AreaLightElement->GetLightShape() ];


	FString XmlString = TEXT("<") + FString(DATASMITH_AREALIGHTSHAPE) + TEXT(" type=\"") + ShapeTypeString + TEXT("\"");
	XmlString += TEXT(" width=\"") + FString::SanitizeFloat( AreaLightElement->GetWidth() ) + TEXT("\"");
	XmlString += TEXT(" length=\"") + FString::SanitizeFloat( AreaLightElement->GetLength() ) + TEXT("\"");

	check( (int32)AreaLightElement->GetLightType() < UE_ARRAY_COUNT( DatasmithAreaLightTypeStrings ) );
	FString LightTypeString = DatasmithAreaLightTypeStrings[ (int32)AreaLightElement->GetLightType() ];

	XmlString += TEXT(" ") + FString(DATASMITH_AREALIGHTTYPE) + TEXT("=\"") + LightTypeString + TEXT("\"");

	XmlString += TEXT("/>") + FString( LINE_TERMINATOR );

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WritePostProcessElement(const TSharedPtr< IDatasmithPostProcessElement >& PostProcessElement, FArchive& Archive, int32 Indent)
{
	if ( !PostProcessElement.IsValid() )
	{
		return;
	}

	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") DATASMITH_POSTPRODUCTIONNAME TEXT(">") LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	if (PostProcessElement->GetTemperature() != 6500.0f)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_POSTPRODUCTIONTEMP, PostProcessElement->GetTemperature());
	}

	if (PostProcessElement->GetVignette() != 0.f)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_POSTPRODUCTIONVIGNETTE, PostProcessElement->GetVignette());
	}

	if (PostProcessElement->GetColorFilter().R != 0.f || PostProcessElement->GetColorFilter().G != 0.f || PostProcessElement->GetColorFilter().B != 0.f)
	{
		WriteRGB(Archive, Indent + 1, DATASMITH_POSTPRODUCTIONCOLOR, PostProcessElement->GetColorFilter());
	}

	if (PostProcessElement->GetSaturation() != 1.f)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_POSTPRODUCTIONSATURATION, PostProcessElement->GetSaturation());
	}

	if (PostProcessElement->GetCameraISO() > 0.f)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_POSTPRODUCTIONCAMERAISO, PostProcessElement->GetCameraISO());
	}

	if (PostProcessElement->GetCameraShutterSpeed() > 0.f)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_POSTPRODUCTIONSHUTTERSPEED, PostProcessElement->GetCameraShutterSpeed());
	}

	if (PostProcessElement->GetDepthOfFieldFstop() > 0.f)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_FSTOP, PostProcessElement->GetDepthOfFieldFstop());
	}

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_POSTPRODUCTIONNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WritePostProcessVolumeElement(const TSharedPtr< IDatasmithPostProcessVolumeElement >& PostElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_POSTPROCESSVOLUME) + TEXT(" name=\"") + SanitizeXMLText( PostElement->GetName() ) + ("\"");

	XmlString += GetLabelAndLayer(PostElement) + TEXT(">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	WriteTransform(PostElement, Archive, Indent + 1);

	WriteValue(Archive, Indent + 1, DATASMITH_ENABLED, PostElement->GetEnabled());
	WriteValue(Archive, Indent + 1, DATASMITH_POSTPROCESSVOLUME_UNBOUND, PostElement->GetUnbound());

	WritePostProcessElement(PostElement->GetSettings(), Archive, Indent + 1);

	WriteActorTags(PostElement, Archive, Indent);

	WriteActorChildren(PostElement, Archive, Indent);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_POSTPROCESSVOLUME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteCameraActorElement(const TSharedPtr< IDatasmithCameraActorElement >& CameraElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_CAMERANAME) + TEXT(" name=\"") + SanitizeXMLText( CameraElement->GetName() ) + TEXT("\"");

	XmlString += GetLabelAndLayer(CameraElement) + TEXT(">") + LINE_TERMINATOR;

	if ( FCString::Strlen( CameraElement->GetLookAtActor() ) > 0 )
	{
		AppendXMLChild( XmlString, Indent + 1, DATASMITH_LOOKAT, DATASMITH_ACTORNAME, CameraElement->GetLookAtActor() );
	}

	SerializeToArchive( Archive, XmlString );

	WriteTransform(CameraElement, Archive, Indent + 1);

	WriteValue(Archive, Indent + 1, DATASMITH_SENSORWIDTH, CameraElement->GetSensorWidth());
	WriteValue(Archive, Indent + 1, DATASMITH_SENSORASPECT, CameraElement->GetSensorAspectRatio());
	WriteBool(Archive, Indent + 1, DATASMITH_DEPTHOFFIELD, CameraElement->GetEnableDepthOfField());
	WriteValue(Archive, Indent + 1, DATASMITH_FOCUSDISTANCE, CameraElement->GetFocusDistance());
	WriteValue(Archive, Indent + 1, DATASMITH_FSTOP, CameraElement->GetFStop());
	WriteValue(Archive, Indent + 1, DATASMITH_FOCALLENGTH, CameraElement->GetFocalLength());
	WriteBool(Archive, Indent + 1, DATASMITH_LOOKATROLL, CameraElement->GetLookAtAllowRoll());

	WritePostProcessElement(CameraElement->GetPostProcess(), Archive, Indent + 1);

	WriteActorTags(CameraElement, Archive, Indent);

	WriteActorChildren(CameraElement, Archive, Indent);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_CAMERANAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteEnvironmentElement(const TSharedPtr< IDatasmithEnvironmentElement >& EnvironmentElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_ENVIRONMENTNAME) + TEXT(" name=\"") + SanitizeXMLText( EnvironmentElement->GetName() ) + ("\"");
	XmlString += GetLabelAndLayer(EnvironmentElement) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteCompTex(EnvironmentElement->GetEnvironmentComp(), Archive, Indent + 1);
	WriteBool(Archive, Indent + 1, DATASMITH_ENVILLUMINATIONMAP, EnvironmentElement->GetIsIlluminationMap());

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_ENVIRONMENTNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteShaderElement(const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<") + FString(DATASMITH_SHADERNAME) + TEXT(" name=\"") + SanitizeXMLText( FDatasmithUtils::SanitizeFileName(ShaderElement->GetName()) ) + TEXT("\">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	if (ShaderElement->GetIsStackedLayer())
	{
		WriteBool(Archive, Indent + 1, DATASMITH_STACKLAYER, true);
	}

	if (ShaderElement->GetBlendMode() != EDatasmithBlendMode::Alpha)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_BLENDMODE, (float)ShaderElement->GetBlendMode());
	}

	if (ShaderElement->GetIOR() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_IORVALUENAME, (float)ShaderElement->GetIOR());
	}

	if (ShaderElement->GetIORk() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_IORKVALUENAME, (float)ShaderElement->GetIORk());
	}

	if (ShaderElement->GetIORRefra() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_REFRAIORVALUENAME, (float)ShaderElement->GetIORRefra());
	}

	if (ShaderElement->GetBumpAmount() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_BUMPVALUENAME, (float)ShaderElement->GetBumpAmount());
	}

	if (ShaderElement->GetEmitTemperature() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_EMITTEMPNAME, (float)ShaderElement->GetEmitTemperature());
	}

	if (ShaderElement->GetEmitPower() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_EMITVALUENAME, (float)ShaderElement->GetEmitPower());
	}

	if (ShaderElement->GetLightOnly())
	{
		WriteBool(Archive, Indent + 1, DATASMITH_EMITONLYVALUENAME, ShaderElement->GetLightOnly());
	}

	if (ShaderElement->GetTwoSided())
	{
		WriteBool(Archive, Indent + 1, DATASMITH_TWOSIDEDVALUENAME, ShaderElement->GetTwoSided());
	}

	if (ShaderElement->GetMetal() > 0)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_METALVALUENAME, (float)ShaderElement->GetMetal());
	}

	if (ShaderElement->GetUseEmissiveForDynamicAreaLighting())
	{
		WriteBool(Archive, Indent + 1, DATASMITH_DYNAMICEMISSIVE, ShaderElement->GetUseEmissiveForDynamicAreaLighting());
	}

	if (ShaderElement->GetShaderUsage() != EDatasmithShaderUsage::Surface)
	{
		WriteValue(Archive, Indent + 1, DATASMITH_SHADERUSAGE, (float)ShaderElement->GetShaderUsage());
	}

	WriteCompTex(ShaderElement->GetDiffuseComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetRefleComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetRoughnessComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetNormalComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetBumpComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetTransComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetMaskComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetMetalComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetEmitComp(), Archive, Indent + 1);
	WriteCompTex(ShaderElement->GetWeightComp(), Archive, Indent + 1);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_SHADERNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteCompTex(const TSharedPtr< IDatasmithCompositeTexture >& Comp, FArchive& Archive, int32 Indent)
{
	if (Comp->GetMode() == EDatasmithCompMode::Regular)
	{
		if (Comp->GetParamSurfacesCount() > 0)
		{
			int32 j = Comp->GetParamSurfacesCount() - 1;
			if (Comp->GetUseTexture(j))
			{
				WriteTexture(Archive, Indent, Comp->GetBaseTextureName(), Comp->GetParamTexture(j), Comp->GetParamTextureSampler(j));
			}
			if (Comp->GetUseColor(j))
			{
				WriteRGB(Archive, Indent, Comp->GetBaseColName(), Comp->GetParamColor(j));
			}
		}
		if (Comp->GetParamVal1Count() > 0)
		{
			int32 i = Comp->GetParamVal1Count() - 1;
			WriteValue(Archive, Indent, Comp->GetBaseValName(), Comp->GetParamVal1(i).Key, true, Comp->GetParamVal1(i).Value);
		}
	}
	else
	{
		WriteIndent(Archive, Indent);

		FString XmlString = TEXT("<") + FString( Comp->GetBaseCompName() ) + FString::Printf( TEXT(" mode=\"%d\" "), (int32)Comp->GetMode() ) + CompModeToText(Comp->GetMode()) + TEXT(">") + LINE_TERMINATOR;
		SerializeToArchive( Archive, XmlString );

		for (int32 j = 0; j < Comp->GetParamSurfacesCount(); j++)
		{
			if (Comp->GetUseTexture(j))
			{
				WriteTexture(Archive, Indent + 1, DATASMITH_TEXTURENAME, Comp->GetParamTexture(j), Comp->GetParamTextureSampler(j));
			}

			if (Comp->GetUseColor(j))
			{
				WriteRGB(Archive, Indent + 1, DATASMITH_COLORNAME, Comp->GetParamColor(j));
			}

			if (Comp->GetUseComposite(j))
			{
				WriteCompTex(Comp->GetParamSubComposite(j), Archive, Indent + 1);
			}
		}

		for (int32 i = 0; i < Comp->GetParamMaskSurfacesCount(); i++)
		{
			TSharedPtr<IDatasmithCompositeTexture>& SubComp = const_cast< TSharedPtr<IDatasmithCompositeTexture>& >( Comp->GetParamMaskSubComposite(i) ); // TODO : Remove const_cast, I'm not sure if this is the right place to rename the SubComp anyway.
			SubComp->SetBaseNames(DATASMITH_MASKNAME, DATASMITH_MASKCOLOR, DATASMITH_VALUE1NAME, DATASMITH_MASKCOMPNAME);
			WriteCompTex(Comp->GetParamMaskSubComposite(i), Archive, Indent + 1);
		}

		for (int32 i = 0; i < Comp->GetParamVal1Count(); i++)
		{
			WriteValue(Archive, Indent + 1, DATASMITH_VALUE1NAME, Comp->GetParamVal1(i).Key, true, Comp->GetParamVal1(i).Value);
		}

		for (int32 i = 0; i < Comp->GetParamVal2Count(); i++)
		{
			WriteValue(Archive, Indent + 1, DATASMITH_VALUE2NAME, Comp->GetParamVal2(i).Key, true, Comp->GetParamVal2(i).Value);
		}

		WriteIndent(Archive, Indent);

		XmlString = TEXT("</") + FString( Comp->GetBaseCompName() ) + TEXT(">") + LINE_TERMINATOR;
		SerializeToArchive( Archive, XmlString );
	}
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialElement( TSharedPtr< IDatasmithMaterialElement >& MaterialElement, FArchive& Archive, int32 Indent)
{
	if (MaterialElement->GetShadersCount() < 1) // invalid material
	{
		return;
	}

	WriteIndent(Archive, Indent);

	FString XmlString = "<";
	XmlString += FString(DATASMITH_MATERIALNAME) + TEXT(" name=\"") + SanitizeXMLText(MaterialElement->GetName()) + TEXT("\"");
	XmlString += TEXT(" label=\"") + SanitizeXMLText(MaterialElement->GetLabel()) + TEXT("\"");
	XmlString += FString(">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	for (int32 i = MaterialElement->GetShadersCount() - 1; i > 0; i--)
	{
		int32 DupeNames = 0;
		for (int32 j = i - 1; j >= 0; j--)
		{
			if (FString(MaterialElement->GetShader(j)->GetName()) == MaterialElement->GetShader(i)->GetName())
			{
				DupeNames++;
			}
		}

		if (DupeNames > 0)
		{
			FString ss = MaterialElement->GetShader(i)->GetName() + FString::Printf( TEXT("_%d"), DupeNames );
			MaterialElement->GetShader(i)->SetName( *ss );
		}
	}

	for (int32 i = MaterialElement->GetShadersCount() - 1; i >= 0; i--)
	{
		WriteShaderElement(MaterialElement->GetShader(i), Archive, Indent + 1);
	}

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_MATERIALNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialInstanceElement(TSharedPtr< IDatasmithMaterialInstanceElement >& MaterialInstanceElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = "<";
	XmlString += FString(DATASMITH_MATERIALINSTANCENAME) + TEXT(" name=\"") + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(MaterialInstanceElement->GetName())) + TEXT("\" ");
	XmlString += TEXT(" label=\"") + SanitizeXMLText(MaterialInstanceElement->GetLabel()) + TEXT("\" ");
	XmlString += FString(DATASMITH_MATERIALINSTANCETYPE) + TEXT("=\"") + FString::FromInt( (int32)MaterialInstanceElement->GetMaterialType() ) + TEXT("\" ");
	XmlString += FString(DATASMITH_MATERIALINSTANCEQUALITY) + TEXT("=\"") + FString::FromInt( (int32)MaterialInstanceElement->GetQuality() ) + TEXT("\" ");

	if ( MaterialInstanceElement->GetMaterialType() == EDatasmithReferenceMaterialType::Custom )
	{
		XmlString += FString(DATASMITH_MATERIALINSTANCEPATHNAME) + TEXT("=\"") + SanitizeXMLText( MaterialInstanceElement->GetCustomMaterialPathName() ) + TEXT("\" ");
	}

	XmlString += FString(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteKeyValueProperties( *MaterialInstanceElement, Archive, Indent );

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</") + FString(DATASMITH_MATERIALINSTANCENAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteDecalMaterialElement(TSharedPtr< IDatasmithDecalMaterialElement >& DecalMaterialElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = "<";
	XmlString += FString(DATASMITH_DECALMATERIALNAME) + TEXT(" name=\"") + SanitizeXMLText(FDatasmithUtils::SanitizeFileName(DecalMaterialElement->GetName())) + TEXT("\" ");
	XmlString += TEXT(" label=\"") + SanitizeXMLText(DecalMaterialElement->GetLabel()) + TEXT("\" ");

	XmlString += FString(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteIndent(Archive, Indent + 1);
	XmlString = TEXT("<") + FString(DATASMITH_DIFFUSETEXNAME) + TEXT(" PathName=\"") + SanitizeXMLText(DecalMaterialElement->GetDiffuseTexturePathName()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteIndent(Archive, Indent + 1);
	XmlString = TEXT("<") + FString(DATASMITH_NORMALTEXNAME) + TEXT(" PathName=\"") + SanitizeXMLText(DecalMaterialElement->GetNormalTexturePathName()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</") + FString(DATASMITH_DECALMATERIALNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteUEPbrMaterialElement( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = "<";
	XmlString += FString(DATASMITH_UEPBRMATERIALNAME) + TEXT(" name=\"") + SanitizeXMLText( MaterialElement->GetName() ) + TEXT("\"");

	if ( FCString::Strcmp( MaterialElement->GetLabel(), MaterialElement->GetName() ) != 0 )
	{
		XmlString += TEXT(" label=\"") + SanitizeXMLText( MaterialElement->GetLabel() ) + TEXT("\"");
	}

	if ( FCString::Strcmp( MaterialElement->GetParentLabel(), MaterialElement->GetLabel() ) != 0 )
	{
		XmlString += TEXT(" ") + FString( DATASMITH_PARENTMATERIALLABEL ) + TEXT("=\"") + SanitizeXMLText( MaterialElement->GetParentLabel() ) + TEXT("\"");
	}

	XmlString += FString( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteUEPbrMaterialExpressions( MaterialElement, Archive, Indent + 1 );

	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetBaseColor(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetMetallic(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetRoughness(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetSpecular(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetEmissiveColor(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetOpacity(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetNormal(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetRefraction(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetAmbientOcclusion(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetClearCoat(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetClearCoatRoughness(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetWorldPositionOffset(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, MaterialElement->GetMaterialAttributes(), Archive, Indent + 1 );

	if ( MaterialElement->GetUseMaterialAttributes() )
	{
		WriteBool( Archive, Indent + 1, DATASMITH_USEMATERIALATTRIBUTESNAME, MaterialElement->GetUseMaterialAttributes() );
	}
	if ( MaterialElement->GetTwoSided() )
	{
		WriteBool( Archive, Indent + 1, DATASMITH_TWOSIDEDVALUENAME, MaterialElement->GetTwoSided() );
	}
	if ( MaterialElement->GetMaterialFunctionOnly() )
	{
		WriteBool( Archive, Indent + 1, DATASMITH_FUNCTIONLYVALUENAME, MaterialElement->GetMaterialFunctionOnly() );
	}
	WriteValue( Archive, Indent + 1, DATASMITH_BLENDMODE, MaterialElement->GetBlendMode() );
	WriteValue( Archive, Indent + 1, DATASMITH_OPACITYMASKCLIPVALUE, MaterialElement->GetOpacityMaskClipValue() );
	WriteValue( Archive, Indent + 1, DATASMITH_TRANSLUCENCYLIGHTINGMODE, MaterialElement->GetTranslucencyLightingMode() );

	if ( MaterialElement->GetShadingModel() != EDatasmithShadingModel::DefaultLit && (int32)MaterialElement->GetShadingModel() < UE_ARRAY_COUNT( DatasmithShadingModelStrings ) )
	{
		XmlString.Empty();
		AppendXMLChild( XmlString, Indent + 1, DATASMITH_SHADINGMODEL, TEXT("value"), DatasmithShadingModelStrings[ (int32)MaterialElement->GetShadingModel() ] );
		SerializeToArchive( Archive, XmlString );
	}

	WriteIndent( Archive, Indent );
	XmlString = TEXT("</") + FString(DATASMITH_UEPBRMATERIALNAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteUEPbrMaterialExpressions( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );
	FString XmlString = FString( "<Expressions>" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	for ( int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ++ExpressionIndex )
	{
		const IDatasmithMaterialExpression* MaterialExpression = MaterialElement->GetExpression( ExpressionIndex );

		if ( MaterialExpression )
		{
			if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Texture ) )
			{
				const IDatasmithMaterialExpressionTexture* ExpressionTexture = static_cast< const IDatasmithMaterialExpressionTexture* >( MaterialExpression );

				WriteMaterialExpressionTexture( MaterialElement, *ExpressionTexture, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::TextureCoordinate ) )
			{
				const IDatasmithMaterialExpressionTextureCoordinate* ExpressionTextureCoordinate = static_cast< const IDatasmithMaterialExpressionTextureCoordinate* >( MaterialExpression );

				WriteMaterialExpressionTextureCoordinate( MaterialElement, *ExpressionTextureCoordinate, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FlattenNormal ) )
			{
				const IDatasmithMaterialExpressionFlattenNormal* FlattenNormal = static_cast< const IDatasmithMaterialExpressionFlattenNormal* >( MaterialExpression );

				WriteMaterialExpressionFlattenNormal( MaterialElement, *FlattenNormal, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantBool ) )
			{
				const IDatasmithMaterialExpressionBool* ConstantBool = static_cast< const IDatasmithMaterialExpressionBool* >( MaterialExpression );
				WriteMaterialExpressionBool( MaterialElement, *ConstantBool, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantColor ) )
			{
				const IDatasmithMaterialExpressionColor* ConstantColor = static_cast< const IDatasmithMaterialExpressionColor* >( MaterialExpression );
				WriteMaterialExpressionColor( MaterialElement, *ConstantColor, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantScalar ) )
			{
				const IDatasmithMaterialExpressionScalar* ConstantScalar = static_cast< const IDatasmithMaterialExpressionScalar* >( MaterialExpression );
				WriteMaterialExpressionScalar( MaterialElement, *ConstantScalar, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Generic ) )
			{
				const IDatasmithMaterialExpressionGeneric* GenericExpression = static_cast< const IDatasmithMaterialExpressionGeneric* >( MaterialExpression );
				WriteMaterialExpressionGeneric( MaterialElement, *GenericExpression, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FunctionCall ) )
			{
				const IDatasmithMaterialExpressionFunctionCall* FunctionCall = static_cast< const IDatasmithMaterialExpressionFunctionCall* >( MaterialExpression );
				WriteMaterialExpressionFunctionCall( MaterialElement, *FunctionCall, Archive, Indent + 1 );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Custom ) )
			{
				const IDatasmithMaterialExpressionCustom* Expression = static_cast< const IDatasmithMaterialExpressionCustom* >( MaterialExpression );
				WriteMaterialExpressionCustom( MaterialElement, *Expression, Archive, Indent + 1 );
			}
		}
	}

	WriteIndent( Archive, Indent );
	XmlString = TEXT("</") + FString( "Expressions" ) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteUEPbrMaterialExpressionInput( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithExpressionInput& ExpressionInput,
	FArchive& Archive, int32 Indent )
{
	if ( const IDatasmithMaterialExpression* Expression = ExpressionInput.GetExpression() )
	{
		WriteIndent( Archive, Indent );

		FString XmlString = TEXT("<Input Name=\"") + FString( ExpressionInput.GetName() ) + TEXT("\" ");
		XmlString += TEXT("expression=\"") + FString::FromInt( MaterialElement->GetExpressionIndex( Expression ) ) + TEXT("\" OutputIndex=\"") +
			FString::FromInt( ExpressionInput.GetOutputIndex() ) + TEXT("\"/>") + LINE_TERMINATOR;

		SerializeToArchive( Archive, XmlString );
	}
}

void FDatasmithSceneXmlWriterImpl::AppendMaterialExpressionAttributes( const IDatasmithMaterialExpression& Expression, FString& XmlString )
{
	XmlString += FString( "Name=\"" ) + Expression.GetName() + FString( "\"" );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionFlattenNormal( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionFlattenNormal& FlattenNormal, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = FString( "<FlattenNormal>" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteUEPbrMaterialExpressionInput( MaterialElement, FlattenNormal.GetNormal(), Archive, Indent + 1 );
	WriteUEPbrMaterialExpressionInput( MaterialElement, FlattenNormal.GetFlatness(), Archive, Indent + 1 );

	WriteIndent( Archive, Indent );
	XmlString = TEXT("</") + FString( "FlattenNormal" ) + TEXT(">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionTexture( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionTexture& TextureExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = "<Texture ";
	AppendMaterialExpressionAttributes( TextureExpression, XmlString );
	XmlString += FString( " PathName=\"" ) + TextureExpression.GetTexturePathName() + FString( "\">" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	WriteUEPbrMaterialExpressionInput( MaterialElement, TextureExpression.GetInputCoordinate(), Archive, Indent + 1 );

	WriteIndent( Archive, Indent );
	XmlString = TEXT("</") + FString( "Texture" ) + TEXT(">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionTextureCoordinate( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionTextureCoordinate& TextureCoordinateExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = TEXT("<TextureCoordinate Index=\"") + FString::FromInt( TextureCoordinateExpression.GetCoordinateIndex() ) +
		TEXT("\" UTiling=\"") + FString::SanitizeFloat( TextureCoordinateExpression.GetUTiling() ) +
		TEXT("\" VTiling=\"") + FString::SanitizeFloat( TextureCoordinateExpression.GetVTiling() ) +
		TEXT("\"/>") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionBool( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionBool& BoolExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = "<Bool ";
	AppendMaterialExpressionAttributes( BoolExpression, XmlString );
	XmlString += TEXT(" constant=\"" ) + LexToString( BoolExpression.GetBool() ) + TEXT("\"");
	XmlString += FString( "/>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionColor( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionColor& ColorExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = "<Color ";
	AppendMaterialExpressionAttributes( ColorExpression, XmlString );
	XmlString += TEXT(" constant=\"" ) + ColorExpression.GetColor().ToString() + TEXT("\"");
	XmlString += FString( "/>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionScalar( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionScalar& ScalarExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = "<Scalar ";
	AppendMaterialExpressionAttributes( ScalarExpression, XmlString );
	XmlString += TEXT(" constant=\"" ) + FString::SanitizeFloat( ScalarExpression.GetScalar() ) + TEXT("\"");
	XmlString += FString( "/>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionGeneric( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionGeneric& GenericExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = TEXT("<") + FString( GenericExpression.GetExpressionName() ) + FString( " " );
	AppendMaterialExpressionAttributes( GenericExpression, XmlString );
	XmlString += FString( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	WriteKeyValueProperties( GenericExpression, Archive, Indent );

	for ( int32 InputIndex = 0 ; InputIndex < GenericExpression.GetInputCount(); ++InputIndex )
	{
		WriteUEPbrMaterialExpressionInput( MaterialElement, *GenericExpression.GetInput( InputIndex ), Archive, Indent + 1 );
	}

	WriteIndent( Archive, Indent );
	XmlString = TEXT("</") + FString( GenericExpression.GetExpressionName() ) + TEXT(">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionFunctionCall( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionFunctionCall& FunctionCall, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	FString XmlString = TEXT("<FunctionCall Function=\"") + FString( FunctionCall.GetFunctionPathName() ) + TEXT("\"");
	XmlString += FString( ">" ) + LINE_TERMINATOR;
	SerializeToArchive( Archive, XmlString );

	for ( int32 InputIndex = 0 ; InputIndex < FunctionCall.GetInputCount(); ++InputIndex )
	{
		WriteUEPbrMaterialExpressionInput( MaterialElement, *FunctionCall.GetInput( InputIndex ), Archive, Indent + 1 );
	}

	WriteIndent( Archive, Indent );
	XmlString = FString( "</FunctionCall>" ) + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );
}

void FDatasmithSceneXmlWriterImpl::WriteMaterialExpressionCustom( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const IDatasmithMaterialExpressionCustom& CustomExpression, FArchive& Archive, int32 Indent )
{
	WriteIndent( Archive, Indent );

	SerializeToArchive( Archive, TEXT("<Custom ") );
	FString XmlString;
	AppendMaterialExpressionAttributes( CustomExpression, XmlString );
	XmlString += FString::Printf( TEXT(" OutputType=\"%d\""), (uint32)CustomExpression.GetOutputType());
	XmlString += FString::Printf( TEXT(" Description=\"%s\""), *SanitizeXMLText(CustomExpression.GetDescription()));
	SerializeToArchive( Archive, XmlString );
	SerializeToArchive( Archive, TEXT(">") LINE_TERMINATOR );

	WriteIndent( Archive, Indent + 1 );
	SerializeToArchive( Archive, TEXT("<Code>") );
	SerializeToArchive( Archive, SanitizeXMLText(CustomExpression.GetCode()) );
	SerializeToArchive( Archive, TEXT("</Code>") LINE_TERMINATOR );

	for ( int32 PathIndex = 0 ; PathIndex < CustomExpression.GetIncludeFilePathCount(); ++PathIndex )
	{
		WriteIndent( Archive, Indent + 1 );
		FString Path = SanitizeXMLText(CustomExpression.GetIncludeFilePath(PathIndex));
		SerializeToArchive( Archive, FString::Printf( TEXT("<Include path=\"%s\"/>") LINE_TERMINATOR, *Path));
	}

	for ( int32 Index = 0 ; Index < CustomExpression.GetAdditionalDefineCount(); ++Index )
	{
		WriteIndent( Archive, Indent + 1 );
		FString Define = SanitizeXMLText(CustomExpression.GetAdditionalDefine(Index));
		SerializeToArchive( Archive, FString::Printf( TEXT("<Define value=\"%s\"/>") LINE_TERMINATOR, *Define));
	}

	for ( int32 Index = 0 ; Index < CustomExpression.GetArgumentNameCount(); ++Index )
	{
		WriteIndent( Archive, Indent + 1 );
		FString ArgName = SanitizeXMLText(CustomExpression.GetArgumentName(Index));
		SerializeToArchive( Archive, FString::Printf( TEXT("<Arg index=\"%d\" name=\"%s\" />") LINE_TERMINATOR, Index, *ArgName));
	}

	for ( int32 InputIndex = 0 ; InputIndex < CustomExpression.GetInputCount(); ++InputIndex )
	{
		WriteUEPbrMaterialExpressionInput( MaterialElement, *CustomExpression.GetInput( InputIndex ), Archive, Indent + 1 );
	}

	WriteIndent( Archive, Indent );
	SerializeToArchive( Archive,  TEXT("</Custom>") LINE_TERMINATOR );
}

void FDatasmithSceneXmlWriterImpl::WriteTextureElement(const TSharedPtr< IDatasmithTextureElement >& TextureElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	FString XmlString = TEXT("<Texture name=\"") + SanitizeXMLText( FDatasmithUtils::SanitizeFileName( TextureElement->GetName() ) );

	if ( !FString( TextureElement->GetLabel() ).IsEmpty() )
	{
		XmlString += TEXT("\" label=\"") + SanitizeXMLText( TextureElement->GetLabel() );
	}

	XmlString += FString::Printf( TEXT("\" texturemode=\"%i"), (int)TextureElement->GetTextureMode() )
		+ FString::Printf( TEXT("\" texturefilter=\"%i"), (int)TextureElement->GetTextureFilter() )
		+ FString::Printf( TEXT("\" textureaddressx=\"%i"), (int)TextureElement->GetTextureAddressX() )
		+ FString::Printf( TEXT("\" textureaddressy=\"%i"), (int)TextureElement->GetTextureAddressY() )
		+ FString::Printf( TEXT("\" rgbcurve=\"%f"), TextureElement->GetRGBCurve() )
		+ FString::Printf( TEXT("\" srgb=\"%i"), (int)TextureElement->GetSRGB() )
		+ TEXT("\" file=\"") + FString( TextureElement->GetFile() ) + TEXT("\">") + LINE_TERMINATOR;

	SerializeToArchive( Archive, XmlString );

	WriteIndent(Archive, Indent + 1);

	XmlString = TEXT("<") + FString(DATASMITH_HASH) + TEXT(" value=\"") + LexToString(TextureElement->GetFileHash()) + TEXT("\"/>") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteIndent(Archive, Indent);

	XmlString = TEXT("</") + FString(DATASMITH_TEXTURENAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);
}

void FDatasmithSceneXmlWriterImpl::WriteMetaDataElement(const TSharedPtr< IDatasmithMetaDataElement >& MetaDataElement, FArchive& Archive, int32 Indent)
{
	WriteIndent(Archive, Indent);

	// Save the associated element as "ElementType.ElementName"
	FString ElementName = SanitizeXMLText(MetaDataElement->GetName());
	FString ReferenceName = ElementName;

	FString TypeString;
	const TSharedPtr<IDatasmithElement>& AssociatedElement = MetaDataElement->GetAssociatedElement();
	EDatasmithElementType ElementType = EDatasmithElementType::None;
	if (AssociatedElement.IsValid())
	{
		ReferenceName = AssociatedElement->GetName();
		if (AssociatedElement->IsA(EDatasmithElementType::Actor))
		{
			TypeString = TEXT("Actor");
		}
		else if (AssociatedElement->IsA(EDatasmithElementType::Texture))
		{
			TypeString = TEXT("Texture");
		}
		else if (AssociatedElement->IsA(EDatasmithElementType::BaseMaterial))
		{
			TypeString = TEXT("Material");
		}
		else if (AssociatedElement->IsA(EDatasmithElementType::StaticMesh))
		{
			TypeString = TEXT("StaticMesh");
		}
		else
		{
			ensure(false);
		}
	}

	FString XmlString = "<";
	XmlString += FString(DATASMITH_METADATANAME) + TEXT(" name=\"") + ElementName + TEXT("\" ");
	XmlString += FString(DATASMITH_REFERENCENAME) + TEXT("=\"") + TypeString + TEXT(".") + ReferenceName + TEXT("\">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);

	WriteKeyValueProperties(*MetaDataElement, Archive, Indent);

	WriteIndent(Archive, Indent);
	XmlString = TEXT("</") + FString(DATASMITH_METADATANAME) + TEXT(">") + LINE_TERMINATOR;
	SerializeToArchive(Archive, XmlString);
}

template< typename ElementType >
void FDatasmithSceneXmlWriterImpl::WriteKeyValueProperties(const ElementType& Element, FArchive& Archive, int32 Indent)
{
	for ( int32 i = 0; i < Element.GetPropertiesCount(); ++i )
	{
		const TSharedPtr< IDatasmithKeyValueProperty >& Property = Element.GetProperty(i);

		WriteIndent(Archive, Indent + 1);
		FString XmlString = TEXT("<") + FString(DATASMITH_KEYVALUEPROPERTYNAME) + TEXT(" name=\"") + SanitizeXMLText((Property->GetName())) + TEXT("\"")
			+ TEXT(" type=\"") + KeyValuePropertyTypeStrings[ (int32)Property->GetPropertyType() ] + TEXT("\"")
			+ TEXT(" val=\"")  + SanitizeXMLText(Property->GetValue()) + TEXT("\"/>") + LINE_TERMINATOR;

		SerializeToArchive( Archive, XmlString );
	}
}

FString FDatasmithSceneXmlWriterImpl::SanitizeXMLText(FString InString)
{
	InString.ReplaceInline( TEXT("&"), TEXT("&amp;"), ESearchCase::CaseSensitive );
	InString.ReplaceInline( TEXT("<"), TEXT("&lt;"), ESearchCase::CaseSensitive );
	InString.ReplaceInline( TEXT(">"), TEXT("&gt;"), ESearchCase::CaseSensitive );
	InString.ReplaceInline( TEXT("\""), TEXT("&quot;"), ESearchCase::CaseSensitive );
	InString.ReplaceInline( TEXT("'"), TEXT("&apos;"), ESearchCase::CaseSensitive );

	return InString;
}

FString FDatasmithSceneXmlWriterImpl::CompModeToText(EDatasmithCompMode Mode)
{
	switch (Mode)
	{
	case EDatasmithCompMode::Regular:
		return TEXT("compMode=\"Regular\" ");
	case EDatasmithCompMode::Mix:
		return TEXT("compMode=\"Mix\" ");
	case EDatasmithCompMode::Fresnel:
		return TEXT("compMode=\"Fresnel\" ");
	case EDatasmithCompMode::Ior:
		return TEXT("compMode=\"IOR\" ");
	case EDatasmithCompMode::ColorCorrectGamma:
		return TEXT("compMode=\"CCGamma\" ");
	case EDatasmithCompMode::ColorCorrectContrast:
		return TEXT("compMode=\"CCContrast\" ");
	case EDatasmithCompMode::Multiply:
		return TEXT("compMode=\"Multiply\" ");
	case EDatasmithCompMode::Composite:
		return TEXT("compMode=\"Composite\" ");
	default:
		return TEXT("compMode=\"Undefined\" ");
	}
}

void FDatasmithSceneXmlWriter::Serialize( TSharedRef< IDatasmithScene > DatasmithScene, FArchive& Archive )
{
	// explicitly set the locale to have consistent exports
	FDatasmithLocaleScope CLocaleScope;

	FString XmlString = FString( "<DatasmithUnrealScene>" ) + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);

	XmlString = FString::Printf( TEXT("<%s>%s</%s>"), DATASMITH_EXPORTERVERSION, *FDatasmithUtils::GetDatasmithFormatVersionAsString(), DATASMITH_EXPORTERVERSION ) + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::AppendIndent( XmlString, 1 );
	XmlString += FString::Printf( TEXT("<%s>%s</%s>"), DATASMITH_EXPORTERSDKVERSION, *FDatasmithUtils::GetEnterpriseVersionAsString(true), DATASMITH_EXPORTERSDKVERSION ) + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);

	XmlString = TEXT("<Host>") + FDatasmithSceneXmlWriterImpl::SanitizeXMLText( DatasmithScene->GetHost() ) + TEXT("</Host>") + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);

	// Add Vendor, ProductName, ProductVersion
	XmlString = TEXT("<Application Vendor=\"") + FDatasmithSceneXmlWriterImpl::SanitizeXMLText( DatasmithScene->GetVendor() )
		+ TEXT("\" ProductName=\"") + FDatasmithSceneXmlWriterImpl::SanitizeXMLText( DatasmithScene->GetProductName() )
		+ TEXT("\" ProductVersion=\"") + FDatasmithSceneXmlWriterImpl::SanitizeXMLText( DatasmithScene->GetProductVersion() ) + TEXT("\"/>") + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive(Archive, XmlString);

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);

	XmlString = TEXT("<ResourcePath>") + FString( DatasmithScene->GetResourcePath() ) + TEXT("</ResourcePath>") + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);

	// Add UserID doing the export and the OS used
	FString OSVersion;
	FString OSSubVersion;
	FPlatformMisc::GetOSVersions(OSVersion, OSSubVersion);

	XmlString = TEXT("<User ID=\"") + FPlatformMisc::GetLoginId() + TEXT("\" OS=\"") + OSVersion + TEXT("\"/>") + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive(Archive, XmlString);

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);

	// Geolocation
	{
		FVector Geolocation = DatasmithScene->GetGeolocation();

		XmlString = FString::Printf( TEXT("<%s"), DATASMITH_GEOLOCATION);

		auto WriteGeolocationComponent = [&XmlString](const TCHAR* AttrName, double Value)
		{
			if (Value != TNumericLimits<double>::Max())
			{
				XmlString += FString::Printf( TEXT(" %s=\"%s\""), AttrName, *FString::SanitizeFloat(Value));
			}
		};
		WriteGeolocationComponent(DATASMITH_GEOLOCATION_LATITUDE, Geolocation.X);
		WriteGeolocationComponent(DATASMITH_GEOLOCATION_LONGITUDE, Geolocation.Y);
		WriteGeolocationComponent(DATASMITH_GEOLOCATION_ELEVATION, Geolocation.Z);
		XmlString += TEXT("/>");
		XmlString += LINE_TERMINATOR;

		FDatasmithSceneXmlWriterImpl::SerializeToArchive(Archive, XmlString);
	}

	FDatasmithSceneXmlWriterImpl::WritePostProcessElement( DatasmithScene->GetPostProcess(), Archive, 1 );

	bool bHasIlluminationEnvironment = false;
	for ( int32 i = DatasmithScene->GetActorsCount() - 1; i >= 0; i-- )
	{
		if ( DatasmithScene->GetActor(i)->IsA( EDatasmithElementType::EnvironmentLight ) )
		{
			TSharedPtr< IDatasmithEnvironmentElement > EnvironmentElement = StaticCastSharedPtr< IDatasmithEnvironmentElement >( DatasmithScene->GetActor(i) );

			bool bIsIlluminationMap = EnvironmentElement->GetIsIlluminationMap();
			if ( bIsIlluminationMap )
			{
				bHasIlluminationEnvironment = true;
				break;
			}
		}
	}

	if (!bHasIlluminationEnvironment && DatasmithScene->GetUsePhysicalSky())
	{
		FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);
		XmlString = TEXT("<") + FString(DATASMITH_PHYSICALSKYNAME) + TEXT(" enabled=\"1\"/>") + LINE_TERMINATOR;
		FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );
	}

	for ( int32 MeshIndex = 0; MeshIndex < DatasmithScene->GetMeshesCount(); ++MeshIndex )
	{
		TSharedPtr< IDatasmithMeshElement > Mesh = DatasmithScene->GetMesh( MeshIndex );
		FDatasmithSceneXmlWriterImpl::WriteMeshElement( Mesh, Archive, 1 );
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for ( int32 Index = 0; Index < DatasmithScene->GetClothesCount(); ++Index )  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		const TSharedPtr< IDatasmithClothElement >& Cloth = DatasmithScene->GetCloth( Index );
		FDatasmithSceneXmlWriterImpl::WriteClothElement( Cloth, Archive, 1 );
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for ( int32 ActorIndex = 0; ActorIndex < DatasmithScene->GetActorsCount(); ++ActorIndex )
	{
		TSharedPtr< IDatasmithActorElement > Actor = DatasmithScene->GetActor( ActorIndex );
		FDatasmithSceneXmlWriterImpl::WriteActorElement(Actor, Archive, 1);
	}

	for ( int32 TextureIndex = 0; TextureIndex < DatasmithScene->GetTexturesCount(); ++TextureIndex )
	{
		TSharedPtr< IDatasmithTextureElement > Texture = DatasmithScene->GetTexture( TextureIndex );
		FDatasmithSceneXmlWriterImpl::WriteTextureElement(Texture, Archive, 1);
	}

	for ( int32 MaterialIndex = 0; MaterialIndex < DatasmithScene->GetMaterialsCount(); ++MaterialIndex )
	{
		TSharedPtr< IDatasmithBaseMaterialElement > BaseMaterialElement = DatasmithScene->GetMaterial( MaterialIndex );

		if ( BaseMaterialElement->IsA( EDatasmithElementType::Material ) )
		{
			TSharedPtr< IDatasmithMaterialElement > MaterialElement = StaticCastSharedPtr< IDatasmithMaterialElement >( BaseMaterialElement );
			FDatasmithSceneXmlWriterImpl::WriteMaterialElement( MaterialElement, Archive, 1 );
		}
		else if ( BaseMaterialElement->IsA( EDatasmithElementType::MaterialInstance ) )
		{
			TSharedPtr< IDatasmithMaterialInstanceElement > MaterialInstanceElement = StaticCastSharedPtr< IDatasmithMaterialInstanceElement >( BaseMaterialElement );
			FDatasmithSceneXmlWriterImpl::WriteMaterialInstanceElement( MaterialInstanceElement, Archive, 1 );
		}
		else if ( BaseMaterialElement->IsA( EDatasmithElementType::DecalMaterial ) )
		{
			TSharedPtr< IDatasmithDecalMaterialElement > DecalMaterialElement = StaticCastSharedPtr< IDatasmithDecalMaterialElement >( BaseMaterialElement );
			FDatasmithSceneXmlWriterImpl::WriteDecalMaterialElement( DecalMaterialElement, Archive, 1 );
		}
		else if ( BaseMaterialElement->IsA( EDatasmithElementType::UEPbrMaterial ) )
		{
			TSharedRef< IDatasmithUEPbrMaterialElement > UEPbrMaterialElement = StaticCastSharedRef< IDatasmithUEPbrMaterialElement >( BaseMaterialElement.ToSharedRef() );
			FDatasmithSceneXmlWriterImpl::WriteUEPbrMaterialElement( UEPbrMaterialElement, Archive, 1 );
		}
	}

	for ( int32 SequenceIndex = 0; SequenceIndex < DatasmithScene->GetLevelSequencesCount(); ++SequenceIndex )
	{
		TSharedPtr< IDatasmithLevelSequenceElement > Sequence = DatasmithScene->GetLevelSequence( SequenceIndex );
		if (Sequence.IsValid())
		{
			FDatasmithSceneXmlWriterImpl::WriteLevelSequenceElement(Sequence.ToSharedRef(), Archive, 1);
		}
	}

	for ( int32 LevelVariantSetsIndex = 0; LevelVariantSetsIndex < DatasmithScene->GetLevelVariantSetsCount(); ++LevelVariantSetsIndex )
	{
		TSharedPtr< IDatasmithLevelVariantSetsElement > LevelVariantSets = DatasmithScene->GetLevelVariantSets( LevelVariantSetsIndex );
		if ( LevelVariantSets.IsValid() )
		{
			FDatasmithSceneXmlWriterImpl::WriteLevelVariantSetsElement( LevelVariantSets.ToSharedRef(), Archive, 1 );
		}
	}

	for ( int32 MetaDataIndex = 0; MetaDataIndex < DatasmithScene->GetMetaDataCount(); ++MetaDataIndex )
	{
		TSharedPtr< IDatasmithMetaDataElement > MetaDataElement = DatasmithScene->GetMetaData( MetaDataIndex );
		FDatasmithSceneXmlWriterImpl::WriteMetaDataElement(MetaDataElement, Archive, 1);
	}

	FDatasmithSceneXmlWriterImpl::WriteIndent(Archive, 1);
	XmlString = TEXT("<") + FString(DATASMITH_EXPORT) + TEXT(" ") + FString(DATASMITH_EXPORTDURATION) + FString::Printf(TEXT("=\"%d\"/>"), DatasmithScene->GetExportDuration()) + LINE_TERMINATOR;
	FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );

	XmlString = TEXT("</DatasmithUnrealScene>");
	FDatasmithSceneXmlWriterImpl::SerializeToArchive( Archive, XmlString );
}