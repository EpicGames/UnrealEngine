// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithTypes.h"
#include "DatasmithDefinitions.h"
#include "Math/UnrealMathUtility.h"

FDatasmithTransformFrameInfo FDatasmithTransformFrameInfo::InvalidFrameInfo(MIN_int32, 0., 0., 0.);

bool FDatasmithTransformFrameInfo::IsValid() const
{
	return FrameNumber != MIN_int32;
}

bool FDatasmithTransformFrameInfo::operator==(const FDatasmithTransformFrameInfo& Other) const
{
	return FMath::IsNearlyEqual(X, Other.X, DOUBLE_KINDA_SMALL_NUMBER) &&
	       FMath::IsNearlyEqual(Y, Other.Y, DOUBLE_KINDA_SMALL_NUMBER) &&
	       FMath::IsNearlyEqual(Z, Other.Z, DOUBLE_KINDA_SMALL_NUMBER);
}

FDatasmithVisibilityFrameInfo FDatasmithVisibilityFrameInfo::InvalidFrameInfo(MIN_int32, false);

bool FDatasmithVisibilityFrameInfo::IsValid() const
{
	return FrameNumber != MIN_int32;
}

bool FDatasmithVisibilityFrameInfo::operator==(const FDatasmithVisibilityFrameInfo& Other) const
{
	return bVisible == Other.bVisible;
}
