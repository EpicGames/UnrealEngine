// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithUtils.h"

#include "DatasmithCore.h"
#include "DatasmithDefinitions.h"
#include "DatasmithAnimationElements.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithMesh.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithVariantElements.h"
#include "IDatasmithSceneElements.h"

#include "Algo/MinElement.h"
#include "Algo/MaxElement.h"
#include "Algo/Sort.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Math/UnrealMath.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "RawMesh.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UObject/NameTypes.h"
#include "UVMapSettings.h"

static const float DATASMITH_FORMAT_VERSION = 0.24f; // Major.Minor - A change in the major version means that backward compatibility is broken

// List of invalid names copied from Engine/Source/Runtime/Core/Private/Misc/FileHelper.cpp
static const FString InvalidNames[] = {
	TEXT("CON"), TEXT("PRN"), TEXT("AUX"), TEXT("CLOCK$"), TEXT("NUL"), TEXT("NONE"),
	TEXT("COM1"), TEXT("COM2"), TEXT("COM3"), TEXT("COM4"), TEXT("COM5"), TEXT("COM6"), TEXT("COM7"), TEXT("COM8"), TEXT("COM9"),
	TEXT("LPT1"), TEXT("LPT2"), TEXT("LPT3"), TEXT("LPT4"), TEXT("LPT5"), TEXT("LPT6"), TEXT("LPT7"), TEXT("LPT8"), TEXT("LPT9")
};

template<typename T, int32 Size>
static constexpr int GetArrayLength(T(&)[Size]) { return Size; }

void FDatasmithUtils::SanitizeNameInplace(FString& InString)
{
	static const TCHAR Original[] = TEXT("ÀÁÂÃÄÅÆÇÈÉÊËÌÍÎÏÐÑÒÓÔÕÖ×ØÙÚÛÜÝÞßàáâãäåæçèéêëìíîïðñòóôõö÷øùúûüýþÿБбВвГгДдЁёЖжЗзИиЙйКкЛлМмНнОоПпРрСсТтУуФфХхЦцЧчШшЩщЪъЫыЬьЭэЮюЯя'\"");
	static const TCHAR Modified[] = TEXT("AAAAAAECEEEEIIIIDNOOOOOx0UUUUYPsaaaaaaeceeeeiiiiOnoooood0uuuuypyBbVvGgDdEeJjZzIiYyKkLlMmNnOoPpRrSsTtUuFfJjTtCcSsSs__ii__EeYyYy__");
	static_assert(!PLATFORM_DESKTOP || GetArrayLength(Original) == GetArrayLength(Modified), "array size mismatch");

	for (int32 i = 0; i < GetArrayLength(Original); i++)
	{
		InString.ReplaceCharInline(Original[i], Modified[i], ESearchCase::CaseSensitive);
	}

	// Also remove control character and other oddities
	for (TCHAR& InChar : InString)
	{
		InChar = FChar::IsPrint(InChar) ? InChar : TEXT('_');
	}
}

void FDatasmithUtils::SanitizeStringInplace(FString& InString)
{
	for (TCHAR& Char : InString)
	{
		if (!FChar::IsPrint(Char) && !FChar::IsWhitespace(Char))
		{
			Char = TEXT('_');
		}
	}
}

FString FDatasmithUtils::SanitizeName(FString InString)
{
	SanitizeNameInplace(InString);
	return InString;
}

FString FDatasmithUtils::SanitizeObjectName(FString InString)
{
	// List of invalid characters taken from ObjectTools::SanitizeObjectName
	// Also prevent character not allowed in filenames as we use labels as package name
	const FString Invalid = INVALID_OBJECTNAME_CHARACTERS TEXT("*<>?\\·");

	if ( InString.IsEmpty() )
	{
		return InString;
	}

	for (int32 i = 0; i < Invalid.Len(); i++)
	{
		InString.ReplaceCharInline(Invalid[i], TCHAR('_'), ESearchCase::CaseSensitive);
	}

	SanitizeNameInplace(InString);

	// Unreal does not support names starting with '_'
	if (InString[0] == TEXT('_'))
	{
		InString.InsertAt(0, TEXT("Object"));
	}

	// Object's Name equals to "[InvalidNames]" or formatted like "[InvalidNames][_]%d" will generate a crash on save or reload of the asset. See UE-66320.
	// Consequently, in that case, "_SAFE" is appended to the name to avoid the crash but keep the original name
	// Note that the format (uppercase, lowercase) of "[InvalidNames]" does not matter.
	for (const FString& InvalidName : InvalidNames)
	{
		if (InString.StartsWith(InvalidName, ESearchCase::IgnoreCase))
		{
			const int32 InvalidNameLen = InvalidName.Len();
			if (InString.Len() == InvalidNameLen)
			{
				InString += TEXT("_SAFE");
			}
			else
			{
				const int32 InStringLen = InString.Len();
				int32 Index = InString[InvalidNameLen] == TEXT('_') ? InvalidNameLen + 1 : InvalidNameLen;
				for (; Index < InStringLen; ++Index)
				{
					if (!FChar::IsDigit(InString[Index]))
					{
						break;
					}
				}

				// Input name is formatted as "[InvalidNames][_]%d". Append "_SAFE"
				if (Index == InStringLen)
				{
					InString += TEXT("_SAFE");
				}
			}

			break;
		}
	}

	return InString;
}

FString FDatasmithUtils::SanitizeFileName(FString InString)
{
	const FString Invalid = TEXT(" \"'*:.,;<>?/\\|&$·#");
	for (int32 i = 0; i < Invalid.Len(); i++)
	{
		InString.ReplaceCharInline(Invalid[i], TCHAR('_'), ESearchCase::CaseSensitive);
	}
	SanitizeNameInplace(InString);
	return InString;
}

void FDatasmithUtils::GetCleanFilenameAndExtension(const FString& InFilePath, FString& OutFilename, FString& OutExtension)
{
	FString BaseFile = FPaths::GetCleanFilename(InFilePath);
	BaseFile.Split(TEXT("."), &OutFilename, &OutExtension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

	if (!OutExtension.IsEmpty() && FCString::IsNumeric(*OutExtension))
	{
		BaseFile = OutFilename;
		BaseFile.Split(TEXT("."), &OutFilename, &OutExtension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		OutExtension = OutExtension + TEXT(".*");
		return;
	}

	if (OutExtension.IsEmpty())
	{
		OutFilename = BaseFile;
	}
}

int32 FDatasmithUtils::GetEnterpriseVersionAsInt()
{
	const int32 PatchVersion = FEngineVersion::Current().GetPatch();
	const int32 MinorVersion = FEngineVersion::Current().GetMinor() * 10;
	int32 MinorNumberOfDigits = 1;
	for (int32 Version = MinorVersion; Version /= 10; MinorNumberOfDigits++);

	const int32 MajorVersion = FEngineVersion::Current().GetMajor() * FMath::Pow(10.f, MinorNumberOfDigits);

	return MajorVersion + MinorVersion + PatchVersion;
}

FString FDatasmithUtils::GetEnterpriseVersionAsString(bool bWithChangelist)
{
	EVersionComponent Format = (bWithChangelist && FEngineVersion::Current().HasChangelist()) ? EVersionComponent::Changelist : EVersionComponent::Patch;
	return FEngineVersion::Current().ToString(Format);
}

float FDatasmithUtils::GetDatasmithFormatVersionAsFloat()
{
	return DATASMITH_FORMAT_VERSION;
}

int32 FDatasmithUtils::GetDatasmithFormatVersionAsInt()
{
	return (int32)DATASMITH_FORMAT_VERSION * 100;
}

FString FDatasmithUtils::GetDatasmithFormatVersionAsString()
{
	return FString::SanitizeFloat(DATASMITH_FORMAT_VERSION, 2);
}

const TCHAR* FDatasmithUtils::GetFileExtension()
{
	return TEXT("udatasmith");
}

const TCHAR* FDatasmithUtils::GetLongAppName()
{
	return TEXT("Unreal Datasmith");
}

const TCHAR* FDatasmithUtils::GetShortAppName()
{
	return TEXT("Datasmith");
}

float FDatasmithUtils::AreaTriangle3D(const FVector3f& v0, const FVector3f& v1, const FVector3f& v2)
{
	FVector3f TriangleNormal = (v1 - v2) ^ (v0 - v2);
	return TriangleNormal.Size() * 0.5f;
}

bool FDatasmithMeshUtils::ToRawMesh(const FDatasmithMesh& Mesh, FRawMesh& RawMesh, bool bValidateRawMesh)
{
	RawMesh.Empty();

	if (Mesh.GetVerticesCount() == 0 || Mesh.GetFacesCount() == 0)
	{
		return false;
	}

	RawMesh.VertexPositions.Reserve( Mesh.GetVerticesCount() );

	for ( int32 i = 0; i < Mesh.GetVerticesCount(); ++i )
	{
		RawMesh.VertexPositions.Add( Mesh.GetVertex( i ) );
	}

	RawMesh.FaceMaterialIndices.Reserve( Mesh.GetFacesCount() );
	RawMesh.FaceSmoothingMasks.Reserve( Mesh.GetFacesCount() );
	RawMesh.WedgeIndices.Reserve( Mesh.GetFacesCount() * 3 );
	RawMesh.WedgeTangentZ.Reserve( Mesh.GetFacesCount() * 3 );

	for ( int32 FaceIndex = 0; FaceIndex < Mesh.GetFacesCount(); ++FaceIndex )
	{
		int32 Vertex1;
		int32 Vertex2;
		int32 Vertex3;
		int32 MaterialId;

		Mesh.GetFace( FaceIndex, Vertex1, Vertex2, Vertex3, MaterialId );

		RawMesh.FaceMaterialIndices.Add( MaterialId );
		RawMesh.FaceSmoothingMasks.Add( Mesh.GetFaceSmoothingMask( FaceIndex ) );

		RawMesh.WedgeIndices.Add( Vertex1 );
		RawMesh.WedgeIndices.Add( Vertex2 );
		RawMesh.WedgeIndices.Add( Vertex3 );

		for ( int32 j = 0; j < 3; ++j )
		{
			RawMesh.WedgeTangentZ.Add( Mesh.GetNormal( FaceIndex * 3 + j ) );
		}
	}

	for ( int32 UVChannel = 0; UVChannel < FMath::Min( Mesh.GetUVChannelsCount(), (int32)MAX_MESH_TEXTURE_COORDS ) ; ++UVChannel )
	{
		RawMesh.WedgeTexCoords[ UVChannel ].Reserve( Mesh.GetFacesCount() * 3 );

		for ( int32 FaceIndex = 0; FaceIndex < Mesh.GetFacesCount(); ++FaceIndex )
		{
			int32 UVIndex1;
			int32 UVIndex2;
			int32 UVIndex3;

			Mesh.GetFaceUV( FaceIndex, UVChannel, UVIndex1, UVIndex2, UVIndex3 );

			RawMesh.WedgeTexCoords[ UVChannel ].Add( FVector2f(Mesh.GetUV( UVChannel, UVIndex1 )) );
			RawMesh.WedgeTexCoords[ UVChannel ].Add( FVector2f(Mesh.GetUV( UVChannel, UVIndex2 )) );
			RawMesh.WedgeTexCoords[ UVChannel ].Add( FVector2f(Mesh.GetUV( UVChannel, UVIndex3 )) );
		}
	}


	RawMesh.WedgeColors.Reserve( Mesh.GetVertexColorCount() );
	for ( int32 i = 0; i < Mesh.GetVertexColorCount(); i++ )
	{
		RawMesh.WedgeColors.Add( Mesh.GetVertexColor( i ) );
	}

	// Verify RawMesh is actually valid;
	if (bValidateRawMesh && !RawMesh.IsValid())
	{
		RawMesh.Empty();
		return false;
	}

	return true;
}

bool FDatasmithMeshUtils::ToMeshDescription(FDatasmithMesh& DsMesh, FMeshDescription& MeshDescription, EUvGenerationPolicy UvGen)
{
	FRawMesh RawMesh;
	bool bGenerateDefaultUvs = DsMesh.GetUVChannelsCount() == 0 && UvGen != EUvGenerationPolicy::Ignore;
	if (bGenerateDefaultUvs)
	{
		DsMesh.AddUVChannel();
		DsMesh.SetUVCount(0, 1); // we need one valid uv value so that default UVIndex (which is 0) points to a valid location
	}

	if (!ToRawMesh(DsMesh, RawMesh))
	{
		return false;
	}

	// ToRawMesh() produces meshes whose material indices are in fact Ids, eg. can be any random int, instead of the compact set {0, 1, 2, ...}.
	// we have to remap ids to compact indices in order to generate MeshDescriptions with the correct amount of polygon groups.
	// The following steps ensure that
	// - Slot names are FNames that are the literal representation of the original materialId (so that the importer can map materials)
	// - MeshDescription creation receives a RawMesh with a compact set of materialIndices (in RawMesh.FaceMaterialIndices)

	TMap<int32, FName> MaterialMap;
	TSet<int32> MaterialIds{RawMesh.FaceMaterialIndices};

	int32* Min = Algo::MinElement(MaterialIds);
	int32* Max = Algo::MaxElement(MaterialIds);
	bool bCanUseIdsDirectly = Min && Max && *Min == 0 && *Max == MaterialIds.Num()-1; // We can skip the remapping when ids are already indices

	if (bCanUseIdsDirectly)
	{
		for (int32 MaterialId : MaterialIds)
		{
			MaterialMap.Add(MaterialId, *FString::FromInt(MaterialId));
		}
	}
	else
	{
		TMap<int32, int32> IDsToIndices;
		for (int32 MaterialId : MaterialIds)
		{
			int32 MaterialIndex = MaterialMap.Num();
			IDsToIndices.Add(MaterialId, MaterialIndex);

			MaterialMap.Add(MaterialIndex, *FString::FromInt(MaterialId));
		}
		for (auto& OldId : RawMesh.FaceMaterialIndices)
		{
			OldId = IDsToIndices[OldId];
		}
	}

	FStaticMeshAttributes Attributes(MeshDescription);
	Attributes.Register();
	FStaticMeshOperations::ConvertFromRawMesh(RawMesh, MeshDescription, MaterialMap);

	if (bGenerateDefaultUvs)
	{
		FBox MeshBoundingBox = MeshDescription.ComputeBoundingBox();
		FUVMapParameters UVParameters(MeshBoundingBox.GetCenter(), FQuat::Identity, MeshBoundingBox.GetSize(), FVector::OneVector, FVector2D::UnitVector);
		TMap<FVertexInstanceID, FVector2D> TexCoords;
		FStaticMeshOperations::GenerateBoxUV(MeshDescription, UVParameters, TexCoords);

		TVertexInstanceAttributesRef<FVector2f> UVs = FStaticMeshAttributes(MeshDescription).GetVertexInstanceUVs();
		for (const auto& Pair : TexCoords)
		{
			UVs.Set(Pair.Key, 0, (FVector2f)Pair.Value);
		}
	}
	return true;
}

bool FDatasmithMeshUtils::IsUVChannelValid(const FDatasmithMesh& DsMesh, const int32 Channel)
{
	for (int32 FaceIndex = 0, FacesCount = DsMesh.GetFacesCount(); FaceIndex < FacesCount; ++FaceIndex)
	{
		int32 UV[3];
		DsMesh.GetFaceUV(FaceIndex, Channel, UV[0], UV[1], UV[2]);

		FVector2D UVCoords[3] = {
			DsMesh.GetUV(Channel, UV[0]),
			DsMesh.GetUV(Channel, UV[1]),
			DsMesh.GetUV(Channel, UV[2]) };

		float TriangleArea = FMath::Abs(((UVCoords[0].X * (UVCoords[1].Y - UVCoords[2].Y))
			+ (UVCoords[1].X * (UVCoords[2].Y - UVCoords[0].Y))
			+ (UVCoords[2].X * (UVCoords[0].Y - UVCoords[1].Y))) * 0.5f);

		if (TriangleArea <= SMALL_NUMBER)
		{
			return false;
		}
	}

	return true;
}

void CreateDefaultUVs( FDatasmithMesh& Mesh )
{
	if ( Mesh.GetUVChannelsCount() > 0 )
	{
		return;
	}

	// Get the mesh description to generate BoxUV.
	FMeshDescription MeshDescription;
	FStaticMeshAttributes(MeshDescription).Register();
	FDatasmithMeshUtils::ToMeshDescription(Mesh, MeshDescription, FDatasmithMeshUtils::Ignore);
	FUVMapParameters UVParameters(
		FVector(Mesh.GetExtents().GetCenter()),
		FQuat::Identity,
		FVector(Mesh.GetExtents().GetSize()),
		FVector::OneVector,
		FVector2D::UnitVector
	);
	TMap<FVertexInstanceID, FVector2D> TexCoords;
	FStaticMeshOperations::GenerateBoxUV(MeshDescription, UVParameters, TexCoords);

	// Put the results in a map to determine the number of unique values.
	TMap<FVector2D, TArray<int32>> UniqueTexCoordMap;
	for (const TPair<FVertexInstanceID, FVector2D>& Pair : TexCoords)
	{
		TArray<int32>& MappedIndices = UniqueTexCoordMap.FindOrAdd(Pair.Value);
		MappedIndices.Add(Pair.Key.GetValue());
	}

	//Set the UV values
	Mesh.AddUVChannel();
	Mesh.SetUVCount(0, UniqueTexCoordMap.Num());
	int32 UVIndex = 0;
	TArray<int32> IndicesMapping;
	IndicesMapping.AddZeroed(TexCoords.Num());
	for (const TPair<FVector2D, TArray<int32>>& UniqueCoordPair : UniqueTexCoordMap)
	{
		Mesh.SetUV(0, UVIndex, UniqueCoordPair.Key.X, UniqueCoordPair.Key.Y);
		for (int32 IndicesIndex : UniqueCoordPair.Value)
		{
			IndicesMapping[IndicesIndex] = UVIndex;
		}
		UVIndex++;
	}

	//Map the UV indices.
	for (int32 FaceIndex = 0; FaceIndex < Mesh.GetFacesCount(); ++FaceIndex)
	{
		const int32 IndicesOffset = FaceIndex * 3;
		check(IndicesOffset + 2 < IndicesMapping.Num());
		Mesh.SetFaceUV(FaceIndex, 0, IndicesMapping[IndicesOffset + 0], IndicesMapping[IndicesOffset + 1], IndicesMapping[IndicesOffset + 2]);
	}
}

void FDatasmithMeshUtils::CreateDefaultUVsWithLOD(FDatasmithMesh& Mesh)
{
	CreateDefaultUVs( Mesh );

	for ( int32 LODIndex = 0; LODIndex < Mesh.GetLODsCount(); ++LODIndex )
	{
		if ( FDatasmithMesh* LODMesh = Mesh.GetLOD( LODIndex ) )
		{
			CreateDefaultUVs( *LODMesh );
		}
	}
}

void FDatasmithMeshUtils::ExtractVertexPositions(const FMeshDescription& Mesh, TArray<FVector3f>& OutPositions)
{
	FStaticMeshConstAttributes Attributes(Mesh);
	const TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	if (VertexPositions.IsValid())
	{
		OutPositions.Reserve(VertexPositions.GetNumElements());
		for (const FVertexID VertexID : Mesh.Vertices().GetElementIDs())
		{
			OutPositions.Add(VertexPositions[VertexID]);
		}
	}
}

bool FDatasmithTextureUtils::CalculateTextureHash(const TSharedPtr<class IDatasmithTextureElement>& TextureElement)
{
	if (TextureElement)
	{
		TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileReader(TextureElement->GetFile()));
		if (Archive)
		{
			TextureElement->SetFileHash(FMD5Hash::HashFileFromArchive(Archive.Get()));
			return true;
		}
	}
	return false;
}

void FDatasmithTextureUtils::CalculateTextureHashes(const TSharedPtr<IDatasmithScene>& Scene)
{
	if (Scene)
	{
		for (int i = 0; i < Scene->GetTexturesCount(); ++i)
		{
			CalculateTextureHash(Scene->GetTexture(i));
		}
	}
}

ETransformChannelComponents FDatasmithAnimationUtils::GetChannelTypeComponents(EDatasmithTransformChannels Channels, EDatasmithTransformType TransformType)
{
	static_assert((uint16)EDatasmithTransformChannels::TranslationX == 0x001, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::TranslationY == 0x002, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::TranslationZ == 0x004, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::RotationX == 0x008, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::RotationY == 0x010, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::RotationZ == 0x020, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::ScaleX == 0x040, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::ScaleY == 0x080, "INVALID_ENUM_VALUE");
	static_assert((uint16)EDatasmithTransformChannels::ScaleZ == 0x100, "INVALID_ENUM_VALUE");

	ETransformChannelComponents Result = ETransformChannelComponents::None;

	switch (TransformType)
	{
	case EDatasmithTransformType::Translation:
		Result |= (ETransformChannelComponents) (((uint16)Channels) & 0xFF);
		break;
	case EDatasmithTransformType::Rotation:
		Result |= (ETransformChannelComponents) (((uint16)Channels >> 3) & 0xFF);
		break;
	case EDatasmithTransformType::Scale:
		Result |= (ETransformChannelComponents) (((uint16)Channels >> 6) & 0xFF);
		break;
	default:
		break;
	}

	return Result;
}

EDatasmithTransformChannels FDatasmithAnimationUtils::SetChannelTypeComponents(ETransformChannelComponents Components, EDatasmithTransformType TransformType)
{
	EDatasmithTransformChannels Result = EDatasmithTransformChannels::None;

	switch (TransformType)
	{
	case EDatasmithTransformType::Translation:
		Result |= (EDatasmithTransformChannels) ((uint16)Components);
		break;
	case EDatasmithTransformType::Rotation:
		Result |= (EDatasmithTransformChannels) ((uint16)Components << 3);
		break;
	case EDatasmithTransformType::Scale:
		Result |= (EDatasmithTransformChannels) ((uint16)Components << 6);
		break;
	default:
		break;
	}

	return Result;
}

namespace DatasmithSceneUtilsImpl
{
	template<typename IElementType>
	void GetAllActorsChildRecursive(const TSharedPtr<IDatasmithActorElement>& ActorElement, EDatasmithElementType ElementType, TArray<TSharedPtr<IElementType>>& OutResult)
	{
		if (ActorElement->IsA(ElementType))
		{
			OutResult.Add(StaticCastSharedPtr<IElementType>(ActorElement));
		}

		const int32 ChildrenCount = ActorElement->GetChildrenCount();
		for (int32 ChildIndex = 0; ChildIndex < ChildrenCount; ++ChildIndex)
		{
			DatasmithSceneUtilsImpl::GetAllActorsChildRecursive(ActorElement->GetChild(ChildIndex), ElementType, OutResult);
		}
	}

	template<typename IElementType>
	void GetAllActorsChildRecursive(const TSharedPtr<IDatasmithScene>& Scene, EDatasmithElementType ElementType, TArray<TSharedPtr<IElementType>>& OutResult)
	{
		const int32 ActorsCount = Scene->GetActorsCount();
		for (int32 ActorIndex = 0; ActorIndex < ActorsCount; ++ActorIndex)
		{
			DatasmithSceneUtilsImpl::GetAllActorsChildRecursive(Scene->GetActor(ActorIndex), ElementType, OutResult);
		}
	}
}

TArray<TSharedPtr<IDatasmithCameraActorElement>> FDatasmithSceneUtils::GetAllCameraActorsFromScene(const TSharedPtr<IDatasmithScene>& Scene)
{
	TArray<TSharedPtr<IDatasmithCameraActorElement>> Result;
	if (Scene)
	{
		Result.Reserve(Scene->GetActorsCount());
		DatasmithSceneUtilsImpl::GetAllActorsChildRecursive(Scene, EDatasmithElementType::Camera, Result);
	}

	return Result;
}

TArray<TSharedPtr<IDatasmithLightActorElement>> FDatasmithSceneUtils::GetAllLightActorsFromScene(const TSharedPtr<IDatasmithScene>& Scene)
{
	TArray<TSharedPtr<IDatasmithLightActorElement>> Result;
	if (Scene)
	{
		Result.Reserve(Scene->GetActorsCount());
		DatasmithSceneUtilsImpl::GetAllActorsChildRecursive(Scene, EDatasmithElementType::Light, Result);
	}

	return Result;
}

TArray<TSharedPtr<IDatasmithMeshActorElement>> FDatasmithSceneUtils::GetAllMeshActorsFromScene(const TSharedPtr<IDatasmithScene>& Scene)
{
	TArray<TSharedPtr<IDatasmithMeshActorElement>> Result;
	if (Scene)
	{
		Result.Reserve(Scene->GetActorsCount());
		DatasmithSceneUtilsImpl::GetAllActorsChildRecursive(Scene, EDatasmithElementType::StaticMeshActor, Result);
	}

	return Result;
}

TArray< TSharedPtr< IDatasmithCustomActorElement> > FDatasmithSceneUtils::GetAllCustomActorsFromScene(const TSharedPtr<IDatasmithScene>& Scene)
{
	TArray<TSharedPtr<IDatasmithCustomActorElement>> Result;
	if (Scene)
	{
		Result.Reserve(Scene->GetActorsCount());
		DatasmithSceneUtilsImpl::GetAllActorsChildRecursive(Scene, EDatasmithElementType::CustomActor, Result);
	}
	return Result;
}

namespace DatasmithSceneUtilsImpl
{
	bool FindActorHierarchy(const TSharedPtr<IDatasmithActorElement>& ActorElement, const TSharedPtr<IDatasmithActorElement>& ToFind, FDatasmithSceneUtils::TActorHierarchy& OutHierarchy)
	{
		if (ActorElement == ToFind)
		{
			return true;
		}

		int32 ChildrenCount = ActorElement->GetChildrenCount();
		for (int32 ChildIndex = 0; ChildIndex < ChildrenCount; ++ChildIndex)
		{
			const TSharedPtr< IDatasmithActorElement > Child = ActorElement->GetChild(ChildIndex);

			bool bFound = FindActorHierarchy(Child, ToFind, OutHierarchy);
			if (bFound)
			{
				OutHierarchy.Add(ActorElement);
				return true;
			}
		}
		return false;
	}
}

bool FDatasmithSceneUtils::FindActorHierarchy(const IDatasmithScene* Scene, const TSharedPtr<IDatasmithActorElement>& ToFind, TActorHierarchy& OutHierarchy)
{
	bool bResult = false;
	OutHierarchy.Reset();
	if (Scene != nullptr && ToFind.IsValid())
	{
		const int32 ActorsCount = Scene->GetActorsCount();
		for (int32 ActorIndex = 0; ActorIndex  < ActorsCount; ++ActorIndex)
		{
			const TSharedPtr< IDatasmithActorElement>& ActorElement = Scene->GetActor(ActorIndex);
			bResult = DatasmithSceneUtilsImpl::FindActorHierarchy(ActorElement, ToFind, OutHierarchy);
			if (bResult)
			{
				OutHierarchy.Add(ActorElement);
				break;
			}
		}
	}

	if (bResult)
	{
		Algo::Reverse(OutHierarchy);
	}
	return bResult;
}

bool FDatasmithSceneUtils::IsMaterialIDUsedInScene(const TSharedPtr<IDatasmithScene>& Scene, const TSharedPtr<IDatasmithMaterialIDElement>& MaterialElement)
{
	if (!Scene)
	{
		return false;
	}

	TArray<TSharedPtr<IDatasmithMeshActorElement>> AllMeshActors = FDatasmithSceneUtils::GetAllMeshActorsFromScene(Scene);
	for (const TSharedPtr<IDatasmithMeshActorElement>& MeshActor : AllMeshActors)
	{
		const int32 MaterialCount = MeshActor->GetMaterialOverridesCount();
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			if (MeshActor->GetMaterialOverride(MaterialIndex) == MaterialElement)
			{
				return true;
			}
		}
	}

	TArray<TSharedPtr<IDatasmithLightActorElement>> AllLightActors = FDatasmithSceneUtils::GetAllLightActorsFromScene(Scene);
	for (const TSharedPtr<IDatasmithLightActorElement>& LightActor : AllLightActors)
	{
		if (LightActor->GetLightFunctionMaterial() == MaterialElement)
		{
			return true;
		}
	}
	return false;
}

bool FDatasmithSceneUtils::IsPostProcessUsedInScene(const TSharedPtr<IDatasmithScene>& Scene, const TSharedPtr<IDatasmithPostProcessElement>& PostProcessElement)
{
	if (!Scene)
	{
		return false;
	}

	if (Scene->GetPostProcess() == PostProcessElement)
	{
		return true;
	}

	TArray<TSharedPtr<IDatasmithCameraActorElement>> AllCameraActors = GetAllCameraActorsFromScene(Scene);
	for (const TSharedPtr<IDatasmithCameraActorElement>& CameraActor : AllCameraActors)
	{
		if (CameraActor->GetPostProcess() == PostProcessElement)
		{
			return true;
		}
	}
	return false;
}

namespace DatasmithSceneUtilsImpl
{
	void FixIesTextures(IDatasmithScene& Scene, const TSharedPtr< IDatasmithActorElement>& InActor)
	{
		if (InActor->IsA(EDatasmithElementType::Light))
		{
			TSharedPtr<IDatasmithLightActorElement> LightActor = StaticCastSharedPtr< IDatasmithLightActorElement >(InActor);

			FString IESFilePath(LightActor->GetIesFile());

			if (!IESFilePath.IsEmpty() && FCString::Strlen(LightActor->GetIesTexturePathName()) == 0)
			{
				FString TextureName = FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(IESFilePath) + TEXT("_IES"));
				TSharedPtr<IDatasmithTextureElement> TexturePtr;

				for (int32 Index = 0; Index < Scene.GetTexturesCount(); ++Index)
				{
					if (TextureName.Equals(Scene.GetTexture(Index)->GetName()))
					{
						TexturePtr = Scene.GetTexture(Index);
						break;
					}
				}

				if(!TexturePtr.IsValid())
				{
					// Create a Datasmith texture element.
					TexturePtr = FDatasmithSceneFactory::CreateTexture(*TextureName);

					// Set the texture label used in the Unreal UI.
					TexturePtr->SetLabel(*TextureName);

					// Set the Datasmith texture mode.
					TexturePtr->SetTextureMode(EDatasmithTextureMode::Ies);

					// Set the Datasmith texture file path.
					TexturePtr->SetFile(*IESFilePath);

					// Add the texture to the Datasmith scene.
					Scene.AddTexture(TexturePtr);
				}

				// The Datasmith light is controlled by a IES definition file.
				LightActor->SetUseIes(true);

				// Set the IES definition file path of the Datasmith light.
				LightActor->SetIesTexturePathName(TexturePtr->GetName());
				LightActor->SetIesFile(TEXT(""));
			}
		}

		int32 ChildrenCount = InActor->GetChildrenCount();
		for (int32 i = 0; i < ChildrenCount; ++i)
		{
			FixIesTextures(Scene, InActor->GetChild(i));
		}
	}

	EDatasmithTextureMode GetTextureModeFromPropertyName(const FString& PropertyName)
	{
		if (PropertyName.Find(TEXT("BUMP")) != INDEX_NONE)
		{
			return EDatasmithTextureMode::Bump;
		}
		else if (PropertyName.Find(TEXT("SPECULAR")) != INDEX_NONE)
		{
			return EDatasmithTextureMode::Specular;
		}
		else if (PropertyName.Find(TEXT("NORMAL")) != INDEX_NONE)
		{
			return EDatasmithTextureMode::Normal;
		}

		return EDatasmithTextureMode::Diffuse;
	};

	void CheckMaterialInstanceTextures( IDatasmithScene& Scene )
	{
		TSet<FString> ProcessedTextures;

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();


		for( int32 MaterialIndex = 0; MaterialIndex < Scene.GetMaterialsCount(); ++MaterialIndex )
		{
			const TSharedPtr< IDatasmithBaseMaterialElement >& BaseMaterial = Scene.GetMaterial( MaterialIndex );

			if ( BaseMaterial->IsA( EDatasmithElementType::MaterialInstance ) )
			{
				const TSharedPtr< IDatasmithMaterialInstanceElement >& Material = StaticCastSharedPtr< IDatasmithMaterialInstanceElement >( BaseMaterial );

				for ( int32 i = 0; i < Material->GetPropertiesCount(); ++i )
				{
					TSharedPtr< IDatasmithKeyValueProperty > Property = Material->GetProperty(i);

					if (Property->GetPropertyType() == EDatasmithKeyValuePropertyType::Texture && FCString::Strlen(Property->GetValue()) > 0)
					{
						FString TexturePathName = Property->GetValue();

						// If TexturePathName is a path to a file on disk
						if (TexturePathName[0] != '/' && PlatformFile.FileExists( *TexturePathName ))
						{
							// Add TextureElement associated to TexturePathName if it has not been yet
							if (ProcessedTextures.Find(TexturePathName) == nullptr)
							{

								TSharedPtr< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture(*FPaths::GetBaseFilename(TexturePathName));

								TextureElement->SetTextureMode( GetTextureModeFromPropertyName(Property->GetName()) );
								TextureElement->SetFile( *TexturePathName );

								Scene.AddTexture( TextureElement );

								ProcessedTextures.Add(TexturePathName);
							}

							Property->SetValue( *FPaths::GetBaseFilename(TexturePathName) );
						}
					}
				}
			}
		}
	}

	void CleanUpEnvironments( TSharedPtr<IDatasmithScene> Scene )
	{
		if (!Scene)
		{
			return;
		}

		// Remove unsupported environments
		for (int32 Index = Scene->GetActorsCount() - 1; Index >= 0; --Index)
		{
			if ( Scene->GetActor(Index)->IsA( EDatasmithElementType::EnvironmentLight ) )
			{
				TSharedPtr< IDatasmithEnvironmentElement > EnvironmentElement = StaticCastSharedPtr< IDatasmithEnvironmentElement >( Scene->GetActor(Index) );

				if ( EnvironmentElement->GetEnvironmentComp()->GetMode() != EDatasmithCompMode::Regular || EnvironmentElement->GetEnvironmentComp()->GetParamSurfacesCount() != 1 )
				{
					UE_LOGF(LogDatasmith, Warning, "Environment %ls removed because it is not supported yet", EnvironmentElement->GetName());
					Scene->RemoveActor( EnvironmentElement, EDatasmithActorRemovalRule::RemoveChildren );
				}
				else if ( !EnvironmentElement->GetEnvironmentComp()->GetUseTexture(0) )
				{
					UE_LOGF(LogDatasmith, Warning, "Environment %ls removed because it is not supported yet", EnvironmentElement->GetName());
					Scene->RemoveActor( EnvironmentElement, EDatasmithActorRemovalRule::RemoveChildren );
				}
			}
		}

		// Keep only one environment with an illumination map and without
		for ( int32 Index = Scene->GetActorsCount() - 1; Index >= 0; --Index )
		{
			if ( Scene->GetActor(Index)->IsA( EDatasmithElementType::EnvironmentLight ) )
			{
				TSharedPtr< IDatasmithEnvironmentElement > EnvironmentElement = StaticCastSharedPtr< IDatasmithEnvironmentElement >( Scene->GetActor(Index) );

				bool bIsIlluminationMap = EnvironmentElement->GetIsIlluminationMap();

				bool bIsADuplicate = false;
				for ( int32 PastIndex = 0; PastIndex < Index; ++PastIndex )
				{
					if (Scene->GetActor(PastIndex)->IsA(EDatasmithElementType::EnvironmentLight))
					{
						TSharedPtr< IDatasmithEnvironmentElement > PreviousEnvElement = StaticCastSharedPtr< IDatasmithEnvironmentElement >( Scene->GetActor(PastIndex) );
						if (PreviousEnvElement->GetIsIlluminationMap() == bIsIlluminationMap)
						{
							bIsADuplicate = true;
							break;
						}
					}
				}

				if ( bIsADuplicate )
				{
					UE_LOGF(LogDatasmith, Warning, "Environment %ls removed because only one environment of its type is supported", EnvironmentElement->GetName());
					Scene->RemoveActor( EnvironmentElement, EDatasmithActorRemovalRule::RemoveChildren );
				}
			}
		}
	}

	const FString TexturePrefix( TEXT( "Texture." ) );
	const FString MaterialPrefix( TEXT( "Material." ) );
	const FString MeshPrefix( TEXT( "Mesh." ) );

	struct FDatasmithSceneCleaner
	{
		TSet<TSharedPtr<IDatasmithMeshElement>> ReferencedMeshes;
		TSet<TSharedPtr<IDatasmithBaseMaterialElement>> ReferencedMaterials;
		TSet<TSharedPtr<IDatasmithBaseMaterialElement>> FunctionMaterials;
		TSet<FString> ReferencedTextures;
		TSet<FString> ActorsInScene;

		TMap<FString, TSharedPtr<IDatasmithElement>> AssetElementMapping;

		TSharedPtr<IDatasmithScene> Scene;

		FDatasmithSceneCleaner(TSharedPtr<IDatasmithScene> InScene)
			: Scene(InScene)
		{
		}

		void ScanMaterialIDElement(const IDatasmithMaterialIDElement* MaterialIDElement)
		{
			if (MaterialIDElement)
			{
				if (TSharedPtr<IDatasmithElement>* MaterialElementPtr = AssetElementMapping.Find(MaterialPrefix + MaterialIDElement->GetName()))
				{
					TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = StaticCastSharedPtr<IDatasmithBaseMaterialElement>(*MaterialElementPtr);

					ReferencedMaterials.Add(MaterialElement);
				}
			}
		}

		void ScanMeshActorElement(IDatasmithMeshActorElement* MeshActorElement)
		{
			if (FCString::Strlen(MeshActorElement->GetStaticMeshPathName()) == 0)
			{
				return;
			}

			FString StaticMeshPathName(MeshActorElement->GetStaticMeshPathName());

			// If mesh actor refers to a UE asset, nothing to do
			if (StaticMeshPathName[0] == '/')
			{
				return;
			}

			if (TSharedPtr<IDatasmithElement>* MeshElementPtr = AssetElementMapping.Find(MeshPrefix + StaticMeshPathName))
			{
				TSharedPtr<IDatasmithMeshElement> MeshElement = StaticCastSharedPtr<IDatasmithMeshElement>(*MeshElementPtr);

				ReferencedMeshes.Add(MeshElement);

				for (int32 Index = 0; Index < MeshActorElement->GetMaterialOverridesCount(); ++Index)
				{
					ScanMaterialIDElement( MeshActorElement->GetMaterialOverride(Index).Get() );
				}
			}
		}

		void ScanLightActorElement(IDatasmithLightActorElement* LightActorElement)
		{
			if (LightActorElement->GetUseIes() && FCString::Strlen(LightActorElement->GetIesTexturePathName()) > 0)
			{
				FString TexturePathName(LightActorElement->GetIesTexturePathName());

				if (TexturePathName[0] != '/')
				{
					if (TSharedPtr<IDatasmithElement>* TextureElementPtr = AssetElementMapping.Find(TexturePrefix + TexturePathName))
					{
						ReferencedTextures.Add((*TextureElementPtr)->GetName());
					}
				}
			}

			ScanMaterialIDElement( LightActorElement->GetLightFunctionMaterial().Get() );
		}

		void ScanDecalActorElement(IDatasmithDecalActorElement* DecalActorElement)
		{
			if (const TCHAR* DecalMaterialName = DecalActorElement->GetDecalMaterialPathName())
			{
				if (TSharedPtr<IDatasmithElement>* MaterialElementPtr = AssetElementMapping.Find(MaterialPrefix + DecalMaterialName))
				{
					TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = StaticCastSharedPtr<IDatasmithBaseMaterialElement>(*MaterialElementPtr);
					ReferencedMaterials.Add(MaterialElement);
				}
			}
		}

		void ParseSceneActor( const TSharedPtr<IDatasmithActorElement>& ActorElement )
		{
			if (!ActorElement.IsValid())
			{
				return;
			}

			ActorsInScene.Add(ActorElement->GetName());

			if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
			{
				ScanMeshActorElement(static_cast<IDatasmithMeshActorElement*>(ActorElement.Get()));
			}
			else if (ActorElement->IsA(EDatasmithElementType::Light))
			{
				ScanLightActorElement(static_cast<IDatasmithLightActorElement*>(ActorElement.Get()));
			}
			else if (ActorElement->IsA(EDatasmithElementType::Decal))
			{
				ScanDecalActorElement(static_cast<IDatasmithDecalActorElement*>(ActorElement.Get()));
			}

			for (int32 Index = 0; Index < ActorElement->GetChildrenCount(); ++Index)
			{
				ParseSceneActor( ActorElement->GetChild(Index) );
			}
		}

		void ScanMeshElement(TSharedPtr<IDatasmithMeshElement >& MeshElement)
		{
			for (int32 Index = 0; Index < MeshElement->GetMaterialSlotCount(); Index++)
			{
				ScanMaterialIDElement( MeshElement->GetMaterialSlotAt(Index).Get() );
			}
		}

		void ScanMaterialInstanceElement(IDatasmithMaterialInstanceElement* MaterialElement)
		{
			for ( int32 Index = 0; Index < MaterialElement->GetPropertiesCount(); ++Index )
			{
				TSharedPtr< IDatasmithKeyValueProperty > Property = MaterialElement->GetProperty(Index);

				if (Property->GetPropertyType() == EDatasmithKeyValuePropertyType::Texture && FCString::Strlen(Property->GetValue()) > 0)
				{
					FString TexturePathName = Property->GetValue();

					if (TexturePathName[0] != '/')
					{
						ReferencedTextures.Add(TexturePathName);
					}
				}
			}
		}

		void ScanDecalMaterialElement(IDatasmithDecalMaterialElement* MaterialElement)
		{
			if (const TCHAR* DiffuseTextureName = MaterialElement->GetDiffuseTexturePathName())
			{
				if (DiffuseTextureName[0] != '/')
				{
					ReferencedTextures.Add(DiffuseTextureName);
				}
			}
			if (const TCHAR* NormalTextureName = MaterialElement->GetNormalTexturePathName())
			{
				if (NormalTextureName[0] != '/')
				{
					ReferencedTextures.Add(NormalTextureName);
				}
			}
		}

		void ScanPbrMaterialElement(IDatasmithUEPbrMaterialElement* MaterialElement)
		{
			TFunction<void(IDatasmithMaterialExpression*)> ParseExpressionElement;
			ParseExpressionElement = [this, &ParseExpressionElement](IDatasmithMaterialExpression* ExpressionElement) -> void
			{
				if (ExpressionElement)
				{
					if (ExpressionElement->IsSubType(EDatasmithMaterialExpressionType::Texture))
					{
						const IDatasmithMaterialExpressionTexture* TextureExpression = static_cast<IDatasmithMaterialExpressionTexture*>(ExpressionElement);
						if (FCString::Strlen(TextureExpression->GetTexturePathName()) > 0)
						{
							FString TexturePathName(TextureExpression->GetTexturePathName());
							if (TexturePathName[0] != '/')
							{
								this->ReferencedTextures.Add(TexturePathName);
							}
						}
					}
					else if (ExpressionElement->IsSubType(EDatasmithMaterialExpressionType::Generic))
					{
						const IDatasmithMaterialExpressionGeneric* GenericExpression = static_cast<IDatasmithMaterialExpressionGeneric*>(ExpressionElement);

						for ( int32 PropertyIndex = 0; PropertyIndex < GenericExpression->GetPropertiesCount(); ++PropertyIndex )
						{
							if ( const TSharedPtr< IDatasmithKeyValueProperty >& Property = GenericExpression->GetProperty( PropertyIndex ) )
							{
								if ( Property->GetPropertyType() == EDatasmithKeyValuePropertyType::Texture )
								{
									this->ReferencedTextures.Add( Property->GetValue() );
								}
							}
						}
					}
					else if (ExpressionElement->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
					{
						const IDatasmithMaterialExpressionFunctionCall* FunctionExpression = static_cast<IDatasmithMaterialExpressionFunctionCall*>(ExpressionElement);
						if (FCString::Strlen(FunctionExpression->GetFunctionPathName()) > 0)
						{
							FString FunctionPathName(FunctionExpression->GetFunctionPathName());
							if (FunctionPathName[0] != '/')
							{
								if (TSharedPtr<IDatasmithElement>* MaterialElementPtr = this->AssetElementMapping.Find(MaterialPrefix + FunctionPathName))
								{
									TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = StaticCastSharedPtr<IDatasmithBaseMaterialElement>(*MaterialElementPtr);

									this->ReferencedMaterials.Add(MaterialElement);
									this->ScanPbrMaterialElement(static_cast< IDatasmithUEPbrMaterialElement* >( MaterialElement.Get() ));
								}
							}
						}
					}

					for (int32 InputIndex = 0; InputIndex < ExpressionElement->GetInputCount(); ++InputIndex)
					{
						ParseExpressionElement(ExpressionElement->GetInput(InputIndex)->GetExpression());
					}
				}
			};

			ParseExpressionElement(MaterialElement->GetBaseColor().GetExpression());
			ParseExpressionElement(MaterialElement->GetSpecular().GetExpression());
			ParseExpressionElement(MaterialElement->GetNormal().GetExpression());
			ParseExpressionElement(MaterialElement->GetMetallic().GetExpression());
			ParseExpressionElement(MaterialElement->GetRoughness().GetExpression());
			ParseExpressionElement(MaterialElement->GetEmissiveColor().GetExpression());
			ParseExpressionElement(MaterialElement->GetRefraction().GetExpression());
			ParseExpressionElement(MaterialElement->GetAmbientOcclusion().GetExpression());
			ParseExpressionElement(MaterialElement->GetOpacity().GetExpression());

			if ( MaterialElement->GetUseMaterialAttributes() )
			{
				ParseExpressionElement(MaterialElement->GetMaterialAttributes().GetExpression());
			}
		}

		void ScanCompositeTexture( IDatasmithCompositeTexture* CompositeTexture )
		{
			if ( !CompositeTexture )
			{
				return;
			}

			for (int32 i = 0; i < CompositeTexture->GetParamSurfacesCount(); ++i)
			{
				const FString Texture = CompositeTexture->GetParamTexture(i);

				if ( !Texture.IsEmpty() && Algo::Find( ReferencedTextures, Texture ) == nullptr )
				{
					this->ReferencedTextures.Add( Texture );
					this->ReferencedTextures.Add( Texture + TEXT("_Tex") );
				}
			}

			for (int32 i = 0; i < CompositeTexture->GetParamMaskSurfacesCount(); i++)
			{
				ScanCompositeTexture( CompositeTexture->GetParamMaskSubComposite(i).Get() );
			}

			for (int32 i = 0; i < CompositeTexture->GetParamSurfacesCount(); i++)
			{
				ScanCompositeTexture( CompositeTexture->GetParamSubComposite(i).Get() );
			}
		};

		void ScanLegacyMaterialElement(IDatasmithMaterialElement* MaterialElement)
		{
			if ( !MaterialElement )
			{
				return;
			}

			for (int32 j = 0; j < MaterialElement->GetShadersCount(); ++j )
			{
				if ( const TSharedPtr< IDatasmithShaderElement >& Shader = MaterialElement->GetShader(j) )
				{
					ScanCompositeTexture( Shader->GetDiffuseComp().Get() );
					ScanCompositeTexture( Shader->GetRefleComp().Get() );
					ScanCompositeTexture( Shader->GetRoughnessComp().Get() );
					ScanCompositeTexture( Shader->GetNormalComp().Get() );
					ScanCompositeTexture( Shader->GetBumpComp().Get() );
					ScanCompositeTexture( Shader->GetTransComp().Get() );
					ScanCompositeTexture( Shader->GetMaskComp().Get() );
					ScanCompositeTexture( Shader->GetMetalComp().Get() );
					ScanCompositeTexture( Shader->GetEmitComp().Get() );
					ScanCompositeTexture( Shader->GetWeightComp().Get() );
				}
			}
		}

		void ScanVariant( TSharedPtr<IDatasmithVariantElement> Variant )
		{
			for (int32 BindingIndex = 0; BindingIndex < Variant->GetActorBindingsCount(); ++BindingIndex)
			{
				TSharedPtr<IDatasmithActorBindingElement> ActorBinding = Variant->GetActorBinding(BindingIndex);

				for (int32 PropertyIndex = 0; PropertyIndex < ActorBinding->GetPropertyCapturesCount(); ++PropertyIndex)
				{
					TSharedPtr<IDatasmithBasePropertyCaptureElement> BasePropCaptureElement = ActorBinding->GetPropertyCapture(PropertyIndex);
					if (BasePropCaptureElement->IsSubType(EDatasmithElementVariantSubType::ObjectPropertyCapture))
					{
						// Mark all materials used in this actor binding as referenced
						TSharedPtr<IDatasmithObjectPropertyCaptureElement> PropCaptureElement = StaticCastSharedPtr<IDatasmithObjectPropertyCaptureElement>(BasePropCaptureElement);
						TSharedPtr<IDatasmithElement> TargetElement = PropCaptureElement->GetRecordedObject().Pin();
						if (TargetElement.IsValid() && TargetElement->IsA(EDatasmithElementType::BaseMaterial))
						{
							TSharedPtr<IDatasmithBaseMaterialElement> TargetMaterialElement = StaticCastSharedPtr<IDatasmithBaseMaterialElement>(TargetElement);
							if (TargetMaterialElement.IsValid())
							{
								ReferencedMaterials.Add(TargetMaterialElement);
							}
						}
					}
				}
			}
		}

		void ScanLevelVariantSet( TSharedPtr<IDatasmithLevelVariantSetsElement> LevelVariantSets )
		{
			for (int32 VariantSetIndex = 0; VariantSetIndex < LevelVariantSets->GetVariantSetsCount(); ++VariantSetIndex)
			{
				TSharedPtr<IDatasmithVariantSetElement> VariantSet = LevelVariantSets->GetVariantSet(VariantSetIndex);

				for (int32 VariantIndex = 0; VariantIndex < VariantSet->GetVariantsCount(); ++VariantIndex)
				{
					TSharedPtr<IDatasmithVariantElement> Variant = VariantSet->GetVariant(VariantIndex);
					ScanVariant(Variant);
				}
			}
		}

		void Initialize()
		{
			int32 AssetElementCount = Scene->GetTexturesCount() + Scene->GetMaterialsCount() +
				Scene->GetMeshesCount() + Scene->GetLevelSequencesCount();

			AssetElementMapping.Reserve( AssetElementCount );

			TFunction<void(TSharedPtr<IDatasmithElement>&&, const FString&)> AddAsset;
			AddAsset = [this](TSharedPtr<IDatasmithElement>&& InElementPtr, const FString& AssetPrefix) -> void
			{
				if (InElementPtr)
				{
					AssetElementMapping.Add(AssetPrefix + InElementPtr->GetName(), MoveTemp(InElementPtr));
				}
			};

			for (int32 Index = 0; Index < Scene->GetTexturesCount(); ++Index)
			{
				AddAsset(Scene->GetTexture(Index), TexturePrefix);
			}

			for (int32 Index = 0; Index < Scene->GetMaterialsCount(); ++Index)
			{
				AddAsset(Scene->GetMaterial(Index), MaterialPrefix);
			}

			for (int32 Index = 0; Index < Scene->GetMeshesCount(); ++Index)
			{
				AddAsset(Scene->GetMesh(Index), MeshPrefix);
			}
		}

		void Clean()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithSceneCleaner::Clean);

			Initialize();

			for (int32 Index = 0; Index < Scene->GetActorsCount(); ++Index)
			{
				ParseSceneActor( Scene->GetActor(Index) );
			}

			for (int32 Index = 0; Index < Scene->GetLevelVariantSetsCount(); ++Index)
			{
				ScanLevelVariantSet( Scene->GetLevelVariantSets(Index) );
			}

			for (TSharedPtr<IDatasmithMeshElement >& MeshElement : ReferencedMeshes)
			{
				ScanMeshElement(MeshElement);
			}

			TSet< TSharedPtr<IDatasmithBaseMaterialElement > > CopyOfReferencedMaterials = ReferencedMaterials; // We might discover more referenced materials so iterate on a copy of the set
			for (TSharedPtr<IDatasmithBaseMaterialElement >& MaterialElement : CopyOfReferencedMaterials)
			{
				if ( MaterialElement->IsA( EDatasmithElementType::UEPbrMaterial ) )
				{
					ScanPbrMaterialElement(static_cast< IDatasmithUEPbrMaterialElement* >( MaterialElement.Get() ));
				}
				else if ( MaterialElement->IsA( EDatasmithElementType::MaterialInstance ) )
				{
					ScanMaterialInstanceElement(static_cast< IDatasmithMaterialInstanceElement* >( MaterialElement.Get() ));
				}
				else if (MaterialElement->IsA(EDatasmithElementType::DecalMaterial))
				{
					ScanDecalMaterialElement(static_cast<IDatasmithDecalMaterialElement*>(MaterialElement.Get()));
				}
				else if ( MaterialElement->IsA( EDatasmithElementType::Material ) )
				{
					ScanLegacyMaterialElement(static_cast< IDatasmithMaterialElement* >( MaterialElement.Get() ));
				}
			}

			for ( int32 ActorIndex = 0; ActorIndex < Scene->GetActorsCount(); ++ActorIndex )
			{
				const TSharedPtr< IDatasmithActorElement >& Actor = Scene->GetActor( ActorIndex );

				if ( Actor->IsA( EDatasmithElementType::EnvironmentLight ) )
				{
					if ( TSharedPtr< IDatasmithEnvironmentElement > EnvironmentElement = StaticCastSharedPtr< IDatasmithEnvironmentElement >( Actor ) )
					{
						ScanCompositeTexture( EnvironmentElement->GetEnvironmentComp().Get() );
					}
				}
			}

			for (int32 Index = Scene->GetMeshesCount() - 1; Index >= 0; --Index)
			{
				const TSharedPtr< IDatasmithMeshElement >& MeshElement = Scene->GetMesh(Index);
				if ( !ReferencedMeshes.Contains(MeshElement) )
				{
					UE_LOGF(LogDatasmith, Warning, "Mesh element %ls removed because it is unused", MeshElement->GetName());
					Scene->RemoveMesh(MeshElement);
				}
			}

			for (int32 Index = Scene->GetMaterialsCount() - 1; Index >= 0; --Index)
			{
				const TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement = Scene->GetMaterial(Index);
				if ( !ReferencedMaterials.Contains(MaterialElement) )
				{
					UE_LOGF(LogDatasmith, Warning, "Material element %ls removed because it is unused", MaterialElement->GetName());
					Scene->RemoveMaterial(MaterialElement);
				}
			}

			for (int32 Index = Scene->GetTexturesCount() - 1; Index >= 0; --Index)
			{
				const TSharedPtr< IDatasmithTextureElement >& TextureElement = Scene->GetTexture(Index);
				if ( !ReferencedTextures.Contains(TextureElement->GetName()) )
				{
					UE_LOGF(LogDatasmith, Warning, "Texture element %ls removed because it is unused", TextureElement->GetName());
					Scene->RemoveTexture(TextureElement);
				}
			}

			CleanUpLevelSequences();

			// Remove variant sets referring actors which are not in the scene
			// #ue_liveupdate: Add code to fully clean up the VariantSetsElement itself
			for (int32 Index = Scene->GetLevelVariantSetsCount() - 1; Index >= 0; --Index)
			{
				TSharedPtr< IDatasmithLevelVariantSetsElement > VariantSetsElement = Scene->GetLevelVariantSets(Index);

				bool bValidVariantSets = false;

				for (int32 VariantSetIndex = 0; VariantSetIndex < VariantSetsElement->GetVariantSetsCount() && !bValidVariantSets; ++VariantSetIndex)
				{
					TSharedPtr<IDatasmithVariantSetElement> VariantSetElement = VariantSetsElement->GetVariantSet(VariantSetIndex);
					if (!VariantSetElement)
					{
						continue;
					}

					for (int32 VariantIndex = 0; VariantIndex < VariantSetElement->GetVariantsCount() && !bValidVariantSets; ++VariantIndex)
					{
						TSharedPtr<IDatasmithVariantElement> Variant = VariantSetElement->GetVariant(VariantIndex);
						if (!Variant)
						{
							continue;
						}

						for (int32 BindingIndex = 0; BindingIndex < Variant->GetActorBindingsCount() && !bValidVariantSets; ++BindingIndex)
						{
							TSharedPtr<IDatasmithActorBindingElement> Binding = Variant->GetActorBinding(BindingIndex);
							if (!Binding || !Binding->GetActor())
							{
								continue;
							}

							bValidVariantSets = ActorsInScene.Contains(Binding->GetActor()->GetName());
						}
					}
				}

				if (!bValidVariantSets)
				{
					UE_LOGF(LogDatasmith, Warning, "VariantSets element %ls removed because it references no actor part of the scene", VariantSetsElement->GetName());
					Scene->RemoveLevelVariantSets(VariantSetsElement);
				}
			}
		}

		int32 OptimizeTransformFrames( const TSharedRef<IDatasmithTransformAnimationElement>& Animation, EDatasmithTransformType TransformType )
		{
			int32 NumFrames = Animation->GetFramesCount(TransformType);
			if (NumFrames > 3)
			{
				// First pass: determine which redundant frames can be removed safely
				TArray<int32> FramesToDelete;
				for (int32 FrameIndex = 1; FrameIndex < NumFrames - 2; ++FrameIndex)
				{
					const FDatasmithTransformFrameInfo& PreviousFrameInfo = Animation->GetFrame(TransformType, FrameIndex - 1);
					const FDatasmithTransformFrameInfo& CurrentFrameInfo = Animation->GetFrame(TransformType, FrameIndex);
					const FDatasmithTransformFrameInfo& NextFrameInfo = Animation->GetFrame(TransformType, FrameIndex + 1);

					// Remove the in-between frames that have the same transform as the previous and following frames
					// Need to keep the frames on the boundaries of sharp transitions to avoid interpolated frames at import
					if (CurrentFrameInfo.IsValid() && PreviousFrameInfo.IsValid() && NextFrameInfo.IsValid() && CurrentFrameInfo == PreviousFrameInfo && CurrentFrameInfo == NextFrameInfo)
					{
						FramesToDelete.Add(FrameIndex);
					}
				}
				// Second pass: remove the frames determined in the previous pass
				for (int32 FrameIndex = FramesToDelete.Num() - 1; FrameIndex > 0; --FrameIndex)
				{
					Animation->RemoveFrame(TransformType, FramesToDelete[FrameIndex]);
				}
			}
			// Note that a one-frame animation could be an instantaneous state change (eg. teleport), so keep it
			return Animation->GetFramesCount(TransformType);
		}

		void CleanUpLevelSequences()
		{
			int32 LevelSequencesLastIndex = Scene->GetLevelSequencesCount() - 1;

			// Remove level seequences without animation
			for (int32 SequenceIndex = LevelSequencesLastIndex; SequenceIndex >= 0; --SequenceIndex)
			{
				TSharedPtr< IDatasmithLevelSequenceElement > LevelSequence = Scene->GetLevelSequence(SequenceIndex);

				if (!LevelSequence.IsValid())
				{
					continue;
				}

				int32 NumAnims = LevelSequence->GetAnimationsCount();
				for (int32 AnimIndex = NumAnims - 1; AnimIndex >= 0; --AnimIndex)
				{
					TSharedPtr< IDatasmithBaseAnimationElement > Animation = LevelSequence->GetAnimation(AnimIndex);
					if (Animation.IsValid() && Animation->IsA(EDatasmithElementType::Animation) && Animation->IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
					{
						const TSharedRef< IDatasmithTransformAnimationElement > TransformAnimation = StaticCastSharedRef< IDatasmithTransformAnimationElement >(Animation.ToSharedRef());

						// Optimize the frames for each transform type
						int32 NumFrames = OptimizeTransformFrames(TransformAnimation, EDatasmithTransformType::Translation);
						NumFrames += OptimizeTransformFrames(TransformAnimation, EDatasmithTransformType::Rotation);
						NumFrames += OptimizeTransformFrames(TransformAnimation, EDatasmithTransformType::Scale);

						// Remove animation that has no frame
						if (NumFrames == 0)
						{
							LevelSequence->RemoveAnimation(TransformAnimation);
						}
					}
				}

				if (LevelSequence->GetAnimationsCount() == 0)
				{
					Scene->RemoveLevelSequence(LevelSequence.ToSharedRef());
				}
			}

			// The last valid index can change in the previous loop
			LevelSequencesLastIndex = Scene->GetLevelSequencesCount() - 1;

			// Process sequences without sub-sequence animation first
			TSet<TSharedPtr< IDatasmithLevelSequenceElement >> ValidSequences;
			for (int32 Index =  LevelSequencesLastIndex; Index >= 0; --Index)
			{
				TSharedPtr< IDatasmithLevelSequenceElement > SequenceElement = Scene->GetLevelSequence(Index);
				bool bValidSequence = false;

				const int32 NumAnimations = SequenceElement->GetAnimationsCount();
				for (int32 AnimIndex = 0; AnimIndex < NumAnimations && !bValidSequence; ++AnimIndex)
				{
					TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement = SequenceElement->GetAnimation(AnimIndex);
					if (!AnimationElement)
					{
						continue;
					}

					if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
					{
						const IDatasmithTransformAnimationElement* TransformAnimation = static_cast<IDatasmithTransformAnimationElement*>(AnimationElement.Get());
						bValidSequence = ActorsInScene.Contains(TransformAnimation->GetName());
					}
					else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::VisibilityAnimation))
					{
						const IDatasmithVisibilityAnimationElement* VisibilityAnimation = static_cast<IDatasmithVisibilityAnimationElement*>(AnimationElement.Get());
						bValidSequence = ActorsInScene.Contains(VisibilityAnimation->GetName());
					}
				}

				if (bValidSequence)
				{
					ValidSequences.Add(SequenceElement);
				}
			}

			for (int32 Index = LevelSequencesLastIndex; Index >= 0; --Index)
			{
				TSharedPtr< IDatasmithLevelSequenceElement > SequenceElement = Scene->GetLevelSequence(Index);
				if (ValidSequences.Contains(SequenceElement))
				{
					continue;
				}

				bool bValidSequence = false;

				const int32 NumAnimations = SequenceElement->GetAnimationsCount();
				for (int32 AnimIndex = 0; AnimIndex < NumAnimations && !bValidSequence; ++AnimIndex)
				{
					TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement = SequenceElement->GetAnimation(AnimIndex);
					if (!AnimationElement)
					{
						continue;
					}

					if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
					{
						const IDatasmithTransformAnimationElement* TransformAnimation = static_cast<IDatasmithTransformAnimationElement*>(AnimationElement.Get());
						bValidSequence = ActorsInScene.Contains(TransformAnimation->GetName());
					}
					else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::VisibilityAnimation))
					{
						const IDatasmithVisibilityAnimationElement* VisibilityAnimation = static_cast<IDatasmithVisibilityAnimationElement*>(AnimationElement.Get());
						bValidSequence = ActorsInScene.Contains(VisibilityAnimation->GetName());
					}
					else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::SubsequenceAnimation))
					{
						TSharedRef<IDatasmithSubsequenceAnimationElement> SubsequenceAnimation = StaticCastSharedRef<IDatasmithSubsequenceAnimationElement>(AnimationElement.ToSharedRef());
						bValidSequence = ValidSequences.Contains(SubsequenceAnimation->GetSubsequence().Pin());
					}
				}

				if (!bValidSequence)
				{
					UE_LOGF(LogDatasmith, Warning, "LevelSequence element %ls removed because it references no actor part of the scene", SequenceElement->GetName());
					Scene->RemoveLevelSequence(SequenceElement.ToSharedRef());
				}
			}
		}
	};
}

void FDatasmithSceneUtils::CleanUpScene(TSharedRef<IDatasmithScene> Scene, bool bRemoveUnused)
{
	using namespace DatasmithSceneUtilsImpl;

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithSceneUtils::CleanUpScene);

	for (int32 Index = 0; Index < Scene->GetActorsCount(); ++Index)
	{
		FixIesTextures(*Scene, Scene->GetActor(Index));
	}

	CheckMaterialInstanceTextures(*Scene);

	CleanUpEnvironments(Scene);

	if (bRemoveUnused)
	{
		FDatasmithSceneCleaner SceneCleaner(Scene);
		SceneCleaner.Clean();
	}
}

FDatasmithUniqueNameProviderBase::FDatasmithUniqueNameProviderBase(const FDatasmithUniqueNameProviderBase& Other)
{
	FScopeLock LockOther(&Other.CriticalSection);

	FrequentlyUsedNames = Other.FrequentlyUsedNames;
}

FDatasmithUniqueNameProviderBase::FDatasmithUniqueNameProviderBase(FDatasmithUniqueNameProviderBase&& Other)
{
	FScopeLock LockOther(&Other.CriticalSection);

	FrequentlyUsedNames = MoveTemp(Other.FrequentlyUsedNames);
}

FDatasmithUniqueNameProviderBase* FDatasmithUniqueNameProviderBase::operator=(const FDatasmithUniqueNameProviderBase& Other)
{
	// Guarantee lock order
	TArray< FCriticalSection*, TInlineAllocator<2> > OrderedCriticalSections;
	OrderedCriticalSections.Add( &CriticalSection );
	OrderedCriticalSections.Add( &Other.CriticalSection );

	Algo::Sort( OrderedCriticalSections );

	FScopeLock LockFirst( OrderedCriticalSections[0] );
	FScopeLock LockSecond( OrderedCriticalSections[1] );

	FrequentlyUsedNames = Other.FrequentlyUsedNames;

	return this;
}

FDatasmithUniqueNameProviderBase* FDatasmithUniqueNameProviderBase::operator=(FDatasmithUniqueNameProviderBase&& Other)
{
	// Guarantee lock order
	TArray< FCriticalSection*, TInlineAllocator<2> > OrderedCriticalSections;
	OrderedCriticalSections.Add( &CriticalSection );
	OrderedCriticalSections.Add( &Other.CriticalSection );

	Algo::Sort( OrderedCriticalSections );

	FScopeLock LockFirst( OrderedCriticalSections[0] );
	FScopeLock LockSecond( OrderedCriticalSections[1] );

	FrequentlyUsedNames = MoveTemp(Other.FrequentlyUsedNames);

	return this;
}

FString FDatasmithUniqueNameProviderBase::GenerateUniqueName(const FString& InBaseName, int32 CharBudget)
{
	FScopeLock Lock( &CriticalSection );

	const int32 FrequentlyUsedThreshold = 5; // don't saturate the table with uncommon names

	for (int32 CurrentBaseNameCharBudget = CharBudget; CurrentBaseNameCharBudget >= 1; --CurrentBaseNameCharBudget)
	{
		FString ShortName = InBaseName.Left(CurrentBaseNameCharBudget);

		if (!Contains(ShortName))
		{
			AddExistingName(ShortName);
			return ShortName;
		}

		// use the frequently used label info to avoid useless indices iterations
		int32 LastKnownIndex = 1;
		int32* FreqIndexPtr = FrequentlyUsedNames.Find(ShortName);
		if (FreqIndexPtr != nullptr && *FreqIndexPtr > LastKnownIndex)
		{
			LastKnownIndex = *FreqIndexPtr;
		}

		// loop to find an available indexed name
		FString NumberedName;
		do
		{
			++LastKnownIndex;
			NumberedName = FString::Printf(TEXT("%s_%d"), *ShortName, LastKnownIndex);
		} while (NumberedName.Len() <= CharBudget && Contains(NumberedName));

		if (NumberedName.Len() > CharBudget)
		{
			continue;
		}

		// update frequently used names
		if (FreqIndexPtr != nullptr)
		{
			*FreqIndexPtr = LastKnownIndex;
		}
		else if (LastKnownIndex > FrequentlyUsedThreshold)
		{
			FrequentlyUsedNames.Add(ShortName, LastKnownIndex);
		}

		AddExistingName(NumberedName);
		return NumberedName;
	}

	UE_LOGF(LogDatasmith, Warning, "Cannot generate a unique name from '%ls'.", *InBaseName);
	return {};
}

void FDatasmithUniqueNameProviderBase::Clear()
{
	FScopeLock Lock( &CriticalSection );
	FrequentlyUsedNames.Empty();
}

FTransform FDatasmithUtils::ConvertTransform(EModelCoordSystem SourceCoordSystem, const FTransform& LocalTransform)
{
	// convert to UE coords
	static const FTransform RightHanded(FRotator(0.0f, 0.0f, 0.0f), FVector(0.0f, 0.0f, 0.0f), FVector(-1.0f, 1.0f, 1.0f));
	static const FTransform RightHandedLegacy(FRotator(0.0f, 0.0f, 0.0f), FVector(0.0f, 0.0f, 0.0f), FVector(1.0f, -1.0f, 1.0f));
	static const FTransform YUpMatrix(FMatrix(FVector(0.0f, 0.0f, 1.0f), FVector(1.0f, 0.0f, 0.0f), FVector(0.0f, 1.0f, 0.0f), FVector(0.0f, 0.0f, 0.0f)));
	static const FTransform YUpMatrixInv(YUpMatrix.Inverse());

	switch (SourceCoordSystem)
	{
	case EModelCoordSystem::ZUp_RightHanded:
	{
		return RightHanded * LocalTransform * RightHanded;
	}
	case EModelCoordSystem::YUp_LeftHanded:
	{
		return YUpMatrix * LocalTransform * YUpMatrixInv;
	}
	case EModelCoordSystem::YUp_RightHanded:
	{
		return RightHanded * YUpMatrix * LocalTransform * YUpMatrixInv * RightHanded;
	}
	case EModelCoordSystem::ZUp_RightHanded_FBXLegacy:
	{
		return RightHandedLegacy * LocalTransform * RightHandedLegacy;
	}
	default:
	{
		return LocalTransform;
	}
	}
}



