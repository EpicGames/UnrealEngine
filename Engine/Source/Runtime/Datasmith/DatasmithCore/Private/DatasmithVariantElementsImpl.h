// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"
#include "DatasmithSceneElementsImpl.h"
#include "DatasmithVariantElements.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

template< typename InterfaceType >
class FDatasmithBaseVariantElementImpl : public FDatasmithElementImpl< InterfaceType >
{
public:
	using FDatasmithElementImpl< InterfaceType >::FDatasmithElementImpl;

	virtual bool IsSubType( const EDatasmithElementVariantSubType VariantSubType ) const override
	{
		return FDatasmithElementImpl< InterfaceType >::IsSubTypeInternal( (uint64)VariantSubType );
	}
};

template< typename InterfaceType >
class FDatasmithBasePropertyCaptureElementImpl : public FDatasmithBaseVariantElementImpl< InterfaceType >
{
public:
	explicit FDatasmithBasePropertyCaptureElementImpl(EDatasmithElementVariantSubType InSubType = EDatasmithElementVariantSubType::PropertyCapture)
		: FDatasmithBaseVariantElementImpl< InterfaceType >(nullptr, EDatasmithElementType::Variant, (uint64)InSubType)
		, Category(EDatasmithPropertyCategory::Undefined)
	{
	}

	virtual void SetPropertyPath(const FString& Path) override
	{
		PropertyPath = Path;
	}

	virtual const FString& GetPropertyPath() const override
	{
		return PropertyPath;
	}

	virtual void SetCategory(EDatasmithPropertyCategory InCategory) override
	{
		Category = InCategory;
	}

	virtual EDatasmithPropertyCategory GetCategory() const override
	{
		return Category;
	}

private:
	FString PropertyPath;
	EDatasmithPropertyCategory Category;
};

class FDatasmithPropertyCaptureElementImpl : public FDatasmithBasePropertyCaptureElementImpl< IDatasmithPropertyCaptureElement >
{
public:
	explicit FDatasmithPropertyCaptureElementImpl()
		: FDatasmithBasePropertyCaptureElementImpl(EDatasmithElementVariantSubType::PropertyCapture)
	{
	}

	virtual void SetRecordedData(const uint8* InData, int32 InNumBytes)
	{
		Data.SetNumUninitialized(InNumBytes);
		FMemory::Memcpy(Data.GetData(), InData, InNumBytes);
	}

	virtual const TArray<uint8>& GetRecordedData() const
	{
		return Data;
	}

protected:
	TArray<uint8> Data;
};

class FDatasmithObjectPropertyCaptureElementImpl : public FDatasmithBasePropertyCaptureElementImpl< IDatasmithObjectPropertyCaptureElement >
{
public:
	explicit FDatasmithObjectPropertyCaptureElementImpl()
		: FDatasmithBasePropertyCaptureElementImpl(EDatasmithElementVariantSubType::ObjectPropertyCapture)
	{
	}

	virtual void SetRecordedObject(TWeakPtr<IDatasmithElement> InObject)
	{
		Object = InObject;
	}

	virtual TWeakPtr<IDatasmithElement> GetRecordedObject() const
	{
		return Object;
	}

private:
	TWeakPtr<IDatasmithElement> Object;
};

class FDatasmithActorBindingElementImpl : public FDatasmithBaseVariantElementImpl< IDatasmithActorBindingElement >
{
public:
	explicit FDatasmithActorBindingElementImpl()
		: FDatasmithBaseVariantElementImpl(nullptr, EDatasmithElementType::Variant, (uint64)EDatasmithElementVariantSubType::ActorBinding)
	{
	}

	virtual void SetActor(TSharedPtr<IDatasmithActorElement> InActor) override
	{
		Actor = InActor;
	}

	virtual TSharedPtr<IDatasmithActorElement> GetActor() const override
	{
		return Actor;
	}

	virtual void AddPropertyCapture(const TSharedRef< IDatasmithBasePropertyCaptureElement >& Prop) override
	{
		PropertyCaptures.Add(Prop);
	}

	virtual int32 GetPropertyCapturesCount() const override
	{
		return PropertyCaptures.Num();
	}

	virtual TSharedPtr< IDatasmithBasePropertyCaptureElement > GetPropertyCapture(int32 InIndex) override
	{
		return PropertyCaptures.IsValidIndex(InIndex) ? PropertyCaptures[InIndex] : TSharedPtr< IDatasmithBasePropertyCaptureElement >();
	}

	virtual void RemovePropertyCapture(const TSharedRef< IDatasmithBasePropertyCaptureElement >& Prop) override
	{
		PropertyCaptures.Remove(Prop);
	}

private:
	TSharedPtr<IDatasmithActorElement> Actor;
	TArray< TSharedRef< IDatasmithBasePropertyCaptureElement > > PropertyCaptures;
};

class FDatasmithVariantElementImpl : public FDatasmithBaseVariantElementImpl< IDatasmithVariantElement >
{
public:
	explicit FDatasmithVariantElementImpl(const TCHAR* InName)
		: FDatasmithBaseVariantElementImpl(InName, EDatasmithElementType::Variant, (uint64)EDatasmithElementVariantSubType::Variant)
	{
	}

	virtual void AddActorBinding(const TSharedRef< IDatasmithActorBindingElement >& Binding) override
	{
		Bindings.Add(Binding);
	}

	virtual int32 GetActorBindingsCount() const override
	{
		return Bindings.Num();
	}

	virtual TSharedPtr< IDatasmithActorBindingElement > GetActorBinding(int32 InIndex) override
	{
		return Bindings.IsValidIndex(InIndex) ? Bindings[InIndex] : TSharedPtr< IDatasmithActorBindingElement >();
	}

	virtual void RemoveActorBinding(const TSharedRef< IDatasmithActorBindingElement >& Binding) override
	{
		Bindings.Remove(Binding);
	}

private:
	TArray< TSharedRef< IDatasmithActorBindingElement > > Bindings;
};

class FDatasmithVariantSetElementImpl : public FDatasmithBaseVariantElementImpl< IDatasmithVariantSetElement >
{
public:
	explicit FDatasmithVariantSetElementImpl(const TCHAR* InName)
		: FDatasmithBaseVariantElementImpl(InName, EDatasmithElementType::Variant, (uint64)EDatasmithElementVariantSubType::VariantSet)
	{
	}

	virtual void AddVariant(const TSharedRef< IDatasmithVariantElement >& Variant) override
	{
		Variants.Add(Variant);
	}

	virtual int32 GetVariantsCount() const override
	{
		return Variants.Num();
	}

	virtual TSharedPtr< IDatasmithVariantElement > GetVariant(int32 InIndex) override
	{
		return Variants.IsValidIndex(InIndex) ? Variants[InIndex] : TSharedPtr< IDatasmithVariantElement >();
	}

	virtual void RemoveVariant(const TSharedRef< IDatasmithVariantElement >& Variant) override
	{
		Variants.Remove(Variant);
	}

private:
	TArray< TSharedRef< IDatasmithVariantElement > > Variants;
};

class FDatasmithLevelVariantSetsElementImpl : public FDatasmithBaseVariantElementImpl< IDatasmithLevelVariantSetsElement  >
{
public:
	explicit FDatasmithLevelVariantSetsElementImpl(const TCHAR* InName)
		: FDatasmithBaseVariantElementImpl(InName, EDatasmithElementType::Variant, (uint64)EDatasmithElementVariantSubType::LevelVariantSets)
	{
	}

	virtual void AddVariantSet(const TSharedRef< IDatasmithVariantSetElement >& VariantSet) override
	{
		VariantSets.Add(VariantSet);
	}

	virtual int32 GetVariantSetsCount() const override
	{
		return VariantSets.Num();
	}

	virtual TSharedPtr< IDatasmithVariantSetElement > GetVariantSet(int32 InIndex) override
	{
		return VariantSets.IsValidIndex(InIndex) ? VariantSets[InIndex] : TSharedPtr< IDatasmithVariantSetElement >();
	}

	virtual void RemoveVariantSet(const TSharedRef< IDatasmithVariantSetElement >& VariantSet) override
	{
		VariantSets.Remove(VariantSet);
	}

private:
	TArray< TSharedRef< IDatasmithVariantSetElement > > VariantSets;
};








