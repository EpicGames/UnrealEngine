// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLink/DatasmithDirectLinkTools.h"

#include "DatasmithSceneXmlWriter.h"
#include "IDatasmithSceneElements.h"
#include "DirectLinkMisc.h"
#include "HAL/FileManager.h"



const TCHAR* GetElementTypeName(const IDatasmithElement* Element)
{
	if (Element == nullptr)
	{
		return TEXT("<nullptr>");
	}
#define DS_ELEMENT_TYPE(x) \
	if (Element->IsA(x))   \
	{                      \
		return TEXT(#x);   \
	}
	DS_ELEMENT_TYPE(EDatasmithElementType::Variant)
	DS_ELEMENT_TYPE(EDatasmithElementType::Animation)
	DS_ELEMENT_TYPE(EDatasmithElementType::LevelSequence)
	DS_ELEMENT_TYPE(EDatasmithElementType::PostProcessVolume)
	DS_ELEMENT_TYPE(EDatasmithElementType::UEPbrMaterial)
	DS_ELEMENT_TYPE(EDatasmithElementType::Landscape)
	DS_ELEMENT_TYPE(EDatasmithElementType::Material)
	DS_ELEMENT_TYPE(EDatasmithElementType::CustomActor)
	DS_ELEMENT_TYPE(EDatasmithElementType::MetaData)
	DS_ELEMENT_TYPE(EDatasmithElementType::Scene)
	DS_ELEMENT_TYPE(EDatasmithElementType::PostProcess)
	DS_ELEMENT_TYPE(EDatasmithElementType::MaterialId)
	DS_ELEMENT_TYPE(EDatasmithElementType::Texture)
	DS_ELEMENT_TYPE(EDatasmithElementType::KeyValueProperty)
	DS_ELEMENT_TYPE(EDatasmithElementType::MaterialInstance)
	DS_ELEMENT_TYPE(EDatasmithElementType::BaseMaterial)
	DS_ELEMENT_TYPE(EDatasmithElementType::Shader)
	DS_ELEMENT_TYPE(EDatasmithElementType::Camera)
	DS_ELEMENT_TYPE(EDatasmithElementType::EnvironmentLight)
	DS_ELEMENT_TYPE(EDatasmithElementType::LightmassPortal)
	DS_ELEMENT_TYPE(EDatasmithElementType::AreaLight)
	DS_ELEMENT_TYPE(EDatasmithElementType::DirectionalLight)
	DS_ELEMENT_TYPE(EDatasmithElementType::SpotLight)
	DS_ELEMENT_TYPE(EDatasmithElementType::PointLight)
	DS_ELEMENT_TYPE(EDatasmithElementType::Light)
	DS_ELEMENT_TYPE(EDatasmithElementType::StaticMeshActor)
	DS_ELEMENT_TYPE(EDatasmithElementType::Actor)
	DS_ELEMENT_TYPE(EDatasmithElementType::HierarchicalInstanceStaticMesh)
	DS_ELEMENT_TYPE(EDatasmithElementType::StaticMesh)
	DS_ELEMENT_TYPE(EDatasmithElementType::None)
#undef DS_ELEMENT_TYPE
	return TEXT("<unknown>");
}


void DumpDatasmithScene(const TSharedRef<IDatasmithScene>& Scene, const TCHAR* BaseName)
{
	const FString& DumpPath = DirectLink::GetDumpPath();
	if (DumpPath.IsEmpty())
	{
		return;
	}

	FString SceneIdStr;
	if (Scene->GetSharedState())
	{
		SceneIdStr = FString::Printf(TEXT(".%08X"), Scene->GetSharedState()->GetGuid().A);
	}
	FString FileName = DumpPath / BaseName + SceneIdStr + TEXT(".directlink.udatasmith");
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*FileName));
	if (!Ar.IsValid())
	{
		return;
	}

	FDatasmithSceneXmlWriter Writer;
	Writer.Serialize(Scene, *Ar);
}


