// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLink/DatasmithSceneReceiver.h"


#include "DatasmithCore.h"
#include "DatasmithMaterialElementsImpl.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneGraphSharedState.h"
#include "IDatasmithSceneElements.h"
#include "DirectLink/DatasmithDirectLinkTools.h"

#include "DirectLinkCommon.h"
#include "DirectLinkElementSnapshot.h"
#include "DirectLinkLog.h"
#include "DirectLinkParameterStore.h"
#include "DirectLinkSceneSnapshot.h"


/* #ue_directlink_design
 *
 * the scene (aka sharedState) should be the owner of existing node.
 * Resolution of existing node should be provided by the scene, not by the DeltaConsumer.
 */


TSharedPtr<DirectLink::ISceneGraphNode> FDatasmithSceneReceiver::FDatasmithElementPointers::AsSharedPtr(DirectLink::FSceneGraphId NodeId)
{
	if (TSharedPtr<IDatasmithElement>* ElementPtr = ElementsSharedPtrs.Find(NodeId))
	{
		return *ElementPtr;
	}
	return nullptr;
}


FDatasmithSceneReceiver::FDatasmithSceneReceiver()
{
	Current = MakeUnique<FSceneState>();
}


TSharedPtr<IDatasmithScene> FDatasmithSceneReceiver::GetScene() const
{
	return Current->Scene;
}


void FDatasmithSceneReceiver::FinalSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot)
{
	FSceneHashTable OldHashTable = MoveTemp(Current->HashTable);
	DirectLink::FSceneIdentifier OldSceneId = Current->SceneId;
	Current = ParseSnapshot(SceneSnapshot);

	if (!Current->Scene)
	{
		return;
	}

#if 1
	if (ChangeListener)
	{
		// Diff -> lazy eval API should be better
		ChangeListener->OnOpenDelta();

		if (OldSceneId.SceneGuid != Current->SceneId.SceneGuid || !OldSceneId.SceneGuid.IsValid())
		{
			ChangeListener->OnNewScene(Current->SceneId);
		}

		auto OldItr = OldHashTable.ElementHashes.CreateConstIterator();
		auto NewItr = Current->HashTable.ElementHashes.CreateConstIterator();

		auto AddedElement = [this](const auto& Itr)
		{
			ChangeListener->OnAddElement(Itr.Key(), Current->Elements.ElementsSharedPtrs[Itr.Key()]);
		};

		auto RemovedElement = [this](const auto& Itr)
		{
			ChangeListener->OnRemovedElement(Itr.Key());
		};

		auto ChangedElement = [this](const auto& Itr)
		{
			ChangeListener->OnChangedElement(Itr.Key(), Current->Elements.ElementsSharedPtrs[Itr.Key()]);
		};

		while (true)
		{
			if (!OldItr)
			{
				while(NewItr)
				{
					AddedElement(NewItr);
					++NewItr;
				}
				break;
			}

			if (!NewItr)
			{
				while (OldItr)
				{
					RemovedElement(OldItr);
					++OldItr;
				}
				break;
			}

			if (OldItr.Key() < NewItr.Key())
			{
				RemovedElement(OldItr);
				++OldItr;
				continue;
			}

			if (OldItr.Key() > NewItr.Key())
			{
				AddedElement(NewItr);
				++NewItr;
				continue;
			}

			if (OldItr.Value() != NewItr.Value())
			{
				ChangedElement(NewItr);
			}
			++OldItr;
			++NewItr;
		}

		ChangeListener->OnCloseDelta();
	}
#else
	if (ChangeListener)
	{
		ChangeListener->OnOpenDelta();
		{
			ChangeListener->OnNewScene(Current->SceneId);
			for (const auto& Pair : Current->Elements.ElementsSharedPtrs)
			{
				ChangeListener->OnAddElement(Pair.Value);
			}
		}
		ChangeListener->OnCloseDelta();
	}
#endif
}

TUniquePtr<FDatasmithSceneReceiver::FSceneState> FDatasmithSceneReceiver::ParseSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot)
{
	TUniquePtr<FSceneState> NewSceneState = MakeUnique<FSceneState>();
	NewSceneState->HashTable = FSceneHashTable::FromSceneSnapshot(SceneSnapshot);
	NewSceneState->SceneId = SceneSnapshot.SceneId;

	TArray<FFinalizableNode> Nodes;
	Nodes.Reserve(SceneSnapshot.Elements.Num());

	TSharedPtr<FDatasmithSceneGraphSharedState> SceneSharedState = MakeShared<FDatasmithSceneGraphSharedState>(SceneSnapshot.SceneId);

	const EDatasmithElementType UnsupportedTypes = EDatasmithElementType::Material;
	EDatasmithElementType FoundUnsupportedTypes =  EDatasmithElementType::None;

	for (const auto& KV : SceneSnapshot.Elements)
	{
		const DirectLink::FElementSnapshot& ElementSnapshot = KV.Value.Get();
		DirectLink::FSceneGraphId NodeId = ElementSnapshot.GetNodeId();

		FString Name;
		if (!ElementSnapshot.GetValueAs("Name", Name))
		{
			UE_LOGF(LogDatasmith, Error, "Directlink snapshot parsing failed: missing element name for node #%d.", NodeId);
			return NewSceneState;
		}

		uint64 Type = 0;
		if (!ElementSnapshot.GetValueAs("Type", Type))
		{
			UE_LOGF(LogDatasmith, Error, "Directlink snapshot parsing failed: missing element type info for node '%ls'", *Name);
			return NewSceneState;
		}

		uint64 Subtype = 0;
		if (!ElementSnapshot.GetValueAs("Subtype", Subtype))
		{
			UE_LOGF(LogDatasmith, Error, "Directlink snapshot parsing failed: missing element subtype info for node '%ls'", *Name);
			return NewSceneState;
		}

		// derived types have several bits sets.
		// -> keep the leftmost bit, which is the value of the (most-derived) class understood by CreateElement
		// eg. this transforms 'Actor|StaticMeshActor' into 'StaticMeshActor'
		// Well, of course it's not exact...
		// Type &= (uint64)~EDatasmithElementType::BaseMaterial; // remove that flag as it always has a child anyway, and its order is impractical.
		EDatasmithElementType PureType = EDatasmithElementType(uint64(1) << FPlatformMath::FloorLog2_64(Type));

		if (EnumHasAnyFlags(PureType, UnsupportedTypes))
		{
			// We can't skip any unsupported type, because DirectLink needs to validate that each exported node is also imported.
			// So we must remove those types in a latter step.
			FoundUnsupportedTypes |= PureType;
		}

		TSharedPtr<IDatasmithElement> Element = FDatasmithSceneFactory::CreateElement(PureType, Subtype, *Name);
		if (!Element)
		{
			UE_LOG(LogDatasmith, Error, TEXT("Directlink snapshot parsing failed: cannot create element '%s' of type %" UINT64_FMT), *Name, Type);
			return NewSceneState;
		}

		Element->SetSharedState(SceneSharedState);
		Element->SetNodeId(NodeId); // #ue_directlink_design nope, only the Scene SharedState has this right
		NewSceneState->Elements.Add(NodeId, Element);

		const TCHAR* ElementTypeName = GetElementTypeName(Element.Get());

		FFinalizableNode& Node = Nodes.AddDefaulted_GetRef();
		Node.Element = Element;
		Node.Snapshot = &ElementSnapshot;
	}

	// all nodes are created, link refs
	for (FFinalizableNode& Node : Nodes)
	{
		Node.Snapshot->UpdateNodeReferences(NewSceneState->Elements, *Node.Element);
	}

	// set data
	for (FFinalizableNode& Node : Nodes)
	{
		Node.Snapshot->UpdateNodeData(*Node.Element);
	}

	// detect graph root
	for (FFinalizableNode& Node : Nodes)
	{
		if (Node.Element->IsA(EDatasmithElementType::Scene))
		{
			NewSceneState->Scene = StaticCastSharedPtr<IDatasmithScene>(Node.Element);

			if (ensure(NewSceneState->Scene))
			{
				DumpDatasmithScene(NewSceneState->Scene.ToSharedRef(), TEXT("received"));
			}
			break;
		}
	}

	if (!NewSceneState->Scene)
	{
		UE_LOGF(LogDatasmith, Error, "Directlink snapshot parsing failed: cannot find the scene root element.");
		return NewSceneState;
	}

	if (FoundUnsupportedTypes != EDatasmithElementType::None && NewSceneState->Scene)
	{
		IDatasmithScene& Scene = *NewSceneState->Scene;

		if (EnumHasAllFlags(FoundUnsupportedTypes, EDatasmithElementType::Material))
		{
			UE_LOGF(LogDatasmith, Warning, "Datasmith scene \"%ls\" imported with DirectLink contains deprecated IDatasmithMaterialElement, they will be ignored.", Scene.GetName());
			for (int MaterialIndex = Scene.GetMaterialsCount() - 1; 0 <= MaterialIndex; --MaterialIndex)
			{
				TSharedPtr<IDatasmithBaseMaterialElement> CurrentMaterial = Scene.GetMaterial(MaterialIndex);
				if (CurrentMaterial && CurrentMaterial->IsA(UnsupportedTypes))
				{
					Scene.RemoveMaterialAt(MaterialIndex);
				}
			}
		}
	}

	return NewSceneState;
}


FDatasmithSceneReceiver::FSceneHashTable FDatasmithSceneReceiver::FSceneHashTable::FromSceneSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot)
{
	FSceneHashTable Out;
	for (const auto& Pair : SceneSnapshot.Elements)
	{
		const DirectLink::FSceneGraphId& Id = Pair.Key;
		const TSharedRef<DirectLink::FElementSnapshot> ElementSnapshot = Pair.Value;

		Out.ElementHashes.Emplace(Id, ElementSnapshot->GetHash());
	}

	Out.ElementHashes.KeySort(TLess<>());
	return Out;
}

