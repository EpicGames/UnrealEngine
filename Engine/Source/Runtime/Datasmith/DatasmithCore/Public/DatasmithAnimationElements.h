// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDatasmithSceneElements.h"

#include "Misc/FrameNumber.h"
#include "Misc/SecureHash.h"
#include "Templates/SharedPointer.h"

class IDatasmithLevelSequenceElement;

class IDatasmithBaseAnimationElement : public IDatasmithElement
{
public:
	virtual bool IsSubType( const EDatasmithElementAnimationSubType AnimSubType ) const = 0;

	/** Set how the actor should behave once its animation completes */
	virtual void SetCompletionMode(EDatasmithCompletionMode CompletionMode) = 0;

	/** Get how the actor behaves once this animation is complete */
	virtual EDatasmithCompletionMode GetCompletionMode() const = 0;
};

/** IDatasmithTransformAnimationElement holds the frames for an actor transform animation */
class IDatasmithTransformAnimationElement : public IDatasmithBaseAnimationElement
{
public:
	virtual ~IDatasmithTransformAnimationElement() = default;

	/** Add a frame of the given transform type to the animation */
	virtual void AddFrame(EDatasmithTransformType TransformType, const FDatasmithTransformFrameInfo& FrameInfo) = 0;

	/** Return the number of frames of the given transform type in the animation */
	virtual int32 GetFramesCount(EDatasmithTransformType TransformType) const = 0;

	/** Set the interpolation mode of the given transform type in the animation */
	virtual void SetCurveInterpMode(EDatasmithTransformType TransformType, EDatasmithCurveInterpMode CurveInterpMode) = 0;

	/** Get the interpolation mode of the given transform type in the animation */
	virtual EDatasmithCurveInterpMode GetCurveInterpMode(EDatasmithTransformType TransformType) const = 0;

	/** Return the frame of the given transform type at the given index or an invalid frame if the index was out of bounds */
	virtual const FDatasmithTransformFrameInfo& GetFrame(EDatasmithTransformType TransformType, int32 Index) const = 0;

	/** Remove the frame of the given transform type at the given index from the animation */
	virtual void RemoveFrame(EDatasmithTransformType TransformType, int32 Index) = 0;

	/** Gets which channels of this animation will be moved to the animation assets or serialized. All channels are enabled by default */
	virtual EDatasmithTransformChannels GetEnabledTransformChannels() const = 0;

	/** Sets which channels of this animation will be moved to the animation assets or serialized. All channels are enabled by default */
	virtual void SetEnabledTransformChannels(EDatasmithTransformChannels Channels) = 0;
};

/** IDatasmithVisibilityAnimationElement holds the frames for an actor's visibility animation */
class IDatasmithVisibilityAnimationElement : public IDatasmithBaseAnimationElement
{
public:
	virtual ~IDatasmithVisibilityAnimationElement() = default;

	/** Add a frame to the animation */
	virtual void AddFrame(const FDatasmithVisibilityFrameInfo& FrameInfo) = 0;

	/** Return the number of frames of the animation */
	virtual int32 GetFramesCount() const = 0;

	/** Set the interpolation mode of the animation */
	virtual void SetCurveInterpMode(EDatasmithCurveInterpMode CurveInterpMode) = 0;

	/** Get the interpolation mode of the animation */
	virtual EDatasmithCurveInterpMode GetCurveInterpMode() const = 0;

	/** Return the frame of the animation at the given index or an invalid frame if the index was out of bounds */
	virtual const FDatasmithVisibilityFrameInfo& GetFrame(int32 Index) const = 0;

	/** Remove the frame at the given index from the animation */
	virtual void RemoveFrame(int32 Index) = 0;

	/** Sets whether this animation will be duplicated to all children (recursively) when imported */
	virtual void SetPropagateToChildren(bool bPropagate) = 0;

	/** Gets whether this animation will be duplicated to all children (recursively) when imported */
	virtual bool GetPropagateToChildren() const = 0;
};

/** IDatasmithSubsequenceAnimationElement holds a reference to a IDatasmithLevelSequenceElement, to be played as a subsequence */
class IDatasmithSubsequenceAnimationElement : public IDatasmithBaseAnimationElement
{
public:
	virtual ~IDatasmithSubsequenceAnimationElement() = default;

	/** Get the frame where the subsequence starts */
	virtual FFrameNumber GetStartTime() const = 0;

	/** Set the frame where the subsequence starts */
	virtual void SetStartTime(FFrameNumber InStartTime) = 0;

	/** Get the subsequence duration in number of frames */
	virtual int32 GetDuration() const = 0;

	/** Set the subsequence duration in number of frames */
	virtual void SetDuration(int32 InDuration) = 0;

	/** Get the time scale used for the subsequence */
	virtual float GetTimeScale() const = 0;

	/** Set the time scale used for the subsequence */
	virtual void SetTimeScale(float InTimeScale) = 0;

	/** Get a pointer to the subsequence that this element references */
	virtual TWeakPtr<IDatasmithLevelSequenceElement> GetSubsequence() const = 0;

	/** Set the subsequence that this element references */
	virtual void SetSubsequence(TWeakPtr<IDatasmithLevelSequenceElement> InSubsequence) = 0;
};

/** IDatasmithLevelSequenceElement holds a set of animations */
class IDatasmithLevelSequenceElement : public IDatasmithElement
{
public:
	virtual ~IDatasmithLevelSequenceElement() = default;

	/** Get the output filename, it can be absolute or relative to the scene file */
	virtual const TCHAR* GetFile() const = 0;

	/** Set the output filename, it can be absolute or relative to the scene file */
	virtual void SetFile(const TCHAR* InFile) = 0;

	/** Return a MD5 hash of the content of the Level Sequence Element. Used in CalculateElementHash to quickly identify Element with identical content */
	virtual FMD5Hash GetFileHash() const = 0;

	/** Set the MD5 hash of the Level Sequence file. This should be a hash of its content. */
	virtual void SetFileHash(FMD5Hash Hash) = 0;

	/* Gets the frame rate for the animations in the level sequence */
	virtual float GetFrameRate() const = 0;

	/* Sets the frame rate for the animations in the level sequence */
	virtual void SetFrameRate(float FramePerSecs) = 0;

	/**
	 * Adds an animation to the level sequence.
	 *
	 * @param InAnimation the animation to add
	 */
	virtual void AddAnimation(const TSharedRef< IDatasmithBaseAnimationElement >& InAnimation) = 0;

	/** Returns the number of animations in the level sequence */
	virtual int32 GetAnimationsCount() const = 0;

	/** Returns the animation using this index */
	virtual TSharedPtr< IDatasmithBaseAnimationElement > GetAnimation(int32 InIndex) = 0;

	/**
	 * Removes an animation from the level sequence.
	 *
	 * @param InAnimation the animation to remove
	 */
	virtual void RemoveAnimation(const TSharedRef< IDatasmithBaseAnimationElement >& InAnimation) = 0;
};
