// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

#define UE_API DATASMITHCORE_API

class FArchive;
class IDatasmithLevelSequenceElement;

#define DATASMITH_ANIMATION_EXTENSION TEXT(".udsanim")

class FDatasmithAnimationSerializer
{
public:
	UE_API bool Serialize(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TCHAR* FilePath, bool bDebugFormat = false);
	UE_API bool Deserialize(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequence, const TCHAR* FilePath);
};

#undef UE_API
