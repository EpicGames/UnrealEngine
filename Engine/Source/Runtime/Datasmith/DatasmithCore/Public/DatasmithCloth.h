// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/TVariant.h"
#include "UObject/NameTypes.h"
#include "DatasmithCloth.generated.h"

#define UE_API DATASMITHCORE_API

class FDatasmithMesh;
class UActorComponent;


struct UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FParameterData
{
	FString Name;

	enum class ETarget { Vertex };
	ETarget Target = ETarget::Vertex; // (also drives the expected number of values)

	TVariant<TArray<float>, TArray<double>> Data;

public:
	friend FArchive& operator<<(FArchive& Ar, FParameterData& ParameterData);
};


class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API FDatasmithClothPattern
{
public:
	TArray<FVector2f> SimPosition;
	TArray<FVector3f> SimRestPosition;
	TArray<uint32> SimTriangleIndices;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<FParameterData> Parameters;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	bool IsValid() const { return SimRestPosition.Num() == SimPosition.Num() && SimTriangleIndices.Num() % 3 == 0 && SimTriangleIndices.Num(); }
	friend FArchive& operator<<(FArchive& Ar, FDatasmithClothPattern& Pattern);
};


struct UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FDatasmithClothSewingInfo
{
	uint32 Seam0PanelIndex = 0;
	uint32 Seam1PanelIndex = 0;
	TArray<uint32> Seam0MeshIndices;
	TArray<uint32> Seam1MeshIndices;

	friend FArchive& operator<<(FArchive& Ar, FDatasmithClothSewingInfo& Sewing);
};


class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FDatasmithClothPresetProperty
{
public:
	FName Name;
	double Value;

public:
	friend FArchive& operator<<(FArchive& Ar, FDatasmithClothPresetProperty& Property);
};


class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API FDatasmithClothPresetPropertySet
{
public:
	FString SetName;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<FDatasmithClothPresetProperty> Properties;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	friend FArchive& operator<<(FArchive& Ar, FDatasmithClothPresetPropertySet& PropertySet);
};


class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API FDatasmithCloth
{
public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<FDatasmithClothPattern> Patterns;
	TArray<FDatasmithClothSewingInfo> Sewing;
	TArray<FDatasmithClothPresetPropertySet> PropertySets;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	friend FArchive& operator<<(FArchive& Ar, FDatasmithCloth& Cloth);
};


/** Modular cloth asset factory base class. */
UCLASS(MinimalAPI, Abstract)
class UDatasmithClothAssetFactory : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UDatasmithClothAssetFactory() = default;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual ~UDatasmithClothAssetFactory() = default;

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual UObject* CreateClothAsset(UObject* Outer, const FName& Name, EObjectFlags Flags) const
	PURE_VIRTUAL(UDatasmithClothAssetFactory::CreateClothAsset, return nullptr;);

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual UObject* DuplicateClothAsset(UObject* ClothAsset, UObject* Outer, const FName& Name) const
	PURE_VIRTUAL(UDatasmithClothAssetFactory::DuplicateClothAsset, return nullptr;);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual void InitializeClothAsset(UObject* ClothAsset, const FDatasmithCloth& DatasmithCloth) const
	PURE_VIRTUAL(UDatasmithClothAssetFactory::InitializeClothAsset, );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


/** Modular cloth component factory base class. */
UCLASS(MinimalAPI, Abstract)
class UDatasmithClothComponentFactory : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UDatasmithClothComponentFactory() = default;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual ~UDatasmithClothComponentFactory() = default;

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual USceneComponent* CreateClothComponent(UObject* Outer) const
	PURE_VIRTUAL(UDatasmithClothComponentFactory::CreateClothComponent, return nullptr;);

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual void InitializeClothComponent(USceneComponent* ClothComponent, UObject* ClothAsset, USceneComponent* RootComponent) const
	PURE_VIRTUAL(UDatasmithClothComponentFactory::InitializeClothComponent, );
};


/** A modular interface to provide factory classes to initialize cloth assets and components. */
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") IDatasmithClothFactoryClassesProvider : public IModularFeature
{
public:
	inline static const FName FeatureName = "IDatasmithClothFactoryClassesProvider";

	IDatasmithClothFactoryClassesProvider() = default;
	virtual ~IDatasmithClothFactoryClassesProvider() = default;

	virtual FName GetName() const = 0;

	virtual TSubclassOf<UDatasmithClothAssetFactory> GetClothAssetFactoryClass() const = 0;
	virtual TSubclassOf<UDatasmithClothComponentFactory> GetClothComponentFactoryClass() const = 0;
};

#undef UE_API
