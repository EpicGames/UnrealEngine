// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/EnumClassFlags.h"

#ifndef WITH_COREUOBJECT
	#define WITH_COREUOBJECT 0
#endif

#if WITH_COREUOBJECT
	#include "DatasmithDefinitions.generated.h"
#else
	#define UENUM(...)
#endif // WITH_COREUOBJECT

/** Element type is used to identify its class like Mesh, Actor, Light, etc. */
enum class EDatasmithElementType : uint64
{
	None                           = 0ull,
	StaticMesh                     = 1ull <<  0,
	Actor                          = 1ull <<  1,
	StaticMeshActor                = 1ull <<  2,
	Light                          = 1ull <<  3,
	PointLight                     = 1ull <<  4,
	SpotLight                      = 1ull <<  5,
	DirectionalLight               = 1ull <<  6,
	AreaLight                      = 1ull <<  7,
	LightmassPortal                = 1ull <<  8,
	EnvironmentLight               = 1ull <<  9,
	Camera                         = 1ull << 10,
	Shader                         = 1ull << 11,
	BaseMaterial                   = 1ull << 12,
	MasterMaterial UE_DEPRECATED(5.1, "EDatasmithElementType::MasterMaterial will not be supported in 5.2. Please use EDatasmithElementType::MaterialInstance instead.") = 1ull << 13,
	MaterialInstance               = 1ull << 13,
	KeyValueProperty               = 1ull << 14,
	Texture                        = 1ull << 15,
	MaterialId                     = 1ull << 16,
	PostProcess                    = 1ull << 17,
	Scene                          = 1ull << 18,
	MetaData                       = 1ull << 19,
	CustomActor                    = 1ull << 20,
	Material                       = 1ull << 21,
	Landscape                      = 1ull << 22,
	UEPbrMaterial                  = 1ull << 23,
	PostProcessVolume              = 1ull << 24,
	LevelSequence                  = 1ull << 25,
	Animation                      = 1ull << 26,
	HierarchicalInstanceStaticMesh = 1ull << 27,
	Variant                        = 1ull << 28,
	Decal                          = 1ull << 29,
	DecalMaterial                  = 1ull << 30,
	MaterialExpression             = 1ull << 31,
	MaterialExpressionInput        = 1ull << 32,
	MaterialExpressionOutput       = 1ull << 33,
	Cloth      UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") = 1ull << 34,
	ClothActor UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") = 1ull << 35,
};

ENUM_CLASS_FLAGS( EDatasmithElementType ); // Define bitwise operators for EDatasmithElementType

/** Subtype of the Animation EDatasmithElementType, containing base, transform, visibility animations and etc. */
enum class EDatasmithElementAnimationSubType : uint64
{
	BaseAnimation = 0,
	TransformAnimation = 1 << 0,
	VisibilityAnimation = 1 << 1,
	SubsequenceAnimation = 1 << 2,
};

ENUM_CLASS_FLAGS( EDatasmithElementAnimationSubType ); // Define bitwise operators for EDatasmithElementAnimationSubtype

/** Subtype of the Variant EDatasmithElementType, containing property value, variant, variant set, etc. */
enum class EDatasmithElementVariantSubType : uint64
{
	None = 0,
	LevelVariantSets = 1 << 0,
	VariantSet = 1 << 1,
	Variant = 1 << 2,
	ActorBinding = 1 << 3,
	PropertyCapture = 1 << 4,
	ObjectPropertyCapture = 1 << 5,
};

ENUM_CLASS_FLAGS( EDatasmithElementVariantSubType ); // Define bitwise operators for EDatasmithElementVariantSubType

/** Subtype of the MaterialExpression EDatasmithElementType, containing property value, variant, variant set, etc. */
enum class EDatasmithMaterialExpressionType : uint8
{
	ConstantBool,
	ConstantColor,
	ConstantScalar,
	FlattenNormal,
	FunctionCall,
	Generic,
	Texture,
	TextureCoordinate,
	Custom,

	None = 255
};


/**
 * Describes a set of channels from a transform animation. Used to enable/disable those channels on import/export.
 * The values defined in EDatasmithTransformChannels should mirror the analogues in EMovieSceneTransformChannel
 */
enum class EDatasmithTransformChannels : uint16
{
	None			= 0x000,

	TranslationX 	= 0x001,
	TranslationY 	= 0x002,
	TranslationZ 	= 0x004,
	Translation 	= TranslationX | TranslationY | TranslationZ,

	RotationX 		= 0x008,
	RotationY 		= 0x010,
	RotationZ 		= 0x020,
	Rotation 		= RotationX | RotationY | RotationZ,

	ScaleX 			= 0x040,
	ScaleY 			= 0x080,
	ScaleZ 			= 0x100,
	Scale 			= ScaleX | ScaleY | ScaleZ,

	All				= Translation | Rotation | Scale,
};

ENUM_CLASS_FLAGS( EDatasmithTransformChannels ); // Define bitwise operators for EDatasmithTransformChannels

/** Different supported light shapes */
UENUM(BlueprintType)
enum class EDatasmithLightShape : uint8
{
	Rectangle,
	Disc,
	Sphere,
	Cylinder,
	None
};

static const TCHAR* DatasmithAreaLightShapeStrings[] = { TEXT("Rectangle"), TEXT("Disc"), TEXT("Sphere"), TEXT("Cylinder"), TEXT("None") };

enum class EDatasmithAreaLightType
{
	Point,
	Spot,
	IES_DEPRECATED,
	Rect
};

static const TCHAR* DatasmithAreaLightTypeStrings[] = { TEXT("Point"), TEXT("Spot"), TEXT("IES"), TEXT("Rect") };

/** Light intensity units */
enum class EDatasmithLightUnits
{
	Unitless,
	Candelas,
	Lumens,
	EV,
};

/** Different usage for textures.  Note: Preserve enum order. */
UENUM()
enum class EDatasmithTextureMode : uint8
{
	Diffuse,
	Specular,
	Normal,
	NormalGreenInv,
	UNUSED_Displace,
	Other,
	Bump,
	Ies
};

/** Texture filtering for textures. */
UENUM()
enum class EDatasmithTextureFilter : uint8
{
	Nearest,
	Bilinear,
	Trilinear,
	/** Use setting from the Texture Group. */
	Default
};

/** Texture address mode for textures.  Note: Preserve enum order. */
UENUM()
enum class EDatasmithTextureAddress : uint8
{
	Wrap,
	Clamp,
	Mirror
};

/** Texture format for raw data importing. */
UENUM()
enum class EDatasmithTextureFormat : uint8
{
	PNG,
	JPEG
};

/**
 * Texture color space.
 * Default: Leave at whatever is default for the texture mode
 * sRGB: Enable the sRGB boolean regardless of texture mode
 * Linear: Disable the sRGB boolean regardless of texture mode
 */
UENUM()
enum class EDatasmithColorSpace : uint8
{
	Default,
	sRGB,
	Linear,
};

/**
 * Regular: lambertians, glossy materials and almost every type of material but glass, metal or highly reflective.
 * Glass: glass material, it should have appropriate index of refraction and transparency
 * Metal: to be considered a metal material it should have a proper reflective ior
 * MixedMetal: highly reflective non-metallic materials
 */
enum class EDatasmithMaterialMode
{
	Regular,
	Glass,
	Metal,
	MixedMetal
};

enum class EDatasmithReferenceMaterialType : uint8
{
	/** Let Datasmith figure which reference material to use */
	Auto,
	Opaque,
	Transparent,
	ClearCoat,
	/** Instantiate a reference material from a specified one */
	Custom,
	/** Material has a transparent cutout map */
	CutOut,
	Emissive,
	Decal,
	/** Dummy element to count the number of types */
	Count
};

enum class EDatasmithReferenceMaterialQuality : uint8
{
	High,
	Low,
	/** Dummy element to count the number of qualities */
	Count
};

/**
 * Different methods for mixing textures:
 * Just one texture
 * Mix blended by weight
 * Fresnel using a fresnel weight expression
 * Ior using a fresnel weight expression where its curve is defined by the ior value
 * ColorCorrectGamma color correct over the texture
 * ColorCorrectContrast color correct over the texture
 * Multiply simple multiplication of textures
 * Composite blending used common image editor modes
 */
enum class EDatasmithCompMode
{
	Regular,
	Mix,
	Fresnel,
	Ior,
	ColorCorrectGamma,
	ColorCorrectContrast,
	Multiply,
	Composite
};

/** classic blend modes used in image editors */
enum class EDatasmithCompositeCompMode
{
	Alpha,
	Average,
	Add,
	Sub,
	Mult,
	Burn,
	Dodge,
	Darken,
	Difference,
	Exclusion,
	HardLight,
	Lighten,
	Screen,
	LinearBurn,
	LinearDodge,
	LinearLight,
	Overlay,
	PinLight,
	SoftLight,
	Hue,
	Saturation,
	Color,
	Value
};

/** material blend modes */
enum class EDatasmithBlendMode
{
	Alpha,
	ClearCoat,
	Screen,
	Softlight
};

/** material shader data Types */
// see ECustomMaterialOutputType
enum class EDatasmithShaderDataType
{
	Float1 = 1,
	Float2 = 2,
	Float3 = 3,
	Float4 = 4,
	MaterialAttribute = 5,
};

/** Key-value property */
UENUM(BlueprintType)
enum class EDatasmithKeyValuePropertyType : uint8
{
	String,
	Color,
	Float,
	Bool,
	Texture,
	Vector,
	Integer
};

/**
 * Analog to UE material domain, besides UE has some other modes currently we only support Surface and LightFunction.
 * since GUI, postproduction materials and so are out of the scope of Datasmith
 */
enum class EDatasmithShaderUsage
{
	Surface,
	LightFunction
};

static const TCHAR* DatasmithShadingModelStrings[] = { TEXT("DefaultLit"), TEXT("ThinTranslucent"), TEXT("Subsurface"), TEXT("ClearCoat"), TEXT("Unlit") };

enum class EDatasmithShadingModel : uint8
{
	DefaultLit,
	ThinTranslucent,
	Subsurface,
	ClearCoat,
	Unlit
};

UENUM()
enum class EDatasmithActorRemovalRule : uint8
{
	/** Remove also the actors children */
	RemoveChildren,

	/** Keeps current relative transform as the relative transform to the new parent. */
	KeepChildrenAndKeepRelativeTransform,

	/** Automatically calculates the relative transform such that the attached component maintains the same world transform. */
	//KeepChildrenAndKeepWorldTransform,
};

UENUM()
enum class EDatasmithActorAttachmentRule : uint8
{
	/** Keeps current relative transform as the relative transform to the new parent. */
	KeepRelativeTransform,

	/** The attached actor or component will maintain the same world transform. */
	KeepWorldTransform,
};

/** Supported transform types for animations */
enum class EDatasmithTransformType : uint8
{
	Translation,
	Rotation,
	Scale,
	Count
};

/**
 * Describes how an animated node should behave after its animation has completed
 * Mirrors EMovieSceneCompletionMode
 */
UENUM()
enum class EDatasmithCompletionMode : uint8
{
	KeepState,
	RestoreState,
	ProjectDefault,
};

/**
 * Describes a category of an UPropertyValue asset, indicating types of
 * properties that require special handling for any reason.
 * Mirrors EPropertyValueCategory
 */
UENUM()
enum class EDatasmithPropertyCategory : uint8
{
	Undefined = 0,
	Generic = 1,
	RelativeLocation = 2,
	RelativeRotation = 4,
	RelativeScale3D = 8,
	Visibility = 16,
	Material = 32,
	Color = 64,
	Option = 128
};
ENUM_CLASS_FLAGS(EDatasmithPropertyCategory)

static const TCHAR* KeyValuePropertyTypeStrings[] = { TEXT("String"), TEXT("Color"), TEXT("Float"), TEXT("Bool"), TEXT("Texture"), TEXT("Vector"), TEXT("Integer") };

// HOST NAME
#define DATASMITH_HOSTNAME						TEXT("Host")

// DATASMITH EXPORTER VERSION
#define DATASMITH_EXPORTERVERSION				TEXT("Version")
#define DATASMITH_EXPORTERSDKVERSION			TEXT("SDKVersion")

// APPLICATION INFO
#define DATASMITH_APPLICATION					TEXT("Application")
#define DATASMITH_VENDOR						TEXT("Vendor")
#define DATASMITH_PRODUCTNAME					TEXT("ProductName")
#define DATASMITH_PRODUCTVERSION				TEXT("ProductVersion")

// USER INFO
#define DATASMITH_USER							TEXT("User")
#define DATASMITH_USERID						TEXT("ID")
#define DATASMITH_USEROS						TEXT("OS")

// EXPORT INFO
#define DATASMITH_EXPORT						TEXT("Export")
#define DATASMITH_EXPORTDURATION				TEXT("Duration")
#define DATASMITH_RESOURCEPATH					TEXT("ResourcePath")

// SCENE GEOLOCATION
#define DATASMITH_GEOLOCATION					TEXT("Geolocation")
#define DATASMITH_GEOLOCATION_LATITUDE			TEXT("lat")
#define DATASMITH_GEOLOCATION_LONGITUDE			TEXT("lon")
#define DATASMITH_GEOLOCATION_ELEVATION			TEXT("ele")

//ELEMENTS
#define DATASMITH_HASH							TEXT("Hash")
#define DATASMITH_ENABLED						TEXT("Enabled")

//STATIC MESHES
#define DATASMITH_STATICMESHNAME				TEXT("StaticMesh")
#define DATASMITH_LIGHTMAPCOORDINATEINDEX		TEXT("LightmapCoordinateIndex")
#define DATASMITH_LIGHTMAPUVSOURCE				TEXT("LightmapUV")
#define DATASMITH_MATERIAL						TEXT("Material")

#define DATASMITH_ACTORNAME						TEXT("Actor")

#define DATASMITH_CLOTH							TEXT("Cloth")  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
#define DATASMITH_CLOTHACTORNAME				TEXT("ClothActor")  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")

//ACTOR MESHES
#define DATASMITH_ACTORMESHNAME					TEXT("ActorMesh")

//ACTOR HIERARCHICAL INSTANCED STATIC MESH
#define DATASMITH_ACTORHIERARCHICALINSTANCEDMESHNAME		TEXT("ActorHierarchicalInstancedStaticMesh")

//LEVEL SEQUENCES
#define DATASMITH_LEVELSEQUENCENAME				TEXT("LevelSequence")

// VARIANTS
#define DATASMITH_LEVELVARIANTSETSNAME			TEXT("LevelVariantSets")
#define DATASMITH_VARIANTSETNAME				TEXT("VariantSet")
#define DATASMITH_VARIANTNAME					TEXT("Variant")
#define DATASMITH_ACTORBINDINGNAME				TEXT("ActorBinding")
#define DATASMITH_PROPERTYCAPTURENAME			TEXT("PropertyCapture")
#define DATASMITH_OBJECTPROPERTYCAPTURENAME		TEXT("ObjectPropertyCapture")

//LIGHTS
#define DATASMITH_LIGHTNAME						TEXT("Light")
#define DATASMITH_POINTLIGHTNAME				TEXT("PointLight")
#define DATASMITH_SPOTLIGHTNAME					TEXT("SpotLight")
#define DATASMITH_AREALIGHTNAME					TEXT("AreaLight")
#define DATASMITH_PORTALLIGHTNAME				TEXT("SkyPortalLight")
#define DATASMITH_DIRECTLIGHTNAME				TEXT("DirectionalLight")
#define DATASMITH_PHYSICALSKYNAME				TEXT("PhysicalSky")

#define DATASMITH_LIGHTCOLORNAME				TEXT("Color")
#define DATASMITH_LIGHTUSETEMPNAME				TEXT("usetemp")
#define DATASMITH_LIGHTTEMPNAME					TEXT("temperature")
#define DATASMITH_LIGHTIESNAME					TEXT("IES")
#define DATASMITH_LIGHTIESTEXTURENAME			TEXT("IESTexture")
#define DATASMITH_LIGHTIESBRIGHTNAME			TEXT("IESbrightness")
#define DATASMITH_LIGHTIESROTATION				TEXT("IESrotation")
#define DATASMITH_LIGHTINTENSITYNAME			TEXT("Intensity")
#define DATASMITH_LIGHTINTENSITYUNITSNAME		TEXT("IntensityUnits")
#define DATASMITH_LIGHTSOURCESIZENAME			TEXT("SourceSize")
#define DATASMITH_LIGHTSOURCELENGTHNAME			TEXT("SourceLength")
#define DATASMITH_LIGHTATTENUATIONRADIUSNAME	TEXT("AttenuationRadius")
#define DATASMITH_LIGHTINNERRADIUSNAME			TEXT("InnerConeAngle")
#define DATASMITH_LIGHTOUTERRADIUSNAME			TEXT("OuterConeAngle")
#define DATASMITH_LIGHTMATERIAL					TEXT("Material")

#define DATASMITH_AREALIGHTSHAPE				TEXT("Shape")
#define DATASMITH_AREALIGHTDISTRIBUTION			TEXT("Distribution") // Deprecated
#define DATASMITH_AREALIGHTTYPE					TEXT("LightType")

//POSTPRODUCTION
#define DATASMITH_POSTPRODUCTIONNAME			TEXT("Post")
#define DATASMITH_POSTPRODUCTIONTEMP			TEXT("Temperature")
#define DATASMITH_POSTPRODUCTIONCOLOR			TEXT("Color")
#define DATASMITH_POSTPRODUCTIONDISTANCE		TEXT("Distance")
#define DATASMITH_POSTPRODUCTIONVIGNETTE		TEXT("Vignette")
#define DATASMITH_POSTPRODUCTIONSATURATION		TEXT("Saturation")
#define DATASMITH_POSTPRODUCTIONCAMERAISO		TEXT("CameraISO")
#define DATASMITH_POSTPRODUCTIONSHUTTERSPEED	TEXT("ShutterSpeed")

//CAMERAS
#define DATASMITH_CAMERANAME					TEXT("Camera")
#define DATASMITH_SENSORWIDTH					TEXT("SensorWidth")
#define DATASMITH_SENSORASPECT					TEXT("SensorAspectRatio")
#define DATASMITH_DEPTHOFFIELD					TEXT("DepthOfField")
#define DATASMITH_FOCUSDISTANCE					TEXT("FocusDistance")
#define DATASMITH_FSTOP							TEXT("FStop")
#define DATASMITH_FOCALLENGTH					TEXT("FocalLength")
#define DATASMITH_LOOKAT						TEXT("LookAt")
#define DATASMITH_LOOKATROLL					TEXT("LookAtRollAllowed")

//CUSTOM ACTOR
#define DATASMITH_CUSTOMACTORNAME				TEXT("CustomActor")
#define DATASMITH_CUSTOMACTORPATHNAME			TEXT("PathName")

//DECAL ACTOR
#define DATASMITH_DECALACTORNAME				TEXT("DecalActor")

// LANDSCAPE
#define DATASMITH_LANDSCAPENAME					TEXT("Landscape")
#define DATASMITH_HEIGHTMAPNAME					TEXT("Heightmap")
#define DATASMITH_PATHNAME						TEXT("PathName")

// POST PROCESS VOLUME
#define DATASMITH_POSTPROCESSVOLUME				TEXT("PostProcessVolume")
#define DATASMITH_POSTPROCESSVOLUME_UNBOUND		TEXT("Unbound")

// METADATA
#define DATASMITH_METADATANAME					TEXT("MetaData")
#define DATASMITH_REFERENCENAME					TEXT("reference")

//KEY-VALUE
#define DATASMITH_KEYVALUEPROPERTYNAME			TEXT("KeyValueProperty")

//TEXTURES
#define DATASMITH_TEXTUREMODE					TEXT("TextureMode")
#define DATASMITH_TEXTURERESIZE					TEXT("AllowResize")

//MATERIALS
#define DATASMITH_SHADERNAME					TEXT("Shader")
#define DATASMITH_MATERIALNAME					TEXT("Material")
#define DATASMITH_PARENTMATERIALLABEL			TEXT("ParentLabel")
#define DATASMITH_UEPBRMATERIALNAME				TEXT("UEPbrMaterial")

#define DATASMITH_MATERIALINSTANCENAME			TEXT("MaterialInstance")
#define DATASMITH_MATERIALINSTANCETYPE			TEXT("Type")
#define DATASMITH_MATERIALINSTANCEQUALITY		TEXT("Quality")
#define DATASMITH_MATERIALINSTANCEPATHNAME		TEXT("PathName")

#define DATASMITH_TEXTURENAME					TEXT("Texture")
#define DATASMITH_TEXTURECOMPNAME				TEXT("Texturecomp")
#define DATASMITH_COLORNAME						TEXT("Color")
#define DATASMITH_MASKNAME						TEXT("Mask")
#define DATASMITH_MASKCOLOR						TEXT("MaskColor")
#define DATASMITH_MASKCOMPNAME					TEXT("Maskcomp")
#define DATASMITH_VALUE1NAME					TEXT("Value1")
#define DATASMITH_VALUE2NAME					TEXT("Value2")
#define DATASMITH_ENVIRONMENTNAME				TEXT("Environment")

#define DATASMITH_DIFFUSETEXNAME				TEXT("Diffuse")
#define DATASMITH_DIFFUSECOLNAME				TEXT("Diffusecolor")
#define DATASMITH_DIFFUSECOMPNAME				TEXT("Diffusecomp")

#define DATASMITH_REFLETEXNAME					TEXT("Reflectance")
#define DATASMITH_REFLECOLNAME					TEXT("Reflectancecolor")
#define DATASMITH_REFLECOMPNAME					TEXT("Reflectancecomp")

#define DATASMITH_ROUGHNESSTEXNAME				TEXT("Roughness")
#define DATASMITH_ROUGHNESSVALUENAME			TEXT("Roughnessval")
#define DATASMITH_ROUGHNESSCOMPNAME				TEXT("Roughnesscomp")

#define DATASMITH_CLIPTEXNAME					TEXT("Clip")
#define DATASMITH_CLIPCOMPNAME					TEXT("Clipcomp")

#define DATASMITH_TRANSPTEXNAME					TEXT("RefractionTransparency")
#define DATASMITH_TRANSPCOLNAME					TEXT("RefractionTransparencycolor")
#define DATASMITH_TRANSPCOMPNAME				TEXT("RefractionTransparencycomp")

#define DATASMITH_NORMALTEXNAME					TEXT("Normal")
#define DATASMITH_NORMALCOMPNAME				TEXT("Normalcomp")
#define DATASMITH_BUMPTEXNAME					TEXT("Bump")
#define DATASMITH_BUMPCOMPNAME					TEXT("Bumpcomp")

#define DATASMITH_USEMATERIALATTRIBUTESNAME		TEXT("UseMaterialAttributes")
#define DATASMITH_FUNCTIONLYVALUENAME			TEXT("FunctionOnly")
#define DATASMITH_TWOSIDEDVALUENAME				TEXT("TwoSided")
#define DATASMITH_BUMPVALUENAME					TEXT("Bumpval")
#define DATASMITH_IORVALUENAME					TEXT("IOR")
#define DATASMITH_IORKVALUENAME					TEXT("IORk")
#define DATASMITH_REFRAIORVALUENAME				TEXT("IORRefraction")

#define DATASMITH_METALTEXNAME					TEXT("Metal")
#define DATASMITH_METALVALUENAME				TEXT("Metalval")
#define DATASMITH_METALCOMPNAME					TEXT("Metalcomp")

#define DATASMITH_EMITTEXNAME					TEXT("Emittance")
#define DATASMITH_EMITCOLNAME					TEXT("Emittancecolor")
#define DATASMITH_EMITCOMPNAME					TEXT("Emittancecomp")
#define DATASMITH_EMITTEMPNAME					TEXT("Emittancetemp")
#define DATASMITH_EMITVALUENAME					TEXT("Emittanceval")
#define DATASMITH_EMITONLYVALUENAME				TEXT("EmitOnly")
#define DATASMITH_DYNAMICEMISSIVE				TEXT("DynamicEmissive")
#define DATASMITH_SHADERUSAGE					TEXT("ShaderUsage")

#define DATASMITH_WEIGHTTEXNAME					TEXT("Weight")
#define DATASMITH_WEIGHTCOLNAME					TEXT("Weightcolor")
#define DATASMITH_WEIGHTCOMPNAME				TEXT("Weightcomp")
#define DATASMITH_WEIGHTVALUENAME				TEXT("Weightval")

#define DATASMITH_STACKLAYER					TEXT("Stacked")
#define DATASMITH_BLENDMODE						TEXT("Blendmode")
#define DATASMITH_OPACITYMASKCLIPVALUE			TEXT("OpacityMaskClipValue")
#define DATASMITH_SHADINGMODEL					TEXT("ShadingModel")
#define DATASMITH_TRANSLUCENCYLIGHTINGMODE		TEXT("TranslucencyLightingMode")

#define DATASMITH_ENVILLUMINATIONMAP			TEXT("Illuminate")

#define DATASMITH_DECALMATERIALNAME				TEXT("DecalMaterial")


enum class EDatasmithActorMobilityType : uint8
{
	Static = 0,
	Stationary,
	Movable
};

static const TCHAR* DatasmithActorMobilityTypeStrings[] = { TEXT("Static"), TEXT("Stationary"), TEXT("Movable") };
