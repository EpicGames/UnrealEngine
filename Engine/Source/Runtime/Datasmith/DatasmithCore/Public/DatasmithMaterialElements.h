// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IDatasmithSceneElements.h"

class IDatasmithMaterialExpression;

class IDatasmithExpressionInput : public IDatasmithElement
{
public:
	UE_DEPRECATED(4.27, "IDatasmithExpressionInput now derive from IDatasmithElement, please use GetName() instead")
	const TCHAR* GetInputName() const { return GetName(); }

	virtual IDatasmithMaterialExpression* GetExpression() = 0;
	virtual const IDatasmithMaterialExpression* GetExpression() const = 0;
	virtual void SetExpression( IDatasmithMaterialExpression* InExpression ) = 0;

	virtual int32 GetOutputIndex() const = 0;
	virtual void SetOutputIndex( int32 InOutputIndex ) = 0;
};

class IDatasmithExpressionOutput : public IDatasmithElement
{
public:
	virtual ~IDatasmithExpressionOutput() = default;

	UE_DEPRECATED(4.27, "IDatasmithExpressionOutput now derive from IDatasmithElement, please use GetName() instead")
	const TCHAR* GetOutputName() const { return GetName(); }
	UE_DEPRECATED(4.27, "IDatasmithExpressionOutput now derive from IDatasmithElement, please use SetName() instead")
	void SetOutputName( const TCHAR* InOutputName ) { SetName( InOutputName ); }
};

/**
 * Base class for representing an expression in the material graph of a IDatasmithUEPbrMaterial.
 * Setting a name to the expression is optional but allows to automatically create material instances during the import, which is faster and more optimal.
 */
class IDatasmithMaterialExpression : public IDatasmithElement
{
public:
	virtual ~IDatasmithMaterialExpression() = default;

	//Needed while we have the deprecated IsA() implementation to avoid declaration conflict.
	using IDatasmithElement::IsA;

	UE_DEPRECATED(4.27, "Deprecated, please use GetExpressionType() instead")
	EDatasmithMaterialExpressionType GetType() const { return GetExpressionType(); }
	virtual EDatasmithMaterialExpressionType GetExpressionType() const = 0;
	UE_DEPRECATED(4.27, "IDatasmithMaterialExpression now derive from IDatasmithElement, please use IsSubType() or GetExpressionType() instead")
	bool IsA( const EDatasmithMaterialExpressionType ExpressionType ) const { return IsSubType(ExpressionType); }

	virtual bool IsSubType( const EDatasmithMaterialExpressionType ExpressionType ) const = 0;

	/** Connects the default output to an expression input */
	virtual void ConnectExpression( IDatasmithExpressionInput& ExpressionInput ) = 0;

	/** Connects a specific output to an expression input */
	virtual void ConnectExpression( IDatasmithExpressionInput& ExpressionInput, int32 OutputIndex ) = 0;

	virtual int32 GetInputCount() const = 0;
	virtual IDatasmithExpressionInput* GetInput( int32 Index ) = 0;
	virtual const IDatasmithExpressionInput* GetInput( int32 Index ) const = 0;

	/** The output index to use by default for this expression when connecting it to other inputs. */
	virtual int32 GetDefaultOutputIndex() const = 0;
	virtual void SetDefaultOutputIndex( int32 OutputIndex ) = 0;

	/** Reset the expression to its default values, and disconnect input expressions */
	virtual void ResetExpression() = 0;
};

class IDatasmithExpressionParameter : public IDatasmithMaterialExpression
{
public:
	virtual ~IDatasmithExpressionParameter() = default;

	virtual const TCHAR* GetGroupName() const = 0;
	virtual void SetGroupName( const TCHAR* InGroupName ) = 0;
};

/**
 * Represents a UMaterialExpressionStaticBoolParameter
 */
class IDatasmithMaterialExpressionBool : public IDatasmithExpressionParameter
{
public:
	virtual bool& GetBool() = 0;
	virtual const bool& GetBool() const = 0;
};

class IDatasmithMaterialExpressionColor : public IDatasmithExpressionParameter
{
public:
	virtual FLinearColor& GetColor() = 0;
	virtual const FLinearColor& GetColor() const = 0;
};

class IDatasmithMaterialExpressionScalar : public IDatasmithExpressionParameter
{
public:
	virtual float& GetScalar() = 0;
	virtual const float& GetScalar() const = 0;
};

class IDatasmithMaterialExpressionTexture : public IDatasmithExpressionParameter
{
public:
	virtual const TCHAR* GetTexturePathName() const = 0;
	virtual void SetTexturePathName( const TCHAR* InTexturePathName ) = 0;

	/**
	 * Inputs
	 */
	virtual IDatasmithExpressionInput& GetInputCoordinate() = 0;
	virtual const IDatasmithExpressionInput& GetInputCoordinate() const = 0;

	/**
	 * Outputs:
	 * - RGB
	 * - R
	 * - G
	 * - B
	 * - A
	 */
};

class IDatasmithMaterialExpressionTextureCoordinate : public IDatasmithMaterialExpression
{
public:
	virtual int32 GetCoordinateIndex() const = 0;
	virtual void SetCoordinateIndex( int32 InCoordinateIndex ) = 0;

	virtual float GetUTiling() const = 0;
	virtual void SetUTiling( float InUTiling ) = 0;

	virtual float GetVTiling() const = 0;
	virtual void SetVTiling( float InVTiling ) = 0;
};

class IDatasmithMaterialExpressionFlattenNormal : public IDatasmithMaterialExpression
{
public:
	/**
	 * Inputs
	 */
	virtual IDatasmithExpressionInput& GetNormal() = 0;
	virtual const IDatasmithExpressionInput& GetNormal() const = 0;

	virtual IDatasmithExpressionInput& GetFlatness() = 0;
	virtual const IDatasmithExpressionInput& GetFlatness() const = 0;

	/**
	 * Outputs:
	 * - RGB
	 */
};

// see UMaterialExpressionCustom
class IDatasmithMaterialExpressionCustom : public IDatasmithMaterialExpression
{
public:
	virtual void SetCode(const TCHAR* InCode) = 0;
	virtual const TCHAR* GetCode() const = 0;

	virtual void SetOutputType(EDatasmithShaderDataType InOutputType) = 0;
	virtual EDatasmithShaderDataType GetOutputType() const = 0;

	virtual void SetDescription(const TCHAR* InDescription) = 0;
	virtual const TCHAR* GetDescription() const = 0;

	virtual int32 GetIncludeFilePathCount() const = 0;
	virtual void AddIncludeFilePath(const TCHAR* Path) = 0;
	virtual const TCHAR* GetIncludeFilePath(int32 Index) const = 0;

	virtual int32 GetAdditionalDefineCount() const = 0;
	virtual void AddAdditionalDefine(const TCHAR* Define) = 0;
	virtual const TCHAR* GetAdditionalDefine(int32 Index) const = 0;

	virtual int32 GetArgumentNameCount() const = 0;
	virtual void SetArgumentName(int32 ArgIndex, const TCHAR* ArgName) = 0;
	virtual const TCHAR* GetArgumentName(int32 ArgIndex) const = 0;
};

class IDatasmithMaterialExpressionGeneric : public IDatasmithMaterialExpression
{
public:
	virtual void SetExpressionName( const TCHAR* InExpressionName ) = 0;
	virtual const TCHAR* GetExpressionName() const = 0;

	/** Get the total amount of properties in this expression */
	virtual int32 GetPropertiesCount() const = 0;

	/** Get the property i-th of this expression */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 i) const = 0;
	virtual TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 i) = 0;

	/** Get a property by its name if it exists */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName(const TCHAR* Name) const = 0;
	virtual TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName(const TCHAR* Name) = 0;

	/** Add a property to this expression*/
	virtual void AddProperty( const TSharedPtr< IDatasmithKeyValueProperty >& Property ) = 0;
};

class IDatasmithMaterialExpressionFunctionCall : public IDatasmithMaterialExpression
{
public:
	virtual void SetFunctionPathName( const TCHAR* InFunctionPathName ) = 0;
	virtual const TCHAR* GetFunctionPathName() const = 0;

};

class IDatasmithUEPbrMaterialElement : public IDatasmithBaseMaterialElement
{
public:
	virtual IDatasmithExpressionInput& GetBaseColor() = 0;
	virtual IDatasmithExpressionInput& GetMetallic() = 0;
	virtual IDatasmithExpressionInput& GetSpecular() = 0;
	virtual IDatasmithExpressionInput& GetRoughness() = 0;
	virtual IDatasmithExpressionInput& GetEmissiveColor() = 0;
	virtual IDatasmithExpressionInput& GetOpacity() = 0;
	virtual IDatasmithExpressionInput& GetNormal() = 0;
	virtual IDatasmithExpressionInput& GetRefraction() = 0;
	virtual IDatasmithExpressionInput& GetAmbientOcclusion() = 0;
	virtual IDatasmithExpressionInput& GetClearCoat() = 0;
	virtual IDatasmithExpressionInput& GetClearCoatRoughness() = 0;
	virtual IDatasmithExpressionInput& GetWorldPositionOffset() = 0;
	virtual IDatasmithExpressionInput& GetMaterialAttributes() = 0;

	/** InBlendMode must match the values of EBlendMode from EngineTypes.h */
	virtual int GetBlendMode() const = 0;
	virtual void SetBlendMode( int InBlendMode ) = 0;

	virtual bool GetTwoSided() const = 0;
	virtual void SetTwoSided( bool bTwoSided ) = 0;

	virtual bool GetIsThinSurface() const = 0;
	virtual void SetIsThinSurface(bool bIsThinSurface) = 0;

	virtual bool GetUseMaterialAttributes() const = 0;
	virtual void SetUseMaterialAttributes( bool bInUseMaterialAttributes ) = 0;

	/** If a material is only referenced by other materials then it is only used as a material function and there is no need to instantiate it. */
	virtual bool GetMaterialFunctionOnly() const = 0;
	virtual void SetMaterialFunctionOnly(bool bInMaterialFunctionOnly) = 0;

	virtual float GetOpacityMaskClipValue() const = 0;
	virtual void SetOpacityMaskClipValue(float InClipValue) = 0;

	virtual int GetTranslucencyLightingMode() const = 0;
	/** InMode must match the values of ETranslucencyLightingMode from EngineTypes.h */
	virtual void SetTranslucencyLightingMode(int InMode) = 0;

	virtual int32 GetExpressionsCount() const = 0;
	virtual IDatasmithMaterialExpression* GetExpression( int32 Index ) = 0;
	virtual int32 GetExpressionIndex( const IDatasmithMaterialExpression* Expression ) const = 0;

	virtual IDatasmithMaterialExpression* AddMaterialExpression( const EDatasmithMaterialExpressionType ExpressionType ) = 0;

	template< typename T >
	T* AddMaterialExpression()
	{
		return nullptr;
	}

	/** Reset all expression to their default values and remove all connections */
	virtual void ResetExpressionGraph( bool bRemoveAllExpressions ) = 0;

	/** If a parent material is generated from this material, this will be its label. If none, the instance and the parent will have the same label. */
	virtual void SetParentLabel( const TCHAR* InParentLabel ) = 0;
	virtual const TCHAR* GetParentLabel() const = 0;

	virtual void SetShadingModel( const EDatasmithShadingModel InShadingModel ) = 0;
	virtual EDatasmithShadingModel GetShadingModel() const = 0;
};

template<>
inline IDatasmithMaterialExpressionBool* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionBool >()
{
	return static_cast< IDatasmithMaterialExpressionBool* >( AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantBool ) );
}

template<>
inline IDatasmithMaterialExpressionColor* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionColor >()
{
	return static_cast< IDatasmithMaterialExpressionColor* >( AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantColor ) );
}

template<>
inline IDatasmithMaterialExpressionFlattenNormal* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionFlattenNormal >()
{
	return static_cast< IDatasmithMaterialExpressionFlattenNormal* >( AddMaterialExpression( EDatasmithMaterialExpressionType::FlattenNormal ) );
}

template<>
inline IDatasmithMaterialExpressionFunctionCall* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >()
{
	return static_cast< IDatasmithMaterialExpressionFunctionCall* >( AddMaterialExpression( EDatasmithMaterialExpressionType::FunctionCall ) );
}

template<>
inline IDatasmithMaterialExpressionGeneric* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionGeneric >()
{
	return static_cast< IDatasmithMaterialExpressionGeneric* >( AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
}

template<>
inline IDatasmithMaterialExpressionScalar* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionScalar >()
{
	return static_cast< IDatasmithMaterialExpressionScalar* >( AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar ) );
}

template<>
inline IDatasmithMaterialExpressionTexture* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionTexture >()
{
	return static_cast< IDatasmithMaterialExpressionTexture* >( AddMaterialExpression( EDatasmithMaterialExpressionType::Texture ) );
}

template<>
inline IDatasmithMaterialExpressionTextureCoordinate* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionTextureCoordinate >()
{
	return static_cast< IDatasmithMaterialExpressionTextureCoordinate* >( AddMaterialExpression( EDatasmithMaterialExpressionType::TextureCoordinate ) );
}

template<>
inline IDatasmithMaterialExpressionCustom* IDatasmithUEPbrMaterialElement::AddMaterialExpression< IDatasmithMaterialExpressionCustom >()
{
	return static_cast< IDatasmithMaterialExpressionCustom* >( AddMaterialExpression( EDatasmithMaterialExpressionType::Custom ) );
}
