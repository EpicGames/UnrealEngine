// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithDefinitions.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class IDatasmithMaterialExpression;
class IDatasmithMaterialExpressionTexture;
class IDatasmithUEPbrMaterialElement;

namespace DatasmithMaterialsUtils
{
	struct FUVEditParameters
	{
		FUVEditParameters()
			: UVOffset( FVector2D::ZeroVector )
			, UVTiling( FVector2D::UnitVector )
			, RotationPivot( 0.f, 1.f, 0.f )
			, RotationAngle( 0.f )
			, ChannelIndex( 0 )
			, bIsUsingRealWorldScale( false )
			, bMirrorU( false )
			, bMirrorV( false )
		{
		}

		// UV space has its origin at top-left, with U going left-to-right, V top-to-bottom
		FVector2D UVOffset;
		FVector2D UVTiling;

		// W rotation center
		FVector RotationPivot;

		// W rotation in degrees (rotation is counterclockwise)
		float RotationAngle;

		// UV channel to use
		uint8 ChannelIndex;

		// Enable "Real-World Scale" behavior as in 3ds max
		bool bIsUsingRealWorldScale;

		// Enable mirroring of texture in U and V
		bool bMirrorU;
		bool bMirrorV;
	};

	/**
	 * Generate material expressions on a given DatasmithUEPbrMaterialElement to output a texture with UV settings applied to it
	 * @param MaterialElement		The UEPbrMaterialElement on which to create the material expressions
	 * @param ParameterName			The display name for the Texture expression
	 * @param TextureMapPath		The texture to use for the Texture expression
	 * @param UVParameters			The UVEditParameters to apply
	 * @return						Texture expression with UVEdit applied to it; nullptr if no TextureMapPath is specified
	 */
	DATASMITHCORE_API IDatasmithMaterialExpressionTexture* CreateTextureExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const TCHAR* ParameterName, const TCHAR* TextureMapPath, const FUVEditParameters& UVParameters );

	/**
	 * Generate material expressions on a given DatasmithUEPbrMaterialElement that output a color or scalar interpolated with a MaterialExpression
	 * @param MaterialElement		The UEPbrMaterialElement on which to create the material expressions
	 * @param ParameterName			The display name for the Color or Scalar input
	 * @param Color					The color to interpolate the Expression with. If no color is given, use the scalar
	 * @param Scalar				The scalar to interpolate the Expression with. If no color or scalar are given, use the Expression only
	 * @param Expression			The MaterialExpression to interpolate the Color or Scalar with
	 * @param Weight				The weight of the Expression in the interpolation
	 * @param TextureMode			The TextureMode of the Expression to determine if it requires normal flattening (bump and normal mode) or interpolation
	 * @return						Material expression that outputs a Color or Scalar interpolated with Expression; nullptr if no Color, Scalar or Expression are given
	 */
	DATASMITHCORE_API IDatasmithMaterialExpression* CreateWeightedMaterialExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, const TCHAR* ParameterName, TOptional< FLinearColor > Color, TOptional< float > Scalar, IDatasmithMaterialExpression* Expression, float Weight, EDatasmithTextureMode TextureMode = EDatasmithTextureMode::Diffuse );

	DATASMITHCORE_API FLinearColor TemperatureToColor(float Kelvin);
}
