// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"

#include "Misc/SecureHash.h"

#define UE_API DATASMITHCORE_API


class FDatasmithMesh
{
public:
	UE_API FDatasmithMesh();
	UE_API ~FDatasmithMesh();

	UE_API FDatasmithMesh( const FDatasmithMesh& Other );
	UE_API FDatasmithMesh( FDatasmithMesh&& Other );

	UE_API FDatasmithMesh& operator=( const FDatasmithMesh& Other );
	UE_API FDatasmithMesh& operator=( FDatasmithMesh&& Other );

	// Calculate mesh data hash(doesn't include name)
	UE_API FMD5Hash CalculateHash() const;

	UE_API void SetName(const TCHAR* InName);
	UE_API const TCHAR* GetName() const;

	//--------------------------
	// Faces
	//--------------------------
	/** Setting the amount of faces is mandatory before filling the array */
	UE_API void SetFacesCount(int32 NumFaces);

	/** Retrieves the amount of faces */
	UE_API int32 GetFacesCount() const;

	/**
	 * Sets the geometry of the face
	 *
	 * @param Index		value to choose the face that will be affected
	 * @param Vertex1	Index of the first geometric vertex that defines this face
	 * @param Vertex2	Index of the second geometric vertex that defines this face
	 * @param Vertex3	Index of the third geometric vertex that defines this face
	 */
	UE_API void SetFace(int32 Index, int32 Vertex1, int32 Vertex2, int32 Vertex3, int32 MaterialId = 0);
	UE_API void GetFace(int32 Index, int32& Vertex1, int32& Vertex2, int32& Vertex3, int32& MaterialId) const;

	/**
	 * Sets the smoothing mask of the face
	 *
	 * @param Index			Index of the face that will be affected
	 * @param SmoothingMask	32 bits mask, 0 means no smoothing
	 */
	UE_API void SetFaceSmoothingMask(int32 Index, uint32 SmoothingMask);

	/**
	 * Gets the smoothing mask of a face
	 *
	 * @param Index		Index of the face to retrieve the smoothing mask from
	 */
	UE_API uint32 GetFaceSmoothingMask(int32 Index) const;

	UE_API int32 GetMaterialsCount() const;
	UE_API bool IsMaterialIdUsed(int32 MaterialId) const;

	//--------------------------
	// Vertices
	//--------------------------
	/** Setting the amount of geometric vertices is mandatory before filling the array */
	UE_API void SetVerticesCount(int32 NumVerts);

	/** Retrieves the amount of geometric vertices. The validity of the vertex data is not guaranteed */
	UE_API int32 GetVerticesCount() const;

	/**
	 * Sets the 3d position of the vertex
	 *
	 * @param Index value to choose the vertex that will be affected
	 * @x position on the x axis
	 * @y position on the y axis
	 * @z position on the z axis
	 */
	UE_API void SetVertex(int32 Index, float X, float Y, float Z);
	UE_API FVector3f GetVertex(int32 Index) const;

	//--------------------------
	// Normals
	//--------------------------
	/**
	 * Sets the 3d normal
	 *
	 * @param Index value to choose the normal that will be affected
	 * @x direction on the x axis
	 * @y direction on the y axis
	 * @z direction on the z axis
	 */
	UE_API void SetNormal(int32 Index, float X, float Y, float Z);
	UE_API FVector3f GetNormal(int32 Index) const;

	//--------------------------
	// UVs
	//--------------------------
	/**
	 * Sets the amount of UV channels on this mesh
	 *
	 * @param ChannelCount	The number of UV channels
	 */
	UE_API void SetUVChannelsCount(int32 ChannelCount);

	/**
	 * Add a UV channel at the end
	 */
	UE_API void AddUVChannel();

	/**
	 * Remove the last UV channel
	 */
	UE_API void RemoveUVChannel();

	/** Gets the amount of UV channels on this mesh */
	UE_API int32 GetUVChannelsCount() const;

	/**
	 * Setting the amount of UV coordinates on the channel is mandatory before filling the array
	 *
	 * @param Channel	The channel to set the numbers of UVs
	 * @param NumVerts	The number of UVs in the channel
	 */
	UE_API void SetUVCount(int32 Channel, int32 NumVerts);

	/** Retrieves the amount of UV coordinates on the channel. The validity of the vertex data is not guaranteed */
	UE_API int32 GetUVCount(int32 Channel) const;

	/**
	 * Sets the 2d position of the UV vertex for the first uv mapping
	 *
	 * @param Index	Value to choose the vertex that will be affected
	 * @param U		Horizontal coordinate
	 * @param V		Vertical coordinate
	 */
	UE_API void SetUV(int32 Channel, int32 Index, double U, double V);

	/**
	 * Get the hash for a uv channel
	 */
	UE_API uint32 GetHashForUVChannel(int32 Channel) const;

	/**
	 * Gets the UV coordinates for a channel
	 *
	 * @param Channel	The channel we want to retrieve the UVs from
	 * @param Index		The UV coordinates to retrieve
	 */
	UE_API FVector2D GetUV(int32 Channel, int32 Index) const;

	/**
	 * Sets the channel UV coordinates of the face
	 *
	 * @param Index		Index of the face that will be affected
	 * @param Channel	UV channel, starting at 0
	 * @param Vertex1	Index of the first uv vertex that defines this face
	 * @param Vertex2	Index of the second uv vertex that defines this face
	 * @param Vertex3	Index of the third uv vertex that defines this face
	 */
	UE_API void SetFaceUV(int32 Index, int32 Channel, int32 Vertex1, int32 Vertex2, int32 Vertex3);

	/**
	 * Gets the UV coordinates of the vertices of a face
	 *
	 * @param Index		Index of the face to retrieve the UVs from
	 * @param Channel	UV channel, starting at 0
	 * @param Vertex1	Index of the first uv vertex that defines this face
	 * @param Vertex2	Index of the second uv vertex that defines this face
	 * @param Vertex3	Index of the third uv vertex that defines this face
	 */
	UE_API void GetFaceUV(int32 Index, int32 Channel, int32& Vertex1, int32& Vertex2, int32& Vertex3) const;

	/** Get the number of vertex color */
	UE_API int32 GetVertexColorCount() const;

	/**
	 * Set a vertex color
	 *
	 * @param Index		Index of the vertex color that will be affected
	 * @param Color		The color for the vertex
	 */
	UE_API void SetVertexColor(int32 Index, const FColor& Color);

	/**
	 * Get the color for a vertex
	 *
	 * @param Index		Index of the vertex color to retrieve
	 */
	UE_API FColor GetVertexColor(int32 Index) const;

	/**
	 * Sets the UV channel that will be used as the source UVs for lightmap UVs generation at import, defaults to channel 0.
	 * Will be overwritten during Mesh export if we choose to generate the lightmap source UVs.
	 */
	UE_API void SetLightmapSourceUVChannel(int32 Channel);

	/** Gets the UV channel that will be used for lightmap UVs generation at import */
	UE_API int32 GetLightmapSourceUVChannel() const;

	//--------------------------
	// LODs
	//--------------------------

	/** Adds a LOD mesh to this base LOD mesh */
	UE_API void AddLOD( const FDatasmithMesh& InLODMesh );
	UE_API void AddLOD( FDatasmithMesh&& InLODMesh );
	UE_API int32 GetLODsCount() const;

	/** Gets the FDatasmithMesh LOD at the given index, if the index is invalid returns nullptr */
	UE_API FDatasmithMesh* GetLOD( int32 Index );
	UE_API const FDatasmithMesh* GetLOD( int32 Index ) const;

	//--------------------------
	// Misc
	//--------------------------
	/** Returns the total surface area */
	UE_API float ComputeArea() const;

	/** Returns the bounding box containing all vertices of this mesh */
	UE_API FBox3f GetExtents() const;

private:
	class FDatasmithMeshImpl;

	FDatasmithMeshImpl* Impl;
};

#undef UE_API
