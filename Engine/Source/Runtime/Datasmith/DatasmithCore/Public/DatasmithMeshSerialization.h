// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithCore.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "MeshDescription.h"
#include "Misc/SecureHash.h"
#include "DatasmithCloth.h"  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")

#define UE_API DATASMITHCORE_API

class FArchive;


struct FDatasmithMeshModels
{
	FString MeshName;
	bool bIsCollisionMesh = false;
	TArray<FMeshDescription> SourceModels;

	friend FArchive& operator<<(FArchive& Ar, FDatasmithMeshModels& Models);
};

struct FDatasmithPackedMeshes
{
	TArray<FDatasmithMeshModels> Meshes;

	UE_API FMD5Hash Serialize(FArchive& Ar, bool bSaveCompressed=true);
};

DATASMITHCORE_API FDatasmithPackedMeshes GetDatasmithMeshFromFile(const FString& MeshPath);



PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API FDatasmithClothInfo
{
	FDatasmithCloth Cloth;
	friend FArchive& operator<<(FArchive& Ar, FDatasmithClothInfo& Info);
};

struct UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API FDatasmithPackedCloths
{
	TArray<FDatasmithClothInfo> ClothInfos;

	FMD5Hash Serialize(FArchive& Ar, bool bSaveCompressed=true);
};

UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
DATASMITHCORE_API FDatasmithPackedCloths GetDatasmithClothFromFile(const FString& Path);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
