// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "MeshDescription.h"
#include "RawMesh.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DatasmithMeshUObject.generated.h"

#define UE_API DATASMITHCORE_API

class FArchive;

USTRUCT()
struct FDatasmithMeshSourceModel
{
	GENERATED_BODY()

public:
	void SerializeBulkData(FArchive& Ar, UObject* Owner);

	FRawMeshBulkData RawMeshBulkData;
};

UCLASS(MinimalAPI)
class UDatasmithMesh : public UObject
{
	GENERATED_BODY()

public:
	static const TCHAR* GetFileExtension() { return TEXT("udsmesh"); }

	UE_API virtual void Serialize(FArchive& Ar) override;

	UPROPERTY()
	FString MeshName;

	UPROPERTY()
	bool bIsCollisionMesh;

	UPROPERTY()
	TArray< FDatasmithMeshSourceModel > SourceModels;
};

#undef UE_API
