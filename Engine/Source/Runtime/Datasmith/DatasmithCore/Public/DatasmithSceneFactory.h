// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithAnimationElements.h"
#include "DatasmithDefinitions.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithVariantElements.h"
#include "HAL/Platform.h"
#include "IDatasmithSceneElements.h"
#include "Templates/SharedPointer.h"

#define UE_API DATASMITHCORE_API

/**
 * Factory to create the scene elements used for the export and import process.
 * The shared pointer returned is the only one existing at that time.
 * Make sure to hang onto it until the scene element isn't needed anymore.
 */
class FDatasmithSceneFactory
{
public:
	static UE_API TSharedPtr< IDatasmithElement > CreateElement( EDatasmithElementType InType, const TCHAR* InName );

	static UE_API TSharedPtr< IDatasmithElement > CreateElement( EDatasmithElementType InType, uint64 InSubType, const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithActorElement > CreateActor( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithCameraActorElement > CreateCameraActor( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithCompositeTexture > CreateCompositeTexture();

	static UE_API TSharedRef< IDatasmithCustomActorElement > CreateCustomActor( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithLandscapeElement > CreateLandscape( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithPostProcessVolumeElement > CreatePostProcessVolume( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithEnvironmentElement > CreateEnvironment( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithPointLightElement > CreatePointLight( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithSpotLightElement > CreateSpotLight( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithDirectionalLightElement > CreateDirectionalLight( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithAreaLightElement > CreateAreaLight( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithLightmassPortalElement > CreateLightmassPortal( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithKeyValueProperty > CreateKeyValueProperty( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithMeshElement > CreateMesh( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithMeshActorElement > CreateMeshActor(const TCHAR* InName);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static UE_API TSharedRef< IDatasmithClothElement > CreateCloth( const TCHAR* InName );

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static UE_API TSharedRef< IDatasmithClothActorElement > CreateClothActor(const TCHAR* InName);
UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement > CreateHierarchicalInstanceStaticMeshActor( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithMaterialElement > CreateMaterial( const TCHAR* InName );

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "FDatasmithSceneFactory::CreateMasterMaterial will not be supported in 5.2. Please use FDatasmithSceneFactory::CreateMaterialInstance instead.")
	static TSharedRef< IDatasmithMasterMaterialElement > CreateMasterMaterial(const TCHAR* InName)
	{
		return CreateMaterialInstance(InName);
	}
UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static TSharedRef< IDatasmithMaterialInstanceElement > CreateMaterialInstance(const TCHAR* InName);

	static UE_API TSharedRef< IDatasmithUEPbrMaterialElement > CreateUEPbrMaterial( const TCHAR* InName );

	/**
	 * Creates a MaterialExpression from the given type.
	 * Warning: This function's main purpose is to allow the creation of serialized material expressions.
	 *          Creating and adding new expressions to a UEPbrMaterial should be done with the IDatasmithUEPbrMaterialElement::AddMaterialExpression() function.
	 */
	static UE_API TSharedPtr< IDatasmithMaterialExpression > CreateMaterialExpression( EDatasmithMaterialExpressionType MaterialExpression );

	/**
	 * Creates an ExpressionInput from the given type.
	 * Warning: This function's main purpose is to allow the creation of serialized expression inputs. It should not be used to add inputs to a material expression.
	 */
	static UE_API TSharedRef< IDatasmithExpressionInput > CreateExpressionInput( const TCHAR* InName );

	/**
	 * Creates an ExpressionOutput from the given type.
	 * Warning: This function's main purpose is to allow the creation of serialized expression outputs. It should not be used to add outputs to a material expression.
	 */
	static UE_API TSharedRef< IDatasmithExpressionOutput > CreateExpressionOutput( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithMetaDataElement > CreateMetaData( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithMaterialIDElement > CreateMaterialId( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithPostProcessElement > CreatePostProcess();

	static UE_API TSharedRef< IDatasmithShaderElement > CreateShader( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithTextureElement > CreateTexture( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithLevelSequenceElement > CreateLevelSequence( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithTransformAnimationElement > CreateTransformAnimation( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithVisibilityAnimationElement > CreateVisibilityAnimation( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithSubsequenceAnimationElement > CreateSubsequenceAnimation( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithLevelVariantSetsElement > CreateLevelVariantSets( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithVariantSetElement > CreateVariantSet( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithVariantElement > CreateVariant( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithActorBindingElement > CreateActorBinding();
	static UE_API TSharedRef< IDatasmithPropertyCaptureElement > CreatePropertyCapture();
	static UE_API TSharedRef< IDatasmithObjectPropertyCaptureElement > CreateObjectPropertyCapture();

	static UE_API TSharedRef< IDatasmithDecalActorElement > CreateDecalActor( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithDecalMaterialElement > CreateDecalMaterial( const TCHAR* InName );

	static UE_API TSharedRef< IDatasmithScene > CreateScene( const TCHAR* InName );
	static UE_API TSharedRef< IDatasmithScene > DuplicateScene( const TSharedRef< IDatasmithScene >& InScene );
};

#undef UE_API
