// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithVariantElements.h"
#include "IDatasmithSceneElements.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API DATASMITHCORE_API

class FDatasmithTextureSampler;
class FXmlFile;
class FXmlNode;
class UTexture;
struct FLinearColor;

class FDatasmithSceneXmlReader
{
public:
	// Force non-inline constructor and destructor to prevent instantiating TUniquePtr< FXmlFile > with an incomplete FXmlFile type (forward declared)
	UE_API FDatasmithSceneXmlReader();
	UE_API ~FDatasmithSceneXmlReader();

	UE_API bool ParseFile(const FString& InFilename, TSharedRef< IDatasmithScene >& OutScene, bool bInAppend = false);
	UE_API bool ParseBuffer(const FString& XmlBuffer, TSharedRef< IDatasmithScene >& OutScene, bool bInAppend = false);

private:
	bool ParseXmlFile(TSharedRef< IDatasmithScene >& OutScene, bool bInAppend = false);
	void PatchUpVersion(TSharedRef< IDatasmithScene >& OutScene) const;

	[[nodiscard]] FString UnsanitizeXMLText(const FString& InString) const;

	template<typename T> T ValueFromString(const FString& InString) const = delete;
	FVector VectorFromNode(FXmlNode* InNode, const TCHAR* XName, const TCHAR* YName, const TCHAR* ZName) const;
	FQuat QuatFromHexString(const FString& HexString) const;
	FQuat QuatFromNode(FXmlNode* InNode) const;
	FTransform ParseTransform(FXmlNode* InNode) const;
	void ParseTransform(FXmlNode* InNode, TSharedPtr< IDatasmithActorElement >& OutElement) const;

	void ParseElement(FXmlNode* InNode, TSharedRef<IDatasmithElement> OutElement) const;
	void ParseLevelSequence(FXmlNode* InNode, const TSharedRef<IDatasmithLevelSequenceElement>& OutElement) const;
	void ParseLevelVariantSets( FXmlNode* InNode, const TSharedRef<IDatasmithLevelVariantSetsElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const;
	void ParseVariantSet( FXmlNode* InNode, const TSharedRef<IDatasmithVariantSetElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const;
	void ParseVariant( FXmlNode* InNode, const TSharedRef<IDatasmithVariantElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const;
	void ParseActorBinding( FXmlNode* InNode, const TSharedRef<IDatasmithActorBindingElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const;
	void ParsePropertyCapture( FXmlNode* InNode, const TSharedRef<IDatasmithPropertyCaptureElement>& OutElement ) const;
	void ParseObjectPropertyCapture( FXmlNode* InNode, const TSharedRef<IDatasmithObjectPropertyCaptureElement>& OutElement, const TMap< FString, TSharedPtr<IDatasmithElement> >& Objects ) const;
	void ParseMesh(FXmlNode* InNode, TSharedPtr<IDatasmithMeshElement>& OutElement) const;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	void ParseCloth(FXmlNode* InNode, TSharedPtr<IDatasmithClothElement>& OutElement) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	void ParseTextureElement(FXmlNode* InNode, TSharedPtr<IDatasmithTextureElement>& OutElement) const;
	void ParseTexture(FXmlNode* InNode, FString& OutTextureFilename, FDatasmithTextureSampler& OutTextureUV) const;
	void ParseActor(FXmlNode* InNode, TSharedPtr<IDatasmithActorElement>& InOutElement, TSharedRef< IDatasmithScene > Scene, TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors) const;
	void ParseMeshActor(FXmlNode* InNode, TSharedPtr<IDatasmithMeshActorElement>& OutElement, TSharedRef< IDatasmithScene > Scene) const;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	void ParseClothActor(FXmlNode* InNode, TSharedPtr<IDatasmithClothActorElement>& OutElement, TSharedRef< IDatasmithScene > Scene) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	void ParseHierarchicalInstancedStaticMeshActor(FXmlNode* InNode, TSharedPtr<IDatasmithHierarchicalInstancedStaticMeshActorElement>& OutElement, TSharedRef< IDatasmithScene > Scene) const;
	void ParseLight(FXmlNode* InNode, TSharedPtr<IDatasmithLightActorElement>& OutElement) const;
	void ParseCamera(FXmlNode* InNode, TSharedPtr<IDatasmithCameraActorElement>& OutElement) const;
	void ParsePostProcess(FXmlNode* InNode, const TSharedPtr< IDatasmithPostProcessElement >& Element) const;
	void ParsePostProcessVolume(FXmlNode* InNode, const TSharedRef< IDatasmithPostProcessVolumeElement >& Element) const;
	void ParseColor(FXmlNode* InNode, FLinearColor& OutColor) const;
	void ParseComp(FXmlNode* InNode, TSharedPtr< IDatasmithCompositeTexture >& OutCompTexture, bool bInIsNormal = false) const;
	void ParseMaterial(FXmlNode* InNode, TSharedPtr< IDatasmithMaterialElement >& OutElement) const;
	void ParseMaterialInstance(FXmlNode* InNode, TSharedPtr< IDatasmithMaterialInstanceElement >& OutElement) const;
	void ParseDecalMaterial(FXmlNode* InNode, TSharedPtr< IDatasmithDecalMaterialElement >& OutElement) const;
	void ParseUEPbrMaterial(FXmlNode* InNode, TSharedPtr< IDatasmithUEPbrMaterialElement >& OutElement) const;
	void ParseCustomActor(FXmlNode* InNode, TSharedPtr< IDatasmithCustomActorElement >& OutElement) const;
	void ParseMetaData(FXmlNode* InNode, TSharedPtr< IDatasmithMetaDataElement >& OutElement, const TSharedRef< IDatasmithScene >& InScene, TMap< FString, TSharedPtr<IDatasmithActorElement> >& Actors) const;
	void ParseLandscape(FXmlNode* InNode, TSharedRef< IDatasmithLandscapeElement >& OutElement) const;

	template< typename ElementType >
	void ParseKeyValueProperties(const FXmlNode* InNode, ElementType& OutElement) const;

	bool LoadFromFile(const FString& InFilename);
	bool LoadFromBuffer(const FString& XmlBuffer);

	template< typename ExpressionInputType >
	void ParseExpressionInput(const FXmlNode* InNode, TSharedPtr< IDatasmithUEPbrMaterialElement >& OutElement, ExpressionInputType& ExpressionInput) const;

	TUniquePtr< FXmlFile > XmlFile;
	FString ProjectPath;
};

#undef UE_API
