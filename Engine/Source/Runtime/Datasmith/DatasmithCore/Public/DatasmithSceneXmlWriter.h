// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API DATASMITHCORE_API

class FArchive;
class IDatasmithScene;

class FDatasmithSceneXmlWriter
{
public:
	UE_API void Serialize( TSharedRef< IDatasmithScene > DatasmithScene, FArchive& Archive );
};

#undef UE_API
