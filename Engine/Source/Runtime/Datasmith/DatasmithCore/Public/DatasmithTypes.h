// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "Math/Vector.h"

#define UE_API DATASMITHCORE_API

enum class EDatasmithCurveInterpMode
{
	Linear,
	Constant,
	Cubic
};

class FDatasmithTextureSampler
{
public:
	FDatasmithTextureSampler()
	{
		CoordinateIndex = 0;
		ScaleX = 1.0f;
		ScaleY = 1.0f;
		OffsetX = 0.0f;
		OffsetY = 0.0f;
		Rotation = 0.0f;
		bInvert = false;
		Multiplier = 1.0f;
		OutputChannel = 0;
		bCroppedTexture = false;
		MirrorX = 0;
		MirrorY = 0;
	}

	FDatasmithTextureSampler(int InCoordinateIndex, float InSx, float InSy, float InOx, float InOy, float InRotation, float InMultiplier, bool bInInvert, int InOutputChannel, bool InCroppedTexture, int InMirrorX, int InMirrorY)
	{
		CoordinateIndex = InCoordinateIndex;
		ScaleX = InSx;
		ScaleY = InSy;
		OffsetX = InOx;
		OffsetY = InOy;
		Rotation = InRotation;
		Multiplier = InMultiplier;
		bInvert = bInInvert;
		OutputChannel = InOutputChannel;
		bCroppedTexture = InCroppedTexture;
		MirrorX = InMirrorX;
		MirrorY = InMirrorY;
	}

	// UV coordinate index
	int CoordinateIndex;

	// UV horizontal scale
	float ScaleX;
	// UV vertical scale
	float ScaleY;

	// UV horizontal offset [0,1] range
	float OffsetX;
	// UV vertical offset [0,1] range
	float OffsetY;

	// UV rotation [0,1] range
	float Rotation;

	// texture multiplier
	float Multiplier;

	// force texture invert
	bool bInvert;

	// color channel to be connected
	int OutputChannel;

	// flag to enable UV cropping
	bool bCroppedTexture;

	// UV Mirror 0 No mirror, X mirror tilling otherwise
	int MirrorX;
	// UV Mirror 0 No mirror, X mirror tilling otherwise
	int MirrorY;
};

/**
 * FDatasmithTransformFrameInfo holds the data for the transform values of a frame
 * The transform values must be relative to the parent
 * The rotation is represented as Euler angles in degrees
 */
struct FDatasmithTransformFrameInfo
{
	FDatasmithTransformFrameInfo(int32 InFrameNumber, const FVector& InVec)
		: FrameNumber(InFrameNumber)
		, X(InVec.X)
		, Y(InVec.Y)
		, Z(InVec.Z)
	{}

	FDatasmithTransformFrameInfo(int32 InFrameNumber, const FQuat& InQuat)
		: FrameNumber(InFrameNumber)
	{
		const FVector EulerAngles = InQuat.Euler();
		X = EulerAngles.X;
		Y = EulerAngles.Y;
		Z = EulerAngles.Z;
	}

	FDatasmithTransformFrameInfo(int32 InFrameNumber, double InX, double InY, double InZ)
		: FrameNumber(InFrameNumber)
		, X(InX)
		, Y(InY)
		, Z(InZ)
	{}

	UE_API bool operator==(const FDatasmithTransformFrameInfo& Other) const;
	UE_API bool IsValid() const;

	int32 FrameNumber;

	// For translation and scale, X, Y, Z are the transform values of the respective components
	// For rotation, they represent rotation around the X, Y, Z axis (roll, pitch, yaw respectively) in degrees
	double X;
	double Y;
	double Z;

	static UE_API FDatasmithTransformFrameInfo InvalidFrameInfo;
};

/**
 * FDatasmithVisibilityFrameInfo holds the visibility value for a frame
 */
struct FDatasmithVisibilityFrameInfo
{
	FDatasmithVisibilityFrameInfo(int32 InFrameNumber, bool bInVisible)
		: FrameNumber(InFrameNumber)
		, bVisible(bInVisible)
	{}

	UE_API bool operator==(const FDatasmithVisibilityFrameInfo& Other) const;
	UE_API bool IsValid() const;

	int32 FrameNumber;
	bool bVisible;

	static UE_API FDatasmithVisibilityFrameInfo InvalidFrameInfo;
};

#undef UE_API
