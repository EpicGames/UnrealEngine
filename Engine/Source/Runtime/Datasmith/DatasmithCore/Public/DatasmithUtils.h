// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "DatasmithDefinitions.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "Math/Matrix.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

#include <stdint.h>

#define UE_API DATASMITHCORE_API

class FDatasmithMesh;
class IDatasmithActorElement;
class IDatasmithScene;
struct FMeshDescription;
struct FRawMesh;

class FDatasmithUtils
{
public:
	static UE_API void SanitizeNameInplace(FString& InString);
	static UE_API FString SanitizeName(FString InString);
	static UE_API FString SanitizeObjectName(FString InString);
	static UE_API FString SanitizeFileName(FString InString);

	/** Remove all non-printable characters from the string */
	static UE_API void SanitizeStringInplace(FString& InString);

	static UE_API int32 GetEnterpriseVersionAsInt();
	static UE_API FString GetEnterpriseVersionAsString(bool bWithChangelist=false);

	/** Returns the Datasmith data format version */
	static UE_API float GetDatasmithFormatVersionAsFloat();
	static UE_API int32 GetDatasmithFormatVersionAsInt();
	static UE_API FString GetDatasmithFormatVersionAsString();

	/** Returns the file extension without the dot, of the DatasmithScene. Currently udatasmith */
	static UE_API const TCHAR* GetFileExtension();

	/**
	 * Gets the file name and its extension.
	 * In case of extension like asm.1, the return name is the part before the full extension (asm.1)
	 * The right part of extension must to be a numeric value
	 *
	 * @param	InFilePath the path of the file
	 * @param	OutFilename the filename (without extension), minus any path information.
	 * @param	OutExtension the extension
	 */
	static UE_API void GetCleanFilenameAndExtension(const FString& InFilePath, FString& OutFilename, FString& OutExtension);

	/** Returns the long name of Datasmith */
	static UE_API const TCHAR* GetLongAppName();
	/** Returns the abbreviated name of Datasmith */
	static UE_API const TCHAR* GetShortAppName();

	/** Computes the area of a triangle */
	static UE_API float AreaTriangle3D(const FVector3f& v0, const FVector3f& v1, const FVector3f& v2);

	enum class EModelCoordSystem : uint8
	{
		ZUp_LeftHanded,
		ZUp_RightHanded,
		YUp_LeftHanded,
		YUp_RightHanded,
		ZUp_RightHanded_FBXLegacy,
	};

	template<typename VecType>
	static void ConvertVectorArray(EModelCoordSystem ModelCoordSys, TArray<VecType>& Array)
	{
		switch (ModelCoordSys)
		{
		case EModelCoordSystem::YUp_LeftHanded:
			for (VecType& Vector : Array)
			{
				Vector.Set(Vector[2], Vector[0], Vector[1]);
			}
			break;

		case EModelCoordSystem::YUp_RightHanded:
			for (VecType& Vector : Array)
			{
				Vector.Set(-Vector[2], Vector[0], Vector[1]);
			}
			break;

		case EModelCoordSystem::ZUp_RightHanded:
			for (VecType& Vector : Array)
			{
				Vector.Set(-Vector[0], Vector[1], Vector[2]);
			}
			break;

		case EModelCoordSystem::ZUp_RightHanded_FBXLegacy:
			for (VecType& Vector : Array)
			{
				Vector.Set(Vector[0], -Vector[1], Vector[2]);
			}
			break;

		case EModelCoordSystem::ZUp_LeftHanded:
		default:
			break;
		}
	}

	template<typename VecType>
	static VecType ConvertVector(EModelCoordSystem ModelCoordSys, const VecType& V)
	{
		switch (ModelCoordSys)
		{
		case EModelCoordSystem::YUp_LeftHanded:
			return VecType(V[2], V[0], V[1]);

		case EModelCoordSystem::YUp_RightHanded:
			return VecType(-V[2], V[0], V[1]);

		case EModelCoordSystem::ZUp_RightHanded:
			return VecType(-V[0], V[1], V[2]);

		case EModelCoordSystem::ZUp_RightHanded_FBXLegacy:
			return VecType(V[0], -V[1], V[2]);

		case EModelCoordSystem::ZUp_LeftHanded:
		default:
			return VecType(V[0], V[1], V[2]);
		}
	}

	UE_API FTransform static ConvertTransform(EModelCoordSystem SourceCoordSystem, const FTransform& LocalTransform);

	template<typename Type>
	static UE::Math::TMatrix<Type> GetSymmetricMatrix(const UE::Math::TVector<Type>& Origin, const UE::Math::TVector<Type>& Normal)
	{
		using namespace UE::Math;
		//Calculate symmetry matrix
		//(Px, Py, Pz) = normal
		// -Px*Px + Pz*Pz + Py*Py  |  - 2 * Px * Py           |  - 2 * Px * Pz
		// - 2 * Py * Px           |  - Py*Py + Px*Px + Pz*Pz |  - 2 * Py * Pz
		// - 2 * Pz * Px           |  - 2 * Pz * Py           |  - Pz*Pz + Py*Py + Px*Px

		TVector<Type> LocOrigin = Origin;

		Type NormalXSqr = Normal.X * Normal.X;
		Type NormalYSqr = Normal.Y * Normal.Y;
		Type NormalZSqr = Normal.Z * Normal.Z;

		TMatrix<Type> OSymmetricMatrix;
		OSymmetricMatrix.SetIdentity();
		TVector<Type> Axis0(-NormalXSqr + NormalZSqr + NormalYSqr, -2 * Normal.X * Normal.Y, -2 * Normal.X * Normal.Z);
		TVector<Type> Axis1(-2 * Normal.Y * Normal.X, -NormalYSqr + NormalXSqr + NormalZSqr, -2 * Normal.Y * Normal.Z);
		TVector<Type> Axis2(-2 * Normal.Z * Normal.X, -2 * Normal.Z * Normal.Y, -NormalZSqr + NormalYSqr + NormalXSqr);
		OSymmetricMatrix.SetAxes(&Axis0, &Axis1, &Axis2);

		TMatrix<Type> SymmetricMatrix;
		SymmetricMatrix.SetIdentity();

		//Translate to 0, 0, 0
		LocOrigin *= -1.;
		SymmetricMatrix.SetOrigin(LocOrigin);

		//// Apply Symmetric
		SymmetricMatrix *= OSymmetricMatrix;

		//Translate to original position
		LocOrigin *= -1.;
		TMatrix<Type> OrigTranslation;
		OrigTranslation.SetIdentity();
		OrigTranslation.SetOrigin(LocOrigin);
		SymmetricMatrix *= OrigTranslation;

		return SymmetricMatrix;
	}
};

class FDatasmithMeshUtils
{
	/**
	 * @param bValidateRawMesh this boolean indicate if the raw must be valid.
	 * For example a collision mesh don't need to have a valid raw mesh
	 */
	static bool ToRawMesh(const FDatasmithMesh& Mesh, FRawMesh& RawMesh, bool bValidateRawMesh = true);

public:
	enum EUvGenerationPolicy
	{
		Ignore, // do not generate a default UV. Conversion fails when no UV is available
		GenerateBox, // generate a Box UV mapping on the resulting MeshDescription (see also FStaticMeshOperations::GenerateBoxUV)
	};
	static UE_API bool ToMeshDescription(FDatasmithMesh& DsMesh, FMeshDescription& MeshDescription, EUvGenerationPolicy UvGen=EUvGenerationPolicy::Ignore);

	/**
	 * Validates that the given UV Channel does not contain a degenerated triangle.
	 *
	 * @param DsMesh	The DatasmithMesh to validate.
	 * @param Channel	The UV channel to validate, starting at 0
	 */
	static UE_API bool IsUVChannelValid(const FDatasmithMesh& DsMesh, const int32 Channel);

	/**
	 * Generate simple UV data at channel 0 for the base mesh and it's various LOD variants.
	 *
	 * @param Mesh    The DatasmithMesh in which the UV data will be created.
	 */
	static UE_API void CreateDefaultUVsWithLOD(FDatasmithMesh& Mesh);

	/**
	 * Build an array of point from a MeshDescription
	 */
	static UE_API void ExtractVertexPositions(const FMeshDescription& Mesh, TArray<FVector3f>& OutPositions);
};

enum class EDSTextureUtilsError : int32
{
	NoError = 0,
	FileNotFound = -1,
	InvalidFileType = -2,
	FileReadIssue = -3,
	InvalidData = -4,
	FreeImageNotFound = -5,
	FileNotSaved = -6,
	ResizeFailed = -7,
};

/**
 * NoResize: Keep original size
 * NearestPowerOfTwo: resizes to the nearest power of two value (recommended)
 * PreviousPowerOfTwo: it decreases the value to the previous power of two
 * NextPowerOfTwo: it increases the value to the next power of two
 */
enum class EDSResizeTextureMode
{
	NoResize,
	NearestPowerOfTwo,
	PreviousPowerOfTwo,
	NextPowerOfTwo
};

class FDatasmithTextureUtils
{
public:
	static UE_API bool CalculateTextureHash(const TSharedPtr<class IDatasmithTextureElement>& TextureElement);
	static UE_API void CalculateTextureHashes(const TSharedPtr<class IDatasmithScene>& Scene);
};

/**
 * Enum mainly used to describe which components of a transform animation are enabled. Should mostly be
 * used with FDatasmithAnimationUtils::GetChannelTypeComponents
 */
enum class ETransformChannelComponents : uint8
{
	None = 0x00,
	X	 = 0x01,
	Y    = 0x02,
	Z    = 0x04,
	All  = X | Y | Z,
};
ENUM_CLASS_FLAGS(ETransformChannelComponents);

class FDatasmithAnimationUtils
{
public:
	/** Utility to help handling the components of a channel independently of the transform type **/
	static UE_API ETransformChannelComponents GetChannelTypeComponents(EDatasmithTransformChannels Channels, EDatasmithTransformType TransformType);

	/** Utility to help assembling a transform type's components into a EDatasmithTransformChannels enum **/
	static UE_API EDatasmithTransformChannels SetChannelTypeComponents(ETransformChannelComponents Components, EDatasmithTransformType TransformType);
};

class FDatasmithSceneUtils
{
public:
	static UE_API TArray<TSharedPtr<class IDatasmithCameraActorElement>> GetAllCameraActorsFromScene(const TSharedPtr<class IDatasmithScene>& Scene);
	static UE_API TArray<TSharedPtr<class IDatasmithLightActorElement>> GetAllLightActorsFromScene(const TSharedPtr<class IDatasmithScene>& Scene);
	static UE_API TArray<TSharedPtr<class IDatasmithMeshActorElement>> GetAllMeshActorsFromScene(const TSharedPtr<class IDatasmithScene>& Scene);
	static UE_API TArray<TSharedPtr<class IDatasmithCustomActorElement>> GetAllCustomActorsFromScene(const TSharedPtr<class IDatasmithScene>& Scene);

	using TActorHierarchy = TArray<TSharedPtr<class IDatasmithActorElement>, TInlineAllocator<8>>;
	static UE_API bool FindActorHierarchy(const IDatasmithScene* Scene, const TSharedPtr<IDatasmithActorElement>& ToFind, TActorHierarchy& OutHierarchy);

	static UE_API bool IsMaterialIDUsedInScene(const TSharedPtr<class IDatasmithScene>& Scene, const TSharedPtr<class IDatasmithMaterialIDElement>& MaterialElement);
	static UE_API bool IsPostProcessUsedInScene(const TSharedPtr<class IDatasmithScene>& Scene, const TSharedPtr<class IDatasmithPostProcessElement>& PostProcessElement);

	/**
	 * Fixes all missing references, remove all unused meshes, materials, textures, etc
	 * @param Scene Scene to perform the cleanup on
	 * @param bRemoveUnused Indicates if the cleanup includes the removal of unused assets
	 */
	static UE_API void CleanUpScene(TSharedRef<class IDatasmithScene> Scene, bool bRemoveUnused = false);
};

/**
 * Based on a table of frequently used names, this class generates unique names
 * with a good complexity when the number of name is important.
 * @note: This abstact class allows various implementation of the cache of known name.
 * Implementation could use a simple TSet, or reuse existing specific cache structure
 */
class FDatasmithUniqueNameProviderBase
{
public:
	FDatasmithUniqueNameProviderBase() = default;
	UE_API FDatasmithUniqueNameProviderBase(const FDatasmithUniqueNameProviderBase& Other);
	UE_API FDatasmithUniqueNameProviderBase(FDatasmithUniqueNameProviderBase&& Other);

	virtual ~FDatasmithUniqueNameProviderBase() = default;

	UE_API FDatasmithUniqueNameProviderBase* operator=(const FDatasmithUniqueNameProviderBase& Other);
	UE_API FDatasmithUniqueNameProviderBase* operator=(FDatasmithUniqueNameProviderBase&& Other);

	/**
	 * Generates a unique name
	 * @param BaseName Name that will be suffixed with an index to be unique
	 * @param CharBudget Max character allowed in the name.
	 * @return FString unique name. Calling "Contains()" with this name will be false
	 */
	UE_API FString GenerateUniqueName(const FString& BaseName, int32 CharBudget=INT32_MAX);

	/**
	 * Register a name as known
	 * @param Name name to register
	 */
	virtual void AddExistingName(const FString& Name) = 0;

	/**
	 * Remove a name from the list of existing name
	 * @param Name name to unregister
	 */
	virtual void RemoveExistingName(const FString& Name) = 0;

	/**
	 * Flushes all known names
	 */
	UE_API virtual void Clear();

protected:

	/**
	 * Check if the given name is already registered
	 *
	 * @param Name name to test
	 * @return true if the name is in the cache
	 */
	virtual bool Contains(const FString& Name) = 0;

private:
	TMap<FString, int32> FrequentlyUsedNames;
	mutable FCriticalSection CriticalSection;
};

/**
 * Name provider with internal cache implemented with a simple TSet
 */
class FDatasmithUniqueNameProvider : public FDatasmithUniqueNameProviderBase
{
public:
	using Super = FDatasmithUniqueNameProviderBase;

	void Reserve( int32 NumberOfName ) { KnownNames.Reserve(NumberOfName); }

	virtual void AddExistingName(const FString& Name) override { KnownNames.Add(Name); }
	virtual void RemoveExistingName(const FString& Name) override { KnownNames.Remove(Name); }

	virtual void Clear() override { Super::Clear(); KnownNames.Empty(); }

protected:
	virtual bool Contains(const FString& Name) override { return KnownNames.Contains(Name); }

private:
	TSet<FString> KnownNames;
};

#undef UE_API
