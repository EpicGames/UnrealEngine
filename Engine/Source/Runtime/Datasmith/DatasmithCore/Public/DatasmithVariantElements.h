// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithDefinitions.h"
#include "IDatasmithSceneElements.h"

#include "Templates/SharedPointer.h"

class IDatasmithBaseVariantElement : public IDatasmithElement
{
public:
	virtual bool IsSubType( const EDatasmithElementVariantSubType VariantSubType ) const = 0;
};

class IDatasmithBasePropertyCaptureElement : public IDatasmithBaseVariantElement
{
public:
	virtual void SetPropertyPath(const FString& Path) = 0;
	virtual const FString& GetPropertyPath() const = 0;

	virtual void SetCategory(EDatasmithPropertyCategory Category) = 0;
	virtual EDatasmithPropertyCategory GetCategory() const = 0;
};

class IDatasmithPropertyCaptureElement : public IDatasmithBasePropertyCaptureElement
{
public:
	virtual void SetRecordedData(const uint8* Data, int32 NumBytes) = 0;
	virtual const TArray<uint8>& GetRecordedData() const = 0;
};

class IDatasmithObjectPropertyCaptureElement : public IDatasmithBasePropertyCaptureElement
{
public:
	virtual void SetRecordedObject(TWeakPtr<IDatasmithElement> Object) = 0;
	virtual TWeakPtr<IDatasmithElement> GetRecordedObject() const = 0;
};

class IDatasmithActorBindingElement : public IDatasmithBaseVariantElement
{
public:
	virtual void SetActor(TSharedPtr<IDatasmithActorElement> Actor) = 0;
	virtual TSharedPtr<IDatasmithActorElement> GetActor() const = 0;

	virtual void AddPropertyCapture(const TSharedRef<IDatasmithBasePropertyCaptureElement>& Prop) = 0;
	virtual int32 GetPropertyCapturesCount() const = 0;
	virtual TSharedPtr<IDatasmithBasePropertyCaptureElement> GetPropertyCapture(int32 Index) = 0;
	virtual void RemovePropertyCapture(const TSharedRef<IDatasmithBasePropertyCaptureElement>& Prop) = 0;
};

class IDatasmithVariantElement : public IDatasmithBaseVariantElement
{
public:
	virtual void AddActorBinding(const TSharedRef<IDatasmithActorBindingElement>& Binding) = 0;
	virtual int32 GetActorBindingsCount() const = 0;
	virtual TSharedPtr<IDatasmithActorBindingElement> GetActorBinding(int32 Index) = 0;
	virtual void RemoveActorBinding(const TSharedRef<IDatasmithActorBindingElement>& Binding) = 0;
};

class IDatasmithVariantSetElement : public IDatasmithBaseVariantElement
{
public:
	virtual void AddVariant(const TSharedRef<IDatasmithVariantElement>& Variant) = 0;
	virtual int32 GetVariantsCount() const = 0;
	virtual TSharedPtr<IDatasmithVariantElement> GetVariant(int32 Index) = 0;
	virtual void RemoveVariant(const TSharedRef<IDatasmithVariantElement>& Variant) = 0;
};

class IDatasmithLevelVariantSetsElement : public IDatasmithBaseVariantElement
{
public:
	virtual void AddVariantSet(const TSharedRef<IDatasmithVariantSetElement>& VariantSet) = 0;
	virtual int32 GetVariantSetsCount() const = 0;
	virtual TSharedPtr<IDatasmithVariantSetElement> GetVariantSet(int32 Index) = 0;
	virtual void RemoveVariantSet(const TSharedRef<IDatasmithVariantSetElement>& VariantSet) = 0;
};
