// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IDatasmithSceneElements.h"
#include "Templates/SharedPointer.h"

class IDatasmithElement;
class IDatasmithScene;


DATASMITHCORE_API const TCHAR* GetElementTypeName(const IDatasmithElement* Element);

DATASMITHCORE_API void DumpDatasmithScene(const TSharedRef<IDatasmithScene>& Scene, const TCHAR* BaseName);
