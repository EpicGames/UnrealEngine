// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "DirectLinkCommon.h"
#include "DirectLinkDeltaConsumer.h"
#include "DirectLinkSceneGraphNode.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API DATASMITHCORE_API

class IDatasmithElement;
class IDatasmithScene;
namespace DirectLink { class FElementSnapshot; }
namespace DirectLink { class FSceneSnapshot; }

/**
 * This class receives DirectLink scene snapshots, and convert them into a DatasmithScene.
 * A listener can be registered to be notified of a scene modification
 */
class FDatasmithSceneReceiver
	: public DirectLink::ISceneReceiver
{
public:
	class ISceneChangeListener
	{
	public:
		virtual void OnOpenDelta() = 0;
		virtual void OnNewScene(const DirectLink::FSceneIdentifier& SceneId) = 0;
		virtual void OnAddElement(DirectLink::FSceneGraphId, TSharedPtr<IDatasmithElement> Element) = 0;
		virtual void OnChangedElement(DirectLink::FSceneGraphId, TSharedPtr<IDatasmithElement> Element) = 0;
		virtual void OnRemovedElement(DirectLink::FSceneGraphId ElementId) = 0;
		virtual void OnCloseDelta() = 0;
	};

public:
	UE_API FDatasmithSceneReceiver();

	// Register a listner that will be notified of important scene edition events
	void SetChangeListener(ISceneChangeListener* Listener) { ChangeListener = Listener; }

	// Get the reconstructed DatasmithScene. Can be null.
	UE_API TSharedPtr<IDatasmithScene> GetScene() const;

private: // DirectLink::ISceneReceiver API
	UE_API virtual void FinalSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot) override;

	struct FSceneState;

	UE_API TUniquePtr<FSceneState> ParseSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot);

private:
	struct FDatasmithElementPointers : public DirectLink::IReferenceResolutionProvider
	{
		// IReferenceResolutionProvider API
		UE_API virtual TSharedPtr<DirectLink::ISceneGraphNode> AsSharedPtr(DirectLink::FSceneGraphId NodeId) override;
		void Reset() { ElementsSharedPtrs.Reset(); }
		void Remove(DirectLink::FSceneGraphId NodeId) { ElementsSharedPtrs.Remove(NodeId); }
		void Add(DirectLink::FSceneGraphId Id, TSharedPtr<IDatasmithElement> Element) { ElementsSharedPtrs.Add(Id, Element); }
		TMap<DirectLink::FSceneGraphId, TSharedPtr<IDatasmithElement>> ElementsSharedPtrs;
	};

	struct FFinalizableNode
	{
		TSharedPtr<IDatasmithElement> Element;
		const DirectLink::FElementSnapshot* Snapshot;
	};

	ISceneChangeListener* ChangeListener = nullptr;

	struct FSceneHashTable
	{
		TMap<DirectLink::FSceneGraphId, DirectLink::FElementHash> ElementHashes;
		static FSceneHashTable FromSceneSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot);
	};

	struct FSceneState
	{
		DirectLink::FSceneIdentifier SceneId;
		TSharedPtr<IDatasmithScene> Scene;
		FDatasmithElementPointers Elements;
		FSceneHashTable HashTable;
	};

	TUniquePtr<FSceneState> Current;
};


#undef UE_API
