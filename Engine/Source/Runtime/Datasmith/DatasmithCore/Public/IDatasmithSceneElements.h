// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithDefinitions.h"
#include "DatasmithTypes.h"
#include "DirectLinkSceneGraphNode.h"

#include "Containers/Map.h"
#include "Math/Color.h"
#include "Math/Quat.h"
#include "Math/Vector.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/SecureHash.h"
#include "Templates/SharedPointer.h"

#define UE_API DATASMITHCORE_API

class IDatasmithCompositeTexture;
class IDatasmithMaterialIDElement;
class IDatasmithPostProcessElement;
class IDatasmithShaderElement;
class IDatasmithLevelSequenceElement;
class IDatasmithLevelVariantSetsElement;

/** Root class for every element in a Datasmith scene */
class IDatasmithElement : public DirectLink::ISceneGraphNode
{
public:
	/** returns if this DatasmithElement is of a specified type */
	virtual bool IsA(EDatasmithElementType Type) const = 0;

	/** Gets the element name */
	virtual const TCHAR* GetName() const = 0;

	/** Sets the element name */
	virtual void SetName(const TCHAR* InName) = 0;

	/** Gets the element label used in the UI */
	virtual const TCHAR* GetLabel() const = 0;

	/** Sets the element label used in the UI */
	virtual void SetLabel(const TCHAR* InLabel) = 0;

	/**
	 * Return a MD5 hash of the content of the Element. Used to quickly identify Element with identical content.
	 * @param bForce Force recalculation of the hash if it was already cached
	 * @return The MD5 hash of the Element properties
	 */
	virtual FMD5Hash CalculateElementHash(bool bForce) = 0;
};

class IDatasmithKeyValueProperty : public IDatasmithElement
{
public:
	virtual ~IDatasmithKeyValueProperty() = default;

	/** Get the type of this property */
	virtual EDatasmithKeyValuePropertyType GetPropertyType() const = 0;

	/** Set the type of this property */
	virtual void SetPropertyType(EDatasmithKeyValuePropertyType InType) = 0;

	/** Get the value of this property */
	virtual const TCHAR* GetValue() const = 0;

	/** Sets the value of this property */
	virtual void SetValue(const TCHAR* Value) = 0;
};

/** Base definition for Actor Elements like geometry instances, cameras or lights */
class IDatasmithActorElement : public IDatasmithElement
{
public:
	virtual ~IDatasmithActorElement() = default;

	/** Get absolute translation of this entity */
	virtual FVector GetTranslation() const = 0;

	/** Set absolute translation of this entity */
	virtual void SetTranslation(double InX, double InY, double InZ, bool bKeepChildrenRelative = true) = 0;

	/** Set absolute translation of this entity */
	virtual void SetTranslation(const FVector& Value, bool bKeepChildrenRelative = true) = 0;

	/** Get absolute scale of this entity */
	virtual FVector GetScale() const = 0;

	/** Set absolute scale of this entity */
	virtual void SetScale(double InX, double InY, double InZ, bool bKeepChildrenRelative = true) = 0;

	/** Set absolute scale of this entity */
	virtual void SetScale(const FVector& Value, bool bKeepChildrenRelative = true) = 0;

	/** Get rotation (in quaternion format) of this entity */
	virtual FQuat GetRotation() const = 0;

	/** Set rotation (in quaternion format) of this entity */
	virtual void SetRotation(double InX, double InY, double InZ, double InW, bool bKeepChildrenRelative = true) = 0;

	/** Set rotation (in quaternion format) of this entity */
	virtual void SetRotation(const FQuat& Value, bool bKeepChildrenRelative = true) = 0;

	/** Returns the relative transform for this element */
	virtual FTransform GetRelativeTransform() const = 0;

	/** Get the the name of the layer that contains this entity */
	virtual const TCHAR* GetLayer() const = 0;

	/** Set the the the layer that contains this entity, layer will be auto-created from its name */
	virtual void SetLayer(const TCHAR* InLayer) = 0;

	/** Add a new Tag to an Actor element */
	virtual void AddTag(const TCHAR* InTag) = 0;

	/** Remove all Tags on the Actor element */
	virtual void ResetTags() = 0;

	/** Get the number of tags attached to an Actor element */
	virtual int32 GetTagsCount() const = 0;

	/** Get the 'TagIndex'th tag of an Actor element */
	virtual const TCHAR* GetTag(int32 TagIndex) const = 0;

	/** Adds a new child to the Actor Element */
	virtual void AddChild(const TSharedPtr< IDatasmithActorElement >& InChild, EDatasmithActorAttachmentRule AttachementRule = EDatasmithActorAttachmentRule::KeepWorldTransform) = 0;

	/** Get the number of children on this actor */
	virtual int32 GetChildrenCount() const = 0;

	/** Get the InIndex-th child of the mesh actor */
	virtual TSharedPtr< IDatasmithActorElement > GetChild(int32 InIndex) = 0;
	virtual const TSharedPtr< IDatasmithActorElement >& GetChild(int32 InIndex) const = 0;

	virtual void RemoveChild(const TSharedPtr< IDatasmithActorElement >& InChild) = 0;

	/** Get the parent actor of the Actor element, returns invalid TSharedPtr if the Actor is directly under the scene root */
	virtual const TSharedPtr< IDatasmithActorElement >& GetParentActor() const = 0;

	/** Indicates if this actor is a standalone actor or a component, when used in a hierarchy */
	virtual void SetIsAComponent(bool Value) = 0;
	virtual bool IsAComponent() const = 0;

	/** Set a mesh actor's visibility */
	virtual void SetVisibility(bool bInVisibility) = 0;

	/** Get a mesh actor's visibility */
	virtual bool GetVisibility() const = 0;

	/** Set whether an actor's casts shadow */
	virtual void SetCastShadow(bool bInCastShadow) = 0;

	/** Get whether an actor's casts shadow */
	virtual bool GetCastShadow() const = 0;

	/** Set a mesh actor's visibility */
	virtual void SetMobility(EDatasmithActorMobilityType Mobility) = 0;

	/** Get a mesh actor's visibility */
	virtual EDatasmithActorMobilityType GetMobility() const = 0;
};

/**
 * IDatasmithMeshElement defines an actual geometry.
 * It won't add any instance to your scene, you'll need IDatasmithMeshActorElement for this.
 * Notice that several IDatasmithMeshActorElements could use the this geometry.
 */
class IDatasmithMeshElement : public IDatasmithElement
{
public:
	virtual ~IDatasmithMeshElement() = default;

	/** Get the output filename, it can be absolute or relative to the scene file */
	virtual const TCHAR* GetFile() const = 0;

	/** Set the output filename, it can be absolute or relative to the scene file */
	virtual void SetFile(const TCHAR* InFile) = 0;

	/** Return a MD5 hash of the content of the Mesh Element. Used in CalculateElementHash to quickly identify Element with identical content */
	virtual FMD5Hash GetFileHash() const = 0;

	/** Set the MD5 hash of the current mesh file. This should be a hash of its content. */
	virtual void SetFileHash(FMD5Hash Hash) = 0;

	/**
	 * Set surface area and bounding box dimensions to be used on lightmap size calculation.
	 *
	 * @param InArea total surface area
	 * @param InWidth bounding box width
	 * @param InHeight bounding box height
	 * @param InDepth bounding box depth
	 */
	virtual void SetDimensions(float InArea, float InWidth, float InHeight, float InDepth) = 0;

	/** Get the bounding box dimension of the mesh, in a vector in the form of (Width, Height, Depth )*/
	virtual FVector3f GetDimensions() const = 0;

	/** Get the total surface area */
	virtual float GetArea() const = 0;

	/** Get the bounding box width */
	virtual float GetWidth() const = 0;

	/** Get the bounding box height */
	virtual float GetHeight() const = 0;

	/** Get the bounding box depth */
	virtual float GetDepth() const = 0;

	/** Get the UV channel that will be used for the lightmap */
	virtual int32 GetLightmapCoordinateIndex() const = 0;

	/**
	 * Set the UV channel that will be used for the lightmap
	 * Note: If the lightmap coordinate index is something greater than -1 it will make the importer skip the lightmap generation
	 */
	virtual void SetLightmapCoordinateIndex(int32 UVChannel) = 0;

	/** Get the source UV channel that will be used at import to generate the lightmap UVs */
	virtual int32 GetLightmapSourceUV() const = 0;

	/** Set the source UV channel that will be used at import to generate the lightmap UVs */
	virtual void SetLightmapSourceUV(int32 UVChannel) = 0;

	/** Set the material slot Id to use the material MaterialPathName*/
	virtual void SetMaterial(const TCHAR* MaterialPathName, int32 SlotId) = 0;

	/** Get the name of the material mapped to slot Id, return nullptr if slot isn't mapped */
	virtual const TCHAR* GetMaterial(int32 SlotId) const = 0;

	/** Get the number of material slot set on this mesh */
	virtual int32 GetMaterialSlotCount() const = 0;

	/** Get the material mapping for slot Index */
	virtual TSharedPtr<const IDatasmithMaterialIDElement> GetMaterialSlotAt(int32 Index) const = 0;
	virtual TSharedPtr<IDatasmithMaterialIDElement> GetMaterialSlotAt(int32 Index) = 0;

protected:
	/** Get number of LODs */
	virtual int32 GetLODCount() const = 0;

	/** Set number of LODs */
	virtual void SetLODCount(int32 Count) = 0;

	friend class FDatasmithStaticMeshImporter;
};

/**
* IDatasmithClothElement class: experimental class that describes a cloth asset
*/
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API IDatasmithClothElement : public IDatasmithElement
{
public:
	/** Get the FDatasmithCloth resource filename */
	virtual const TCHAR* GetFile() const = 0;

	/** Set the FDatasmithCloth resource filename, it can be absolute or relative to the scene file */
	virtual void SetFile(const TCHAR* InFile) = 0;

// class DATASMITHCORE_API IDatasmithClothPropertiesElement : public IDatasmithElement
};

/**
 * IDatasmithActorElement used in any geometry instance independently if it could be static or movable.
 * It doesn't define the actual geometry, you'll need IDatasmithMeshElement for this.
 * Notice that several IDatasmithMeshActorElements could use the same geometry.
 */
class IDatasmithMeshActorElement : public IDatasmithActorElement
{
public:
	virtual ~IDatasmithMeshActorElement() = default;

	/**
	 * Adds a new material override to the Actor Element
	 *
	 * @param MaterialName name of the material, it should be unique
	 * @param Id material identifier to be used with mesh sub-material indices. Use -1 to override all material slots.
	 */
	virtual void AddMaterialOverride(const TCHAR* MaterialName, int32 Id) = 0;

	/** Adds a new material override to the Actor Element */
	virtual void AddMaterialOverride(const TSharedPtr<IDatasmithMaterialIDElement>& Material) = 0;

	/** Get the amount of material overrides on this mesh */
	virtual int32 GetMaterialOverridesCount() const = 0;

	/** Get the i-th material override of this actor */
	virtual TSharedPtr<IDatasmithMaterialIDElement> GetMaterialOverride(int32 i) = 0;

	/** Get the i-th material override of this actor */
	virtual TSharedPtr<const IDatasmithMaterialIDElement> GetMaterialOverride(int32 i) const = 0;

	/** Remove material from the Actor Element */
	virtual void RemoveMaterialOverride(const TSharedPtr<IDatasmithMaterialIDElement>& Material) = 0;

	/** Remove all material overrides from the Actor Element */
	virtual void ResetMaterialOverrides() = 0;

	/** Get the path name of the StaticMesh associated with the actor */
	virtual const TCHAR* GetStaticMeshPathName() const = 0;

	/**
	 * Set the path name of the StaticMesh that the actor is using
	 * It can be either a package path to refer to an existing mesh or a mesh name to refer to a MeshElement in the DatasmithScene
	 */
	virtual void SetStaticMeshPathName(const TCHAR* InStaticMeshPathName) = 0;
};

class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") DATASMITHCORE_API IDatasmithClothActorElement : public IDatasmithActorElement
{
public:
	virtual void SetCloth(const TCHAR* Cloth) = 0;
	virtual const TCHAR* GetCloth() const = 0;
};

class IDatasmithHierarchicalInstancedStaticMeshActorElement : public IDatasmithMeshActorElement
{
public:
	/**
	 * Get the number of instances
	 * @return the number of instances
	 */
	virtual int32 GetInstancesCount() const = 0;

	/**
	 * Reserve memory for a number of instance.
	 * @param NumInstances The number of instance.
	 * This reduce the overall time needed to add a large number of instances.
	 */
	virtual void ReserveSpaceForInstances(int32 NumIntances) = 0;

	/**
	 * Add an instance
	 * @param Transform the transform of the instance
	 * @return the index of the new instance
	 */
	virtual int32 AddInstance(const FTransform& Transform) = 0;

	/**
	 * Get the transform of a specified instance
	 * @param InstanceIndex The index of the instance
	 * @return The transform of the instance
	 */
	virtual FTransform GetInstance(int32 InstanceIndex) const = 0;

	/**
	 * Remove an instance
	 * @param InstanceIndex The index of the instance to remove
	 * Note that this destruct the order of the instances
	 */
	virtual void RemoveInstance(int32 InstanceIndex) = 0;
};

class IDatasmithLightActorElement : public IDatasmithActorElement
{
public:
	virtual ~IDatasmithLightActorElement() = default;

	/** Return true on light enabled, false otherwise */
	virtual bool IsEnabled() const = 0;

	/** Set enable property of the light */
	virtual void SetEnabled(bool bIsEnabled) = 0;

	/** Get light intensity */
	virtual double GetIntensity() const = 0;

	/** Set light intensity */
	virtual void SetIntensity(double Intensity) = 0;

	/** Get light color on linear mode */
	virtual FLinearColor GetColor() const = 0;

	/** Set light color on linear mode */
	virtual void SetColor(FLinearColor Color) = 0;

	/** Get the light temperature in Kelvin */
	virtual double GetTemperature() const = 0;

	/** Set the light temperature in Kelvin */
	virtual void SetTemperature(double Temperature) = 0;

	/** Get if the light color is controlled by temperature */
	virtual bool GetUseTemperature() const = 0;

	/** Set if the light color is controlled by temperature */
	virtual void SetUseTemperature(bool bUseTemperature) = 0;

	/** Get the path of the Ies definition file - DEPRECATED in 4.26: Replaced with GetIesTexturePathName */
	virtual const TCHAR* GetIesFile() const = 0;

	/** Get the IES texture path */
	virtual const TCHAR* GetIesTexturePathName() const = 0;

	/** Set the path of the Ies definition file - DEPRECATED in 4.26: Replaced with SetIesTexturePathName */
	virtual void SetIesFile(const TCHAR* IesFile) = 0;

	/**
	 * Set the IES texture path
	 * The path is either the name of an element attached to the scene or
	 * the path of a UE texture asset, i.e. /Game/.../TextureAssetName.TextureAssetName
	 */
	virtual void SetIesTexturePathName(const TCHAR* TextureName) = 0;

	/** Set if this light is controlled by Ies definition file */
	virtual bool GetUseIes() const = 0;

	/** Get if this light is controlled by Ies definition file */
	virtual void SetUseIes(bool bUseIes) = 0;

	/** Get the Ies brightness multiplier */
	virtual double GetIesBrightnessScale() const = 0;

	/** Set the Ies brightness multiplier */
	virtual void SetIesBrightnessScale(double IesBrightnessScale) = 0;

	/** Get if the emissive amount of the ies is controlled by the brightness scale */
	virtual bool GetUseIesBrightness() const = 0;

	/** Set if the emissive amount of the ies is controlled by the brightness scale */
	virtual void SetUseIesBrightness(bool bUseIesBrightness) = 0;

	/** Get the rotation applied to the IES shape */
	virtual FQuat GetIesRotation() const = 0;

	/** Set the rotation to apply to the IES shape */
	virtual void SetIesRotation(const FQuat& IesRotation) = 0;

	/** Get emissive material on this light */
	virtual TSharedPtr< IDatasmithMaterialIDElement >& GetLightFunctionMaterial() = 0;

	/** Set emissive material on this light */
	virtual void SetLightFunctionMaterial(const TSharedPtr< IDatasmithMaterialIDElement >& InMaterial) = 0;

	/** Set emissive material on this light */
	virtual void SetLightFunctionMaterial(const TCHAR* InMaterialName) = 0;
};

class IDatasmithPointLightElement : public IDatasmithLightActorElement
{
public:
	virtual void SetIntensityUnits(EDatasmithLightUnits InUnits) = 0;
	virtual EDatasmithLightUnits GetIntensityUnits() const = 0;

	/** Get light radius, width in case of 2D light sources */
	virtual float GetSourceRadius() const = 0;

	/** Set light radius, width in case of 2D light sources */
	virtual void SetSourceRadius(float SourceRadius) = 0;

	/** Get light length only affects 2D shaped lights */
	virtual float GetSourceLength() const = 0;

	/** Set light length only affects 2D shaped lights */
	virtual void SetSourceLength(float SourceLength) = 0;

	/** Get attenuation radius in centimeters */
	virtual float GetAttenuationRadius() const = 0;

	/** Set attenuation radius in centimeters */
	virtual void SetAttenuationRadius(float AttenuationRadius) = 0;
};

class IDatasmithSpotLightElement : public IDatasmithPointLightElement
{
public:
	/** Get the inner cone angle for spot lights in degrees */
	virtual float GetInnerConeAngle() const = 0;

	/** Set the inner cone angle for spot lights in degrees */
	virtual void SetInnerConeAngle(float InnerConeAngle) = 0;

	/** Get the outer cone angle for spot lights in degrees */
	virtual float GetOuterConeAngle() const = 0;

	/** Set the outer cone angle for spot lights in degrees */
	virtual void SetOuterConeAngle(float OuterConeAngle) = 0;
};

class IDatasmithDirectionalLightElement : public IDatasmithLightActorElement
{
};

/**
 * An area light is an emissive shape (light shape) with a light component (light type)
 */
class IDatasmithAreaLightElement : public IDatasmithSpotLightElement
{
public:
	/** Get the light shape Rectangle/Sphere/Disc/Cylinder */
	virtual EDatasmithLightShape GetLightShape() const = 0;

	/** Set the light shape Rectangle/Sphere/Disc/Cylinder */
	virtual void SetLightShape(EDatasmithLightShape Shape) = 0;

	/** Set the type of light for an area light: Point/Spot/Rect */
	virtual void SetLightType(EDatasmithAreaLightType LightType) = 0;
	virtual EDatasmithAreaLightType GetLightType() const = 0;

	/** Set the area light shape size on the Y axis */
	virtual void SetWidth(float InWidth) = 0;
	virtual float GetWidth() const = 0;

	/** Set the area light shape size on the X axis */
	virtual void SetLength(float InLength) = 0;
	virtual float GetLength() const = 0;
};

/**
 * Represents a ALightmassPortal
 *
 * Use the actor scale to drive the portal dimensions
 */
class IDatasmithLightmassPortalElement : public IDatasmithPointLightElement
{
};

class IDatasmithCameraActorElement : public IDatasmithActorElement
{
public:
	virtual ~IDatasmithCameraActorElement() = default;

	/** Get camera sensor width in millimeters */
	virtual float GetSensorWidth() const = 0;

	/** Set camera sensor width in millimeters */
	virtual void SetSensorWidth(float SensorWidth) = 0;

	/** Get framebuffer aspect ratio (width/height) */
	virtual float GetSensorAspectRatio() const = 0;

	/** Set framebuffer aspect ratio (width/height) */
	virtual void SetSensorAspectRatio(float SensorAspectRatio) = 0;

	/** The focus method of the camera, either None (no DoF) or Manual */
	virtual bool GetEnableDepthOfField() const = 0;

	/** The focus method of the camera, either None (no DoF) or Manual */
	virtual void SetEnableDepthOfField(bool bEnableDepthOfField) = 0;

	/** Get camera focus distance in centimeters */
	virtual float GetFocusDistance() const = 0;

	/** Set camera focus distance in centimeters */
	virtual void SetFocusDistance(float FocusDistance) = 0;

	/** Get camera FStop also known as FNumber */
	virtual float GetFStop() const = 0;

	/** Set camera FStop also known as FNumber */
	virtual void SetFStop(float FStop) = 0;

	/** Get camera focal length in millimeters */
	virtual float GetFocalLength() const = 0;

	/** Set camera focal length in millimeters */
	virtual void SetFocalLength(float FocalLength) = 0;

	/** Get camera's postprocess */
	virtual TSharedPtr< IDatasmithPostProcessElement >& GetPostProcess() = 0;

	/** Get camera's postprocess */
	virtual const TSharedPtr< IDatasmithPostProcessElement >& GetPostProcess() const = 0;

	/** Set camera's postprocess */
	virtual void SetPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& PostProcess) = 0;

	/** Get camera look at actor name */
	virtual const TCHAR* GetLookAtActor() const = 0;

	/** Set camera look at actor name */
	virtual void SetLookAtActor(const TCHAR* ActorName) = 0;

	/** Get camera look at allow roll state */
	virtual bool GetLookAtAllowRoll() const = 0;

	/** Set camera look at allow roll state */
	virtual void SetLookAtAllowRoll(bool bAllow) = 0;
};

class IDatasmithCustomActorElement : public IDatasmithActorElement
{
public:
	/** The class name or path to the blueprint to instantiate. */
	virtual const TCHAR* GetClassOrPathName() const = 0;
	virtual void SetClassOrPathName(const TCHAR* InClassOrPathName) = 0;

	/** Get the total amount of properties in this actor */
	virtual int32 GetPropertiesCount() const = 0;

	/** Get the property i-th of this actor */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 i) const = 0;

	/** Get a property by its name if it exists */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName(const TCHAR* Name) const = 0;

	/** Add a property to this actor */
	virtual void AddProperty(const TSharedPtr< IDatasmithKeyValueProperty >& Property) = 0;

	/** Removes a property from this actor, doesn't preserve ordering */
	virtual void RemoveProperty(const TSharedPtr< IDatasmithKeyValueProperty >& Property) = 0;
};

class IDatasmithLandscapeElement : public IDatasmithActorElement
{
public:
	/** The path to the heightmap file */
	virtual void SetHeightmap(const TCHAR* FilePath) = 0;
	virtual const TCHAR* GetHeightmap() const = 0;

	/** The name or path to the material to assign to this landscape layer */
	virtual void SetMaterial(const TCHAR* MaterialPathName) = 0;
	virtual const TCHAR* GetMaterial() const = 0;
};

class IDatasmithMaterialIDElement : public IDatasmithElement
{
public:
	virtual ~IDatasmithMaterialIDElement() = default;

	virtual int32 GetId() const = 0;
	virtual void SetId(int32 Id) = 0;
};

class IDatasmithBaseMaterialElement : public IDatasmithElement
{
};

class IDatasmithMaterialElement : public IDatasmithBaseMaterialElement
{
public:
	virtual ~IDatasmithMaterialElement() = default;

	/** Returns true if the material has only one shader, false otherwise */
	virtual bool IsSingleShaderMaterial() const = 0;

	/** Returns true if the material has a clear coat layer, false otherwise */
	virtual bool IsClearCoatMaterial() const = 0;

	/** Adds a new shader to the material stack */
	virtual void AddShader(const TSharedPtr< IDatasmithShaderElement >& Shader) = 0;

	/** Get the total amount of shaders in this material */
	virtual int32 GetShadersCount() const = 0;

	/** Get the shader i-th of this material */
	virtual TSharedPtr< IDatasmithShaderElement >& GetShader(int32 InIndex) = 0;

	/** Get the shader i-th of this material */
	virtual const TSharedPtr< IDatasmithShaderElement >& GetShader(int32 InIndex) const = 0;
};

class IDatasmithMaterialInstanceElement : public IDatasmithBaseMaterialElement
{
public:
	virtual ~IDatasmithMaterialInstanceElement() = default;

	virtual EDatasmithReferenceMaterialType GetMaterialType() const = 0;
	virtual void SetMaterialType(EDatasmithReferenceMaterialType InType) = 0;

	virtual EDatasmithReferenceMaterialQuality GetQuality() const = 0;
	virtual void SetQuality(EDatasmithReferenceMaterialQuality InQuality) = 0;

	/** Only used when the material type is set to Custom. The path name to an existing material to instantiate. */
	virtual const TCHAR* GetCustomMaterialPathName() const = 0;
	virtual void SetCustomMaterialPathName(const TCHAR* InPathName) = 0;

	/** Get the total amount of properties in this material */
	virtual int32 GetPropertiesCount() const = 0;

	/** Get the property i-th of this material */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 i) const = 0;

	/** Get a property by its name if it exists */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName(const TCHAR* Name) const = 0;

	/** Add a property to this material*/
	virtual void AddProperty(const TSharedPtr< IDatasmithKeyValueProperty >& Property) = 0;
};

class IDatasmithDecalMaterialElement : public IDatasmithBaseMaterialElement
{
public:
	virtual ~IDatasmithDecalMaterialElement() = default;

	/** Get path name of the diffuse texture associated with the material */
	virtual const TCHAR* GetDiffuseTexturePathName() const = 0;

	/**
	 * Set path name of the diffuse texture associated with the material
	 * The path name can be either a package path referring to an existing texture asset
	 * or a texture name referring a TextureElement in the DatasmithScene
	 */
	virtual void SetDiffuseTexturePathName(const TCHAR* DiffuseTexturePathName) = 0;

	/** Get path name of the normal texture associated with the material */
	virtual const TCHAR* GetNormalTexturePathName() const = 0;

	/**
	 * Set path name of the normal texture associated with the material
	 * The path name can be either the name of a texture added to the Datasmith scene
	 * or a path to an Unreal asset
	 */
	virtual void SetNormalTexturePathName(const TCHAR* NormalTexturePathName) = 0;
};

class IDatasmithPostProcessElement : public IDatasmithElement
{
public:
	/** Get color filter temperature in Kelvin */
	virtual float GetTemperature() const = 0;

	/** Set color filter temperature in Kelvin */
	virtual void SetTemperature(float Temperature) = 0;

	/** Set color filter in linear color scale */
	virtual FLinearColor GetColorFilter() const = 0;

	/** Get color filter in linear color scale */
	virtual void SetColorFilter(FLinearColor ColorFilter) = 0;

	/** Get vignette amount */
	virtual float GetVignette() const = 0;

	/** Set vignette amount */
	virtual void SetVignette(float Vignette) = 0;

	/** Get depth of field multiplier */
	virtual float GetDof() const = 0;

	/** Set depth of field multiplier */
	virtual void SetDof(float Dof) = 0;

	/** Get motion blur multiplier */
	virtual float GetMotionBlur() const = 0;

	/** Set motion blur multiplier */
	virtual void SetMotionBlur(float MotionBlur) = 0;

	/** Get color saturation */
	virtual float GetSaturation() const = 0;

	/** Set color saturation */
	virtual void SetSaturation(float Saturation) = 0;

	/** Get camera ISO */
	virtual float GetCameraISO() const = 0;

	/** Set camera ISO */
	virtual void SetCameraISO(float CameraISO) = 0;

	/** The camera shutter speed in 1/seconds (ie: 60 = 1/60s) */
	virtual float GetCameraShutterSpeed() const = 0;
	virtual void SetCameraShutterSpeed(float CameraShutterSpeed) = 0;

	/** Defines the opening of the camera lens, Aperture is 1/fstop, typical lens go down to f/1.2 (large opening), larger numbers reduce the DOF effect */
	virtual float GetDepthOfFieldFstop() const = 0;
	virtual void SetDepthOfFieldFstop(float Fstop) = 0;
};

/** Represents the APostProcessVolume object */
class IDatasmithPostProcessVolumeElement : public IDatasmithActorElement
{
public:
	/** The post process settings to use for this volume. */
	virtual TSharedRef< IDatasmithPostProcessElement > GetSettings() const = 0;
	virtual void SetSettings(const TSharedRef< IDatasmithPostProcessElement >& Settings) = 0;

	/** Whether this volume is enabled or not. */
	virtual bool GetEnabled() const = 0;
	virtual void SetEnabled(bool bEnabled) = 0;

	/** Whether this volume covers the whole world, or just the area inside its bounds. */
	virtual bool GetUnbound() const = 0;
	virtual void SetUnbound(bool bUnbound) = 0;
};

class IDatasmithEnvironmentElement : public IDatasmithLightActorElement
{
public:
	virtual ~IDatasmithEnvironmentElement() = default;

	/** Get the environment map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetEnvironmentComp() = 0;

	/** Get the environment map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetEnvironmentComp() const = 0;

	/** Set the environment map */
	virtual void SetEnvironmentComp(const TSharedPtr<IDatasmithCompositeTexture>& EnvironmentComp) = 0;

	/** Returns true if it is used for illumination, false if it is used as background */
	virtual bool GetIsIlluminationMap() const = 0;

	/** Set true for being used as illumination, false for being used as background */
	virtual void SetIsIlluminationMap(bool bIsIlluminationMap) = 0;
};

class IDatasmithTextureElement : public IDatasmithElement
{
public:
	virtual ~IDatasmithTextureElement() = default;

	/** Get texture filename */
	virtual const TCHAR* GetFile() const = 0;

	/** Set texture filename */
	virtual void SetFile(const TCHAR* File) = 0;

	/** Set the output data buffer, used only when no output filename is set
	 *
	 * @param InData data to load the texture from
	 * @param InDataSize size in bytes of the buffer
	 * @param InFormat texture format(e.g. png or jpeg)
	 *
	 * @note The given data is not freed by the DatasmithImporter.
	 */
	virtual void SetData(const uint8* InData, uint32 InDataSize, EDatasmithTextureFormat InFormat) = 0;

	/** Return the optional data, if loading from memory. Must be callable from any thread. */
	virtual const uint8* GetData(uint32& OutDataSize, EDatasmithTextureFormat& OutFormat) const = 0;

	/** Return a MD5 hash of the content of the Texture Element. Used in CalculateElementHash to quickly identify Element with identical content */
	virtual FMD5Hash GetFileHash() const = 0;

	/** Set the MD5 hash of the current texture file. This should be a hash of its content. */
	virtual void SetFileHash(FMD5Hash Hash) = 0;

	/** Get texture usage */
	virtual EDatasmithTextureMode GetTextureMode() const = 0;

	/** Set texture usage */
	virtual void SetTextureMode(EDatasmithTextureMode Mode) = 0;

	/** Get texture filter */
	virtual EDatasmithTextureFilter GetTextureFilter() const = 0;

	/** Set texture filter */
	virtual void SetTextureFilter(EDatasmithTextureFilter Filter) = 0;

	/** Get texture X axis address mode */
	virtual EDatasmithTextureAddress GetTextureAddressX() const = 0;

	/** Set texture X axis address mode */
	virtual void SetTextureAddressX(EDatasmithTextureAddress Mode) = 0;

	/** Get texture Y axis address mode */
	virtual EDatasmithTextureAddress GetTextureAddressY() const = 0;

	/** Set texture Y axis address mode */
	virtual void SetTextureAddressY(EDatasmithTextureAddress Mode) = 0;

	/** Get allow texture resizing */
	virtual bool GetAllowResize() const = 0;

	/** Set allow texture resizing */
	virtual void SetAllowResize(bool bAllowResize) = 0;

	/** Get texture gamma <= 0 for auto */
	virtual float GetRGBCurve() const = 0;

	/** Set texture gamma <= 0 for auto */
	virtual void SetRGBCurve(const float InRGBCurve) = 0;

	/** Gets the color space of the texture */
	virtual EDatasmithColorSpace GetSRGB() const = 0;

	/** Sets the color space of the texture */
	virtual void SetSRGB(EDatasmithColorSpace Option) = 0;
};

class IDatasmithShaderElement : public IDatasmithElement
{
public:
	/**
	 * Realistic fresnel creates a pretty more complex node tree based on the actual fresnel equation.
	 * If this param is not enabled an approximation will be used.
	 *
	 * It has no effect if bDisableReflectionFresnel is set to true.
	*/
	static UE_API bool bUseRealisticFresnel;

	/** If it is set to true no fresnel effect is applied on reflection and just a constant effect is assigned to the reflection slot */
	static UE_API bool bDisableReflectionFresnel;

	virtual ~IDatasmithShaderElement() = default;

	/** Get the Ior N value, usually Ior K is set to 0 so this will control the entire reflection fresnel effect */
	virtual double GetIOR() const = 0;

	/** Set the Ior N value, usually Ior K is set to 0 so this will control the entire reflection fresnel effect */
	virtual void SetIOR(double Value) = 0;

	/** Get the Ior K effect, this is used for more advanced representations of the reflection fresnel effect */
	virtual double GetIORk() const = 0;

	/** Set the Ior K effect, this is used for more advanced representations of the reflection fresnel effect */
	virtual void SetIORk(double Value) = 0;

	/** Get the InIndex of Refraction value */
	virtual double GetIORRefra() const = 0;

	/** Set the InIndex of Refraction value */
	virtual void SetIORRefra(double Value) = 0;

	/** Get the bump/normal amount */
	virtual double GetBumpAmount() const = 0;

	/** Set the bump/normal amount */
	virtual void SetBumpAmount(double Value) = 0;

	/** Get the two sided material attribute */
	virtual bool GetTwoSided() const = 0;

	/** Set the two sided material attribute */
	virtual void SetTwoSided(bool Value) = 0;

	/** Get the diffuse color in linear space */
	virtual FLinearColor GetDiffuseColor() const = 0;

	/** Set the diffuse color in linear space */
	virtual void SetDiffuseColor(FLinearColor Value) = 0;

	/** Get the diffuse filename */
	virtual const TCHAR* GetDiffuseTexture() const = 0;

	/** Set the diffuse filename */
	virtual void SetDiffuseTexture(const TCHAR* Value) = 0;

	/** Get the diffuse UV coordinates */
	virtual FDatasmithTextureSampler GetDiffTextureSampler() const = 0;

	/** Set the diffuse UV coordinates */
	virtual void SetDiffTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the diffuse compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetDiffuseComp() = 0;

	/** Get the diffuse compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetDiffuseComp() const = 0;

	/** Set the diffuse compound map */
	virtual void SetDiffuseComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the reflectance color in linear space */
	virtual FLinearColor GetReflectanceColor() const = 0;

	/** Set the reflectance color in linear space */
	virtual void SetReflectanceColor(FLinearColor Value) = 0;

	/** Get the reflectance filename */
	virtual const TCHAR* GetReflectanceTexture() const = 0;

	/** Set the reflectance filename */
	virtual void SetReflectanceTexture(const TCHAR* Value) = 0;

	/** Get the reflectance UV coordinates */
	virtual FDatasmithTextureSampler GetRefleTextureSampler() const = 0;

	/** Set the reflectance UV coordinates */
	virtual void SetRefleTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the reflectance compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetRefleComp() = 0;

	/** Get the reflectance compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetRefleComp() const = 0;

	/** Set the reflectance compound map */
	virtual void SetRefleComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the roughness color in linear space */
	virtual double GetRoughness() const = 0;

	/** Set the roughness color in linear space */
	virtual void SetRoughness(double Value) = 0;

	/** Get the roughness filename */
	virtual const TCHAR* GetRoughnessTexture() const = 0;

	/** Set the roughness filename */
	virtual void SetRoughnessTexture(const TCHAR* Value) = 0;

	/** Get the roughness UV coordinates */
	virtual FDatasmithTextureSampler GetRoughTextureSampler() const = 0;

	/** Set the roughness UV coordinates */
	virtual void SetRoughTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the roughness compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetRoughnessComp() = 0;

	/** Get the roughness compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetRoughnessComp() const = 0;

	/** Set the roughness compound map */
	virtual void SetRoughnessComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the normalmapping filename */
	virtual const TCHAR* GetNormalTexture() const = 0;

	/** Set the normalmapping filename */
	virtual void SetNormalTexture(const TCHAR* Value) = 0;

	/** Get the normalmapping UV coordinates */
	virtual FDatasmithTextureSampler GetNormalTextureSampler() const = 0;

	/** Set the normalmapping UV coordinates */
	virtual void SetNormalTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the normalmapping compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetNormalComp() = 0;

	/** Get the normalmapping compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetNormalComp() const = 0;

	/** Set the normalmapping compound map */
	virtual void SetNormalComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the bumpmapping filename */
	virtual const TCHAR* GetBumpTexture() const = 0;

	/** Set the bumpmapping filename */
	virtual void SetBumpTexture(const TCHAR* Value) = 0;

	/** Get the bumpmapping UV coordinates */
	virtual FDatasmithTextureSampler GetBumpTextureSampler() const = 0;

	/** Set the bumpmapping UV coordinates */
	virtual void SetBumpTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the bumpmapping compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetBumpComp() = 0;

	/** Get the bumpmapping compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetBumpComp() const = 0;

	/** Set the bumpmapping compound map */
	virtual void SetBumpComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the transparency color in linear space */
	virtual FLinearColor GetTransparencyColor() const = 0;

	/** Set the transparency color in linear space */
	virtual void SetTransparencyColor(FLinearColor Value) = 0;

	/** Get the transparency filename */
	virtual const TCHAR* GetTransparencyTexture() const = 0;

	/** Set the transparency filename */
	virtual void SetTransparencyTexture(const TCHAR* Value) = 0;

	/** Get the transparency UV coordinates */
	virtual FDatasmithTextureSampler GetTransTextureSampler() const = 0;

	/** Set the transparency UV coordinates */
	virtual void SetTransTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the transparency compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetTransComp() = 0;

	/** Get the transparency compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetTransComp() const = 0;

	/** Set the transparency compound map */
	virtual void SetTransComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the opacity mask filename */
	virtual const TCHAR* GetMaskTexture() const = 0;

	/** Set the opacity mask filename */
	virtual void SetMaskTexture(const TCHAR* Value) = 0;

	/** Get the opacity mask UV coordinates */
	virtual FDatasmithTextureSampler GetMaskTextureSampler() const = 0;

	/** Set the opacity mask UV coordinates */
	virtual void SetMaskTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the opacity mask compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetMaskComp() = 0;

	/** Get the opacity mask compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetMaskComp() const = 0;

	/** Set the opacity mask compound map */
	virtual void SetMaskComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the metalness value */
	virtual double GetMetal() const = 0;

	/** Set the metalness value */
	virtual void SetMetal(double Value) = 0;

	/** Get the metalness filename */
	virtual const TCHAR* GetMetalTexture() const = 0;

	/** Set the metalness filename */
	virtual void SetMetalTexture(const TCHAR* Value) = 0;

	/** Get the metalness UV coordinates */
	virtual FDatasmithTextureSampler GetMetalTextureSampler() const = 0;

	/** Set the metalness UV coordinates */
	virtual void SetMetalTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the metalness compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetMetalComp() = 0;

	/** Get the metalness compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetMetalComp() const = 0;

	/** Set the metalness compound map */
	virtual void SetMetalComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/** Get the emittance color in linear space */
	virtual FLinearColor GetEmitColor() const = 0;

	/** Set the emittance color in linear space */
	virtual void SetEmitColor(FLinearColor Value) = 0;

	/** Get the emittance filename */
	virtual const TCHAR* GetEmitTexture() const = 0;

	/** Set the emittance filename */
	virtual void SetEmitTexture(const TCHAR* Value) = 0;

	/** Get the emittance UV coordinates */
	virtual FDatasmithTextureSampler GetEmitTextureSampler() const = 0;

	/** Set the emittance UV coordinates */
	virtual void SetEmitTextureSampler(FDatasmithTextureSampler Value) = 0;

	/** Get the emittance temperature color */
	virtual double GetEmitTemperature() const = 0;

	/** Set the emittance temperature color */
	virtual void SetEmitTemperature(double Value) = 0;

	/** Get the emittance power in lumens */
	virtual double GetEmitPower() const = 0;

	/** Set the emittance power in lumens */
	virtual void SetEmitPower(double Value) = 0;

	/** Get the emittance compound map */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetEmitComp() = 0;

	/** Get the emittance compound map */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetEmitComp() const = 0;

	/** Set the emittance compound map */
	virtual void SetEmitComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;


	/**
	 * Gets material is used as lighting only.
	 * If true the material sets the lighting mode to Unlit, regular lighting mode otherwise.
	 */
	virtual bool GetLightOnly() const = 0;

	/**
	 * Sets material is used as lighting only.
	 * If true the material sets the lighting mode to Unlit, regular lighting mode otherwise.
	 */
	virtual void SetLightOnly(bool Value) = 0;

	/**
	 * Get the weight color in linear space.
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual FLinearColor GetWeightColor() const = 0;

	/**
	 * Set the weight color in linear space.
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual void SetWeightColor(FLinearColor Value) = 0;

	/**
	 * Get the weight filename
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual const TCHAR* GetWeightTexture() const = 0;

	/**
	 * Set the weight filename
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual void SetWeightTexture(const TCHAR* Value) = 0;

	/**
	 * Get the weight UV coordinates
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual FDatasmithTextureSampler GetWeightTextureSampler() const = 0;

	/**
	 * Set the weight UV coordinates
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual void SetWeightTextureSampler(FDatasmithTextureSampler Value) = 0;

	/**
	 * Get the weight compound map
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetWeightComp() = 0;

	/**
	 * Get the weight compound map
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual const TSharedPtr<IDatasmithCompositeTexture>& GetWeightComp() const = 0;

	/**
	 * Set the weight compound map
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual void SetWeightComp(const TSharedPtr<IDatasmithCompositeTexture>& Value) = 0;

	/**
	 * Get the weight power value
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual double GetWeightValue() const = 0;

	/**
	 * Set the weight power value
	 * Weight color, texture and value are only used for multilayered materials.
	 */
	virtual void SetWeightValue(double Value) = 0;

	/**
	 * Get the blending mode.
	 * It only has effect on multilayered materials and all the layers but layer 0.
	 */
	virtual EDatasmithBlendMode GetBlendMode() const = 0;

	/**
	 * Set the blending mode.
	 * It only has effect on multilayered materials and all the layers but layer 0.
	 */
	virtual void SetBlendMode(EDatasmithBlendMode Value) = 0;

	/**
	 * Get the if this layer is weighted as a stack.
	 * It only has effect on multilayered materials and all the layers but layer 0.
	 */
	virtual bool GetIsStackedLayer() const = 0;

	/**
	 * Set the if this layer is weighted as a stack.
	 * It only has effect on multilayered materials and all the layers but layer 0.
	 */
	virtual void SetIsStackedLayer(bool Value) = 0;

	/** Get the domain of this shader */
	virtual const EDatasmithShaderUsage GetShaderUsage() const = 0;

	/** Set the domain of this shader */
	virtual void SetShaderUsage(EDatasmithShaderUsage InMaterialUsage) = 0;

	/** Set use Emissive for dynamic area lighting */
	virtual const bool GetUseEmissiveForDynamicAreaLighting() const = 0;

	/** Get use Emissive for dynamic area lighting */
	virtual void SetUseEmissiveForDynamicAreaLighting(bool InUseEmissiveForDynamicAreaLighting) = 0;
};

class IDatasmithCompositeTexture
{
public:
	typedef TPair<float, const TCHAR*> ParamVal;

	virtual ~IDatasmithCompositeTexture() = default;

	/**
	 * Gets the validity of the composite texture.
	 * If it returns false probably you should use the regular texture or color.
	 */
	virtual bool IsValid() const = 0;

	/** Gets the composition mode like color correction etc */
	virtual EDatasmithCompMode GetMode() const = 0;

	/** Sets the composition mode like color correction etc */
	virtual void SetMode(EDatasmithCompMode Mode) = 0;

	/** Get the number of surfaces. */
	virtual int32 GetParamSurfacesCount() const = 0;

	/**
	 * Gets texture usage.
	 * If it returns false you should use a value or a color checking GetUseColor(i).
	 */
	virtual bool GetUseTexture(int32 i) = 0;

	/** Get the filename of the i-th texture */
	virtual const TCHAR* GetParamTexture(int32 i) = 0;

	/** Sets the new texture for the index-th item */
	virtual void SetParamTexture(int32 InIndex, const TCHAR* InTexture) = 0;

	/** Get the i-th uv element */
	virtual FDatasmithTextureSampler& GetParamTextureSampler(int32 i) = 0;

	/**
	 * Gets color usage.
	 * If true color is used, else a value is used.
	 */
	virtual bool GetUseColor(int32 i) = 0;

	/** Get the i-th color in linear space */
	virtual const FLinearColor& GetParamColor(int32 i) = 0;

	/** Returns true if composite texture should be used */
	virtual bool GetUseComposite(int32 i) = 0;

	/**
	 * Get the number of value1 parameters.
	 * Some composites will use no values, other types could use only one value, and others could use two values.
	 */
	virtual int32 GetParamVal1Count() const = 0;

	/** Get the i-th Value1 parameter */
	virtual ParamVal GetParamVal1(int32 i) const = 0;

	/** Add a new Value1 parameter */
	virtual void AddParamVal1(ParamVal InParamVal) = 0;

	/**
	 * Get the number of value2 parameters.
	 * Some composites will use no values, other types could use only one value, and others could use two values.
	 */
	virtual int32 GetParamVal2Count() const = 0;

	/** Get the Value2 parameter */
	virtual ParamVal GetParamVal2(int32 i) const = 0;

	/** Add a new Value2 parameter */
	virtual void AddParamVal2(ParamVal InParamVal) = 0;

	/** Get the i-th nested composite texture */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetParamSubComposite(int32 i) = 0;

	/** Adds a new nested composite texture */
	virtual void AddSurface(const TSharedPtr<IDatasmithCompositeTexture>& SubComp) = 0;

	/** Get the amount of layer masks */
	virtual int32 GetParamMaskSurfacesCount() const = 0;

	/** Get the i-th layer mask's filename */
	virtual const TCHAR* GetParamMask(int32 i) = 0;

	/** Adds a new layer mask from its filename */
	virtual void AddMaskSurface(const TCHAR* InMask, const FDatasmithTextureSampler InMaskSampler) = 0;

	/** Get the ith layer mask's uv element */
	virtual FDatasmithTextureSampler GetParamMaskTextureSampler(int32 i) = 0;

	/** Get the ith composite texture inside this composite used as layer mask */
	virtual TSharedPtr<IDatasmithCompositeTexture>& GetParamMaskSubComposite(int32 i) = 0;

	/** Get the i-th color in linear space */
	virtual const FLinearColor& GetParamMaskColor(int32 i) const = 0;

	/** Returns true if composite texture mask should be used */
	virtual bool GetMaskUseComposite(int32 i) const = 0;

	/** Adds a new composite texture inside this composite used as layer mask */
	virtual void AddMaskSurface(const TSharedPtr<IDatasmithCompositeTexture>& MaskSubComp) = 0;

	/** Creates a new surface to be used as mask that will be used as layer inside this composite using a color in linear space. */
	virtual void AddMaskSurface(const FLinearColor& Color) = 0;

	/** Returns the string that identifies the texture element */
	virtual const TCHAR* GetBaseTextureName() const = 0;

	/** Returns the string that identifies the color element */
	virtual const TCHAR* GetBaseColName() const = 0;

	/** Returns the string that identifies the value element */
	virtual const TCHAR* GetBaseValName() const = 0;

	/** Returns the string that identifies the composite element */
	virtual const TCHAR* GetBaseCompName() const = 0;

	/**
	 * Sets the strings that identifies the different elements on this composite
	 *
	 * @param InTextureName	for plain textures
	 * @param InColorName	for color elements
	 * @param InValueName	for regular float values
	 * @param InCompName	for nested composite elements inside this composite
	 */
	virtual void SetBaseNames(const TCHAR* InTextureName, const TCHAR* InColorName, const TCHAR* InValueName, const TCHAR* InCompName) = 0;

	/** Creates a new surface that will be used as layer inside this composite using the texture filename and its uv element. */
	virtual void AddSurface(const TCHAR* Texture, FDatasmithTextureSampler TexUV) = 0;

	/** Creates a new surface that will be used as layer inside this composite using a color in linear space. */
	virtual void AddSurface(const FLinearColor& Color) = 0;

	/** Purges all the surfaces that could be used as layers inside this composite. */
	virtual void ClearSurface() = 0;
};

class IDatasmithMetaDataElement : public IDatasmithElement
{
public:
	/** Gets the Datasmith element that is associated with this meta data, if any */
	virtual const TSharedPtr< IDatasmithElement >& GetAssociatedElement() const = 0;

	/** Sets the Datasmith element that is associated with this meta data, if any */
	virtual void SetAssociatedElement(const TSharedPtr< IDatasmithElement >& Element) = 0;

	/** Get the total amount of properties in this meta data */
	virtual int32 GetPropertiesCount() const = 0;

	/** Get the property i-th of this meta data */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetProperty(int32 i) const = 0;

	/** Get a property by its name if it exists */
	virtual const TSharedPtr< IDatasmithKeyValueProperty >& GetPropertyByName(const TCHAR* Name) const = 0;

	/** Add a property to this meta data */
	virtual void AddProperty(const TSharedPtr< IDatasmithKeyValueProperty >& Property) = 0;

	/** Remove the property from this meta data */
	virtual void RemoveProperty( const TSharedPtr<IDatasmithKeyValueProperty>& Property ) = 0;

	/** Remove all properties in this meta data */
	virtual void ResetProperties() = 0;
};

class IDatasmithDecalActorElement : public IDatasmithCustomActorElement
{
public:
	/** Get the Decal element size */
	virtual FVector GetDimensions() const = 0;

	/** Set the Decal element size */
	virtual void SetDimensions(const FVector&) = 0;

	/** Get the path name of the Material associated with the actor */
	virtual const TCHAR* GetDecalMaterialPathName() const = 0;

	/**
	 * Set the path name of the Material that the Decal actor uses
	 * It can be either a package path referring to an existing material asset
	 * or a material name referring a DecalMaterialElement in the DatasmithScene
	 * If this is not a DecalMaterialElement or a material with its material domain as DeferredDecal
	 * The DecalActor generated in Unreal at import will use its default material
	 */
	virtual void SetDecalMaterialPathName(const TCHAR*) = 0;

	/** Get the order in which Decal element is rendered. */
	virtual int32 GetSortOrder() const = 0;

	/** Set the order in which decal elements are rendered.  Higher values draw later (on top). */
	virtual void SetSortOrder(int32) = 0;
};

class IDatasmithScene : public IDatasmithElement
{
public:
	virtual ~IDatasmithScene() = default;

	/** Resets all the settings on the scene */
	virtual void Reset() = 0;

	/** Returns the name of the host application which created the scene */
	virtual const TCHAR* GetHost() const = 0;

	/**
	 * Sets the name of the host application from which we're exporting from.
	 *
	 * @param InHost	The host application name
	 */
	virtual void SetHost(const TCHAR*) = 0;

	/** Returns the Datasmith format version used to export the scene */
	virtual const TCHAR* GetExporterVersion() const = 0;

	/**
	 * Sets the Datasmith version used to export the scene.
	 * Only needs to be set when parsing the .udatasmith during import
	 *
	 * @param InVersion	The Datasmith version number as string
	 */
	virtual void SetExporterVersion(const TCHAR*) = 0;

	/** Return the enterprise version of the SDK used by the exporter */
	virtual const TCHAR* GetExporterSDKVersion() const = 0;

	/**
	 * Sets the enterprise SDK version used to export the scene.
	 * Only needs to be set when parsing the .udatasmith during import
	 *
	 * @param InVersion	The Datasmith version number as string
	 */
	virtual void SetExporterSDKVersion(const TCHAR*) = 0;

	/** Returns the vendor name of the application used to export the scene */
	virtual const TCHAR* GetVendor() const = 0;

	/**
	 * Sets the vendor name of the application used to export the scene.
	 *
	 * @param InVendor	The application vendor name
	 */
	virtual void SetVendor(const TCHAR*) = 0;

	/** Returns the product name of the application used to export the scene */
	virtual const TCHAR* GetProductName() const = 0;

	/**
	 * Sets the product name of the application used to export the scene.
	 *
	 * @param InProductName	The application name
	 */
	virtual void SetProductName(const TCHAR*) = 0;

	/** Returns the product version of the application used to export the scene */
	virtual const TCHAR* GetProductVersion() const = 0;

	/**
	 * Sets the product version of the application used to export the scene.
	 *
	 * @param InProductVersion	The application version
	 */
	virtual void SetProductVersion(const TCHAR*) = 0;

	/** Returns the ';' separated list of paths where resources are stored */
	virtual const TCHAR* GetResourcePath() const = 0;

	/**
	 * Similar to how the PATH environment variable works, sets list of paths where resources can be stored.
	 *
	 * @param InResoucePath	The ';' separated list of paths
	 */
	virtual void SetResourcePath(const TCHAR*) = 0;

	/** Returns the user identifier who exported the scene */
	virtual const TCHAR* GetUserID() const = 0;

	/**
	 * Sets the user identifier who exported the scene.
	 *
	 * @param InUserID	The user identifier
	 */
	virtual void SetUserID(const TCHAR*) = 0;

	/** Returns the OS name used by user who exported the scene */
	virtual const TCHAR* GetUserOS() const = 0;

	/**
	 * Sets the user's OS name.
	 *
	 * @param InUserOS	The OS name
	 */
	virtual void SetUserOS(const TCHAR*) = 0;

	/** Get Geolocation data of the scene. Where X = Latitude, Y = Longitude, Z = Elevation
	 *   Components are initialized to TNumericLimits<double>::Max() to indicate they are not "set"
	 */
	virtual FVector GetGeolocation() const = 0; 

	virtual void SetGeolocationLatitude(double) = 0;
	virtual void SetGeolocationLongitude(double) = 0;
	virtual void SetGeolocationElevation(double) = 0;

	/** Returns the time taken to export the scene */
	virtual int32 GetExportDuration() const = 0;

	/**
	 * Sets the time taken to export the scene.
	 *
	 * @param InExportDuration	The export duration (in seconds)
	 */
	virtual void SetExportDuration(int32) = 0;

	/**
	 * Physical Sky could be generated in a large amount of modes, like material, lights etc
	 * that's why it has been added as static, just enable it and it is done.
	 * Notice that if a HDRI environment is used this gets disabled.
	 */
	virtual bool GetUsePhysicalSky() const = 0;

	/**
	 * Enable or disable the usage of Physical Sky
	 * Notice that if a HDRI environment is used this gets disabled.
	 */
	virtual void SetUsePhysicalSky(bool bInUsePhysicalSky) = 0;

	/**
	 * Adds a new Mesh to the scene.
	 *
	 *  @param InMesh	the Mesh that will be added
	 */
	virtual void AddMesh(const TSharedPtr< IDatasmithMeshElement >& InMesh) = 0;

	/** Returns the amount of meshes added to the scene */
	virtual int32 GetMeshesCount() const = 0;

	/** Returns the mesh using this index */
	virtual TSharedPtr< IDatasmithMeshElement > GetMesh(int32 InIndex) = 0;

	/** Returns the mesh using this index */
	virtual const TSharedPtr< IDatasmithMeshElement >& GetMesh(int32 InIndex) const = 0;

	/**
	 * Remove a Mesh to the scene.
	 *
	 * @param InMesh	the Mesh that will be removed
	 */
	virtual void RemoveMesh(const TSharedPtr< IDatasmithMeshElement >& InMesh) = 0;

	/**
	 * Removes from the scene the Mesh element at the specified index.
	 */
	virtual void RemoveMeshAt(int32 InIndex) = 0;

	/**
	* Remove all meshes from the scene
	*/
	virtual void EmptyMeshes() = 0;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual void AddCloth(const TSharedPtr< IDatasmithClothElement >& InElement) = 0;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual int32 GetClothesCount() const = 0;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual TSharedPtr< IDatasmithClothElement > GetCloth(int32 InIndex) = 0;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual const TSharedPtr< IDatasmithClothElement >& GetCloth(int32 InIndex) const = 0;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual void RemoveCloth(const TSharedPtr< IDatasmithClothElement >& InElement) = 0;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual void RemoveClothAt(int32 InIndex) = 0;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual void EmptyClothes() = 0;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Adds an Actor to the scene.
	 *
	 * @param InActor the Actor that will be added
	 */
	virtual void AddActor(const TSharedPtr< IDatasmithActorElement >& InActor) = 0;

	/** Returns the amount of actors added to the scene */
	virtual int32 GetActorsCount() const = 0;

	/** Returns the actor using this index */
	virtual TSharedPtr< IDatasmithActorElement > GetActor(int32 InIndex) = 0;

	/** Returns the actor using this index */
	virtual const TSharedPtr< IDatasmithActorElement >& GetActor(int32 InIndex) const = 0;

	/**
	 * Remove Actor from the scene.
	 *
	 * @param InActor the Actor that will be removed
	 */
	virtual void RemoveActor(const TSharedPtr< IDatasmithActorElement >& InActor, EDatasmithActorRemovalRule RemoveRule) = 0;

	/**
	 * Removes from the scene the Actor at the specified index.
	 */
	virtual void RemoveActorAt(int32 InIndex, EDatasmithActorRemovalRule RemoveRule) = 0;

	/**
	 * Adds a new Material to the scene (it won't be applied to any mesh).
	 *
	 * @param InMaterial the Material that will be added
	 */
	virtual void AddMaterial(const TSharedPtr< IDatasmithBaseMaterialElement >& InMaterial) = 0;

	/** Returns the amount of materials added to the scene */
	virtual int32 GetMaterialsCount() const = 0;
	virtual TSharedPtr< IDatasmithBaseMaterialElement > GetMaterial(int32 InIndex) = 0;
	virtual const TSharedPtr< IDatasmithBaseMaterialElement >& GetMaterial(int32 InIndex) const = 0;

	/**
	 * Removes a Material Element from the scene.
	 *
	 * @param InMaterial the Material Element to remove
	 */
	virtual void RemoveMaterial(const TSharedPtr< IDatasmithBaseMaterialElement >& InMaterial) = 0;

	/**
	 * Removes from the scene the Material Element at the specified index.
	 */
	virtual void RemoveMaterialAt(int32 InIndex) = 0;

	/**
	 * Remove all materials from the scene
	 */
	virtual void EmptyMaterials() = 0;

	/**
	 * Adds a new Texture Element to the scene (it won't be applied to any material).
	 *
	 * @param InTexture the Texture Element that will be added
	 */
	virtual void AddTexture(const TSharedPtr< IDatasmithTextureElement >& InTexture) = 0;

	/** Returns the amount of textures added to the scene */
	virtual int32 GetTexturesCount() const = 0;
	virtual TSharedPtr< IDatasmithTextureElement > GetTexture(int32 InIndex) = 0;
	virtual const TSharedPtr< IDatasmithTextureElement >& GetTexture(int32 InIndex) const = 0;

	/**
	 * Removes a Texture Element from the scene.
	 *
	 * @param InTexture the Texture Element that will be removed
	 */
	virtual void RemoveTexture(const TSharedPtr< IDatasmithTextureElement >& InTexture) = 0;

	/**
	 * Removes from the scene the Texture element at the specified index.
	 */
	virtual void RemoveTextureAt(int32 InIndex) = 0;

	/**
	* Remove all textures from the scene
	*/
	virtual void EmptyTextures() = 0;


	/**
	 * Set a new Postprocess for the scene
	 *
	 * @param InPostProcess the Environment that will be added
	 */
	virtual void SetPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& InPostProcess) = 0;
	virtual TSharedPtr< IDatasmithPostProcessElement > GetPostProcess() = 0;
	virtual const TSharedPtr< IDatasmithPostProcessElement >& GetPostProcess() const = 0;

	/**
	 * Add a metadata to the scene
	 * There should be only one metadata per Datasmith element (the element associated with the metadata)
	 */
	virtual void AddMetaData(const TSharedPtr< IDatasmithMetaDataElement >& InMetaData) = 0;

	virtual int32 GetMetaDataCount() const = 0;
	virtual TSharedPtr< IDatasmithMetaDataElement > GetMetaData(int32 InIndex) = 0;
	virtual const TSharedPtr< IDatasmithMetaDataElement >& GetMetaData(int32 InIndex) const = 0;
	virtual TSharedPtr< IDatasmithMetaDataElement > GetMetaData(const TSharedPtr<IDatasmithElement>& Element) = 0;
	virtual const TSharedPtr< IDatasmithMetaDataElement >& GetMetaData(const TSharedPtr<IDatasmithElement>& Element) const = 0;
	virtual void RemoveMetaData( const TSharedPtr<IDatasmithMetaDataElement>& Element ) = 0;
	virtual void RemoveMetaDataAt(int32 InIndex) = 0;

	/**
	 * Adds a level sequence to the scene.
	 *
	 * @param InSequence the level sequence to add
	 */
	virtual void AddLevelSequence(const TSharedRef< IDatasmithLevelSequenceElement >& InSequence) = 0;

	/** Returns the number of level sequences in the scene */
	virtual int32 GetLevelSequencesCount() const = 0;

	/** Returns the level sequence using this index */
	virtual TSharedPtr< IDatasmithLevelSequenceElement > GetLevelSequence(int32 InIndex) = 0;
	virtual const TSharedPtr< IDatasmithLevelSequenceElement >& GetLevelSequence(int32 InIndex) const = 0;

	/**
	 * Removes a level sequence from the scene.
	 *
	 * @param InSequence the level sequence to remove
	 */
	virtual void RemoveLevelSequence(const TSharedRef< IDatasmithLevelSequenceElement>& InSequence) = 0;

	/**
	 * Removes from the scene the level sequence at the specified index.
	 */
	virtual void RemoveLevelSequenceAt(int32 InIndex) = 0;

	/**
	 * Adds a LevelVariantSets to the scene.
	 *
	 * @param InLevelVariantSets the LevelVariantSets to add
	 */
	virtual void AddLevelVariantSets(const TSharedPtr< IDatasmithLevelVariantSetsElement >& InLevelVariantSets) = 0;

	/** Returns the number of LevelVariantSets in the scene */
	virtual int32 GetLevelVariantSetsCount() const = 0;

	/** Returns the LevelVariantSets using this index */
	virtual TSharedPtr< IDatasmithLevelVariantSetsElement > GetLevelVariantSets(int32 InIndex) = 0;
	virtual const TSharedPtr< IDatasmithLevelVariantSetsElement >& GetLevelVariantSets(int32 InIndex) const = 0;

	/**
	 * Removes a LevelVariantSets from the scene.
	 *
	 * @param InLevelVariantSets the LevelVariantSets to remove
	 */
	virtual void RemoveLevelVariantSets(const TSharedPtr< IDatasmithLevelVariantSetsElement>& InLevelVariantSets) = 0;

	/**
	 * Removes from the scene the LevelVariantSets at the specified index.
	 */
	virtual void RemoveLevelVariantSetsAt(int32 InIndex) = 0;

	/** Attach the actor to its new parent. Detach the actor if it was already attached. */
	virtual void AttachActor(const TSharedPtr< IDatasmithActorElement >& NewParent, const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule) = 0;
	/** Attach the actor to the scene root. Detach the actor if it was already attached. */
	virtual void AttachActorToSceneRoot(const TSharedPtr< IDatasmithActorElement >& Child, EDatasmithActorAttachmentRule AttachmentRule) = 0;
};


using IDatasmithMasterMaterialElement UE_DEPRECATED(5.1, "IDatasmithMasterMaterialElement will not be supported in 5.2. Please use IDatasmithMaterialInstanceElement instead.") = IDatasmithMaterialInstanceElement;

#undef UE_API
