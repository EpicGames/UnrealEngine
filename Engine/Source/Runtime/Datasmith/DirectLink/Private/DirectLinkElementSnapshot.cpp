// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkElementSnapshot.h"
#include "DirectLinkParameterStore.h"
#include "DirectLinkSceneGraphNode.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace DirectLink
{

// this constant must never change, it's used as a marker in a byte stream
static constexpr uint8 kMagic = 0xd1;

FElementSnapshot::FElementSnapshot(const ISceneGraphNode& Node)
{
	NodeId = Node.GetNodeId();

	// Data part
	{
		// Static property serialization: nodes can use a store that describes parameters
		const FParameterStore& Store = Node.GetStore();
		DataSnapshot = Store.Snapshot();
	}

	{
		// Custom property serialization: let nodes dynamically add properties
		bool bIsSaving = true;
		FSnapshotProxy Ar(DataSnapshot, bIsSaving);
		Node.CustomSerialize(Ar);
	}

	// Reference part
	const int32 ProxyGroupCount = Node.GetReferenceProxyCount();
	RefSnapshot.Groups.Reserve(ProxyGroupCount);
	for (int32 ProxyIndex = 0; ProxyIndex < ProxyGroupCount; ++ProxyIndex)
	{
		const IReferenceProxy* RefProxy = Node.GetReferenceProxy(ProxyIndex);
		int32 ReferenceCount = RefProxy->Num();

		// Note: It is not an option to skip empty groups here.
		// An empty array is a legitimate value like any other, and must be serialized and sent to the remote.
		// (If we skip empty groups, updates of references will fail to remove references on the remote graph)

		FReferenceSnapshot::FReferenceGroup& ReferenceGroup = RefSnapshot.Groups.AddDefaulted_GetRef();
		ReferenceGroup.Name = Node.GetReferenceProxyName(ProxyIndex);
		ReferenceGroup.ReferencedIds.Reserve(ReferenceCount);

		for (int32 ReferenceIndex = 0; ReferenceIndex < ReferenceCount; ReferenceIndex++)
		{
			if (ISceneGraphNode* Referenced = RefProxy->GetNode(ReferenceIndex))
			{
				ReferenceGroup.ReferencedIds.Add(Referenced->GetNodeId());
			}
		}
	}
}


void FReferenceSnapshot::Serialize(FArchive& Ar)
{
	// Note: Changes to this implementation impacts version handling
	// see DirectLink::kCurrentProtocolVersion and DirectLink::kMinSupportedProtocolVersion
	if (Ar.IsSaving())
	{
		uint32 N = Groups.Num();
		Ar.SerializeIntPacked(N);
		for (FReferenceSnapshot::FReferenceGroup& Group : Groups)
		{
			Ar << Group.Name;
			Ar << Group.ReferencedIds;
		}
	}
	else
	{
		uint32 N = 0;
		Ar.SerializeIntPacked(N);
		Groups.Reserve(N);

		for (uint32 i = 0; i < N; ++i)
		{
			FReferenceSnapshot::FReferenceGroup& Group = Groups.AddDefaulted_GetRef();
			Ar << Group.Name;
			Ar << Group.ReferencedIds;
		}
	}
}

DirectLink::FElementHash FReferenceSnapshot::Hash() const
{
	FElementHash RunningHash = 0;
	for (const FReferenceGroup& RefGroup : Groups)
	{
		const uint8* Buffer = reinterpret_cast<const uint8*>(RefGroup.ReferencedIds.GetData());
		uint32 ByteCount = RefGroup.ReferencedIds.Num() * sizeof(decltype(RefGroup.ReferencedIds)::ElementType);
		RunningHash = FCrc::MemCrc32(Buffer, ByteCount, RunningHash);
	}
	return RunningHash;
}

ESerializationStatus FElementSnapshot::Serialize(FArchive& Ar)
{
	uint8 Magic = kMagic;
	uint8 SerialVersion = GetCurrentProtocolVersion();

	if (Ar.IsSaving())
	{
		Ar << Magic;
		Ar << SerialVersion;
		Ar << (int32&)NodeId;
		DataSnapshot.SerializeAll(Ar);
		RefSnapshot.Serialize(Ar);
		Ar << Magic;
	}
	else
	{
		Ar << Magic;
		if (!ensure(Magic == kMagic))
		{
			return ESerializationStatus::StreamError;
		}

		Ar << SerialVersion;
		if (SerialVersion > GetCurrentProtocolVersion())
		{
			return ESerializationStatus::VersionMaxNotRespected;
		}
		if (SerialVersion < GetMinSupportedProtocolVersion())
		{
			return ESerializationStatus::VersionMinNotRespected;
		}

		Ar << (int32&)NodeId;
		DataSnapshot.SerializeAll(Ar);
		RefSnapshot.Serialize(Ar);

		Ar << Magic;
		if (!ensure(Magic == kMagic))
		{
			return ESerializationStatus::StreamError;
		}
	}

	return ESerializationStatus::Ok;
}

FElementHash FElementSnapshot::GetHash() const
{
	return GetDataHash() ^ GetRefHash();
}

FElementHash FElementSnapshot::GetDataHash() const
{
	if (DataHash == InvalidHash)
	{
		DataHash = DataSnapshot.Hash();
	}
	return DataHash;
}

FElementHash FElementSnapshot::GetRefHash() const
{
	if (RefHash == InvalidHash)
	{
		RefHash = RefSnapshot.Hash();
	}
	return RefHash;
}


void FElementSnapshot::UpdateNodeReferences(IReferenceResolutionProvider& Resolver, ISceneGraphNode& Node) const
{
	Node.UpdateRefs(Resolver, RefSnapshot);
}


void FElementSnapshot::UpdateNodeData(ISceneGraphNode& Node) const
{
	Node.GetStore().Update(DataSnapshot);

	bool bIsSaving = false;
	FSnapshotProxy Ar(const_cast<FParameterStoreSnapshot&>(DataSnapshot), bIsSaving);
	Node.CustomSerialize(Ar);
}


} // namespace DirectLink

