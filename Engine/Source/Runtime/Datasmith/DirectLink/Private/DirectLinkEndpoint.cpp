// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkEndpoint.h"

#include "DirectLinkConnectionRequestHandler.h"
#include "DirectLinkLog.h"
#include "DirectLinkMessages.h"
#include "DirectLinkSceneGraphNode.h"
#include "DirectLinkStreamConnectionPoint.h"
#include "DirectLinkStreamDescription.h"
#include "DirectLinkStreamDestination.h"
#include "DirectLinkStreamReceiver.h"
#include "DirectLinkStreamSender.h"
#include "DirectLinkStreamSource.h"

#include "Async/Async.h"
#include "MessageEndpointBuilder.h"

namespace DirectLink
{

struct
{
	// heartbeat message periodically sent to keep the connections alive
	double HeartbeatThreshold_s                = 5.0;

	// endpoint not seen for a long time:
	bool   bPeriodicalyCleanupTimedOutEndpoint = true;
	double ThresholdEndpointCleanup_s          = 30.0;
	double CleanupOldEndpointPeriod_s          = 10.0;

	// auto connect streams by name
	bool bAutoconnectFromSources               = true;
	bool bAutoconnectFromDestination           = false;
} gConfig;


double gUdpMessagingInitializationTime = -1.;
ECommunicationStatus ValidateCommunicationStatus()
{
	if (!FModuleManager::Get().IsModuleLoaded("UdpMessaging"))
	{
		gUdpMessagingInitializationTime = FPlatformTime::Seconds();
	}
	return (FModuleManager::Get().LoadModule("Messaging")    ? ECommunicationStatus::NoIssue : ECommunicationStatus::ModuleNotLoaded_Messaging)
		 | (FModuleManager::Get().LoadModule("UdpMessaging") ? ECommunicationStatus::NoIssue : ECommunicationStatus::ModuleNotLoaded_UdpMessaging)
		 | (FModuleManager::Get().LoadModule("Networking")   ? ECommunicationStatus::NoIssue : ECommunicationStatus::ModuleNotLoaded_Networking);
}


class FSharedState
{
public:
	FSharedState(const FString& NiceName) : NiceName(NiceName) {}

	mutable FRWLock SourcesLock;
	TArray<TSharedPtr<FStreamSource>> Sources;
	std::atomic<bool> bDirtySources{false};

	mutable FRWLock DestinationsLock;
	TArray<TSharedPtr<FStreamDestination>> Destinations;
	std::atomic<bool> bDirtyDestinations{false};

	mutable FRWLock StreamsLock;
	FStreamPort StreamPortIdGenerator = InvalidStreamPort;
	TArray<FStreamDescription> Streams; // map streamportId -> stream ? array of N ports ?

	// cleared on inner thread loop start
	mutable FRWLock ObserversLock;
	TArray<IEndpointObserver*> Observers;

	mutable FRWLock RawInfoCopyLock;
	FRawInfo RawInfo;

	std::atomic<bool> bInnerThreadShouldRun{false};
	bool bDebugLog = false;
	const FString NiceName; // not locked (wrote once)
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	FStreamDescription* GetStreamByLocalPort(FStreamPort LocalPort, const FRWScopeLock& _);
	void CloseStreamInternal(FStreamDescription& Stream, const FRWScopeLock& _, bool bNotifyRemote=true);
};


/** Inner thread allows async network communication, which avoids user thread to be locked on every sync. */
class FInternalThreadState
{
public:
	FInternalThreadState(FEndpoint& Owner, FSharedState& SharedState) : Owner(Owner), SharedState(SharedState) {}
	bool Init(); // once, any thread
	void Run(); // once, blocking, inner thread only

	FEvent* InnerThreadEvent = nullptr;
	TFuture<void> InnerThreadResult; // allow to join() in the dtr

private:
	void Handle_DeltaMessage(const FDirectLinkMsg_DeltaMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_HaveListMessage(const FDirectLinkMsg_HaveListMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_EndpointLifecycle(const FDirectLinkMsg_EndpointLifecycle& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_QueryEndpointState(const FDirectLinkMsg_QueryEndpointState& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_EndpointState(const FDirectLinkMsg_EndpointState& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_OpenStreamRequest(const FDirectLinkMsg_OpenStreamRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_OpenStreamAnswer(const FDirectLinkMsg_OpenStreamAnswer& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void Handle_CloseStreamRequest(const FDirectLinkMsg_CloseStreamRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Check if a received message is sent by 'this' endpoint.
		* Can be useful to skip handling of own messages. Makes sense in handlers of subscribed messages.
		* @param MaybeRemoteAddress Address of the sender (see Context->GetSender())
		* @returns whether given address is this address */
	bool IsMine(const FMessageAddress& MaybeRemoteAddress) const;

	/**
		* Check if the given address is an incompatible endpoint
		*/
	bool IsIgnoredEndpoint(const FMessageAddress& Address) const;

	/** Note on state replication:
		* On local state edition (eg. when a source is added) the new state is broadcasted.
		* On top of that, the state revision is broadcasted on heartbeats every few seconds.
		* This allow other endpoint to detect when a replicated state is no longer valid, and query an update.
		* This covers all failure case, and is lightweight as only the revision number is frequently broadcasted. */
	void ReplicateState(const FMessageAddress& RemoteEndpointAddress) const;
	void ReplicateState_Broadcast() const;

	FString ToString_dbg() const;

	void UpdateSourceDescription();
	void UpdateDestinationDescription();

	TUniquePtr<IStreamReceiver> MakeReceiver(FGuid SourceGuid, FGuid DestinationGuid, FMessageAddress RemoteAddress, FStreamPort RemotePort);
	TSharedPtr<IStreamSender> MakeSender(FGuid SourceGuid, FMessageAddress RemoteAddress, FStreamPort RemotePort);

	void RemoveEndpoint(const FMessageAddress& RemoteEndpointAddress);
	void MarkRemoteAsSeen(const FMessageAddress& RemoteEndpointAddress);
	void CleanupTimedOutEndpoint();

private:
	FEndpoint& Owner;
	FSharedState& SharedState;

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	TMap<FMessageAddress, FDirectLinkMsg_EndpointState> RemoteEndpointDescriptions;
	FDirectLinkMsg_EndpointState ThisDescription;

	// state replication
	double Now_s = 0;
	double LastHeartbeatTime_s = 0;
	double LastEndpointCleanupTime_s = 0;
	mutable uint32 LastBroadcastedStateRevision = 0;
	TMap<FMessageAddress, double> RemoteLastSeenTime;
	TSet<FMessageAddress> IgnoredEndpoints;
};


FEndpoint::FEndpoint(const FString& InName)
	: SharedStatePtr(MakeUnique<FSharedState>(InName))
	, SharedState(*SharedStatePtr)
	, InternalPtr(MakeUnique<FInternalThreadState>(*this, SharedState))
	, Internal(*InternalPtr)
{
	if (!FPlatformProcess::SupportsMultithreading())
	{
		UE_LOGF(LogDirectLinkNet, Error, "Endpoint '%ls': Unable to start endpoint: support for threads is required for DirectLink.", *SharedState.NiceName);
		return;
	}

	ECommunicationStatus ComStatus = ValidateCommunicationStatus();
	if (ComStatus != ECommunicationStatus::NoIssue)
	{
		UE_LOGF(LogDirectLinkNet, Error, "Endpoint '%ls': Unable to start communication (error code:%d):", *SharedState.NiceName, int(ComStatus));
		return;
	}

	if (Internal.Init())
	{
		UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls' Start internal thread", *SharedState.NiceName);

		Internal.InnerThreadEvent = FPlatformProcess::GetSynchEventFromPool();
		Internal.InnerThreadResult = Async(EAsyncExecution::Thread,
			[&, this]
			{
				FPlatformProcess::SetThreadName(TEXT("DirectLink"));
				Internal.Run();
			}
		);
	}
}


void FEndpoint::SetVerbose(bool bVerbose)
{
	SharedState.bDebugLog = bVerbose;
}


FEndpoint::~FEndpoint()
{
	UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls' closing...", *SharedState.NiceName);
	SharedState.bInnerThreadShouldRun = false;
	if (Internal.InnerThreadEvent)
	{
		Internal.InnerThreadEvent->Trigger();
		Internal.InnerThreadResult.Get(); // join
		FPlatformProcess::ReturnSynchEventToPool(Internal.InnerThreadEvent);
	}

	UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls' closed", *SharedState.NiceName);
}


FSourceHandle FEndpoint::AddSource(const FString& Name, EVisibility Visibility)
{
	FGuid Id;
	{
		FRWScopeLock _(SharedState.SourcesLock, SLT_Write);
		TSharedPtr<FStreamSource>& NewSource = SharedState.Sources.Add_GetRef(MakeShared<FStreamSource>(Name, Visibility));
		Id = NewSource->GetId();
	}

	SharedState.bDirtySources = true;

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls': Source added '%ls'", *SharedState.NiceName, *Name);
	return Id;
}


void FEndpoint::RemoveSource(const FSourceHandle& SourceId)
{
	{
		// first remove linked streams
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		for (FStreamDescription& Stream : SharedState.Streams)
		{
			if (Stream.SourcePoint == SourceId
			 && Stream.Status != EStreamConnectionState::Closed)
			{
				SharedState.CloseStreamInternal(Stream, _);
			}
		}
	}

	int32 RemovedCount = 0;
	{
		FRWScopeLock _(SharedState.SourcesLock, SLT_Write);
		RemovedCount = SharedState.Sources.RemoveAll([&](const auto& Source){return Source->GetId() == SourceId;});
	}

	if (RemovedCount)
	{
		SharedState.bDirtySources = true;
	}
}


void FEndpoint::SetSourceRoot(const FSourceHandle& SourceId, ISceneGraphNode* InRoot, bool bSnapshot)
{
	{
		FRWScopeLock _(SharedState.SourcesLock, SLT_Write);
		for (TSharedPtr<FStreamSource>& Source : SharedState.Sources) // #ue_directlink_cleanup: readonly on array, write on specific source lock
		{
			if (Source->GetId() == SourceId)
			{
				Source->SetRoot(InRoot);

				break;
			}
		}
	}

	if (bSnapshot)
	{
		SnapshotSource(SourceId);
	}
}


void FEndpoint::SnapshotSource(const FSourceHandle& SourceId)
{
	{
		FRWScopeLock _(SharedState.SourcesLock, SLT_Write); // make this a read, and detailed thread-safety inside the source
		for (TSharedPtr<FStreamSource>& Source : SharedState.Sources)
		{
			if (Source->GetId() == SourceId)
			{
				Source->Snapshot();
				break;
			}
		}
	}
}


FDestinationHandle FEndpoint::AddDestination(const FString& Name, EVisibility Visibility, const TSharedPtr<IConnectionRequestHandler>& Provider)
{
	FDestinationHandle Id;
	if (ensure(Provider.IsValid()))
	{
		FRWScopeLock _(SharedState.DestinationsLock, SLT_Write);
		TSharedPtr<FStreamDestination>& NewDest = SharedState.Destinations.Add_GetRef(MakeShared<FStreamDestination>(Name, Visibility, Provider));
		Id = NewDest->GetId();
		SharedState.bDirtyDestinations = true;
	}

	return Id;
}


void FEndpoint::RemoveDestination(const FDestinationHandle& Destination)
{
	{
		// first close associated streams
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		for (FStreamDescription& Stream : SharedState.Streams)
		{
			if (Stream.DestinationPoint == Destination
			 && Stream.Status != EStreamConnectionState::Closed)
			{
				SharedState.CloseStreamInternal(Stream, _);
			}
		}
	}

	int32 RemovedCount = 0;
	{
		FRWScopeLock _(SharedState.DestinationsLock, SLT_Write);
		RemovedCount = SharedState.Destinations.RemoveAll([&](const auto& Dest){return Dest->GetId() == Destination;});
	}

	if (RemovedCount)
	{
		SharedState.bDirtyDestinations = true;
	}
}


FRawInfo FEndpoint::GetRawInfoCopy() const
{
	FRWScopeLock _(SharedState.RawInfoCopyLock, SLT_ReadOnly);
	return SharedState.RawInfo;
}


void FEndpoint::AddEndpointObserver(IEndpointObserver* Observer)
{
	if (Observer)
	{
		FRWScopeLock _(SharedState.ObserversLock, SLT_Write);
		SharedState.Observers.AddUnique(Observer);
	}
}


void FEndpoint::RemoveEndpointObserver(IEndpointObserver* Observer)
{
	FRWScopeLock _(SharedState.ObserversLock, SLT_Write);
	SharedState.Observers.RemoveSwap(Observer);
}


FEndpoint::EOpenStreamResult FEndpoint::OpenStream(const FSourceHandle& SourceId, const FDestinationHandle& DestinationId)
{
	// #ue_directlink: should be an async api
	// #ue_directlink_cleanup Merge with Handle_OpenStreamRequest
	// #ue_directlink_syncprotocol tempo before next allowed request ?

	// check if the stream is already opened
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_ReadOnly);
		for (const FStreamDescription& Stream : SharedState.Streams)
		{
			if (Stream.SourcePoint == SourceId && Stream.DestinationPoint == DestinationId)
			{
				if (Stream.Status == EStreamConnectionState::Active
				 || Stream.Status == EStreamConnectionState::RequestSent)
				{
					// useless case, temp because of the unfinished connection policy.
					// #ue_directlink_connexion Replace with proper policy (user driven connection map) + log if this happen
					return EOpenStreamResult::AlreadyOpened;
				}
			}
		}
	}

	bool bRequestFromSource = false;
	bool bRequestFromDestination = false;
	{
		FRWScopeLock _(SharedState.SourcesLock, SLT_ReadOnly);
		for (TSharedPtr<FStreamSource>& Source : SharedState.Sources)
		{
			if (Source->GetId() == SourceId)
			{
				bRequestFromSource = true;
				break;
			}
		}
	}
	if (!bRequestFromSource)
	{
		// make sure we have the destination
		FRWScopeLock _(SharedState.DestinationsLock, SLT_ReadOnly);
		for (TSharedPtr<FStreamDestination>& Destination : SharedState.Destinations)
		{
			if (Destination->GetId() == DestinationId)
			{
				bRequestFromDestination = true;
				break;
			}
		}
	}

	if (!bRequestFromSource && !bRequestFromDestination)
	{
		// we don't have any side of the connection...
		UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls': Cannot open stream: no source or destination point found.", *SharedState.NiceName);
		return EOpenStreamResult::SourceAndDestinationNotFound;
	}

	if (bRequestFromSource && bRequestFromDestination)
	{
		UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls': Cannot open stream: have source and destination.", *SharedState.NiceName);
		return EOpenStreamResult::Unsuppported;
	}

	// Find Remote address.
	FMessageAddress RemoteAddress;
	{
		const FGuid& RemoteDataPointId = bRequestFromSource ? DestinationId : SourceId;
		// #ue_directlink_cleanup sad that we rely on raw info.
		FRWScopeLock _(SharedState.RawInfoCopyLock, SLT_ReadOnly);
		if (FRawInfo::FDataPointInfo* DataPointInfo = SharedState.RawInfo.DataPointsInfo.Find(RemoteDataPointId))
		{
			if (DataPointInfo->bIsPublic)
			{
				RemoteAddress = DataPointInfo->EndpointAddress;
			}
			else
			{
				UE_LOGF(LogDirectLinkNet, Warning, "Endpoint '%ls': Cannot open stream: Remote connection Point is private.", *SharedState.NiceName);
				return EOpenStreamResult::CannotConnectToPrivate;
			}
		}
	}

	if (RemoteAddress.IsValid())
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		FStreamPort StreamPort = ++SharedState.StreamPortIdGenerator;

		FDirectLinkMsg_OpenStreamRequest* Request = FMessageEndpoint::MakeMessage<FDirectLinkMsg_OpenStreamRequest>();
		Request->bRequestFromSource = bRequestFromSource;
		Request->RequestFromStreamPort = StreamPort;
		Request->SourceGuid = SourceId;
		Request->DestinationGuid = DestinationId;

		UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls': Send FDirectLinkMsg_OpenStreamRequest", *SharedState.NiceName);
		SharedState.MessageEndpoint->Send(Request, RemoteAddress);

		FStreamDescription& NewStream = SharedState.Streams.AddDefaulted_GetRef();
		NewStream.bThisIsSource = bRequestFromSource;
		NewStream.SourcePoint = SourceId;
		NewStream.DestinationPoint = DestinationId;
		NewStream.LocalStreamPort = StreamPort;
		NewStream.RemoteAddress = RemoteAddress;
		NewStream.Status = EStreamConnectionState::RequestSent;
	}
	else
	{
		UE_LOGF(LogDirectLinkNet, Error, "Connection Request failed: no recipent found");
		return EOpenStreamResult::RemoteEndpointNotFound;
	}
	return EOpenStreamResult::Opened;
}


void FEndpoint::CloseStream(const FSourceHandle& SourceId, const FDestinationHandle& DestinationId)
{
	FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
	for (FStreamDescription& Stream : SharedState.Streams)
	{
		if (Stream.SourcePoint == SourceId
		 && Stream.DestinationPoint == DestinationId
		 && Stream.Status != EStreamConnectionState::Closed)
		{
			SharedState.CloseStreamInternal(Stream, _);
		}
	}
}


FString FInternalThreadState::ToString_dbg() const
{
	FString Out;
	{
		Out.Appendf(TEXT("Endpoint '%s' (%s):\n"), *SharedState.NiceName, *MessageEndpoint->GetAddress().ToString());
	}

	auto PrintEndpoint = [&](const FDirectLinkMsg_EndpointState& Endpoint, int32 RemoteEndpointIndex)
	{
		Out.Appendf(TEXT("-- endpoint #%d %s/%d:'%s' \n"),
			RemoteEndpointIndex,
			*Endpoint.ComputerName,
			Endpoint.ProcessId,
			*Endpoint.NiceName
		);

		Out.Appendf(TEXT("-- %d Sources:\n"), Endpoint.Sources.Num());
		int32 SrcIndex = 0;
		for (const FNamedId& Src : Endpoint.Sources)
		{
			Out.Appendf(TEXT("--- Source #%d: '%s' (%08X) %s\n"), SrcIndex, *Src.Name, Src.Id.A,
				Src.bIsPublic ? TEXT("public"):TEXT("private"));
			SrcIndex++;
		}

		Out.Appendf(TEXT("-- %d Destinations:\n"), Endpoint.Destinations.Num());
		int32 DestinationIndex = 0;
		for (const FNamedId& Dest : Endpoint.Destinations)
		{
			Out.Appendf(TEXT("--- Dest #%d: '%s' (%08X) %s\n"), DestinationIndex, *Dest.Name, Dest.Id.A,
				Dest.bIsPublic ? TEXT("public"):TEXT("private"));
			DestinationIndex++;
		}
	};

	Out.Appendf(TEXT("- This:\n"));
	PrintEndpoint(ThisDescription, 0);

	Out.Appendf(TEXT("- Remotes:\n"));
	int32 RemoteEndpointIndex = 0;
	for (const auto& KeyValue : RemoteEndpointDescriptions)
	{
		const FDirectLinkMsg_EndpointState& Remote = KeyValue.Value;
		PrintEndpoint(Remote, RemoteEndpointIndex);
		RemoteEndpointIndex++;
	}

	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_ReadOnly);
		Out.Appendf(TEXT("- %d Streams:\n"), SharedState.Streams.Num());
		for (const FStreamDescription& Stream : SharedState.Streams)
		{
			FGuid LocalPoint = Stream.bThisIsSource ? Stream.SourcePoint : Stream.DestinationPoint;
			FGuid RemotePoint = Stream.bThisIsSource ? Stream.DestinationPoint : Stream.SourcePoint;
			const TCHAR* OrientationText = Stream.bThisIsSource ? TEXT(">>>") : TEXT("<<<");
			const TCHAR* StatusText = TEXT("?"); //Stream.stabThisIsSource ? 'S' : 'D';
			switch (Stream.Status)
			{
				case EStreamConnectionState::Uninitialized: StatusText = TEXT("Uninitialized"); break;
				case EStreamConnectionState::RequestSent:   StatusText = TEXT("RequestSent  "); break;
				case EStreamConnectionState::Active:        StatusText = TEXT("Active       "); break;
				case EStreamConnectionState::Closed:        StatusText = TEXT("Closed       "); break;
			}
			Out.Appendf(TEXT("-- [%s] stream: %08X:%d %s %08X:%d\n"), StatusText, LocalPoint.A, Stream.LocalStreamPort, OrientationText, RemotePoint.A, Stream.RemoteStreamPort);
		}
	}

	return Out;
}


void FInternalThreadState::Handle_DeltaMessage(const FDirectLinkMsg_DeltaMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		// #ue_directlink_cleanup read array, lock specific stream ? TArray<TUniquePtr<>> ?
		// -> decorelate streams descriptions from actualr sender receiver

		FStreamDescription* StreamPtr = SharedState.GetStreamByLocalPort(Message.DestinationStreamPort, _);
		if (StreamPtr == nullptr)
		{
			UE_LOGF(LogDirectLinkNet, Warning, "Endpoint '%ls': Dropped delta message (no stream at port %d)", *SharedState.NiceName, Message.DestinationStreamPort);
			return;
		}

		FStreamDescription& Stream = *StreamPtr;
		bool bIsActive = Stream.Status == EStreamConnectionState::Active;
		bool bIsReceiver = Stream.Receiver.IsValid();
		bool bIsExpectedSender = Stream.RemoteAddress == Context->GetSender();
		if (!bIsActive || !bIsReceiver || !bIsExpectedSender)
		{
			UE_LOGF(LogDirectLinkNet, Warning, "Endpoint '%ls': Dropped delta message (inactive stream used on port %d)", *SharedState.NiceName, Message.DestinationStreamPort);
			return;
		}

		FDirectLinkMsg_DeltaMessage StolenMessage = MoveTemp(const_cast<FDirectLinkMsg_DeltaMessage&>(Message));
		Stream.Receiver->HandleDeltaMessage(StolenMessage);
	}
}


void FInternalThreadState::Handle_HaveListMessage(const FDirectLinkMsg_HaveListMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);

		FStreamDescription* StreamPtr = SharedState.GetStreamByLocalPort(Message.SourceStreamPort, _);
		if (StreamPtr == nullptr)
		{
			UE_LOGF(LogDirectLinkNet, Warning, "Endpoint '%ls': Dropped havelist message (no stream at port %d)", *SharedState.NiceName, Message.SourceStreamPort);
			return;
		}

		FStreamDescription& Stream = *StreamPtr;
		bool bIsActive = Stream.Status == EStreamConnectionState::Active;
		bool bIsSender = Stream.Sender.IsValid();
		bool bIsExpectedRemote = Stream.RemoteAddress == Context->GetSender();
		if (!bIsActive || !bIsSender || !bIsExpectedRemote)
		{
			UE_LOGF(LogDirectLinkNet, Warning, "Endpoint '%ls': Dropped havelist message (inactive stream used on port %d)", *SharedState.NiceName, Message.SourceStreamPort);
			return;
		}

		Stream.Sender->HandleHaveListMessage(Message);
	}
}


void FInternalThreadState::Handle_EndpointLifecycle(const FDirectLinkMsg_EndpointLifecycle& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const FMessageAddress& RemoteEndpointAddress = Context->GetSender();

	if (IsMine(RemoteEndpointAddress) || IsIgnoredEndpoint(RemoteEndpointAddress))
	{
		return;
	}

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "Endpoint '%ls': Handle_EndpointLifecycle", *SharedState.NiceName);

	MarkRemoteAsSeen(RemoteEndpointAddress);
	switch (Message.LifecycleState)
	{
		case FDirectLinkMsg_EndpointLifecycle::ELifecycle::Start:
		{
			// Noop: remote endpoint will broadcast it's state later
			ReplicateState(RemoteEndpointAddress);
			break;
		}

		case FDirectLinkMsg_EndpointLifecycle::ELifecycle::Heartbeat:
		{
			// #ue_directlink_streams handle connection loss, threshold, and last_message_time.
			// if now-last_message_time > threshold -> mark as dead

			FDirectLinkMsg_EndpointState* RemoteState = RemoteEndpointDescriptions.Find(RemoteEndpointAddress);

			bool bIsUpToDate = RemoteState
				&& RemoteState->StateRevision != 0
				&& RemoteState->StateRevision == Message.EndpointStateRevision;

			if (!bIsUpToDate)
			{
				UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls': Send FDirectLinkMsg_QueryEndpointState", *SharedState.NiceName);
				MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FDirectLinkMsg_QueryEndpointState>(), RemoteEndpointAddress);
			}
			break;
		}

		case FDirectLinkMsg_EndpointLifecycle::ELifecycle::Stop:
		{
			RemoveEndpoint(RemoteEndpointAddress);
			break;
		}
	}
}


void FInternalThreadState::Handle_QueryEndpointState(const FDirectLinkMsg_QueryEndpointState& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	ReplicateState(Context->GetSender());
}


void FInternalThreadState::Handle_EndpointState(const FDirectLinkMsg_EndpointState& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const FMessageAddress& RemoteEndpointAddress = Context->GetSender();
	if (IsMine(RemoteEndpointAddress) || IsIgnoredEndpoint(RemoteEndpointAddress))
	{
		return;
	}

	// check protocol compatibility
	if (GetMinSupportedProtocolVersion() > Message.ProtocolVersion
		|| GetCurrentProtocolVersion() < Message.MinProtocolVersion)
	{
		bool bAlreadyIn = false;
		IgnoredEndpoints.Add(RemoteEndpointAddress, &bAlreadyIn);
		UE_CLOGF(!bAlreadyIn, LogDirectLinkNet, Warning, "Endpoint '%ls': Remote Endpoint %ls ignored, incompatible protocol versions. Supported: [%d..%d], Remote [%d..%d]"
			, *SharedState.NiceName, *Message.NiceName
			, GetMinSupportedProtocolVersion(), GetCurrentProtocolVersion()
			, Message.MinProtocolVersion, Message.ProtocolVersion
		);
		return;
		// #ue_directlink_design We could have a fancier handling than a simple 'ghosting'. UI could eg display a grayed out endpoint.
	}

	FDirectLinkMsg_EndpointState& RemoteState = RemoteEndpointDescriptions.FindOrAdd(RemoteEndpointAddress);
	RemoteState = Message;
	MarkRemoteAsSeen(RemoteEndpointAddress);

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls' Handle_EndpointState", *SharedState.NiceName);
	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Log, "%ls", *ToString_dbg());
}


void FInternalThreadState::Handle_OpenStreamRequest(const FDirectLinkMsg_OpenStreamRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	// #ue_directlink_cleanup refuse connection if local connection point is private
	// #ue_directlink_cleanup endpoint messages should be flagged Reliable

	const FMessageAddress& RemoteEndpointAddress = Context->GetSender();

	FDirectLinkMsg_OpenStreamAnswer* Answer = FMessageEndpoint::MakeMessage<FDirectLinkMsg_OpenStreamAnswer>();
	Answer->RecipientStreamPort = Message.RequestFromStreamPort;

	auto DenyConnection = [&](const FString& Reason)
	{
		Answer->bAccepted = false;
		Answer->Error = TEXT("connection already active"); // #ue_directlink_cleanup merge with OpenStream, and enum to text
		UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls': Refused connection: %ls", *SharedState.NiceName, *Reason);
		MessageEndpoint->Send(Answer, RemoteEndpointAddress);
	};

	if (IsIgnoredEndpoint(RemoteEndpointAddress))
	{
		DenyConnection(TEXT("Request from incompatible endpoint"));
		return;
	}

	// first, check if that stream is already opened
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		for (FStreamDescription& Stream : SharedState.Streams)
		{
			if (Stream.SourcePoint == Message.SourceGuid && Stream.DestinationPoint == Message.DestinationGuid)
			{
				// #ue_directlink_cleanup implement a robust handling of duplicated connections, reopened connections, etc...
				if (Stream.Status == EStreamConnectionState::Active)
				{
					DenyConnection(TEXT("Connection already active"));
					return;
				}
			}
		}
	}

	TUniquePtr<IStreamReceiver> NewReceiver;
	TSharedPtr<IStreamSender> NewSender;
	if (Message.bRequestFromSource)
	{
		NewReceiver = MakeReceiver(Message.SourceGuid, Message.DestinationGuid, RemoteEndpointAddress, Message.RequestFromStreamPort);
	}
	else
	{
		NewSender = MakeSender(Message.SourceGuid, RemoteEndpointAddress, Message.RequestFromStreamPort);
	}


	Answer->bAccepted = NewSender.IsValid() || NewReceiver.IsValid();
	if (!Answer->bAccepted)
	{
		DenyConnection(TEXT("Unknown"));
	}
	else
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		FStreamPort StreamPort = ++SharedState.StreamPortIdGenerator;
		Answer->OpenedStreamPort = StreamPort;
		FStreamDescription& NewStream = SharedState.Streams.AddDefaulted_GetRef();
		NewStream.bThisIsSource = !Message.bRequestFromSource;
		NewStream.SourcePoint = Message.SourceGuid;
		NewStream.DestinationPoint = Message.DestinationGuid;
		NewStream.RemoteAddress = RemoteEndpointAddress;
		NewStream.RemoteStreamPort = Message.RequestFromStreamPort;
		NewStream.LocalStreamPort = StreamPort;
		NewStream.Sender = MoveTemp(NewSender);
		NewStream.Receiver = MoveTemp(NewReceiver);
		NewStream.Status = EStreamConnectionState::Active;

		UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls': Accepted connection", *SharedState.NiceName);
		MessageEndpoint->Send(Answer, RemoteEndpointAddress);
	}

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "Endpoint '%ls': Handle_OpenStreamRequest", *SharedState.NiceName);
	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "%ls", *ToString_dbg());
}


void FInternalThreadState::Handle_OpenStreamAnswer(const FDirectLinkMsg_OpenStreamAnswer& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "Endpoint '%ls': Handle_OpenStreamAnswer", *SharedState.NiceName);
	const FMessageAddress& RemoteEndpointAddress = Context->GetSender();

	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		if (FStreamDescription* StreamPtr = SharedState.GetStreamByLocalPort(Message.RecipientStreamPort, _))
		{
			FStreamDescription& Stream = *StreamPtr;
			if (Stream.Status == EStreamConnectionState::RequestSent)
			{
				if (Message.bAccepted)
				{
					Stream.RemoteStreamPort = Message.OpenedStreamPort;
					if (Stream.bThisIsSource)
					{
						Stream.Sender = MakeSender(Stream.SourcePoint, RemoteEndpointAddress, Message.OpenedStreamPort);
					}
					else
					{
						Stream.Receiver = MakeReceiver(Stream.SourcePoint, Stream.DestinationPoint, RemoteEndpointAddress, Message.OpenedStreamPort);
					}

					check(Stream.Receiver || Stream.Sender)
					Stream.Status = EStreamConnectionState::Active;
				}
				else
				{
					Stream.Status = EStreamConnectionState::Closed;
					UE_LOGF(LogDirectLinkNet, Warning, "stream connection refused. %ls", *Message.Error);
				}
			}
		}
		else
		{
			UE_LOGF(LogDirectLinkNet, Warning, "error: no such stream (%d)", Message.RecipientStreamPort);
		}
	}

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "%ls", *ToString_dbg());
}


void FInternalThreadState::Handle_CloseStreamRequest(const FDirectLinkMsg_CloseStreamRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write); // read array, lock specific stream ?
		if (FStreamDescription* StreamPtr = SharedState.GetStreamByLocalPort(Message.RecipientStreamPort, _))
		{
			bool bNotifyRemote = false; // since it's already a request from Remote...
			SharedState.CloseStreamInternal(*StreamPtr, _, bNotifyRemote);
		}
	}

	UE_LOGF(LogDirectLinkNet, Verbose, "Endpoint '%ls': Handle_CloseStreamRequest", *SharedState.NiceName);
	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "%ls", *ToString_dbg());
}


bool FInternalThreadState::IsMine(const FMessageAddress& MaybeRemoteAddress) const
{
	return MessageEndpoint->GetAddress() == MaybeRemoteAddress;
}


bool FInternalThreadState::IsIgnoredEndpoint(const FMessageAddress& MaybeRemoteAddress) const
{
	return IgnoredEndpoints.Contains(MaybeRemoteAddress);
}


void FInternalThreadState::ReplicateState(const FMessageAddress& RemoteEndpointAddress) const
{
	if (MessageEndpoint.IsValid())
	{
		FDirectLinkMsg_EndpointState* EndpointStateMessage = FMessageEndpoint::MakeMessage<FDirectLinkMsg_EndpointState>();
		*EndpointStateMessage = ThisDescription;

		if (RemoteEndpointAddress.IsValid())
		{
			UE_LOGF(LogDirectLinkNet, Verbose, "Endpoint '%ls': Send FDirectLinkMsg_EndpointState", *SharedState.NiceName);
			MessageEndpoint->Send(EndpointStateMessage, RemoteEndpointAddress);
		}
		else
		{
			UE_LOGF(LogDirectLinkNet, Verbose, "Endpoint '%ls': Publish FDirectLinkMsg_EndpointState", *SharedState.NiceName);
			LastBroadcastedStateRevision = EndpointStateMessage->StateRevision;
			MessageEndpoint->Publish(EndpointStateMessage);
		}
	}
}


void FInternalThreadState::ReplicateState_Broadcast() const
{
	FMessageAddress Invalid;
	ReplicateState(Invalid);
}


void FInternalThreadState::UpdateSourceDescription()
{
	{
		ThisDescription.Sources.Reset();
		FRWScopeLock _(SharedState.SourcesLock, SLT_ReadOnly);
		for (const TSharedPtr<FStreamSource>& Source : SharedState.Sources)
		{
			ThisDescription.Sources.Add({Source->GetName(), Source->GetId(), Source->IsPublic()});
		}
	}
	ThisDescription.StateRevision++;
}


void FInternalThreadState::UpdateDestinationDescription()
{
	{
		ThisDescription.Destinations.Reset();
		FRWScopeLock _(SharedState.DestinationsLock, SLT_ReadOnly);
		for (const TSharedPtr<FStreamDestination>& Dest : SharedState.Destinations)
		{
			ThisDescription.Destinations.Add({Dest->GetName(), Dest->GetId(), Dest->IsPublic()});
		}
	}
	ThisDescription.StateRevision++;
}


TUniquePtr<IStreamReceiver> FInternalThreadState::MakeReceiver(FGuid SourceGuid, FGuid DestinationGuid, FMessageAddress RemoteAddress, FStreamPort RemotePort)
{
	{
		FRWScopeLock _(SharedState.DestinationsLock, SLT_ReadOnly);
		for (const TSharedPtr<FStreamDestination>& Dest : SharedState.Destinations)
		{
			if (Dest->GetId() == DestinationGuid)
			{
				const TSharedPtr<IConnectionRequestHandler>& RequestHandler = Dest->GetProvider();
				check(RequestHandler);

				IConnectionRequestHandler::FSourceInformation SourceInfo;
				SourceInfo.Id = SourceGuid;

				if (RequestHandler->CanOpenNewConnection(SourceInfo))
				{
					if (TSharedPtr<ISceneReceiver> DeltaConsumer = RequestHandler->GetSceneReceiver(SourceInfo))
					{
						return MakeUnique<FStreamReceiver>(MessageEndpoint, RemoteAddress, RemotePort, DeltaConsumer.ToSharedRef());
					}
				}
				UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls': Handle_OpenStreamRequest: new connection refused by provider", *SharedState.NiceName);
				break;
			}
		}
	}

	return nullptr;
}


TSharedPtr<IStreamSender> FInternalThreadState::MakeSender(FGuid SourceGuid, FMessageAddress RemoteAddress, FStreamPort RemotePort)
{
	{
		FRWScopeLock _(SharedState.SourcesLock, SLT_ReadOnly);
		for (const TSharedPtr<FStreamSource>& Source : SharedState.Sources)
		{
			if (Source->GetId() == SourceGuid)
			{
				TSharedPtr<FStreamSender> Sender = MakeShared<FStreamSender>(MessageEndpoint, RemoteAddress, RemotePort);
				Source->LinkSender(Sender);
				return Sender;
			}
		}
	}

	return nullptr;
}


void FInternalThreadState::RemoveEndpoint(const FMessageAddress& RemoteEndpointAddress)
{
	if (FDirectLinkMsg_EndpointState* RemoteState = RemoteEndpointDescriptions.Find(RemoteEndpointAddress))
	{
		UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Display, "Endpoint '%ls' removes remote Endpoint '%ls'", *SharedState.NiceName, *RemoteState->NiceName);
	}

	RemoteEndpointDescriptions.Remove(RemoteEndpointAddress);
	RemoteLastSeenTime.Remove(RemoteEndpointAddress);

	// close remaining associated streams
	{
		FRWScopeLock _(SharedState.StreamsLock, SLT_Write);
		for (auto& Stream : SharedState.Streams)
		{
			if (Stream.RemoteAddress == RemoteEndpointAddress
				&& Stream.Status != EStreamConnectionState::Closed)
			{
				UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls': Closed connection  (reason: remote endpoint removed)", *SharedState.NiceName);
				bool bNotifyRemote = false;
				SharedState.CloseStreamInternal(Stream, _, bNotifyRemote);
			}
		}
	}
}


void FInternalThreadState::MarkRemoteAsSeen(const FMessageAddress& RemoteEndpointAddress)
{
	RemoteLastSeenTime.Add(RemoteEndpointAddress, Now_s);
}


void FInternalThreadState::CleanupTimedOutEndpoint()
{
	TArray<FMessageAddress> RemovableEndpoints;
	for (const auto& KV : RemoteEndpointDescriptions)
	{
		if (double* LastSeen = RemoteLastSeenTime.Find(KV.Key))
		{
			if (Now_s - *LastSeen > gConfig.ThresholdEndpointCleanup_s)
			{
				RemovableEndpoints.Add(KV.Key);
				UE_LOGF(LogDirectLinkNet, Log, "Endpoint '%ls': Removed Endpoint %ls (timeout)", *SharedState.NiceName, *KV.Value.NiceName);
			}
		}
	}

	for (const FMessageAddress& RemovableEndpoint : RemovableEndpoints)
	{
		RemoveEndpoint(RemovableEndpoint);
	}
}

FRawInfo::FEndpointInfo FromMsg(const FDirectLinkMsg_EndpointState& Msg)
{
	FRawInfo::FEndpointInfo Info;
	Info.Name = Msg.NiceName;
	FEngineVersion::Parse(Msg.UEVersion, Info.Version);

	for (const auto& S : Msg.Sources)
	{
		Info.Sources.Add({S.Name, S.Id, S.bIsPublic});
	}
	for (const auto& S : Msg.Destinations)
	{
		Info.Destinations.Add({S.Name, S.Id, S.bIsPublic});
	}
	Info.UserName = Msg.UserName;
	Info.ExecutableName = Msg.ExecutableName;
	Info.ComputerName = Msg.ComputerName;
	Info.bIsLocal = Msg.ComputerName == FPlatformProcess::ComputerName();
	Info.ProcessId = Msg.ProcessId;
	return Info;
}


bool FInternalThreadState::Init()
{
	MessageEndpoint = FMessageEndpoint::Builder(TEXT("DirectLinkEndpoint"))
		.Handling<FDirectLinkMsg_DeltaMessage>(this, &FInternalThreadState::Handle_DeltaMessage)
		.Handling<FDirectLinkMsg_HaveListMessage>(this, &FInternalThreadState::Handle_HaveListMessage)
		.Handling<FDirectLinkMsg_EndpointLifecycle>(this, &FInternalThreadState::Handle_EndpointLifecycle)
		.Handling<FDirectLinkMsg_QueryEndpointState>(this, &FInternalThreadState::Handle_QueryEndpointState)
		.Handling<FDirectLinkMsg_EndpointState>(this, &FInternalThreadState::Handle_EndpointState)
		.Handling<FDirectLinkMsg_OpenStreamRequest>(this, &FInternalThreadState::Handle_OpenStreamRequest)
		.Handling<FDirectLinkMsg_OpenStreamAnswer>(this, &FInternalThreadState::Handle_OpenStreamAnswer)
		.Handling<FDirectLinkMsg_CloseStreamRequest>(this, &FInternalThreadState::Handle_CloseStreamRequest)
		.WithInbox();

	if (!ensure(MessageEndpoint.IsValid()))
	{
		return false;
	}

	MessageEndpoint->Subscribe<FDirectLinkMsg_EndpointLifecycle>();
	MessageEndpoint->Subscribe<FDirectLinkMsg_EndpointState>();
	SharedState.MessageEndpoint = MessageEndpoint;
	SharedState.bInnerThreadShouldRun = true;
	Now_s = FPlatformTime::Seconds();
	return true;
}


void FInternalThreadState::Run()
{
	// setup local endpoint description (aka replicated state)
	ThisDescription = FDirectLinkMsg_EndpointState(1, GetMinSupportedProtocolVersion(), GetCurrentProtocolVersion());
	ThisDescription.ComputerName = FPlatformProcess::ComputerName();
	ThisDescription.UserName = FPlatformProcess::UserName();
	ThisDescription.ProcessId = (int32)FPlatformProcess::GetCurrentProcessId();
	ThisDescription.ExecutableName = FPlatformProcess::ExecutableName();
	ThisDescription.NiceName = SharedState.NiceName;

	if (gUdpMessagingInitializationTime > 0.)
	{
		double WaitTime = FMath::Min(FPlatformTime::Seconds() - gUdpMessagingInitializationTime, 0.5);
		if (WaitTime > 0.)
		{
			UE_LOGF(LogDirectLinkNet, Verbose, "Endpoint '%ls': wait after UDP init. (In order to avoid that temporisation, Load 'UdpMessaging' module sooner in the game thread).", *SharedState.NiceName);
			FPlatformProcess::Sleep(WaitTime);
		}
	}

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "Endpoint '%ls': Publishing FDirectLinkMsg_EndpointLifecycle Start", *SharedState.NiceName);
	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FDirectLinkMsg_EndpointLifecycle>(FDirectLinkMsg_EndpointLifecycle::ELifecycle::Start));

	while (SharedState.bInnerThreadShouldRun)
	{
		Now_s = FPlatformTime::Seconds();

		// process local signals
		if (SharedState.bDirtySources.exchange(false))
		{
			UpdateSourceDescription();
		}
		if (SharedState.bDirtyDestinations.exchange(false))
		{
			UpdateDestinationDescription();
		}

		if (LastBroadcastedStateRevision != ThisDescription.StateRevision)
		{
			ReplicateState_Broadcast();
		}

		if (Now_s - LastHeartbeatTime_s > gConfig.HeartbeatThreshold_s)
		{
			UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "Endpoint '%ls': Publishing FDirectLinkMsg_EndpointLifecycle Heartbeat %f", *SharedState.NiceName, Now_s);
			MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FDirectLinkMsg_EndpointLifecycle>(FDirectLinkMsg_EndpointLifecycle::ELifecycle::Heartbeat, ThisDescription.StateRevision));
			LastHeartbeatTime_s = Now_s;
		}

		// consume remote messages
		MessageEndpoint->ProcessInbox();

		// cleanup old endpoints
		if (gConfig.bPeriodicalyCleanupTimedOutEndpoint
		 && (Now_s - LastEndpointCleanupTime_s > gConfig.CleanupOldEndpointPeriod_s))
		{
			CleanupTimedOutEndpoint();
			LastEndpointCleanupTime_s = Now_s;
		}

		// sync send
		{
			FRWScopeLock _(SharedState.StreamsLock, SLT_ReadOnly);
			for (FStreamDescription& Stream : SharedState.Streams)
			{
				if (Stream.Status == EStreamConnectionState::Active
					&& Stream.bThisIsSource && ensure(Stream.Sender.IsValid()))
				{
					Stream.Sender->Tick(Now_s);
				}
			}
		}

		// rebuild description of remote endpoints
		if (true) // #ue_directlink_integration flag based: user-side api driven
		{
			// prepare data - Endpoint part
			TMap<FMessageAddress, FRawInfo::FEndpointInfo> EndpointsInfo;
			EndpointsInfo.Reserve(RemoteEndpointDescriptions.Num());
			for (const auto& KV : RemoteEndpointDescriptions)
			{
				EndpointsInfo.Add(KV.Key, FromMsg(KV.Value));
			}
			FMessageAddress ThisEndpointAddress = MessageEndpoint->GetAddress();
			EndpointsInfo.Add(ThisEndpointAddress, FromMsg(ThisDescription));

			// prepare data - sources and destinations
			TMap<FGuid, FRawInfo::FDataPointInfo> DataPointsInfo;
			auto l = [&](const FDirectLinkMsg_EndpointState& EpDescription, const FMessageAddress& EpAddress, bool bIsLocal)
			{
				for (const auto& Src : EpDescription.Sources)
				{
					DataPointsInfo.Add(Src.Id, FRawInfo::FDataPointInfo{EpAddress, Src.Name, true, bIsLocal, Src.bIsPublic});
				}
				for (const auto& Dst : EpDescription.Destinations)
				{
					DataPointsInfo.Add(Dst.Id, FRawInfo::FDataPointInfo{EpAddress, Dst.Name, false, bIsLocal, Dst.bIsPublic});
				}
			};

			l(ThisDescription, ThisEndpointAddress, true);
			for (const auto& KV : RemoteEndpointDescriptions)
			{
				l(KV.Value, KV.Key, false);
			}

			// prepare data - Streams part
			TArray<FRawInfo::FStreamInfo> StreamsInfo;
			{
				FRWScopeLock _(SharedState.StreamsLock, SLT_ReadOnly);
				StreamsInfo.Reserve(SharedState.Streams.Num());
				for (FStreamDescription& Stream : SharedState.Streams)
				{
					FRawInfo::FStreamInfo StreamInfo;
					StreamInfo.StreamId = Stream.LocalStreamPort;
					StreamInfo.Source = Stream.SourcePoint;
					StreamInfo.Destination = Stream.DestinationPoint;
					StreamInfo.ConnectionState = Stream.Status;
					if (Stream.Status == EStreamConnectionState::Active)
					{
						if (Stream.Sender)
						{
							StreamInfo.CommunicationStatus = Stream.Sender->GetCommunicationStatus();
						}
						else if (ensure(Stream.Receiver))
						{
							StreamInfo.CommunicationStatus = Stream.Receiver->GetCommunicationStatus();
						}
					}
					StreamsInfo.Add(StreamInfo);
				}
			}

			{
				// update info for local observers
				FRWScopeLock _(SharedState.RawInfoCopyLock, SLT_Write);
				SharedState.RawInfo.ThisEndpointAddress = ThisEndpointAddress;
				SharedState.RawInfo.EndpointsInfo = MoveTemp(EndpointsInfo);
				SharedState.RawInfo.DataPointsInfo = MoveTemp(DataPointsInfo);
				SharedState.RawInfo.StreamsInfo = MoveTemp(StreamsInfo);
			}

			{
				// Notify observers
				FRawInfo RawInfo = Owner.GetRawInfoCopy(); // stupid copy, but avoids locking 2 mutexes at once
				FRWScopeLock _(SharedState.ObserversLock, SLT_ReadOnly);
				for (IEndpointObserver* Observer : SharedState.Observers)
				{
					Observer->OnStateChanged(RawInfo);
				}
			}
		}

		// #ue_directlink_connexion temp autoconnect policy.
		// for all local source, connect to all remote dest with the same name
		// reimpl with named broadcast source, and client connect themselves

		if (gConfig.bAutoconnectFromSources || gConfig.bAutoconnectFromDestination)
		{
			TArray<FNamedId> AllSources = gConfig.bAutoconnectFromSources ? ThisDescription.Sources : TArray<FNamedId>{};
			TArray<FNamedId> AllDestinations = gConfig.bAutoconnectFromDestination ? ThisDescription.Destinations : TArray<FNamedId>{};

			for (const auto& KV : RemoteEndpointDescriptions)
			{
				if (gConfig.bAutoconnectFromSources)
				{
					for (const auto& Dst : KV.Value.Destinations)
					{
						if (Dst.bIsPublic) AllDestinations.Add(Dst);
					}
				}
				if (gConfig.bAutoconnectFromDestination)
				{
					for (const auto& Src : KV.Value.Sources)
					{
						if (Src.bIsPublic) AllSources.Add(Src);
					}
				}
			}

			for (const auto& Src : AllSources)
			{
				for (const auto& Dst : AllDestinations)
				{
					if (Src.Name == Dst.Name)
					{
						Owner.OpenStream(Src.Id, Dst.Id);
					}
				}
			}
		}

		if (MessageEndpoint->IsInboxEmpty())
		{
			InnerThreadEvent->Wait(FTimespan::FromMilliseconds(50));
		}
	}

	UE_CLOGF(SharedState.bDebugLog, LogDirectLinkNet, Verbose, "Endpoint '%ls': Publishing FDirectLinkMsg_EndpointLifecycle Stop", *SharedState.NiceName);
	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FDirectLinkMsg_EndpointLifecycle>(FDirectLinkMsg_EndpointLifecycle::ELifecycle::Stop));
	FMessageEndpoint::SafeRelease(MessageEndpoint);
}


FStreamDescription* FSharedState::GetStreamByLocalPort(FStreamPort LocalPort, const FRWScopeLock& _)
{
	// try to skip a lookup
	if (Streams.IsValidIndex(LocalPort-1)
		&& ensure(Streams[LocalPort-1].LocalStreamPort == LocalPort))
	{
		return &Streams[LocalPort-1];
	}

	for (FStreamDescription& Stream : Streams)
	{
		if (Stream.LocalStreamPort == LocalPort)
		{
			return &Stream;
		}
	}
	return nullptr;
}


void FSharedState::CloseStreamInternal(FStreamDescription& Stream, const FRWScopeLock& _, bool bNotifyRemote)
{
	if (Stream.Status == EStreamConnectionState::Closed)
	{
		return;
	}

	if (bNotifyRemote && Stream.RemoteAddress.IsValid())
	{
		UE_CLOGF(bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls': Stream removed", *NiceName);
		FDirectLinkMsg_CloseStreamRequest* Request = FMessageEndpoint::MakeMessage<FDirectLinkMsg_CloseStreamRequest>();
		Request->RecipientStreamPort = Stream.RemoteStreamPort;

		UE_CLOGF(bDebugLog, LogDirectLinkNet, Log, "Endpoint '%ls': Send FDirectLinkMsg_CloseStreamRequest", *NiceName);
		MessageEndpoint->Send(Request, Stream.RemoteAddress);
	}

	// close local stream
	Stream.Status = EStreamConnectionState::Closed;
	Stream.Sender.Reset();
	Stream.Receiver.Reset(); // #ue_directlink_cleanup notify associated scene provider
}

} // namespace DirectLink
