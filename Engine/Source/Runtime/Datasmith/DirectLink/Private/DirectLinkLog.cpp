// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkLog.h"

DEFINE_LOG_CATEGORY(LogDirectLink);
DEFINE_LOG_CATEGORY(LogDirectLinkNet);
