// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkCommon.h"

#include "CoreMinimal.h"
#include "Misc/EngineVersion.h"

#include "DirectLinkMessages.generated.h"

namespace DirectLink
{
class FScenePipeBase;
} // namespace DirectLink




USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_EndpointLifecycle
{
	GENERATED_BODY()

	enum ELifecycle : uint8
	{
		None,
		Start,
		Heartbeat,
		Stop,
	};

	FDirectLinkMsg_EndpointLifecycle(ELifecycle InLifecycleState = ELifecycle::None, uint32 InEndpointStateRevision = 0)
		: LifecycleState(InLifecycleState)
		, EndpointStateRevision(InEndpointStateRevision)
	{}

	UPROPERTY()
	uint8 LifecycleState = ELifecycle::None;

	UPROPERTY()
	uint32 EndpointStateRevision = 0;
};



USTRUCT(meta=(Experimental))
struct FNamedId
{
	GENERATED_BODY();

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FGuid Id;

	UPROPERTY()
	bool bIsPublic = false;
};

USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_EndpointState
{
	GENERATED_BODY();
	FDirectLinkMsg_EndpointState() = default;

	FDirectLinkMsg_EndpointState(uint32 StateRevision, uint32 MinProtocolVersion, uint32 MaxProtocolVersion)
		: StateRevision(StateRevision)
		, MinProtocolVersion(MinProtocolVersion)
		, ProtocolVersion(MaxProtocolVersion)
		, UEVersion(FEngineVersion::Current().ToString())
	{}

	UPROPERTY()
	uint32 StateRevision = 0;

	UPROPERTY()
	uint32 MinProtocolVersion = 0;

	UPROPERTY()
	uint32 ProtocolVersion = 0;

	UPROPERTY()
	FString UEVersion;

	UPROPERTY()
	FString ComputerName;

	UPROPERTY()
	FString UserName;

	UPROPERTY()
	uint32 ProcessId = 0;

	UPROPERTY()
	FString ExecutableName;

	UPROPERTY()
	FString NiceName;

	UPROPERTY()
	TArray<FNamedId> Destinations;

	UPROPERTY()
	TArray<FNamedId> Sources;
};



USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_QueryEndpointState
{
	GENERATED_BODY();
};



USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_OpenStreamRequest
{
	GENERATED_BODY();
	// #ue_directlink_cleanup explicit ctr to force correct init

	UPROPERTY()
	bool bRequestFromSource = false;

	UPROPERTY()
	int32 RequestFromStreamPort = DirectLink::InvalidStreamPort;

	UPROPERTY()
	FGuid SourceGuid;

	UPROPERTY()
	FGuid DestinationGuid;
};



USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_OpenStreamAnswer
{
	GENERATED_BODY();

	UPROPERTY()
	int32 RecipientStreamPort = DirectLink::InvalidStreamPort;

	UPROPERTY()
	bool bAccepted = false;

	UPROPERTY()
	FString Error; // optionnal: may be filled when the request is denied

	UPROPERTY()
	int32 OpenedStreamPort = DirectLink::InvalidStreamPort;
};


USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_CloseStreamRequest
{
	GENERATED_BODY();

	UPROPERTY()
	int32 RecipientStreamPort = DirectLink::InvalidStreamPort;
};



USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_DeltaMessage
{
	GENERATED_BODY();

	enum EKind
	{
		None,
		SetupScene, // setup the stream for a scene id
		OpenDelta,
		SetElements,
		RemoveElements,
		CloseDelta,
	};

	// required for UStructs
	FDirectLinkMsg_DeltaMessage() = default;

	FDirectLinkMsg_DeltaMessage(EKind Kind, DirectLink::FStreamPort DestinationStreamPort, uint32 BatchNumber, uint32 MessageIndex)
		: DestinationStreamPort(DestinationStreamPort)
		, BatchCode(BatchNumber)
		, MessageCode(MessageIndex)
		, Kind(Kind)
	{
	}

	UPROPERTY()
	int32 DestinationStreamPort = DirectLink::InvalidStreamPort;

	UPROPERTY()
	int8 BatchCode = 0;

	UPROPERTY()
	int32 MessageCode = 0;

	UPROPERTY()
	uint8 Kind = EKind::None;

	UPROPERTY()
	bool CompressedPayload = false;

	UPROPERTY()
	TArray<uint8> Payload;
};


USTRUCT(meta=(Experimental))
struct FDirectLinkMsg_HaveListMessage
{
	GENERATED_BODY();

	enum EKind : uint8
	{
		None,
		OpenHaveList, // see Payload
		HaveListElement, // see NodeIds and Hashes
		AckDeltaMessage, // #ue_directlink_cleanup
		CloseHaveList,
	};

	// required for UStructs
	FDirectLinkMsg_HaveListMessage() = default;

	FDirectLinkMsg_HaveListMessage(EKind Kind, DirectLink::FStreamPort SourceStreamPort, uint32 SyncCycle, uint32 MessageIndex)
		: SourceStreamPort(SourceStreamPort)
		, SyncCycle(SyncCycle)
		, MessageCode(MessageIndex)
		, Kind(Kind)
	{
	}

	UPROPERTY()
	int32 SourceStreamPort = 0; // FStreamPort

	UPROPERTY()
	int32 SyncCycle = 0;

	UPROPERTY()
	int32 MessageCode = 0;

	UPROPERTY()
	uint8 Kind = 0;

	UPROPERTY()
	TArray<uint8> Payload;

	UPROPERTY()
	TArray<int32> NodeIds;

	UPROPERTY()
	TArray<int32> Hashes;
};

