// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkMisc.h"

#include "DirectLinkLog.h"
#include "DirectLinkSceneSnapshot.h"
#include "DirectLinkElementSnapshot.h"


#include "HAL/FileManager.h"
#include "Misc/Paths.h"

namespace DirectLink
{

uint8 GetCurrentProtocolVersion() { return 9; }
uint8 GetMinSupportedProtocolVersion() { return 9; }



const FString& GetDumpPath()
{
	static const FString DumpPath = []() -> FString
	{
		const TCHAR* VarName = TEXT("DIRECTLINK_SNAPSHOT_PATH");
		FString Var = FPlatformMisc::GetEnvironmentVariable(VarName);
		FText Text;
		if (!FPaths::ValidatePath(Var, &Text))
		{
			UE_LOGF(LogDirectLink, Warning, "Invalid path '%ls' defined by environment variable %ls (%ls).", *Var, VarName, *Text.ToString());
			return FString();
		}
		return Var;
	}();
	return DumpPath;
}

void DumpSceneSnapshot(FSceneSnapshot& SceneSnapshot, const FString& BaseFileName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::DumpSceneSnapshot);
	const FString& DumpPath = GetDumpPath();
	if (DumpPath.IsEmpty())
	{
		return;
	}

	auto& Elements = SceneSnapshot.Elements;
	auto& SceneId = SceneSnapshot.SceneId;
	FString SceneIdStr = FString::Printf(TEXT(".%08X"), SceneId.SceneGuid.A);
	FString FileName = DumpPath / BaseFileName + SceneIdStr + TEXT(".directlink.scenesnap");
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*FileName));
	if (!Ar.IsValid())
	{
		return;
	}

	auto Write = [&](const FString& Value)
	{
		FTCHARToUTF8 UTF8String( *Value );
		Ar->Serialize( (ANSICHAR*)UTF8String.Get(), UTF8String.Length() );
	};

	Elements.KeySort(TLess<FSceneGraphId>());

	Write(FString::Printf(TEXT("%d elements:\n"), Elements.Num()));

	for (const auto& KV : Elements)
	{
		Write(FString::Printf(
			TEXT("%d -> %08X (data:%08X ref:%08X)\n")
			, KV.Key
			, KV.Value->GetHash()
			, KV.Value->GetDataHash()
			, KV.Value->GetRefHash()
		));
	}
}

} // namespace DirectLink
