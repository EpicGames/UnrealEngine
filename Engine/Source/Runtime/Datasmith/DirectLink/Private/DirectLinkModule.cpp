// Copyright Epic Games, Inc. All Rights Reserved.
#include "DirectLinkModule.h"

#include "DirectLinkEndpoint.h"
#include "DirectLinkLog.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDirectLinkModule, DirectLink);


void FDirectLinkModule::StartupModule()
{
	DirectLink::ValidateCommunicationStatus();
}


