// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkParameterStore.h"

#include "DirectLinkLog.h"
#include "DirectLinkSceneGraphNode.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/UnrealTemplate.h"



namespace DirectLink
{

// -- ps4 fix --
// Fix for a modular build on PS4 platform, where orbis-ld requires the following symbols to be
// accessible when compiling DatasmithCore as a shared object.
// Generated methods are not enough somehow, so they are explicitely defined here, in the cpp,
// which fixes the visibility issue.
FParameterStore::FParameterStore() = default;
FParameterStore::~FParameterStore() = default;
FParameterStore::FParameterStore(const FParameterStore&) = default;
FParameterStore& FParameterStore::operator=(const FParameterStore&) = default;
FParameterStore::FParameterStore(FParameterStore&&) = default;
FParameterStore& FParameterStore::operator=(FParameterStore&&) = default;
// -- end ps4 fix --

uint32 FParameterStore::GetParameterCount() const
{
	return Parameters.Num();
}

int32 FParameterStore::GetParameterIndex(FName Name) const
{
	return Parameters.IndexOfByPredicate([&](const FParameterDetails& Parameter) {
		return Parameter.Name == Name;
	});
}

bool FParameterStore::HasParameterNamed(FName Name) const
{
	return GetParameterIndex(Name) != INDEX_NONE;
}

FName FParameterStore::GetParameterName(int32 Index) const
{
	check(Parameters.IsValidIndex(Index));
	return Parameters[Index].Name;
}

FParameterStoreSnapshot FParameterStore::Snapshot() const
{
	FParameterStoreSnapshot Snap;
	Snap.ReserveParamCount(GetParameterCount());

	for (const FParameterStore::FParameterDetails& ParamDetails : Parameters)
	{
		Snap.AddParam(
			ParamDetails.Name,
			ParamDetails.StorageMethod,
			ParamDetails.StorageLocation
		);
	}

	return Snap;
}

void FParameterStore::Update(const FParameterStoreSnapshot& NewValues)
{
	for (const FParameterStoreSnapshot::FParameterDetails& ParamUpdate : NewValues.Parameters)
	{
		int32 Index = GetParameterIndex(ParamUpdate.Name);
		if (Parameters.IsValidIndex(Index))
		{
			if (ensure(Reflect::GetStoreType(ParamUpdate.StorageMethod) == Reflect::GetStoreType(Parameters[Index].StorageMethod)))
			{
				FMemoryReader Ar(ParamUpdate.Buffer);
				Reflect::SerialAny(Ar, Parameters[Index].StorageLocation, ParamUpdate.StorageMethod);
			}
			else
			{
				UE_LOGF(LogDirectLink, Warning, "Update property issue [%ls]: incompatible types.", *ParamUpdate.Name.ToString());
			}
		}
	}
}

struct FTempBacktrack : FNoncopyable
{
	FTempBacktrack(FArchive& Ar, int64 TempLocation)
		: Ar(Ar)
	{
		RestoreLocation = Ar.Tell();
		Ar.Seek(TempLocation);
	}
	~FTempBacktrack()
	{
		Ar.Seek(RestoreLocation);
	}

private:
	FArchive& Ar;
	uint64 RestoreLocation;
};


void FParameterStoreSnapshot::SerializeAll(FArchive& Ar)
{
	// Note: Changes to this implementation impacts version handling
	// see DirectLink::GetCurrentProtocolVersion and DirectLink::GetMinSupportedProtocolVersion

	if (Ar.IsSaving())
	{
		uint32 ParamCount = Parameters.Num();
		Ar.SerializeIntPacked(ParamCount);

		for (FParameterStoreSnapshot::FParameterDetails& Parameter : Parameters)
		{
			Ar << Parameter.Name;
			Ar << Parameter.StorageMethod;
			uint32 BufferSize = Parameter.Buffer.Num();
			Ar.SerializeIntPacked(BufferSize);
			Ar.Serialize(Parameter.Buffer.GetData(), BufferSize);
		}
	}
	else
	{
		uint32 ParamCount = 0;
		Ar.SerializeIntPacked(ParamCount);

		Parameters.Reserve(ParamCount);
		for (uint32 i = 0; i < ParamCount; ++i)
		{
			FParameterStoreSnapshot::FParameterDetails& Parameter = Parameters.AddDefaulted_GetRef();
			Ar << Parameter.Name;
			Ar << Parameter.StorageMethod;
			uint32 BufferSize = 0;
			Ar.SerializeIntPacked(BufferSize);
			Parameter.Buffer.SetNumUninitialized(BufferSize);

			Ar.Serialize(Parameter.Buffer.GetData(), BufferSize);
		}
	}
}

void FParameterStoreSnapshot::AddParam(FName Name, Reflect::ESerialMethod StorageMethod, void* StorageLocation)
{
	FParameterDetails& SnapParam = Parameters.AddDefaulted_GetRef();
	SnapParam.Name = Name;
	SnapParam.StorageMethod = StorageMethod;
	FMemoryWriter Ar(SnapParam.Buffer);
	Reflect::SerialAny(Ar, StorageLocation, StorageMethod);
	// #ue_directlink_optim: measures shows a mean alloc size of <20 bytes...
	// one buffer per property is clearly an aweful structure perfwise.
	// one buffer for the whole store could be a solution
}

FElementHash FParameterStoreSnapshot::Hash() const
{
	FElementHash RunningHash = 0;
	for (const FParameterDetails& P : Parameters)
	{
		RunningHash = FCrc::MemCrc32(P.Buffer.GetData(), P.Buffer.Num(), RunningHash);
	}
	return RunningHash;
}

} // namespace DirectLink
