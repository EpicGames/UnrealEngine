// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkSceneGraphNode.h"

#include "DirectLinkElementSnapshot.h"
#include "DirectLinkParameterStore.h"


namespace DirectLink
{

FSceneGraphId ISceneGraphNode::RegisterReference(ISceneGraphNode* Referenced)
{
	// #ue_directlink_design
	// for now, basically alloc an ID for a referenced node.
	// next step: register reference edge to the shared context. This could replace the reference visit system as well as providing other services

	if (Referenced == nullptr)
	{
		return 0;
	}

	if (!ensure(SharedState.IsValid()))
	{
		SetSharedState(MakeSharedState());
	}

	Referenced->SetSharedState(SharedState);

	check(Referenced->SceneGraphId != 0)
	return Referenced->SceneGraphId;
}

void ISceneGraphNode::UpdateRefs(class IReferenceResolutionProvider& Resolver, const FReferenceSnapshot& NewRefs)
{
	for (const FNamedReferenceProxy& Proxy : ReferenceProxies)
	{
		for (const FReferenceSnapshot::FReferenceGroup& Group : NewRefs.Groups)
		{
			if (Group.Name == Proxy.Name)
			{
				// The NodeIds in Group.ReferencedIds may not be valid using the given resolver
				// In that case, the invalid NodeIds should not be set on Proxy's view
				// Only valid nodes can be added to it, see TDatasmithReferenceArrayProxy::SetNodes
				TArray<FSceneGraphId> ValidNodeIds;
				ValidNodeIds.Reserve(Group.ReferencedIds.Num());
				for (FSceneGraphId NodeId : Group.ReferencedIds)
				{
					if (TSharedPtr<ISceneGraphNode> Node = Resolver.AsSharedPtr(NodeId))
					{
						ValidNodeIds.Add(NodeId);
					}
				}

				if (!ValidNodeIds.IsEmpty())
				{
					Proxy.View->SetNodes(Resolver, ValidNodeIds);
				}
				break;
			}
		}
	}
}

void ISceneGraphNode::SetSharedState(TSharedPtr<FSceneGraphSharedState> NewSharedState)
{
	if (SharedState != NewSharedState)
	{
		SharedState = NewSharedState;
		if (SharedState.IsValid())
		{
			SceneGraphId = SharedState->MakeId();
		}
	}
}


void ISceneGraphNode::RegisterReferenceProxy(IReferenceProxy& View, FName Name)
{
	ReferenceProxies.Add({Name, &View});
}

int32 ISceneGraphNode::GetReferenceProxyCount() const
{
	return ReferenceProxies.Num();
}

class IReferenceProxy* ISceneGraphNode::GetReferenceProxy(int32 Index) const
{
	return ReferenceProxies.IsValidIndex(Index) ? ReferenceProxies[Index].View : nullptr;
}

FName ISceneGraphNode::GetReferenceProxyName(int32 Index) const
{
	return ReferenceProxies.IsValidIndex(Index) ? ReferenceProxies[Index].Name : NAME_None;
}

} // namespace DirectLink
