// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkScenePipe.h"

#include "DirectLinkElementSnapshot.h"
#include "DirectLinkLog.h"
#include "DirectLinkMessages.h"
#include "DirectLinkSceneSnapshot.h"

#include "MessageEndpoint.h"
#include "Misc/Compression.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace DirectLink
{

namespace Internal
{


int32 GetDeltaMessageTargetSizeByte()
{
	static int32 i = 64*1024;
	return i;
}

enum class ECompressionMethod
{
	ECM_ZLib = 1,
	ECM_Gzip = 2,
	ECM_LZ4  = 3,
};


FName GetMethodName(ECompressionMethod MethodCode)
{
	switch (MethodCode)
	{
		case ECompressionMethod::ECM_ZLib: return NAME_Zlib;
		case ECompressionMethod::ECM_Gzip: return NAME_Gzip;
		case ECompressionMethod::ECM_LZ4:  return NAME_LZ4;
		default: ensure(0); return NAME_None;
	}
}


bool CompressInline(TArray<uint8>& UncompressedData, ECompressionMethod Method)
{
	int32 UncompressedSize = UncompressedData.Num();
	FName MethodName = GetMethodName(Method);
	int32 CompressedSize = FCompression::CompressMemoryBound(MethodName, UncompressedSize);

	TArray<uint8> CompressedData;
	int32 HeaderSize = 5;
	CompressedData.AddUninitialized(CompressedSize + HeaderSize);

	// header
	{
		FMemoryWriter Ar(CompressedData);
		uint8 MethodCode = uint8(Method);
		Ar << MethodCode;
		Ar << UncompressedSize;
		check(HeaderSize == Ar.Tell());
	}

	if (FCompression::CompressMemory(MethodName, CompressedData.GetData() + HeaderSize, CompressedSize, UncompressedData.GetData(), UncompressedSize))
	{
		float CompressionRatio = float(CompressedSize) / UncompressedSize;
		UE_LOGF(LogDirectLinkNet, Verbose, "Message compression: %f%% (%d -> %d, %ls)", 100*CompressionRatio, UncompressedSize, CompressedSize, *MethodName.ToString());
		CompressedData.SetNum(CompressedSize + HeaderSize, EAllowShrinking::Yes);
		UncompressedData = MoveTemp(CompressedData);

		return true;
	}

	UE_LOGF(LogDirectLinkNet, Verbose, "Message compression failed");
	return false;
}

bool DecompressInline(TArray<uint8>& CompressedData)
{
	ECompressionMethod Method;
	uint8* BufferStart = nullptr;
	int32 UncompressedSize = -1;
	int32 CompressedSize = CompressedData.Num();
	int32 HeaderSize = 0;
	{
		FMemoryReader Ar(CompressedData);
		uint8 MethodCode = 0;
		Ar << MethodCode;
		Method = ECompressionMethod(MethodCode);
		Ar << UncompressedSize;
		HeaderSize = Ar.Tell();
	}

	FName MethodName = GetMethodName(Method);
	if (MethodName != NAME_None)
	{
		TArray<uint8> UncompressedData;
		UncompressedData.SetNumUninitialized(UncompressedSize);
		bool Ok = FCompression::UncompressMemory(MethodName, UncompressedData.GetData(), UncompressedData.Num(), CompressedData.GetData() + HeaderSize, CompressedData.Num()-HeaderSize);
		if (Ok)
		{
			CompressedData = MoveTemp(UncompressedData);
			return true;
		}
	}
	return false;
}


// returns true when the message is compressed
bool CompressInline(FDirectLinkMsg_DeltaMessage& UncompressedMessage)
{
	check(UncompressedMessage.CompressedPayload == false);

	if (UncompressedMessage.Payload.Num() > 100)
	{
		UncompressedMessage.CompressedPayload = CompressInline(UncompressedMessage.Payload, ECompressionMethod::ECM_ZLib);
	}
	return UncompressedMessage.CompressedPayload;
}


// returns true when the message is in an uncompressed state
bool DecompressInline(FDirectLinkMsg_DeltaMessage& Message)
{
	if (Message.CompressedPayload)
	{
		if (DecompressInline(Message.Payload))
		{
			Message.CompressedPayload = false;
		}
		else
		{
			UE_LOGF(LogDirectLinkNet, Error, "DeltaMessage: Decompression error");
			return false;
		}
	}
	return true;
}

} // namespace Internal



template<typename MessageType>
void FPipeBase::SendInternal(MessageType* Message, int32 ByteSizeHint)
{
	auto flag = EMessageFlags::Reliable;
	ThisEndpoint->Send(Message, MessageType::StaticStruct(), flag, nullptr, TArrayBuilder<FMessageAddress>().Add(RemoteAddress), FTimespan::Zero(), FDateTime::MaxValue());
}



void FScenePipeToNetwork::SetupScene(FSetupSceneArg& SetupSceneArg)
{
	FDirectLinkMsg_DeltaMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_DeltaMessage>(
		FDirectLinkMsg_DeltaMessage::SetupScene, RemoteStreamPort, 0, 0
	);

	FMemoryWriter Ar(Message->Payload);
	Ar << SetupSceneArg;

	Send(Message);
}


void FScenePipeToNetwork::OpenDelta(FOpenDeltaArg& OpenDeltaArg)
{
	++BatchNumber;
	if (BatchNumber == 0) // skip 0, which would be considered invalid
	{
		++BatchNumber;
	}

	NextMessageNumber = 0;

	FDirectLinkMsg_DeltaMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_DeltaMessage>(
		FDirectLinkMsg_DeltaMessage::OpenDelta,
		RemoteStreamPort, BatchNumber, NextMessageNumber++
	);

	FMemoryWriter Ar(Message->Payload);
	Ar << OpenDeltaArg;

	Send(Message);
}


void FScenePipeToNetwork::Send(FDirectLinkMsg_DeltaMessage* Message)
{
	Internal::CompressInline(*Message);
	SendInternal(Message, Message->Payload.Num());
}


void FScenePipeToNetwork::InitSetElementBuffer()
{
	SetElementBuffer.Reset(Internal::GetDeltaMessageTargetSizeByte());
}


void FScenePipeToNetwork::OnSetElement(FSetElementArg& SetElementArg)
{
	bool bAppend = true;
	FMemoryWriter Ar(SetElementBuffer, false, bAppend);
	ESerializationStatus Status = SetElementArg.Snapshot->Serialize(Ar);
	Ar << SetElementArg.ElementIndexHint;
	check(Status == ESerializationStatus::Ok); // write should never be an issue

	if (SetElementBuffer.Num() >= Internal::GetDeltaMessageTargetSizeByte())
	{
		SendSetElementBuffer();
	}
}


void FScenePipeToNetwork::SendSetElementBuffer()
{
	FDirectLinkMsg_DeltaMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_DeltaMessage>(
		FDirectLinkMsg_DeltaMessage::SetElements,
		RemoteStreamPort, BatchNumber, NextMessageNumber++
	);

	Message->Payload = MoveTemp(SetElementBuffer);
	Send(Message);

	InitSetElementBuffer();
}


void FScenePipeToNetwork::RemoveElements(FRemoveElementsArg& RemoveElementsArg)
{
	FDirectLinkMsg_DeltaMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_DeltaMessage>(
		FDirectLinkMsg_DeltaMessage::RemoveElements,
		RemoteStreamPort, BatchNumber, NextMessageNumber++
	);

	FMemoryWriter Ar(Message->Payload);
	Ar << RemoveElementsArg.Elements;

	Send(Message);
}


void FScenePipeToNetwork::OnCloseDelta(FCloseDeltaArg& CloseDeltaArg)
{
	SendSetElementBuffer();
	FDirectLinkMsg_DeltaMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_DeltaMessage>(
		FDirectLinkMsg_DeltaMessage::CloseDelta,
		RemoteStreamPort, BatchNumber, NextMessageNumber++
	);

	FMemoryWriter Ar(Message->Payload);
	Ar << CloseDeltaArg;

	Send(Message);
}


///////////////////////////////////////////////////////////////////////////////////////////////////


void FScenePipeFromNetwork::HandleDeltaMessage(FDirectLinkMsg_DeltaMessage& Message)
{
	UE_LOGF(LogDirectLinkNet, Verbose, "Delta message received: b:%d m:%d k:%d", Message.BatchCode, Message.MessageCode, Message.Kind);
	if (Message.BatchCode == 0)
	{
		DelegateDeltaMessage(Message);
		return;
	}

	if (CurrentBatchCode == 0)
	{
		CurrentBatchCode = Message.BatchCode;
		NextTransmitableMessageIndex = 0;
	}

	if (Message.BatchCode != CurrentBatchCode)
	{
		UE_LOGF(LogDirectLinkNet, Warning, "Dropped delta message (bad batch code %d, expected %d)", Message.BatchCode, CurrentBatchCode);
		return;
	}

	// consume as much as possible
	if (Message.MessageCode == NextTransmitableMessageIndex)
	{
		DelegateDeltaMessage(Message);
		++NextTransmitableMessageIndex;

		FDirectLinkMsg_DeltaMessage NextMessage;
		while (MessageBuffer.RemoveAndCopyValue(NextTransmitableMessageIndex, NextMessage))
		{
			DelegateDeltaMessage(NextMessage);
			++NextTransmitableMessageIndex;
		}
	}
	else
	{
		MessageBuffer.Add(Message.MessageCode, MoveTemp(Message));
	}
}


void FScenePipeFromNetwork::OnOpenHaveList(const FSceneIdentifier& HaveSceneId, bool bKeepPreviousContent, int32 SyncCycle)
{
	if (!ensure(BufferedHaveListContent == nullptr))
	{
		// we should not be opening a new have list without having the previous one closed
		SendHaveElements();
	}

	BatchNumber = SyncCycle;
	NextMessageNumber = 0;

	FDirectLinkMsg_HaveListMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_HaveListMessage>(
		FDirectLinkMsg_HaveListMessage::EKind::OpenHaveList,
		RemoteStreamPort, BatchNumber, NextMessageNumber++
	);

	FMemoryWriter Ar(Message->Payload);
	check(Ar.IsSaving());
	Ar << const_cast<FSceneIdentifier&>(HaveSceneId);
	Ar << bKeepPreviousContent;

	UE_LOGF(LogDirectLinkNet, Verbose, "Send OpenHave list b:%d m:%d k:%d", Message->SyncCycle, Message->MessageCode, Message->Kind);

	Send(Message);
}


void FScenePipeFromNetwork::OnHaveElement(FSceneGraphId NodeId, FElementHash HaveHash)
{
	if (BufferedHaveListContent == nullptr)
	{
		BufferedHaveListContent = FMessageEndpoint::MakeMessage<FDirectLinkMsg_HaveListMessage>(
			FDirectLinkMsg_HaveListMessage::EKind::HaveListElement,
			RemoteStreamPort, 0, 0
		);
		BufferedHaveListContent->NodeIds.Reserve(BufferSize);
		BufferedHaveListContent->Hashes.Reserve(BufferSize);
	}
	BufferedHaveListContent->NodeIds.Add(NodeId);
	BufferedHaveListContent->Hashes.Add(HaveHash);

	if (BufferedHaveListContent->NodeIds.Num() >= BufferSize)
	{
		SendHaveElements();
	}
}


void FScenePipeFromNetwork::SendHaveElements()
{
	if (BufferedHaveListContent)
	{
		BufferedHaveListContent->SyncCycle = BatchNumber;
		BufferedHaveListContent->MessageCode = NextMessageNumber++;

		ThisEndpoint->Send(BufferedHaveListContent, RemoteAddress);
		BufferedHaveListContent = nullptr;
	}
}


void FScenePipeFromNetwork::OnCloseHaveList()
{
	SendHaveElements();

	FDirectLinkMsg_HaveListMessage* Message = FMessageEndpoint::MakeMessage<FDirectLinkMsg_HaveListMessage>(
		FDirectLinkMsg_HaveListMessage::EKind::CloseHaveList,
		RemoteStreamPort, BatchNumber, NextMessageNumber++
	);

	UE_LOGF(LogDirectLinkNet, Verbose, "Send OpenHave list b:%d m:%d k:%d", Message->SyncCycle, Message->MessageCode, Message->Kind);
	Send(Message);
}


void FScenePipeFromNetwork::DelegateDeltaMessage(FDirectLinkMsg_DeltaMessage& Message)
{
	UE_LOGF(LogDirectLinkNet, Verbose, "Delta message transmited: b:%d m:%d k:%d", Message.BatchCode, Message.MessageCode, Message.Kind);

	// acknowledge message (allows the sender to track communication progress)
	auto* AckMessage = FMessageEndpoint::MakeMessage<FDirectLinkMsg_HaveListMessage>(
		FDirectLinkMsg_HaveListMessage::EKind::AckDeltaMessage,
		RemoteStreamPort, Message.BatchCode, Message.MessageCode
	);
	Send(AckMessage);

	if (!Internal::DecompressInline(Message))
	{
		return; // log by DecompressInline
	}

	// process message
	check(Consumer);
	switch (Message.Kind)
	{
		case FDirectLinkMsg_DeltaMessage::SetupScene:
		{
			CurrentCommunicationStatus.bIsReceiving = true;

			IDeltaConsumer::FSetupSceneArg SetupSceneArg;
			FMemoryReader Ar(Message.Payload);
			Ar << SetupSceneArg;
			Consumer->SetupScene(SetupSceneArg);
			break;
		}

		case FDirectLinkMsg_DeltaMessage::OpenDelta:
		{
			IDeltaConsumer::FOpenDeltaArg OpenDeltaArg;
			FMemoryReader Ar(Message.Payload);
			Ar << OpenDeltaArg;
			CurrentCommunicationStatus.TaskTotal = OpenDeltaArg.ElementCountHint + 1; // "+1" in order to reach 100% artificially on the last message
			Consumer->OpenDelta(OpenDeltaArg);
			break;
		}

		case FDirectLinkMsg_DeltaMessage::SetElements:
		{
			FMemoryReader Ar(Message.Payload);
			bool bSerialOk = true;
			while (bSerialOk && Ar.Tell() < Message.Payload.Num())
			{
				IDeltaConsumer::FSetElementArg SetElementArg;
				SetElementArg.Snapshot = MakeShared<FElementSnapshot>();
				ESerializationStatus SerialResult = SetElementArg.Snapshot->Serialize(Ar);
				Ar << SetElementArg.ElementIndexHint;

				bSerialOk = false;
				switch (SerialResult)
				{
					case ESerializationStatus::Ok:
						Consumer->OnSetElement(SetElementArg);
						CurrentCommunicationStatus.TaskCompleted = SetElementArg.ElementIndexHint;
						bSerialOk = true;
						break;
					case ESerializationStatus::StreamError:
						// #ue_directlink_syncprotocol notify sender of unrecoverable errors
						UE_LOGF(LogDirectLinkNet, Error, "Delta message issue: Stream Error");
						break;
					case ESerializationStatus::VersionMinNotRespected:
						UE_LOGF(LogDirectLinkNet, Error, "Delta message issue: received message version no longer supported");
						break;
					case ESerializationStatus::VersionMaxNotRespected:
						UE_LOGF(LogDirectLinkNet, Error, "Delta message issue: received message version unknown");
						break;
					default:
						ensure(false);
				}
			}
			break;
		}

		case FDirectLinkMsg_DeltaMessage::RemoveElements:
		{
			IDeltaConsumer::FRemoveElementsArg RemoveElementsArg;
			FMemoryReader Ar(Message.Payload);
			Ar << RemoveElementsArg.Elements;
			Consumer->RemoveElements(RemoveElementsArg);
			break;
		}

		case FDirectLinkMsg_DeltaMessage::CloseDelta:
		{
			CurrentCommunicationStatus.bIsReceiving = false;
			CurrentCommunicationStatus.TaskCompleted = CurrentCommunicationStatus.TaskTotal;

			IDeltaConsumer::FCloseDeltaArg CloseDeltaArg;
			FMemoryReader Ar(Message.Payload);
			Ar << CloseDeltaArg;
			Consumer->OnCloseDelta(CloseDeltaArg);
			CurrentBatchCode = 0;
			break;
		}

		case FDirectLinkMsg_DeltaMessage::None:
		default:
			ensure(false);
	}
}


void FScenePipeFromNetwork::Send(FDirectLinkMsg_HaveListMessage* Message)
{
	// #ue_directlink_sync compression ?
	SendInternal(Message, Message->Payload.Num());
}


FArchive& operator<<(FArchive& Ar, FSceneIdentifier& SceneId)
{
	Ar << SceneId.SceneGuid;
	Ar << SceneId.DisplayName;
	return Ar;
}


FArchive& operator<<(FArchive& Ar, IDeltaConsumer::FSetupSceneArg& SetupSceneArg)
{
	Ar << SetupSceneArg.SceneId;
	Ar << SetupSceneArg.bExpectHaveList;
	Ar << SetupSceneArg.SyncCycle;
	return Ar;
}


FArchive& operator<<(FArchive& Ar, IDeltaConsumer::FOpenDeltaArg& OpenDeltaArg)
{
	Ar << OpenDeltaArg.bBasedOnNewScene;
	Ar << OpenDeltaArg.ElementCountHint;
	return Ar;
}


FArchive& operator<<(FArchive& Ar, IDeltaConsumer::FCloseDeltaArg& CloseDeltaArg)
{
	Ar << CloseDeltaArg.bCancelled;
	return Ar;
}


} // namespace DirectLink
