// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkDeltaConsumer.h"
#include "DirectLinkMessages.h"
#include "DirectLinkStreamCommunicationInterface.h"

#include "CoreTypes.h"
#include "IMessageContext.h"

class FMessageEndpoint;
struct FMessageAddress;

namespace DirectLink
{
class FEndpoint;


class FPipeBase
{
protected:
	FPipeBase(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint, const FMessageAddress& RemoteAddress, FStreamPort RemoteStreamPort)
		: ThisEndpoint(ThisEndpoint)
		, RemoteAddress(RemoteAddress)
		, RemoteStreamPort(RemoteStreamPort)
	{
		check(ThisEndpoint);
	}

	template<typename MessageType>
	void SendInternal(MessageType* Message, int32 ByteSizeHint=0);

	// connectivity
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint;
	FMessageAddress RemoteAddress;
	FStreamPort RemoteStreamPort;
};



/**
 * Responsibility: delegate DeltaConsumer/DeltaProducer link over network, including message ordering and acknowledgments.
 */
class FScenePipeToNetwork
	: public FPipeBase
	, public IDeltaConsumer
{
public:
	FScenePipeToNetwork(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint, const FMessageAddress& RemoteAddress, FStreamPort RemoteStreamPort)
		: FPipeBase(ThisEndpoint, RemoteAddress, RemoteStreamPort)
	{
		check(ThisEndpoint);
		InitSetElementBuffer();
	}

public: // IDeltaConsumer API
	virtual void SetDeltaProducer(IDeltaProducer* Producer) override {}
	virtual void SetupScene(FSetupSceneArg& SetupSceneArg) override;
	virtual void OpenDelta(FOpenDeltaArg& OpenDeltaArg) override;
	virtual void OnSetElement(FSetElementArg& SetElementArg) override;
	virtual void RemoveElements(FRemoveElementsArg& RemoveElementsArg) override;
	virtual void OnCloseDelta(FCloseDeltaArg& CloseDeltaArg) override;

public:
	int32 GetSentDeltaMessageCount() const { return NextMessageNumber; }

private:
	void Send(FDirectLinkMsg_DeltaMessage* Message);
	void InitSetElementBuffer();
	void SendSetElementBuffer();

private:
	// message ordering
	int8 BatchNumber = 0;
	int32 NextMessageNumber;
	TArray<uint8> SetElementBuffer;
};


class FScenePipeFromNetwork
	: public FPipeBase
	, public IDeltaProducer
	, public IStreamCommunicationInterface
{
public:
	FScenePipeFromNetwork(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> Sender, const FMessageAddress& RemoteAddress, FStreamPort RemoteStreamPort, const TSharedRef<IDeltaConsumer> Consumer)
		: FPipeBase(Sender, RemoteAddress, RemoteStreamPort)
		, Consumer(Consumer)
	{
		Consumer->SetDeltaProducer(this);
	}

	void HandleDeltaMessage(FDirectLinkMsg_DeltaMessage& Message);

	// delta producer
	virtual void OnOpenHaveList(const FSceneIdentifier& HaveSceneId, bool bKeepPreviousContent, int32 SyncCycle) override;
	virtual void OnHaveElement(FSceneGraphId NodeId, FElementHash HaveHash) override;
	void SendHaveElements();
	virtual void OnCloseHaveList() override;

public: // IStreamCommunicationInterface API
	virtual FCommunicationStatus GetCommunicationStatus() const override { return CurrentCommunicationStatus; }

private:
	// transmits messages to the actual delta consumer, reordered
	void DelegateDeltaMessage(FDirectLinkMsg_DeltaMessage& Message);
	void Send(FDirectLinkMsg_HaveListMessage* Message);

private:
	// sent message ordering
	int32 BatchNumber = 0;
	int32 NextMessageNumber;
	FDirectLinkMsg_HaveListMessage* BufferedHaveListContent = nullptr;
	static constexpr int32 BufferSize = 100;

	// received message ordering
	TMap<int32, FDirectLinkMsg_DeltaMessage> MessageBuffer;
	int32 NextTransmitableMessageIndex = 0;
	int32 CurrentBatchCode = 0;

	TSharedPtr<IDeltaConsumer> Consumer;

	// reporting
	FCommunicationStatus CurrentCommunicationStatus;
};


FArchive& operator << (FArchive& Ar, FSceneIdentifier& SceneId);
FArchive& operator << (FArchive& Ar, IDeltaConsumer::FSetupSceneArg& SetupSceneArg);
FArchive& operator << (FArchive& Ar, IDeltaConsumer::FOpenDeltaArg& OpenDeltaArg);
FArchive& operator << (FArchive& Ar, IDeltaConsumer::FCloseDeltaArg& CloseDeltaArg);

} // namespace DirectLink
