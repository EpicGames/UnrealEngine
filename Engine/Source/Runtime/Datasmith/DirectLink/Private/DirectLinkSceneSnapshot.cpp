// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkSceneSnapshot.h"

#include "DirectLinkLog.h"
#include "DirectLinkElementSnapshot.h"
#include "DirectLinkSceneGraphNode.h"
#include "Async/ParallelFor.h"


namespace DirectLink
{


void RecursiveAddElements(TSet<ISceneGraphNode*>& Nodes, ISceneGraphNode* Element)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::RecursiveAddElements);
	if (Element == nullptr)
	{
		UE_LOGF(LogDirectLink, Warning, "Try to index null element");
		return;
	}

	bool bWasAlreadyThere;
	Nodes.Add(Element, &bWasAlreadyThere);
	if (bWasAlreadyThere)
	{
		return;
	}

	// Recursive
	for (int32 ProxyIndex = 0; ProxyIndex < Element->GetReferenceProxyCount(); ++ProxyIndex)
	{
		IReferenceProxy* RefProxy = Element->GetReferenceProxy(ProxyIndex);
		int32 ReferenceCount = RefProxy->Num();
		for (int32 ReferenceIndex = 0; ReferenceIndex < ReferenceCount; ReferenceIndex++)
		{
			if (ISceneGraphNode* Referenced = RefProxy->GetNode(ReferenceIndex))
			{
				Element->RegisterReference(Referenced);
				RecursiveAddElements(Nodes, Referenced);
			}
		}
	}
}

TSet<ISceneGraphNode*> BuildIndexForScene(ISceneGraphNode* RootElement)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::BuildIndexForScene);
	TSet<ISceneGraphNode*> Nodes;

	if (RootElement)
	{
		if (!RootElement->GetSharedState().IsValid())
		{
			RootElement->SetSharedState(RootElement->MakeSharedState());
			if (!RootElement->GetSharedState().IsValid())
			{
				return Nodes;
			}
		}
		RecursiveAddElements(Nodes, RootElement);
	}

	return Nodes;
}


TSharedPtr<FSceneSnapshot> SnapshotScene(ISceneGraphNode* RootElement)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::SnapshotScene);
	if (RootElement == nullptr)
	{
		return nullptr;
	}

	TSet<ISceneGraphNode*> Nodes = BuildIndexForScene(RootElement);
	check(Nodes.Num() == Nodes.GetMaxIndex()); // We use direct indexing later so no holes are allowed

	if (!ensure(RootElement->GetSharedState().IsValid()))
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::SnapshotScene/Elements);
	TSharedPtr<FSceneSnapshot> SceneSnapshot = MakeShared<FSceneSnapshot>();
	SceneSnapshot->SceneId = RootElement->GetSharedState()->GetSceneId();

	// parallel snapshot generation
	const int32 BatchSize = 64;
	const int32 BatchCount = (Nodes.Num() + BatchSize - 1) / BatchSize;

	struct FBatchContext
	{
		TArray<TPair<FSceneGraphId, TSharedRef<FElementSnapshot>>> OutPairs;
	};
	TArray<FBatchContext> Batches;
	Batches.SetNum(BatchCount);

	ParallelFor(BatchCount, [&](int32 BatchIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::SnapshotScene/Elements/Batch);
		auto& OutPairs = Batches[BatchIndex].OutPairs;
		OutPairs.Reserve(BatchSize);

		const int32 Start = BatchIndex * BatchSize;
		const int32 End = FMath::Min(Start + BatchSize, Nodes.Num()); // one-past end index

		for (int32 NodeIndex = Start; NodeIndex < End; ++NodeIndex)
		{
			ISceneGraphNode* Node = Nodes[FSetElementId::FromInteger(NodeIndex)];
			OutPairs.Emplace(Node->GetNodeId(), MakeShared<FElementSnapshot>(*Node));
		}
	});

	// Join
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DirectLink::SnapshotScene/Elements/Join);
		SceneSnapshot->Elements.Reserve(Nodes.Num());
		for (auto& Batch : Batches)
		{
			for (auto& Pair : Batch.OutPairs)
			{
				SceneSnapshot->Elements.Add(MoveTemp(Pair));
			}
		}
	}

	return SceneSnapshot;
}


} // namespace DirectLink
