// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DirectLinkCommon.h"

namespace DirectLink
{

class IStreamCommunicationInterface
{
public:
	virtual ~IStreamCommunicationInterface() = default;
	virtual FCommunicationStatus GetCommunicationStatus() const = 0;
	virtual void Tick(double Now_s) {}
};



class IStreamReceiver
	: public IStreamCommunicationInterface
{
public:
	virtual ~IStreamReceiver() = default;
	virtual void HandleDeltaMessage(struct FDirectLinkMsg_DeltaMessage& Message) = 0;
};



class IStreamSender
	: public IStreamCommunicationInterface
{
public:
	virtual ~IStreamSender() = default;
	virtual void SetSceneSnapshot(TSharedPtr<class FSceneSnapshot> SceneSnapshot) = 0;
	virtual void HandleHaveListMessage(const struct FDirectLinkMsg_HaveListMessage& Message) = 0;
};

} // namespace DirectLink
