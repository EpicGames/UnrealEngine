// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkCommon.h"
#include "CoreTypes.h"

namespace DirectLink
{


class FStreamConnectionPoint
{
public:
	FStreamConnectionPoint(const FString& Name, EVisibility Visibility)
		: Name(Name)
		, Id(FGuid::NewGuid())
		, Visibility(Visibility)
	{}
	const FString GetName() const { return Name; }
	const FGuid& GetId() const { return Id; }
	EVisibility GetVisibility() const { return Visibility; }
	bool IsPublic() { return GetVisibility() == EVisibility::Public; }

private:
	FString Name;
	FGuid Id;
	EVisibility Visibility;
};


} // namespace DirectLink
