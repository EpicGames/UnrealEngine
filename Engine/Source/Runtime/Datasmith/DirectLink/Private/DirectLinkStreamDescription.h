// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkStreamCommunicationInterface.h"

#include "CoreTypes.h"
#include "IMessageContext.h"



namespace DirectLink
{

struct FStreamDescription
{
	bool bThisIsSource = false;

	FGuid SourcePoint;
	FGuid DestinationPoint;
	FStreamPort LocalStreamPort = 0; // works like an Id within that endpoint
	FMessageAddress RemoteAddress;
	FStreamPort RemoteStreamPort = 0;
	EStreamConnectionState Status = EStreamConnectionState::Uninitialized;

	TUniquePtr<IStreamReceiver> Receiver;
	TSharedPtr<IStreamSender> Sender;
};

} // namespace DirectLink
