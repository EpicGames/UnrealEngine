// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkStreamConnectionPoint.h"

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"


namespace DirectLink
{
class IConnectionRequestHandler;

class FStreamDestination
	: public FStreamConnectionPoint
{
public:
	FStreamDestination(const FString& Name, EVisibility Visibility, const TSharedPtr<IConnectionRequestHandler>& Provider)
		: FStreamConnectionPoint(Name, Visibility)
		, Provider(Provider)
	{}

	const TSharedPtr<IConnectionRequestHandler>& GetProvider() const { return Provider; }

private:
	TSharedPtr<IConnectionRequestHandler> Provider;
};


} // namespace DirectLink
