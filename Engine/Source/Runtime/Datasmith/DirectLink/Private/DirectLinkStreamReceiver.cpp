// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkStreamReceiver.h"

#include "DirectLinkLog.h"
#include "DirectLinkElementSnapshot.h"
#include "DirectLinkMisc.h"
#include "DirectLinkScenePipe.h"
#include "DirectLinkSceneSnapshot.h"


namespace DirectLink
{

class FDeltaReceiver
	: public IDeltaConsumer
{
public:
	FDeltaReceiver(const TSharedRef<ISceneReceiver>& Receiver)
		: Receiver(Receiver)
	{
	}

	virtual void SetDeltaProducer(IDeltaProducer* Producer) override
	{
		DeltaProducer = Producer;
	}

	virtual void SetupScene(FSetupSceneArg& SetupSceneArg) override
	{
		check(DeltaProducer);

		// ignore duplicated requests
		if (SyncCycle == SetupSceneArg.SyncCycle)
		{
			UE_LOGF(LogDirectLinkNet, VeryVerbose, "ignored duplicated FDeltaReceiver::SetupScene %d", SyncCycle);
			return;
		}

		FSceneIdentifier PreviousSceneId = SceneSnapshot.SceneId; // GetSceneIdentifier();
		const FSceneIdentifier& NewSceneId = SetupSceneArg.SceneId;

		SyncCycle = SetupSceneArg.SyncCycle;
		SceneSnapshot = FSceneSnapshot();
		SceneSnapshot.SceneId = NewSceneId;

		// detect new scene data
		bool bKeepCurrentState = PreviousSceneId.SceneGuid.IsValid()
			&& PreviousSceneId.SceneGuid == NewSceneId.SceneGuid;

		if (!bKeepCurrentState)
		{
			// #ue_directlink_feat consumer load from file
		}
		// #ue_directlink_feat: bKeepCurrentState have list delta
		DeltaProducer->OnOpenHaveList(NewSceneId, bKeepCurrentState, SyncCycle);
		for (const auto& Pair : SceneSnapshot.Elements)
		{
			DeltaProducer->OnHaveElement(Pair.Key, Pair.Value->GetHash());
		}
		DeltaProducer->OnCloseHaveList();
	}


	virtual void OpenDelta(FOpenDeltaArg& OpenDeltaArg) override
	{
		if (OpenDeltaArg.bBasedOnNewScene)
		{
			SceneSnapshot.Elements.Reset();
		}
	}

	virtual void OnSetElement(FSetElementArg& SetElementArg) override
	{
		SceneSnapshot.Elements.Add(SetElementArg.Snapshot->GetNodeId(), SetElementArg.Snapshot.ToSharedRef());
	}

	virtual void RemoveElements(FRemoveElementsArg& RemoveElementsArg) override
	{
		for (const FSceneGraphId& NodeId : RemoveElementsArg.Elements)
		{
			SceneSnapshot.Elements.Remove(NodeId);
		}
	}

	virtual void OnCloseDelta(FCloseDeltaArg& CloseDeltaArg) override
	{
		DumpSceneSnapshot(SceneSnapshot, TEXT("receiver"));

		// #ue_directlink_syncprotocol directlink level validation (ref graph? whole scene Hash ?)

		if (!CloseDeltaArg.bCancelled)
		{
			Receiver->FinalSnapshot(SceneSnapshot);
		}
	}

private:
	IDeltaProducer* DeltaProducer = nullptr;
	TSharedRef<ISceneReceiver> Receiver;
	int32 SyncCycle = 0;
	FSceneSnapshot SceneSnapshot;
};


FStreamReceiver::FStreamReceiver(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint, const FMessageAddress& DestinationAddress, FStreamPort ReceiverStreamPort, const TSharedRef<ISceneReceiver>& Receiver)
	:PipeFromNetwork(ThisEndpoint, DestinationAddress, ReceiverStreamPort, MakeShared<FDeltaReceiver>(Receiver))
{
}


void FStreamReceiver::HandleDeltaMessage(FDirectLinkMsg_DeltaMessage& Message)
{
	PipeFromNetwork.HandleDeltaMessage(Message);
}


FCommunicationStatus FStreamReceiver::GetCommunicationStatus() const
{
	return PipeFromNetwork.GetCommunicationStatus();
}


} // namespace DirectLink

