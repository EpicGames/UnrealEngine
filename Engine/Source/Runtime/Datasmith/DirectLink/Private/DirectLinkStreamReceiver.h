// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DirectLinkCommon.h"
#include "DirectLinkScenePipe.h"
#include "DirectLinkStreamCommunicationInterface.h"


class FMessageEndpoint;
struct FMessageAddress;
struct FDirectLinkMsg_DeltaMessage;

namespace DirectLink
{


class FStreamReceiver
	: public IStreamReceiver
{
public:
	FStreamReceiver(
		TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint,
		const FMessageAddress& DestinationAddress,
		FStreamPort ReceiverStreamPort,
		const TSharedRef<ISceneReceiver>& Consumer);

	virtual void HandleDeltaMessage(FDirectLinkMsg_DeltaMessage& Message) override;

	virtual FCommunicationStatus GetCommunicationStatus() const override;

private:
	FScenePipeFromNetwork PipeFromNetwork;
};


} // namespace DirectLink
