// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkStreamSender.h"

#include "DirectLinkElementSnapshot.h"
#include "DirectLinkLog.h"


namespace DirectLink
{

/**
 * Keeps tracks of what the remote have.
 * A Simple id -> Hash table.
 */
class FRemoteSceneView
{
public:
	const FSceneIdentifier& GetSceneId() const { return SceneId; }
	void SetSceneId(const FSceneIdentifier& InSceneId) { SceneId = InSceneId; }
	FElementHash& GetHashRef(FSceneGraphId NodeId) { return HaveList.FindOrAdd(NodeId, InvalidHash); }
	void GetHaveIdsSet(TSet<FSceneGraphId>& OutIds) const { HaveList.GetKeys(OutIds); }

private:
	FSceneIdentifier SceneId;
	TMap<FSceneGraphId, FElementHash> HaveList;
};

/**
 * Handle unordered incoming messages, build a usable haveList structure for the following diff algo.
 */
class FHaveListReceiver
{
public:
	FHaveListReceiver(int32 SyncCycle)
		: SyncCycle(SyncCycle)
		, RemoteScene(MakeUnique<FRemoteSceneView>())
	{}
	bool IsOver() const { return bClosed; }

	// accepts unordered messages
	void HandleMessage(const FDirectLinkMsg_HaveListMessage& Message); // update RemoteView
	TUniquePtr<FRemoteSceneView>& GetRemoteSceneView() { return RemoteScene; }

private:
	void HandleOrderedMessage(const FDirectLinkMsg_HaveListMessage& Message); // update RemoteView

private:
	int32 SyncCycle = 0;
	int32 NextOrderedMessage = 0;
	TSortedMap<int32, FDirectLinkMsg_HaveListMessage> Unordered;
	TUniquePtr<FRemoteSceneView> RemoteScene;
	bool bClosed = false;
};


FStreamSender::FStreamSender(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint, const FMessageAddress& DestinationAddress, FStreamPort ReceiverStreamPort)
	: PipeToNetwork(ThisEndpoint, DestinationAddress, ReceiverStreamPort)
{
}


// allows forward-decl. of TUniquePtrs inner classes
FStreamSender::~FStreamSender() = default;


void FStreamSender::HandleHaveListMessage(const FDirectLinkMsg_HaveListMessage& Message)
{
	if (Message.Kind == FDirectLinkMsg_HaveListMessage::EKind::AckDeltaMessage)
	{
		UE_LOGF(LogDirectLinkNet, Verbose, "Receiver ack message %d", Message.MessageCode);
		CurrentCommunicationStatus.TaskCompleted = FMath::Max(CurrentCommunicationStatus.TaskCompleted, Message.MessageCode + 1); // max, because ack messages are unordered
		return;
	}

	if (!HaveListReceiver)
	{
		UE_LOGF(LogDirectLinkNet, Warning, "dropped unexpected HaveList message.");
		return;
	}

	HaveListReceiver->HandleMessage(Message);
	LastHaveListMessage_s = FPlatformTime::Seconds();
}


void FStreamSender::SetSceneSnapshot(TSharedPtr<FSceneSnapshot> SceneSnapshot)
{
	FScopeLock _(&NextSnapshotLock);
	NextSnapshot = SceneSnapshot;
}


void FStreamSender::Tick(double Now_s)
{
	switch (NextStep)
	{
		case EStep::Idle:
		{
			if (NextSnapshotLock.TryLock())
			{
				Snapshot = NextSnapshot;
				NextSnapshot.Reset();
				NextSnapshotLock.Unlock();

				if (Snapshot)
				{
					++SyncCycle;
					HaveListReceiver = MakeUnique<FHaveListReceiver>(SyncCycle);
					CurrentCommunicationStatus = FCommunicationStatus();
					NextStep = EStep::SetupScene;
					return Tick(Now_s);
				}
			}

			break;
		}

		case EStep::SetupScene:
		{
			CurrentCommunicationStatus.bIsSending = true;

			check(Snapshot.IsValid());
			auto& Local = *Snapshot.Get();
			IDeltaConsumer::FSetupSceneArg SetupSceneArg;
			SetupSceneArg.SceneId = Local.SceneId;
			SetupSceneArg.bExpectHaveList = true;
			SetupSceneArg.SyncCycle = SyncCycle;

			PipeToNetwork.SetupScene(SetupSceneArg);
			LastHaveListMessage_s = FPlatformTime::Seconds();

			NextStep = EStep::ReceiveHaveList;
			break;
		}

		case EStep::ReceiveHaveList: // wait for completed have list
		{
			CurrentCommunicationStatus.bIsSending = false;
			CurrentCommunicationStatus.bIsReceiving = true;

			// waiting for the remote havelist.
			if (ensure(HaveListReceiver) && HaveListReceiver->IsOver())
			{
				RemoteScene = MoveTemp(HaveListReceiver->GetRemoteSceneView());
				NextStep = EStep::GenerateDelta;
				return Tick(Now_s);
			}
			else
			{
				// #ue_directlink_syncprotocol todo: timeout
				// !!! connectivity is not handled here !!!
				// we should never be in that situation as the stream should have been checked before (ping/pong exchange)
				// Sender object may receive a LostConnectivity msg though.

				double ElapsedSinceHaveListRequest_s = FPlatformTime::Seconds() - LastHaveListMessage_s;
				if (ElapsedSinceHaveListRequest_s > 0.1)
				{
					UE_LOGF(LogDirectLinkNet, Warning, "connectivity issue: Have List not received. Retry")
					NextStep = EStep::SetupScene;
					return;
				}
			}

			break;
		}

		case EStep::GenerateDelta:
		{
			CurrentCommunicationStatus.bIsSending = true;
			CurrentCommunicationStatus.bIsReceiving = false;

			check(Snapshot.IsValid());
			FSceneSnapshot& SceneSnapshot = *Snapshot.Get();

			IDeltaConsumer::FOpenDeltaArg OpenDeltaArg;
			OpenDeltaArg.bBasedOnNewScene = false;
			OpenDeltaArg.ElementCountHint = SceneSnapshot.Elements.Num();
			PipeToNetwork.OpenDelta(OpenDeltaArg);

			TSet<FSceneGraphId> LocalIds;
			LocalIds.Reserve(SceneSnapshot.Elements.Num());
			int32 CurrentElementIndex = -1;

			for (auto& RefPair : SceneSnapshot.Elements)
			{
				++CurrentElementIndex;
				FSceneGraphId NodeId = RefPair.Key;
				LocalIds.Add(NodeId);

				FElementSnapshot& ElementSnapshot = RefPair.Value.Get();

				FElementHash NodeHash = ElementSnapshot.GetHash();
				if (NodeHash != InvalidHash)
				{
					FElementHash& HaveHash = RemoteScene->GetHashRef(NodeId);
					if (HaveHash == NodeHash)
					{
						UE_LOGF(LogDirectLinkNet, Verbose, "diff: Skipped %d, have hash match", NodeId);
						continue;
					}
					HaveHash = NodeHash;
				}
				IDeltaConsumer::FSetElementArg SetArg;
				SetArg.Snapshot = RefPair.Value;
				SetArg.ElementIndexHint = CurrentElementIndex;
				PipeToNetwork.OnSetElement(SetArg);
			}

			TSet<FSceneGraphId> RemoteIds;
			RemoteScene->GetHaveIdsSet(RemoteIds);
			TSet<FSceneGraphId> RemovableIds = RemoteIds.Difference(LocalIds);
			if (RemovableIds.Num())
			{
				IDeltaConsumer::FRemoveElementsArg DelArg;
				DelArg.Elements = RemovableIds.Array();
				PipeToNetwork.RemoveElements(DelArg);
			}

			IDeltaConsumer::FCloseDeltaArg CloseArg;
			PipeToNetwork.OnCloseDelta(CloseArg);

			int32 DeltaMessagesCount = PipeToNetwork.GetSentDeltaMessageCount();
			CurrentCommunicationStatus.TaskTotal = DeltaMessagesCount;

			Snapshot.Reset();
			NextStep = EStep::SendDelta;
			break;
		}

		case EStep::SendDelta:
		{
			if (CurrentCommunicationStatus.TaskTotal == CurrentCommunicationStatus.TaskCompleted)
			{
				NextStep = EStep::Synced;
				return Tick(Now_s);
			}
			else
			{
				UE_LOGF(LogDirectLinkNet, Verbose, "diff: waiting for receiver ack (%d/%d)"
					, CurrentCommunicationStatus.TaskCompleted
					, CurrentCommunicationStatus.TaskTotal
				);
			}
			break;
		}

		case EStep::Synced:
		{
			UE_LOGF(LogDirectLinkNet, Verbose, "diff: sync complete! (cycle %d)", SyncCycle);

			CurrentCommunicationStatus.bIsSending = false;
			CurrentCommunicationStatus.bIsReceiving = false;

			NextStep = EStep::Idle;
			return Tick(Now_s);
		}

		default:
			ensure(false);
	}
}


void FHaveListReceiver::HandleMessage(const FDirectLinkMsg_HaveListMessage& Message)
{
	if (Message.SyncCycle != SyncCycle)
	{
		UE_LOGF(LogDirectLinkNet, Warning, "Dropped FHaveListReceiver message, expected syncCycle:%d, Received SyncCycle:%d",
			SyncCycle, Message.SyncCycle);
		return;
	}
	if (bClosed)
	{
		UE_LOGF(LogDirectLinkNet, Warning, "Dropped FHaveListReceiver message, closed FHaveListReceiver struct");
		return;
	}

	if (NextOrderedMessage == Message.MessageCode)
	{
		HandleOrderedMessage(Message);
		NextOrderedMessage++;
		FDirectLinkMsg_HaveListMessage NextMessage;
		while(Unordered.RemoveAndCopyValue(NextOrderedMessage, NextMessage))
		{
			HandleOrderedMessage(MoveTemp(NextMessage));
			NextOrderedMessage++;
		}
	}
	else
	{
		Unordered.Add(Message.MessageCode, Message);
	}
}


void FHaveListReceiver::HandleOrderedMessage(const FDirectLinkMsg_HaveListMessage& Message)
{
	switch (FDirectLinkMsg_HaveListMessage::EKind(Message.Kind))
	{
		case FDirectLinkMsg_HaveListMessage::EKind::OpenHaveList:
		{
			FSceneIdentifier HaveSceneId;
			bool bKeepPreviousContent = false;
			FMemoryReader Ar(Message.Payload);
			Ar << HaveSceneId;
			Ar << bKeepPreviousContent;
			RemoteScene->SetSceneId(HaveSceneId);
			break;
		}

		case FDirectLinkMsg_HaveListMessage::EKind::HaveListElement:
		{
			uint32 ElementCount = FMath::Min(Message.NodeIds.Num(), Message.Hashes.Num());
			for (uint32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
			{
				RemoteScene->GetHashRef(Message.NodeIds[ElementIndex]) = Message.Hashes[ElementIndex];
			}

			break;
		}

		case FDirectLinkMsg_HaveListMessage::EKind::CloseHaveList:
		{
			SyncCycle = 0;
			bClosed = true;
			break;
		}

		case FDirectLinkMsg_HaveListMessage::EKind::None:
		default:
			ensure(false);
			break;
	}
}


} // namespace DirectLink

