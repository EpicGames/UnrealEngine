// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkScenePipe.h"
#include "DirectLinkStreamCommunicationInterface.h"
#include "DirectLinkSceneSnapshot.h"

#include "CoreTypes.h"
#include "Containers/SortedMap.h"


namespace DirectLink
{
class FRemoteSceneView;
class FHaveListReceiver;

/**
 * This is used to sync a Stream over MessageBus. See also: FStreamReceiver
 *
 * It keeps an Hash table of what the remote receiver already have, and diff with that.
 * There is no handling of bad connection in this class. We accept arbitrary delays
 * that can arise with remote slow operation (file load, breakpoint...).
 * Some requests messages can be sent multiple times though, but with a unique
 * 'SyncCycle' value so that the receiver is able to ignore duplicated requests.
 */
class FStreamSender : public IStreamSender
{
public:
	FStreamSender(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ThisEndpoint, const FMessageAddress& DestinationAddress, FStreamPort ReceiverStreamPort);
	~FStreamSender();
	virtual void SetSceneSnapshot(TSharedPtr<FSceneSnapshot> SceneSnapshot) override;
	virtual void Tick(double Now_s) override;
	virtual void HandleHaveListMessage(const FDirectLinkMsg_HaveListMessage& Message) override; // update RemoteView

public: // IStreamCommunicationInterface API
	virtual FCommunicationStatus GetCommunicationStatus() const override { return CurrentCommunicationStatus; }

private:
	enum class EStep
	{
		Idle,
		SetupScene,
		ReceiveHaveList,
		GenerateDelta,
		SendDelta,
		Synced,
	};

	EStep NextStep = EStep::Idle;
	int32 SyncCycle = 0;

	FScenePipeToNetwork PipeToNetwork;
	TUniquePtr<FHaveListReceiver> HaveListReceiver;
	double LastHaveListMessage_s = 0;

	TSharedPtr<FSceneSnapshot> Snapshot;
	FCriticalSection NextSnapshotLock;
	TSharedPtr<FSceneSnapshot> NextSnapshot;

	TUniquePtr<FRemoteSceneView> RemoteScene;

	// Reporting
	FCommunicationStatus CurrentCommunicationStatus;
};


} // namespace DirectLink
