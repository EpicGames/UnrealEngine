// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkStreamSource.h"

#include "DirectLinkMisc.h"
#include "DirectLinkSceneSnapshot.h"
#include "DirectLinkStreamCommunicationInterface.h"
#include "Misc/ScopeRWLock.h"


namespace DirectLink
{

void FStreamSource::SetRoot(ISceneGraphNode* InRoot)
{
	Root = InRoot;
}


void FStreamSource::Snapshot()
{
	TSharedPtr<FSceneSnapshot> NewSnapshot = SnapshotScene(Root);

	DumpSceneSnapshot(*NewSnapshot.Get(), TEXT("source"));

	{
		FRWScopeLock _(CurrentSnapshotLock, SLT_Write);
		CurrentSnapshot = NewSnapshot;
	}

	{
		FRWScopeLock _(SendersLock, SLT_ReadOnly);
		for (TSharedPtr<IStreamSender>& Sender : Senders)
		{
			Sender->SetSceneSnapshot(NewSnapshot);
		}
	}
}


void FStreamSource::LinkSender(const TSharedPtr<IStreamSender>& Sender)
{
	if (ensure(Sender))
	{
		{
			FRWScopeLock _(SendersLock, SLT_Write);
			Senders.Add(Sender);
		}

		{
			FRWScopeLock _(CurrentSnapshotLock, SLT_ReadOnly);
			Sender->SetSceneSnapshot(CurrentSnapshot);
		}
	}
}


} // namespace DirectLink

