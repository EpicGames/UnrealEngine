// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkCommon.h"
#include "DirectLinkStreamConnectionPoint.h"


namespace DirectLink
{
class FSceneSnapshot;
class ISceneGraphNode;
class IStreamSender;


/**
 * Define a content source.
 * A source is linked to N Destinations through Streams, and uses Senders to write on them.
 */
class FStreamSource : public FStreamConnectionPoint
{
public:
	FStreamSource(const FString& Name, EVisibility Visibility)
		: FStreamConnectionPoint(Name, Visibility)
	{}

	// Defines the content, which is a root node and its referenced tree.
	void SetRoot(ISceneGraphNode* InRoot);

	// Snapshot the current state of the scene
	void Snapshot();

	// Link a stream to this source (via a sender)
	void LinkSender(const TSharedPtr<IStreamSender>& Sender);

private:
	ISceneGraphNode* Root = nullptr;
	FRWLock SendersLock;
	TArray<TSharedPtr<IStreamSender>> Senders;
	FRWLock CurrentSnapshotLock;
	TSharedPtr<FSceneSnapshot> CurrentSnapshot;
};


} // namespace DirectLink
