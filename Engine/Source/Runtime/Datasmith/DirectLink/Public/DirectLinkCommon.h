// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"


namespace DirectLink
{

/**
 * Node Id, aka Element Id. Represent a node within a scene.
 * As a scene has a guid, the combination guid/id must be globally unique.
 */
using FSceneGraphId = uint32;
constexpr FSceneGraphId InvalidId = 0;


using FElementHash = uint32;
constexpr FElementHash InvalidHash = 0;


using FStreamPort = uint32;
constexpr FStreamPort InvalidStreamPort = 0;

/**
 * Guid and optional name, used to designate a scene across processes without ambiguity.
 * The name is not necessary to identify a scene, but it offer a better UX
 */
struct FSceneIdentifier
{
	FSceneIdentifier() = default;

	FSceneIdentifier(const FGuid& Id, const FString& Name)
		: SceneGuid(Id)
		, DisplayName(Name)
	{}

	// Id of scene SharedState
	FGuid SceneGuid;

	// Nice user-facing name. Do not expect it to be stable or consistent.
	FString DisplayName;
};


// DirectLink exchanges messages between pairs. Those versions numbers helps making sure pairs are compatible
uint8 GetCurrentProtocolVersion();

// oldest supported version
uint8 GetMinSupportedProtocolVersion();



/** Used by data source and destination to describe how they are discovered by remote endpoints */
enum class EVisibility
{
	Public,    // The connection point can accept connection requests from remote
	Private,   // The connection point is not expected to be contacted from a remote
};

enum class EStreamConnectionState
{
	Uninitialized,
	RequestSent,
	Active,
	Closed,
};

struct FCommunicationStatus
{
	bool IsTransmitting() const { return bIsSending || bIsReceiving; }
	bool IsProgressKnown() const { return TaskTotal > 0; };
	float GetProgress() const { return IsProgressKnown() ? float(FMath::Clamp(TaskCompleted, 0, TaskTotal)) / TaskTotal : 0.0f; };

public:
	bool bIsSending = false;
	bool bIsReceiving = false;
	int32 TaskTotal = 0;
	int32 TaskCompleted = 0;
};

} // namespace DirectLink
