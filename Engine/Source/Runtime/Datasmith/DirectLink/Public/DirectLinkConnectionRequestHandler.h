// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkCommon.h"
#include "DirectLinkDeltaConsumer.h"



namespace DirectLink
{

/**
 * In DirectLink, Source points can request connections on Destinations points.
 * For each destination, an instance of this class is used to accept/refuse incoming
 * connections requests, and provide a SceneReceiver associated to each accepted Source.
 * Each stream (pair Source-Destination) must have a distinct SceneReceiver.
 */
class IConnectionRequestHandler
{
public:
	struct FSourceInformation
	{
		FGuid Id;
	};

public:
	virtual ~IConnectionRequestHandler() = default;

	/**
	 * @param Source    Information about the incoming Source
	 * @return          whether the source can be accepted as input of the Destination
	 */
	virtual bool CanOpenNewConnection(const FSourceInformation& Source) = 0;

	/**
	 * @param Source    Information about the incoming Source
	 * @return          DeltaConsumer dedicated for this source that will receive Delta information from the source
	 */
	virtual TSharedPtr<ISceneReceiver> GetSceneReceiver(const FSourceInformation& Source) = 0;
};


} // namespace DirectLink

