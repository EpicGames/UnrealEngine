// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkCommon.h"


namespace DirectLink
{
class FElementSnapshot;

class IDeltaProducer
{
public:
	virtual ~IDeltaProducer() = default;

	virtual void OnOpenHaveList(const FSceneIdentifier& HaveSceneId, bool bKeepPreviousContent, int32 SyncCycle) = 0;
	virtual void OnHaveElement(FSceneGraphId NodeId, FElementHash HaveHash) = 0;
	virtual void OnCloseHaveList() = 0;
};



class IDeltaConsumer
{
public:
	virtual ~IDeltaConsumer() = default;

	virtual void SetDeltaProducer(IDeltaProducer* Producer) = 0;

	// On SetupScene message, the receiver is expected to send it's HaveList
	struct FSetupSceneArg
	{
		FSceneIdentifier SceneId;
		bool bExpectHaveList;
		int32 SyncCycle;
	};
	virtual void SetupScene(FSetupSceneArg& SetupSceneArg) = 0;


	// signal beginning of a delta
	struct FOpenDeltaArg
	{
		bool bBasedOnNewScene; // start from a fresh scene. (expect only new content)
		uint32 ElementCountHint = 0;
	};
	virtual void OpenDelta(FOpenDeltaArg& OpenDeltaArg) = 0;


	struct FSetElementArg
	{
		TSharedPtr<FElementSnapshot> Snapshot;
		int32 ElementIndexHint = -1;
	};
	virtual void OnSetElement(FSetElementArg& SetElementArg) = 0;


	struct FRemoveElementsArg
	{
		TArray<FSceneGraphId> Elements;
	};
	virtual void RemoveElements(FRemoveElementsArg& RemoveElementsArg) = 0;


	struct FCloseDeltaArg
	{
		bool bCancelled = false; // if an error occured and the delta is unusable
	};
	virtual void OnCloseDelta(FCloseDeltaArg& CloseDeltaArg) = 0;
};



class ISceneReceiver
{
public:
	virtual ~ISceneReceiver() = default;
	virtual void FinalSnapshot(const class FSceneSnapshot& SceneSnapshot) {};
};


} // namespace DirectLink
