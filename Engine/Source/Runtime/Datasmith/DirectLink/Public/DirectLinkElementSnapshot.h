// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DirectLinkCommon.h"
#include "DirectLinkParameterStore.h"
#include "UObject/NameTypes.h"

#define UE_API DIRECTLINK_API

class FArchive;

namespace DirectLink
{
class IReferenceResolutionProvider;
class ISceneGraphNode;


enum class ESerializationStatus
{
	Ok,
	StreamError,
	VersionMinNotRespected,
	VersionMaxNotRespected,
};


struct FReferenceSnapshot
{
	UE_API void Serialize(FArchive& Ar);
	UE_API FElementHash Hash() const;

	struct FReferenceGroup
	{
		FName Name;
		TArray<FSceneGraphId> ReferencedIds;
	};

	TArray<FReferenceGroup> Groups;
};



class FElementSnapshot
{
public:
	FElementSnapshot() = default;
	UE_API FElementSnapshot(const ISceneGraphNode& Node);

	friend FArchive& operator<<(FArchive& Ar, FElementSnapshot& This);

	UE_API ESerializationStatus Serialize(FArchive& Ar);

	UE_API FElementHash GetHash() const;
	UE_API FElementHash GetDataHash() const; // #ue_directlink_sync: serialize hashs
	UE_API FElementHash GetRefHash() const;

	UE_API void UpdateNodeReferences(IReferenceResolutionProvider& Resolver, ISceneGraphNode& Node) const;
	UE_API void UpdateNodeData(ISceneGraphNode& Node) const;

	FSceneGraphId GetNodeId() const { return NodeId; }

	template<typename T>
	bool GetValueAs(FName Name, T& Out) const
	{
		return DataSnapshot.GetValueAs(Name, Out);
	}

private:
	FSceneGraphId NodeId;
	mutable FElementHash DataHash = InvalidHash;
	mutable FElementHash RefHash = InvalidHash;
	FParameterStoreSnapshot DataSnapshot;
	FReferenceSnapshot RefSnapshot;
};

} // namespace DirectLink

#undef UE_API
