// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DirectLinkCommon.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IMessageContext.h"
#include "Misc/EngineVersion.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>

#define UE_API DIRECTLINK_API

class FMessageEndpoint;

namespace DirectLink
{
class ISceneGraphNode;


enum class ECommunicationStatus{
	NoIssue                      = 0,
	ModuleNotLoaded_Messaging    = 1<<0,
	ModuleNotLoaded_UdpMessaging = 1<<1,
	ModuleNotLoaded_Networking   = 1<<2,
};
ENUM_CLASS_FLAGS(ECommunicationStatus)
DIRECTLINK_API ECommunicationStatus ValidateCommunicationStatus();

struct FRawInfo
{
	struct FDataPointId
	{
		FString Name;
		FGuid Id;
		bool bIsPublic = false;
	};

	struct FEndpointInfo
	{
		FEndpointInfo() = default;
		FString Name;
		FEngineVersion Version; // Unreal version. This field is empty when the client predates 4.27.0
		TArray<FDataPointId> Destinations;
		TArray<FDataPointId> Sources;
		FString UserName;
		FString ExecutableName;
		FString ComputerName;
		bool bIsLocal = false;
		uint32 ProcessId = 0;
	};

	struct FDataPointInfo
	{
		FMessageAddress EndpointAddress;
		FString Name;
		bool bIsSource = false; // as opposed to a destination
		bool bIsOnThisEndpoint = false;
		bool bIsPublic = false; // if public, can be displayed as candidate for connection
	};

	struct FStreamInfo
	{
		FStreamPort StreamId = InvalidStreamPort;
		FGuid Source;
		FGuid Destination;
		EStreamConnectionState ConnectionState;
		FCommunicationStatus CommunicationStatus;
	};
	FMessageAddress ThisEndpointAddress;
	TMap<FMessageAddress, FEndpointInfo> EndpointsInfo;
	TMap<FGuid, FDataPointInfo> DataPointsInfo;
	TArray<FStreamInfo> StreamsInfo;
};


class IEndpointObserver
{
public:
	virtual ~IEndpointObserver() = default;

	virtual void OnStateChanged(const FRawInfo& RawInfo) {}
};


using FSourceHandle = FGuid;
using FDestinationHandle = FGuid;



/**
 * FEndpoint class is the main interface for sending and receiving data with
 * the DirectLink system.
 *
 * Instances of this class discover themselves (through MessageBus) and can see
 * Sources and Destinations available on each others. This is automatic and
 * works for multiple applications. With that system, one can expose data from
 * an application and consume it from another.
 *
 * An endpoint can exposes N sources and N Destinations. That being said, a
 * more classic setup is to have an 'exporter' process that only exposes
 * sources, and some 'consumer' process that only have Destinations.
 *
 * As a convention, 'consumers' have the responsibility to handle the
 * connections management. Exporters have the sole responsibility to exposes
 * their data.
 *
 * Setup example:
 *  - Exporter process
 *    - Source "My First Source"
 *    - Source "My Second Source"
 *
 *  - Consumer process
 *    - Destination "Viewport"
 *
 * The consumer process can handle the connections between Sources and
 * Destinations with the OpenStream and CloseStream methods.
 */
class FEndpoint
	: public FNoncopyable
{
public:
	enum class EOpenStreamResult
	{
		Opened,
		AlreadyOpened,
		SourceAndDestinationNotFound,
		RemoteEndpointNotFound,
		Unsuppported,
		CannotConnectToPrivate,
	};

public:
	UE_API FEndpoint(const FString& InName);
	UE_API ~FEndpoint();

	UE_API void SetVerbose(bool bVerbose=true);

	/**
	 * Add a Source to that endpoint. A source can hold content (a scene snapshot)
	 * and is able to stream that content to remote destinations.
	 * @param Name          User facing name for this source.
	 * @param Visibility    Whether that Source is visible to remote endpoints
	 * @return              A Handle required by other Source related methods
	 */
	UE_API FSourceHandle AddSource(const FString& Name, EVisibility Visibility=EVisibility::Public);

	/**
	 * Closes all streams related to this Source and remove it from the endpoint.
	 * @param Source Handle of the source to remove
	 */
	UE_API void RemoveSource(const FSourceHandle& Source);

	/**
	 * Set the root of the content that should be exposed from that source.
	 * @param Source        Handle of the source to setup
	 * @param InRoot        Root of the graph that should be exposed
	 * @param bSnapshot     Whether the graph should be snapshotted right away
	 */
	UE_API void SetSourceRoot(const FSourceHandle& Source, ISceneGraphNode* InRoot, bool bSnapshot);

	/**
	 * Use the Source root to discover the graph and snapshot the content in its current state.
	 * @param Source        Handle of the source to snapshot
	 */
	UE_API void SnapshotSource(const FSourceHandle& Source);


	/**
	 * Add a Destination on that endpoint
	 *
	 * @param Name          Name of that destination
	 * @param Visibility    Whether this destination should be visible to remote endpoints
	 * @param ConnectionRequestHandler Object that will handle connections requests from Sources
	 * @return              A handle to the new destination
	 */
	UE_API FDestinationHandle AddDestination(const FString& Name, EVisibility Visibility, const TSharedPtr<class IConnectionRequestHandler>& ConnectionRequestHandler);

	/**
	 * Disconnect all streams related to that destination and removes the
	 * Destination itself from that endpoint
	 * @param Destination   Handle of the destination to remove
	 */
	UE_API void RemoveDestination(const FDestinationHandle& Destination);

	/**
	 * Get a snapshot of the current state of the DirectLink swarm. It includes
	 * discovered Endpoints, their sources and destinations, and the streams
	 * descriptions.
	 * @return              Description of the last state of swarm
	 */
	UE_API FRawInfo GetRawInfoCopy() const;

	/**
	 * Register an IEndpointObserver that will be notified periodically with the last state of the swarm.
	 * @param Observer      Object that should be notified
	 */
	UE_API void AddEndpointObserver(IEndpointObserver* Observer);

	/**
	 * Removes a previously added observer
	 * @param Observer      Observer to remove
	 */
	UE_API void RemoveEndpointObserver(IEndpointObserver* Observer);

	/**
	 * Open a Stream between a Source and a Destination
	 * @param SourceId      Handle to the Source
	 * @param DestinationId Handle to the Destination
	 * @return              Whether the stream is correctly opened
	 */
	UE_API EOpenStreamResult OpenStream(const FSourceHandle& SourceId, const FDestinationHandle& DestinationId);

	/**
	 * Close a previously opened stream
	 * @param SourceId      Handle to the Source
	 * @param DestinationId Handle to the Destination
	 */
	UE_API void CloseStream(const FSourceHandle& SourceId, const FDestinationHandle& DestinationId);

private:
	TUniquePtr<class FSharedState> SharedStatePtr;
	class FSharedState& SharedState;

	TUniquePtr<class FInternalThreadState> InternalPtr;
	class FInternalThreadState& Internal;
};

} // namespace DirectLink

#undef UE_API
