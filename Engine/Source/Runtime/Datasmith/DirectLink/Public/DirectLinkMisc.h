// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

class IDatasmithElement;
class IDatasmithScene;

namespace DirectLink
{
class FSceneSnapshot;

DIRECTLINK_API const FString& GetDumpPath();

DIRECTLINK_API void DumpSceneSnapshot(FSceneSnapshot& SceneSnapshot, const FString& BaseFileName);
} // namespace DirectLink
