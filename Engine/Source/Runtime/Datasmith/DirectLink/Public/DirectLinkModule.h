// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DIRECTLINK_MODULE_NAME TEXT("DirectLink")

/**
 * The DirectLink system enable multiple application to send and receive data
 * over the network. This is a N to N system that can be used to broadcast or
 * consume content.
 * The main components of that systems:
 *
 * - Endpoint (DirectLink::FEndpoint class)
 * Represent a node on the network, capable of discovering other nodes.
 * It is the main interface for that systems.
 *
 * - Source
 * Declared on an Endpoint, a Source is a Named handle that can hold content.
 *
 * - Destination
 * Similar to a Source, it is a named handle declared on an endpoint, capable
 * of receiving content.
 *
 * - Stream (aka. connection)
 * A stream represents a connection between a Source and a Destination.
 * When a stream is established, the content of the source is kept in sync on
 * the Destination.
 *
 * - Snapshot
 * DirectLink works with snapshots. When a Source is snapshotted, the content
 * graph is discovered from the root, and each reached node is serialized. This
 * snapshot is the content that will be streamed to connected destinations.
 */
class FDirectLinkModule : public IModuleInterface
{
public:
	static inline FDirectLinkModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FDirectLinkModule>(DIRECTLINK_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DIRECTLINK_MODULE_NAME);
	}

	virtual void StartupModule() override;
};
