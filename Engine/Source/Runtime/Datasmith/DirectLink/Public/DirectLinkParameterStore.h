// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "DirectLinkCommon.h"
#include "DirectLinkSceneGraphNode.h"
#include "DirectLinkSerialMethods.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/MemoryReader.h"
#include "UObject/NameTypes.h"

#define UE_API DIRECTLINK_API

class FArchive;


namespace DirectLink
{
class FParameterStore;

template<typename T, typename S = T>
class TStoreKey // rename TReflected ?
{
public:
	TStoreKey(const T& InitialValue = {})
		: NativeValue(InitialValue)
	{}

	const T& Get() const { return NativeValue; }
	T& Get() { return NativeValue; }
	operator const T&() const { return NativeValue; }
	operator T&() { return NativeValue; }

	T& operator=(const T& InValue)
	{
		NativeValue = InValue;
		return NativeValue;
	}

private:
	friend FParameterStore;
	T NativeValue;
	// #ue_directlink_quality dtr: check it was registered
};



/**
 * Diffable, and serializable to a buffer
 */
class FParameterStoreSnapshot
{
public:
	UE_API void SerializeAll(FArchive& Ar);

	template<typename T>
	bool GetValueAs(int32 I, T& Out) const
	{
		if (Parameters.IsValidIndex(I))
		{
			if (Reflect::CanSerializeWithMethod<T>(Parameters[I].StorageMethod))
			{
				FMemoryReader Ar(Parameters[I].Buffer);
				return Reflect::SerialAny(Ar, &Out, Parameters[I].StorageMethod);
			}
		}
		return false;
	}

	template<typename T>
	bool GetValueAs(FName Name, T& Out) const { return GetValueAs(GetParameterIndex(Name), Out); }

	int32 GetParameterIndex(FName ParameterName) const
	{
		return Parameters.IndexOfByPredicate([&](const FParameterDetails& Parameter) {
			return Parameter.Name == ParameterName;
		});
	}

	UE_API void AddParam(FName Name, Reflect::ESerialMethod StorageMethod, void* StorageLocation);

	void ReserveParamCount(uint32 PropCount)
	{
		Parameters.Reserve(PropCount);
	}

	UE_API FElementHash Hash() const;

private:
	friend class FParameterStore; // for FParameterStore::Update
	struct FParameterDetails
	{
		FName Name;
		Reflect::ESerialMethod StorageMethod;
		TArray<uint8> Buffer;
	};
	TArray<FParameterDetails> Parameters;
};



class FParameterStore
{
public:
	UE_API FParameterStore();
	UE_API ~FParameterStore();
	UE_API FParameterStore(const FParameterStore&);
	UE_API FParameterStore& operator=(const FParameterStore&);
	UE_API FParameterStore(FParameterStore&&);
	UE_API FParameterStore& operator=(FParameterStore&&);

	template<typename T, typename S>
	TStoreKey<T, S>& RegisterParameter(TStoreKey<T, S>& Key, FName Name)
	{
		check(HasParameterNamed(Name) == false);

		FParameterDetails& NewParameter = Parameters.AddDefaulted_GetRef();

		NewParameter.Name = Name;
		NewParameter.StorageLocation = &Key.NativeValue;
		static_assert(Reflect::TDefaultSerialMethod<S>::Value != Reflect::ESerialMethod::_NotImplementedYet, "Key type not exposed to serialization");
		NewParameter.StorageMethod = Reflect::TDefaultSerialMethod<S>::Value;

		return Key;
	}

	UE_API uint32 GetParameterCount() const;
	UE_API int32 GetParameterIndex(FName Name) const;
	UE_API bool HasParameterNamed(FName Name) const;
	UE_API FName GetParameterName(int32 Index) const;

	template<typename T>
	bool GetValueAs(FName ParameterName, T& Out) const
	{
		int32 I = GetParameterIndex(ParameterName);
		if (Parameters.IsValidIndex(I))
		{
			if (Reflect::CanSerializeWithMethod<T>(Parameters[I].StorageMethod))
			{
				Out = *(T*)(Parameters[I].StorageLocation);
				return true;
			}
		}
		return false;
	}

	UE_API FParameterStoreSnapshot Snapshot() const;

	UE_API void Update(const FParameterStoreSnapshot& NewValues);

private:
	struct FParameterDetails
	{
		FName Name;
		void* StorageLocation;
		Reflect::ESerialMethod StorageMethod;
	};
	TArray<FParameterDetails> Parameters;
};


class FSnapshotProxy
{
public:
	FSnapshotProxy(FParameterStoreSnapshot& Storage, bool bIsSaving)
		: Storage(Storage)
		, bIsSaving(bIsSaving)
	{}

	bool IsSaving() const { return bIsSaving; }

	template<typename T>
	bool TagSerialize(FName SerialTag, T& Item)
	{
		if (bIsSaving)
		{
			Storage.AddParam(SerialTag, Reflect::TDefaultSerialMethod<T>::Value, &Item);
			return true;
		}
		else
		{
			return Storage.GetValueAs(SerialTag, Item);
		}
	}

private:
	FParameterStoreSnapshot& Storage;
	const bool bIsSaving;
};

} // namespace DirectLink

#undef UE_API
