// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DirectLinkCommon.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API DIRECTLINK_API


namespace DirectLink
{
class FParameterStore;
class FSnapshotProxy;
class IReferenceProxy;
struct FReferenceSnapshot;


/**
 * Data shared by all element of a given scene.
 * The scene is uniquely identified by this element.
 * Within this scene, all elements ids are unique. To ensure this property,
 * this shared state is responsible for the id attribution.
 * Id 0 is considered invalid (see InvalidId).
 */
class FSceneGraphSharedState
{
public:
	FSceneGraphId MakeId() { return ++LastElementId; }
	const FGuid& GetGuid() const { return SceneId.SceneGuid; }
	const FSceneIdentifier& GetSceneId() const { return SceneId; }

protected:
	FSceneGraphId LastElementId = InvalidId;
	FSceneIdentifier SceneId{FGuid::NewGuid(), FString()};
};


/**
 * Represents a scene part.
 * #ue_directlink_doc: full doc pass
 * 	- sharedState concept
 *
 * Notes:
 * - replication of references via DirectLink::FReferenceSnapshot
 * - replication of data by the Store (DirectLink::FParameterStore)
 */
class ISceneGraphNode
{
public:
	virtual ~ISceneGraphNode() = default;

	/// Related nodes share a common object
	virtual TSharedPtr<FSceneGraphSharedState> MakeSharedState() const { return MakeShared<FSceneGraphSharedState>(); }
	virtual TSharedPtr<FSceneGraphSharedState> GetSharedState() const { return SharedState; }
	UE_API void SetSharedState(TSharedPtr<FSceneGraphSharedState> NewSharedState);

	/// data
	virtual const FParameterStore& GetStore() const = 0;
	virtual FParameterStore& GetStore() = 0; // protected ?

	virtual void CustomSerialize(class FSnapshotProxy& Ar) {}
	void CustomSerialize(class FSnapshotProxy& Ar) const { const_cast<ISceneGraphNode*>(this)->CustomSerialize(Ar); }

	/// References
	// This Id is unique within a SceneGraph
	FSceneGraphId GetNodeId() const { return SceneGraphId; }
	//private: friend class ...  #ue_directlink_design -> only accessible from the scene that owns the dep graph
	void SetNodeId(FSceneGraphId Id) { SceneGraphId = Id; }

	UE_API void RegisterReferenceProxy(class IReferenceProxy& View, FName Name);
	UE_API int32 GetReferenceProxyCount() const;
	UE_API class IReferenceProxy* GetReferenceProxy(int32 Index) const;
	UE_API FName GetReferenceProxyName(int32 Index) const;

	UE_API FSceneGraphId RegisterReference(ISceneGraphNode* Referenced);
	UE_API void UpdateRefs(class IReferenceResolutionProvider& Resolver, const FReferenceSnapshot& NewRefs);
	// #ue_directlink_design is node responsible for ref serialization?

private:
	FSceneGraphId SceneGraphId = 0;
	TSharedPtr<FSceneGraphSharedState> SharedState;

	struct FNamedReferenceProxy
	{
		FName Name;
		IReferenceProxy* View;
	};
	TArray<FNamedReferenceProxy> ReferenceProxies;
};

class IReferenceResolutionProvider // #ue_directlink_design: improve: ClaimRef in the ds shared state
{
public:
	virtual ~IReferenceResolutionProvider() = default;
	virtual TSharedPtr<ISceneGraphNode> AsSharedPtr(FSceneGraphId NodeId) { return nullptr; }
};

class IReferenceProxy
{
public:
	virtual ~IReferenceProxy() = default;
	virtual int32 Num() const = 0;
	virtual ISceneGraphNode* GetNode(int32 Index) const = 0;
	virtual void SetNodes(IReferenceResolutionProvider& ResolutionProvider, const TArray<FSceneGraphId>& NodeIds) = 0;
};

} // namespace DirectLink

#undef UE_API
