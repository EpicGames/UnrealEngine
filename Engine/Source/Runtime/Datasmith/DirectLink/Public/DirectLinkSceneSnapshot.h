// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "DirectLinkCommon.h"
#include "Templates/SharedPointer.h"


namespace DirectLink
{
class FElementSnapshot;
class ISceneGraphNode;


class FSceneSnapshot
{
public:
	TMap<FSceneGraphId, TSharedRef<FElementSnapshot>> Elements;
	FSceneIdentifier SceneId;
};


TSet<ISceneGraphNode*> DIRECTLINK_API BuildIndexForScene(ISceneGraphNode* RootElement);
TSharedPtr<FSceneSnapshot> DIRECTLINK_API SnapshotScene(ISceneGraphNode* RootElement);


} // namespace DirectLink
