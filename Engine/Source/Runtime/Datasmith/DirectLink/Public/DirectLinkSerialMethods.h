// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/SecureHash.h"
#include "Serialization/Archive.h"


namespace Reflect
{

// clang-format off
enum EStoreType : uint8
{
	_bool = 1,
	_i8, _i16, _i32, _i64,
	_u8, _u16, _u32, _u64,
	_f32, _f64,
	_str,
	_vector, _vector4, _quat,
	_linearcolor,
	_md5hash,
	_transform,
	_storeTypeLast
};

enum ESerialModifier : uint8
{
	_default = 0 << 5,
	_array   = 1 << 5,
	_alt1    = 2 << 5,
};

static constexpr uint8 _storeTypeMask = 0b0001'1111u;
static constexpr uint8 _modifierMask  = 0b1110'0000u;
static_assert((_storeTypeLast & _modifierMask) == 0, "enum overlap");

enum class ESerialMethod : uint8
{
	None               = 0,
	Bool_Default       = _bool       | _default,
	Uint8_Default      = _u8         | _default,
	Int32_Default      = _i32        | _default,
	Int32_Array        = _i32        | _array,
	Uint32_Default     = _u32        | _default,
	Uint64_Default     = _u64        | _default,
	Uint32_Packed      = _u32        | _alt1,
	String_Default     = _str        | _default,
	String_Array       = _str        | _array,
	Float_Default      = _f32        | _default,
	Float_Array        = _f32        | _array,
	Double_Default     = _f64        | _default,
	Vector_f32         = _vector     | _default, // FVector serialized as FVector3f
	Vector_f64         = _vector     | _alt1,    // FVector serialized as FVector3d (native since 5.0)
	Quat_f32           = _quat       | _default, // FQuat serialized as FQuat4f
	Quat_f64           = _quat       | _alt1,    // FQuat serialized as FQuat4d (native since 5.0)
	LinearColor_Default= _linearcolor| _default,
	MD5Hash_Default    = _md5hash    | _default,
	Transform_Default  = _transform  | _default, // Added in 5.0, uses LWC.
	Transform_Array    = _transform  | _array,   // Added in 5.0, uses LWC.

	_NotImplementedYet = 0xff
};


static constexpr EStoreType GetStoreType(ESerialMethod Method) { return EStoreType(uint8(Method) & _storeTypeMask); }

// This list maps which serialization method should be used for each Type reflected by directlink
template<typename T> struct TDefaultSerialMethod           { constexpr static ESerialMethod Value = ESerialMethod::_NotImplementedYet; };
template<> struct TDefaultSerialMethod<bool>               { constexpr static ESerialMethod Value = ESerialMethod::Uint8_Default;      };
template<> struct TDefaultSerialMethod<uint8>              { constexpr static ESerialMethod Value = ESerialMethod::Uint8_Default;      };
template<> struct TDefaultSerialMethod<int32>              { constexpr static ESerialMethod Value = ESerialMethod::Int32_Default;      };
template<> struct TDefaultSerialMethod<uint32>             { constexpr static ESerialMethod Value = ESerialMethod::Uint32_Default;     };
template<> struct TDefaultSerialMethod<uint64>             { constexpr static ESerialMethod Value = ESerialMethod::Uint64_Default;     };
template<> struct TDefaultSerialMethod<FString>            { constexpr static ESerialMethod Value = ESerialMethod::String_Default;     };
template<> struct TDefaultSerialMethod<float>              { constexpr static ESerialMethod Value = ESerialMethod::Float_Default;      };
template<> struct TDefaultSerialMethod<double>             { constexpr static ESerialMethod Value = ESerialMethod::Double_Default;     };
template<> struct TDefaultSerialMethod<TArray<int32>>      { constexpr static ESerialMethod Value = ESerialMethod::Int32_Array;        };
template<> struct TDefaultSerialMethod<TArray<FString>>    { constexpr static ESerialMethod Value = ESerialMethod::String_Array;       };
template<> struct TDefaultSerialMethod<TArray<float>>      { constexpr static ESerialMethod Value = ESerialMethod::Float_Array;        };
template<> struct TDefaultSerialMethod<FLinearColor>       { constexpr static ESerialMethod Value = ESerialMethod::LinearColor_Default;};
template<> struct TDefaultSerialMethod<FMD5Hash>           { constexpr static ESerialMethod Value = ESerialMethod::MD5Hash_Default;    };
template<> struct TDefaultSerialMethod<FTransform>         { constexpr static ESerialMethod Value = ESerialMethod::Transform_Default;  };
template<> struct TDefaultSerialMethod<TArray<FTransform>> { constexpr static ESerialMethod Value = ESerialMethod::Transform_Array;    };
template<> struct TDefaultSerialMethod<FVector>            { constexpr static ESerialMethod Value = ESerialMethod::Vector_f64;         };
template<> struct TDefaultSerialMethod<FQuat>              { constexpr static ESerialMethod Value = ESerialMethod::Quat_f64;           };


template<typename T> EStoreType GetStoreTypeForType() { return GetStoreType(TDefaultSerialMethod<T>::Value); }
template<typename T> bool CanSerializeWithMethod(ESerialMethod Method) { return GetStoreTypeForType<T>() == GetStoreType(Method); }


// Serialization implementations
template<ESerialMethod Code, typename T>
void Serial(FArchive& Ar, T* Param) { Ar << *Param; }

template<> inline void Serial<ESerialMethod::Uint32_Packed>(FArchive& Ar, uint32* ValuePtr) { Ar.SerializeIntPacked(*ValuePtr); }

template<> inline void Serial<ESerialMethod::Vector_f32>(FArchive& Ar, FVector* ValuePtr)
{
	if (Ar.IsLoading())
	{
		FVector3f Tmp;
		Ar << Tmp;
		*ValuePtr = FVector(Tmp);
	}
	else if (Ar.IsSaving())
	{
		FVector3f Tmp(*ValuePtr);
		Ar << Tmp;
	}
}

template<> inline void Serial<ESerialMethod::Quat_f32>(FArchive& Ar, FQuat* ValuePtr)
{
	if (Ar.IsLoading())
	{
		FQuat4f Tmp;
		Ar << Tmp;
		*ValuePtr = FQuat(Tmp);
	}
	else if (Ar.IsSaving())
	{
		FQuat4f Tmp(*ValuePtr);
		Ar << Tmp;
	}
}


inline bool SerialAny(FArchive& Ar, void* data, ESerialMethod Method)
{
#define SerialAny_Case(enumvalue, casttype) case enumvalue: Serial<enumvalue>(Ar, (casttype*)data); return true;
	switch (Method)
	{
		SerialAny_Case(ESerialMethod::Bool_Default       , bool              );
		SerialAny_Case(ESerialMethod::Uint8_Default      , uint8             );
		SerialAny_Case(ESerialMethod::Int32_Default      , int32             );
		SerialAny_Case(ESerialMethod::Uint32_Default     , uint32            );
		SerialAny_Case(ESerialMethod::Uint64_Default     , uint64            );
		SerialAny_Case(ESerialMethod::Uint32_Packed      , uint32            );
		SerialAny_Case(ESerialMethod::String_Default     , FString           );
		SerialAny_Case(ESerialMethod::Double_Default     , double            );
		SerialAny_Case(ESerialMethod::Float_Default      , float             );
		SerialAny_Case(ESerialMethod::Int32_Array        , TArray<int32>     );
		SerialAny_Case(ESerialMethod::String_Array       , TArray<FString>   );
		SerialAny_Case(ESerialMethod::Float_Array        , TArray<float>     );
		SerialAny_Case(ESerialMethod::Vector_f32         , FVector           );
		SerialAny_Case(ESerialMethod::Vector_f64         , FVector           );
		SerialAny_Case(ESerialMethod::Quat_f32           , FQuat             );
		SerialAny_Case(ESerialMethod::Quat_f64           , FQuat             );
		SerialAny_Case(ESerialMethod::LinearColor_Default, FLinearColor      );
		SerialAny_Case(ESerialMethod::MD5Hash_Default    , FMD5Hash          );
		SerialAny_Case(ESerialMethod::Transform_Default  , FTransform        );
		SerialAny_Case(ESerialMethod::Transform_Array    , TArray<FTransform>);

		case ESerialMethod::None: return true;
		case ESerialMethod::_NotImplementedYet:
		default: ensure(false);
	}
	return false;
#undef SerialAny_Case
}

} // namespace Reflect
